// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Transform.h"
#include "UObject/ObjectPtr.h"

#include "PCGMetadataCommon.generated.h"

class FScriptMapHelper;
class FScriptSetHelper;

typedef int64 PCGMetadataEntryKey;
typedef int32 PCGMetadataAttributeKey;
typedef int32 PCGMetadataValueKey;

const PCGMetadataEntryKey PCGInvalidEntryKey = -1;
const PCGMetadataEntryKey PCGFirstEntryKey = 0;
const PCGMetadataValueKey PCGDefaultValueKey = -1;
const PCGMetadataValueKey PCGNotFoundValueKey = -2;

UENUM()
enum class EPCGMetadataOp : uint8
{
	/** Take the minimum value. */
	Min,
	/** Take the maximum value. */
	Max,
	/** Subtract the values. */
	Sub,
	/** Add the values. */
	Add,
	/** Multiply the values. */
	Mul,
	/** Divide the values. */
	Div,
	/** Pick the source (first) value. */
	SourceValue,
	/** Pick the target (second) value. */
	TargetValue
};

UENUM()
enum class EPCGMetadataFilterMode : uint8
{
	/** The listed attributes will be unchanged by the projection and will not be added from the target data. */
	ExcludeAttributes,
	/** Only the listed attributes will be changed by the projection or added from the target data. */
	IncludeAttributes,
};

UENUM()
enum class EPCGMetadataDomainFlag : uint8
{
	/** Depends on the data. Should map to the same concept before multi-domain metadata. */
	Default = 0,
	
	/** Metadata at the data domain. */
	Data = 1,

	/** Metadata on elements like points on point data and entries on param data. */
	Elements = 2,
	
	/** For invalid domain. */
	Invalid = 254,

	/** For data that can have more domains. */
	Custom = 255
};

USTRUCT(BlueprintType)
struct FPCGMetadataDomainID
{
	GENERATED_BODY();

	FPCGMetadataDomainID() = default;
	
	FPCGMetadataDomainID(EPCGMetadataDomainFlag InFlag, int32 InCustomFlag = -1, FName InDebugName = NAME_None)
		: Flag(InFlag)
		, CustomFlag(InCustomFlag)
		, DebugName(InDebugName)
	{
		check(CustomFlag == -1 || InFlag == EPCGMetadataDomainFlag::Custom);
	}

	bool operator==(const FPCGMetadataDomainID& Other) const
	{
		return Flag == Other.Flag && CustomFlag == Other.CustomFlag;
	}

	bool operator<(const FPCGMetadataDomainID& Other) const
	{
		return Flag < Other.Flag || (Flag == Other.Flag && CustomFlag < Other.CustomFlag);
	}

	friend uint32 GetTypeHash(const FPCGMetadataDomainID& Item)
	{
		return HashCombine(static_cast<uint32>(Item.Flag), static_cast<uint32>(Item.CustomFlag));
	}

	friend FArchive& operator<<(FArchive& Ar, FPCGMetadataDomainID& Item)
	{
		Ar << Item.Flag;
		Ar << Item.CustomFlag;
		return Ar;
	}

	bool IsDefault() const { return Flag == EPCGMetadataDomainFlag::Default; }
	bool IsValid() const { return Flag != EPCGMetadataDomainFlag::Invalid; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "")
	EPCGMetadataDomainFlag Flag = EPCGMetadataDomainFlag::Default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "")
	int32 CustomFlag = -1;

	FName DebugName = NAME_None;
};

namespace PCGMetadataDomainID
{
	extern PCG_API const FPCGMetadataDomainID Default;
	extern PCG_API const FPCGMetadataDomainID Elements;
	extern PCG_API const FPCGMetadataDomainID Data;
	extern PCG_API const FPCGMetadataDomainID Invalid;
}

USTRUCT(BlueprintType)
struct FPCGAttributeIdentifier
{
	GENERATED_BODY();

	FPCGAttributeIdentifier() = default;

	// Needs to be backward compatible with FNames (and everything that can be constructed into a FName).
	template <typename T = FName, std::enable_if_t<std::is_constructible_v<FName, T>, bool> = true>
	FPCGAttributeIdentifier(const T& InName, FPCGMetadataDomainID InMetadataDomainID = PCGMetadataDomainID::Default)
		: Name(InName)
		, MetadataDomain(InMetadataDomainID)
	{}

	template <typename T = FName, std::enable_if_t<std::is_constructible_v<FName, T>, bool> = true>
	FPCGAttributeIdentifier& operator=(const T& InName)
	{
		Name = FName(InName);
		return *this;
	}

	UE_DEPRECATED(5.6, "Explicitly use the Name field.")
	operator FName() const
	{
		return Name;
	}

	bool operator==(const FPCGAttributeIdentifier& Other) const
	{
		return Name == Other.Name && MetadataDomain == Other.MetadataDomain;
	}

	friend uint32 GetTypeHash(const FPCGAttributeIdentifier& Item)
	{
		return HashCombine(GetTypeHash(Item.Name), GetTypeHash(Item.MetadataDomain));
	}
	
	friend FArchive& operator<<(FArchive& Ar, FPCGAttributeIdentifier& Item)
	{
		Ar << Item.Name;
		Ar << Item.MetadataDomain;
		return Ar;
	}
	
	static TSet<FPCGAttributeIdentifier> TransformNameSet(const TSet<FName>& InContainer)
	{
		TSet<FPCGAttributeIdentifier> OutContainer;
		OutContainer.Reserve(InContainer.Num());
		Algo::Transform(InContainer, OutContainer, [](const FName& InAttributeName) { return InAttributeName;});
		return OutContainer;
	}

	template <typename AllocatorType>
	static TArray<FPCGAttributeIdentifier, AllocatorType> TransformNameArray(const TArray<FName, AllocatorType>& InContainer)
	{
		TArray<FPCGAttributeIdentifier, AllocatorType> OutContainer;
		OutContainer.Reserve(InContainer.Num());
		Algo::Transform(InContainer, OutContainer, [](const FName& InAttributeName) { return InAttributeName;});
		return OutContainer;
	}

	template <typename Container>
	static TMap<FPCGMetadataDomainID, TSet<FName>> SplitByDomain(const Container& InContainer)
	{
		TMap<FPCGMetadataDomainID, TSet<FName>> OutMap;
		for (const FPCGAttributeIdentifier& It : InContainer)
		{
			OutMap.FindOrAdd(It.MetadataDomain).Add(It.Name);
		}

		return OutMap;
	}

	FString ToString() const
	{
		if (!MetadataDomain.IsValid())
		{
			return TEXT("INVALID");
		}
		else if (MetadataDomain.IsDefault())
		{
			return Name.ToString();
		}
		else
		{
			return FString::Printf(TEXT("%s.%s"), *MetadataDomain.DebugName.ToString(), *Name.ToString());
		}
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "")
	FName Name = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "", meta = (ShowOnlyInnerProperties))
	FPCGMetadataDomainID MetadataDomain;
};

UENUM(BlueprintType)
enum class EPCGMetadataTypes : uint8
{
	Float = 0,
	Double,
	Integer32,
	Integer64,
	Vector2,
	Vector,
	Vector4,
	Quaternion,
	Transform,
	String,
	Boolean,
	Rotator,
	Name,
	SoftObjectPath,
	SoftClassPath,
	
	EndLegacyTypes UMETA(Hidden),
	
	// Hide all the new types for now, so they are not selectable in the UI
	Byte UMETA(Hidden),
	Text UMETA(Hidden),
	Enum UMETA(Hidden),
	Struct UMETA(Hidden),
	Object UMETA(Hidden),
	SoftObject UMETA(Hidden),
	Class UMETA(Hidden),
	SoftClass UMETA(Hidden),

	Count UMETA(Hidden),

	Unknown = 255 UMETA(Hidden),
};

UENUM()
enum class EPCGMetadataAttributeContainerTypes
{
	None,
	Array,
	Set,
	Map
};

USTRUCT()
struct FPCGMetadataAttributeDesc
{
	GENERATED_BODY();
	
	UPROPERTY()
	FName Name;
	
	UPROPERTY()
	EPCGMetadataTypes ValueType = EPCGMetadataTypes::Unknown;
	
	UPROPERTY()
	TArray<EPCGMetadataAttributeContainerTypes> ContainerTypes;
		
	// Struct/Object/Enum Specific
	UPROPERTY()
	TObjectPtr<const UObject> ValueTypeObject = nullptr;
		
	// Map specific
	UPROPERTY()
	EPCGMetadataTypes KeyType = EPCGMetadataTypes::Unknown;
	
	UPROPERTY()
	TObjectPtr<const UObject> KeyTypeObject = nullptr;
	
	PCG_API bool IsSameType(const FPCGMetadataAttributeDesc& Other) const;
	
	/** Return true if we can write this descriptor type into the other descriptor type. (i.e. if this is a AActor*, it is compatible to UObject*) */
	PCG_API bool IsCompatible(const FPCGMetadataAttributeDesc& Other) const;
	
	PCG_API bool IsValid() const;

	/** Same type and same name */
	PCG_API bool operator==(const FPCGMetadataAttributeDesc& Other) const;

	PCG_API friend int32 GetTypeHash(const FPCGMetadataAttributeDesc& Desc);
	
	// For already predefined structs, replace `EPCGMetadataTypes::Struct` by its predefined type.
	PCG_API void FixLegacyTypeId();
	
	/** Returns true if the attribute is not a container (Array/Set/Map). */
	PCG_API bool IsSingleValue() const;

	/** Returns true if the attribute is an array. */
	PCG_API bool IsArray() const;

	/** Returns true if the attribute is a set. */
	PCG_API bool IsSet() const;

	/** Returns true if the attribute is a map. */
	PCG_API bool IsMap() const;
	
	/** Returns true if the attribute contains an object type. */
	PCG_API bool ContainsObject() const;

	/** Returns the string representing the type of this descriptor. */
	PCG_API FString GetTypeString() const;
	
	/** Convert to an array descriptor. */
	PCG_API FPCGMetadataAttributeDesc ConvertToArray() const;

	/** Convert to a single value descriptor. */
	PCG_API FPCGMetadataAttributeDesc ConvertToSingleValue() const;

	FText GetTypeText() const { return FText::FromString(GetTypeString()); }

	/** Operation to transform a desc that contains objects into their SoftPath counterpart. */
	PCG_API void ConvertObjectsToSoftPath();

	/** Create a descriptor from a Property. */
	static PCG_API FPCGMetadataAttributeDesc CreateFromProperty(const FProperty* InProperty);
};

struct FPCGAttributeProperty
{
public:
	PCG_API FPCGAttributeProperty(const FPCGMetadataAttributeDesc& InAttributeDesc);
	PCG_API ~FPCGAttributeProperty();

	// Can't copy, can move.
	FPCGAttributeProperty(const FPCGAttributeProperty& Other) = delete;
	FPCGAttributeProperty(FPCGAttributeProperty&& Other) = default;

	FPCGAttributeProperty& operator=(const FPCGAttributeProperty& Other) = delete;
	FPCGAttributeProperty& operator=(FPCGAttributeProperty&& Other) = default;

	PCG_API void Copy(void* DestPtr, const void* SrcPtr, int32 Count = 1) const;
	PCG_API bool IsValid() const;
	const FArrayProperty* GetProperty() const { return Property.Get(); }
	bool CompressData() const { return bCompressData; }
	bool IsPlainOldData() const { return bIsPlainOldData; }
	size_t GetInnerElementSize() const { return InnerElementCachedSize; }

	/** Unsafe method that will apply pointer arithmetic to get to the element in the array at the specified index. */
	PCG_API const void* GetPtrInArray(const void* InArrayStart, int32 Index) const;

private:
	TUniquePtr<FArrayProperty> Property;
	size_t InnerElementCachedSize = 0;
	bool bCompressData = false;
	bool bIsPlainOldData = false;
};

namespace PCG::Private
{
	void CopyArray(const FArrayProperty* InProperty, void* DestPtr, const void* SrcPtr, int32 Count);
	TOptional<int32> ComputeHash(const FProperty* InProperty, const void* InValuePtr, const int32 InCount);
	bool CompareArrays(const FProperty* InProperty, const void* LHSPtr, const int32 LHSCount, const void* RHSPtr, const int32 RHSCount);
}
