// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataAttributeTraits.h"

#include "Misc/Zip.h"
#include "Templates/Function.h"
#include "UObject/UnrealType.h"

class FPCGMetadataAttributeBase;
class IPCGAttributeAccessor;

namespace PCG
{
template <typename T> requires (!std::is_trivially_constructible_v<T>)
static void ConstructRange(void* DestPtr, int32 Count)
{
	TArrayView<T> View = MakeArrayView(static_cast<T*>(DestPtr), Count);
	for (T& Element : View)
	{
		new (&Element) T;
	}
}

template <typename T> requires (!std::is_trivially_destructible_v<T>)
static void DestructRange(void* DestPtr, int32 Count)
{
	DestructItems(static_cast<T*>(DestPtr), Count);
}

template <typename T> requires (!std::is_trivially_copyable_v<T>)
static void CopyRange(void* DestPtr, const void* SrcPtr, int32 Count)
{
	TConstArrayView<T> SrcView = MakeArrayView(static_cast<const T*>(SrcPtr), Count);
	TArrayView<T> DestView = MakeArrayView(static_cast<T*>(DestPtr), Count);
	for (auto [SrcElement, DestElement] : UE::Zip(SrcView, DestView))
	{
		DestElement = SrcElement;
	}
}

/**
* Typed erased struct to be able to convert an array type to another. when we do not know the underlying type at compile time.
* Need to know how to construct/destruct/copy the underlying type, and what is its element and alignment size.
*/
class FPCGAccessorBuffer
{
	template <typename T>
	friend class TPCGArrayAccessorWrapper;

public:
	FPCGAccessorBuffer() = default;
	
	PCG_API FPCGAccessorBuffer(const FPCGAccessorBuffer& Other);
	PCG_API FPCGAccessorBuffer& operator=(const FPCGAccessorBuffer& Other);
	PCG_API FPCGAccessorBuffer(FPCGAccessorBuffer&& Other);
	PCG_API FPCGAccessorBuffer& operator=(FPCGAccessorBuffer&& Other);

	PCG_API ~FPCGAccessorBuffer();
	PCG_API void SetupAndAllocate(int32 Count, const FPCGMetadataAttributeDesc& InType);
	PCG_API void Reset(int32 Slack = 0);
	
	PCG_API void* GetOwnedMemoryPtr();
	PCG_API int32 Num() const;
	PCG_API bool IsOwningMemory() const;
	PCG_API const FPCGMetadataAttributeDesc& GetUnderlyingDesc() const;
	
	/** Copy the values passed as pointer, interpreted as the underlying type. InValuesPtr must be of the type passed as parameter. Buffer will be allocated if needed. */
	PCG_API bool CopyFromPointers(TConstArrayView<const void*> InValuesPtr, const FPCGMetadataAttributeDesc& InType);

	/** Will move our own memory to the Other array helper of a given type. Only works if the type matches perfectly. */
	PCG_API void MoveMemoryTo(FScriptArray& Other, const FPCGMetadataAttributeDesc& InType);
	
	template <typename T>
	void SetupFromKnownType()
	{
		ResetInternals();

		Internals.UnderlyingDesc = PCG::Private::GetDefaultAttributeDesc<T>();
		Internals.ElementSize = sizeof(T);
		Internals.AlignmentSize = alignof(T);
		if constexpr (!std::is_trivially_constructible_v<T>)
		{
			Internals.ConstructFunc = &ConstructRange<T>;
		}
		if constexpr (!std::is_trivially_destructible_v<T>)
		{
			Internals.DestructFunc = &DestructRange<T>;
		}
		if constexpr (!std::is_trivially_copyable_v<T>)
		{
			Internals.CopyFunc = &CopyRange<T>;
		}
	}

private:
	using FConstructFunc = void(*)(void* /*DestPtr*/, int32 /*Count*/);
	using FDestructFunc = void(*)(void* /*DestPtr*/, int32 /*Count*/);
	using FCopyFunc = void(*)(void* /*DestPtr*/, const void* /*SrcPtr*/, int32 /*Count*/);
	
	/** Will move our own memory to the Other array. */
	PCG_API void MoveMemoryTo(FScriptArray& Other);

	/** Will copy our own memory to the Other array. */
	PCG_API void CopyMemoryTo(FScriptArray& Other) const;

	PCG_API void Allocate(int32 Count);

	PCG_API void ResetInternals();

	struct FInternals
	{
		FPCGMetadataAttributeDesc UnderlyingDesc;

		// If we know the type, use those accelerated functions that work on ranges
		int32 ElementSize = -1;
		int32 AlignmentSize = -1;
		FConstructFunc ConstructFunc = nullptr;
		FDestructFunc DestructFunc = nullptr;
		FCopyFunc CopyFunc = nullptr;

		// If we do not know the type, we have to rely on the FProperty system, that will go one by one.
		TSharedPtr<FPCGAttributeProperty> UnderlyingProperty;
		TFunction<void(void*, int32)> ComplexConstructFunc;
		TFunction<void(void*, int32)> ComplexDestructFunc;
		TFunction<void(void*, const void*, int32)> ComplexCopyFunc;
	};

	FInternals Internals;

	bool bOwnMemory = false;

	FScriptArray OwnMemory;
};

/**
 * Class meant to be used everytime we need to get an array from attributes/accessors.
 * When requesting a type that is the exact same type as the attribute/accessor underlying type, only the view will be set.
 * If the type need to be converted, the other array will be set, and the view will point on this array.
 * When this object gets deleted, the memory will be freed normally.
 */
template <typename T>
class TPCGArrayAccessorWrapper
{
public:
	using ElementType = T;

	TPCGArrayAccessorWrapper()
	{
		OwnMemory.SetupFromKnownType<T>();
	}

	TPCGArrayAccessorWrapper(const TPCGArrayAccessorWrapper& Other)
	{
		if (Other.IsOwningMemory())
		{
			OwnMemory = Other.OwnMemory;
			SetupViewOnMemory();
		}
		else
		{
			View = Other.View;
		}
	}
	
	TPCGArrayAccessorWrapper& operator=(const TPCGArrayAccessorWrapper& Other)
	{
		if (this == &Other)
		{
			// Nothing to do
			return *this;
		}

		if (Other.IsOwningMemory())
		{
			OwnMemory = Other.OwnMemory;
			SetupViewOnMemory();
		}
		else
		{
			OwnMemory.Reset();
			View = Other.View;
		}

		return *this;
	}
	
	TPCGArrayAccessorWrapper(TPCGArrayAccessorWrapper&& Other)
	{
		if (Other.IsOwningMemory())
		{
			OwnMemory = MoveTemp(Other.OwnMemory);
			SetupViewOnMemory();
		}
		else
		{
			View = Other.View;
		}

		Other.OwnMemory.Reset();
		Other.View = TConstArrayView<T>{};
	}
	
	TPCGArrayAccessorWrapper& operator=(TPCGArrayAccessorWrapper&& Other)
	{
		if (this == &Other)
		{
			// Nothing to do
			return *this;
		}

		if (Other.IsOwningMemory())
		{
			OwnMemory = MoveTemp(Other.OwnMemory);
			SetupViewOnMemory();
		}
		else
		{
			OwnMemory.Reset();
			View = Other.View;
		}

		Other.OwnMemory.Reset();
		Other.View = TConstArrayView<T>{};

		return *this;
	}

	TArrayView<T> AllocateOwnMemory(int32 Num)
	{
		if (Num == 0)
		{
			OwnMemory.Reset();
			View = TConstArrayView<T>{};
			return TArrayView<T>{};
		}
		else
		{
			OwnMemory.Allocate(Num);
			SetupViewOnMemory();
			return MakeArrayView(static_cast<T*>(OwnMemory.GetOwnedMemoryPtr()), OwnMemory.Num());
		}
	}

	bool IsOwningMemory() const { return OwnMemory.IsOwningMemory(); }

	/**
	 * Steal the memory owned by this wrapper. Will also reset the view.
	 */
	TArray<T> StealMemory()
	{
		View = TConstArrayView<T>{};
		TArray<T> Result;
		OwnMemory.MoveMemoryTo(reinterpret_cast<FScriptArray&>(Result));
		return Result;
	}

	/** Access the view on the array. Point to somewhere in memory or to our own memory. */
	TConstArrayView<T> GetView() const { return View; }

	/** Setup the view. If memory was owned, it'll be freed. */
	void SetupView(TConstArrayView<T> InView)
	{
		View = InView;
		OwnMemory.Reset();
	}

private:
	void SetupViewOnMemory()
	{
		View = MakeConstArrayView(static_cast<const T*>(OwnMemory.GetOwnedMemoryPtr()), OwnMemory.Num());
	}

	TConstArrayView<T> View;
	FPCGAccessorBuffer OwnMemory;
};

/**
 * Class meant to wrap around a FScriptSetHelper, to manipulate it like a TSet and have type checking.
 * It is not meant to be initialized by the user, but by the generic attributes (hence why only the default ctor is exposed).
 * Since it contains a FScriptSetHelper, which contains a FScriptSet pointer, lifetime is not guaranteed. 
 * Can convert to a TSet, but this operation will do a full copy, so it might be expensive.
 * Note that it is also read-only, so support only const operations.
 */
template <typename T>
class TScriptSetWrapper
{
public:
	// HACK: We need to use a dummy property so the helper can be default constructed.
	// We always verify that the helper is valid before using it, so in theory this is safe.
	// We also don't care for unicity across dlls, since validity is not checking the dummy property, so static inline is fine.
	static inline FSetProperty Dummy{nullptr, NAME_None};
	
	TScriptSetWrapper()
		: Helper(&Dummy, nullptr, FScriptSet::GetScriptLayout(0, 1))
	{}
	
	bool IsValid() const { return Helper.Set != nullptr; }
	
	const FScriptSetHelper* GetHelper() const { return IsValid() ? &Helper : nullptr; }
	
	bool Contains(const T& Value) const
	{
		return IsValid() && Helper.FindElementIndexFromHash(&Value) != INDEX_NONE;
	}
	
	int32 Num() const { return IsValid() ? Helper.Num() : 0;}
	
	TSet<T> Intersection(const TSet<T>& Other) const
	{
		if (!IsValid())
		{
			return {};
		}
		
		TSet<T> Result;
		for (const T& Value : Other)
		{
			if (Helper.FindElementIndexFromHash(&Value) != INDEX_NONE)
			{
				Result.Add(Value);
			}
		}
		
		return Result;
	}
	
	TSet<T> Union(const TSet<T>& Other) const
	{
		if (!IsValid())
		{
			return Other;
		}
		
		TSet<T> Result = Other;
		Result.Reserve(Other.Num() + Num());
		ForEach([&Result](const T& Value) { Result.Add(Value); });
		
		return Result;
	}
	
	explicit operator TSet<T>() const
	{
		return Union(TSet<T>{});
	}
	
	template <typename FuncType>
	void ForEach(FuncType&& Func) const
	{
		if (!IsValid())
		{
			return;
		}
		
		for (FScriptSetHelper::FIterator It = Helper.CreateIterator(); It; ++It)
		{
			Func(*reinterpret_cast<const T*>(Helper.GetElementPtr(It)));
		}
	}
	
private:
	friend FPCGMetadataAttributeBase;
	friend IPCGAttributeAccessor;
	FScriptSetHelper Helper;
};
	
/**
 * Class meant to wrap around a FScriptMapHelper, to manipulate it like a TMap and have type checking.
 * It is not meant to be initialized by the user, but by the generic attributes (hence why only the default ctor is exposed).
 * Since it contains a FScriptMapHelper, which contains a FScriptMap pointer, lifetime is not guaranteed. 
 * Can convert to a TMap, but this operation will do a full copy, so it might be expensive.
 * Note that it is also read-only, so support only const operations.
 */
template <typename KeyType, typename ValueType>
class TScriptMapWrapper
{
public:
	// HACK: We need to use dummy properties so the helper can be default constructed.
	// We always verify that the helper is valid before using it, so in theory this is safe.
	// We also don't care for unicity across dlls, since validity is not checking the dummy property, so static inline is fine.
	static inline FBoolProperty Dummy{nullptr, NAME_None};
	
	TScriptMapWrapper()
		: Helper(&Dummy, &Dummy, nullptr, FScriptMap::GetScriptLayout(0, 1, 0, 1), EMapPropertyFlags::None)
	{}
	
	bool IsValid() const { return Helper.HeapMap != nullptr; }
	
	const FScriptSetHelper* GetHelper() const { return IsValid() ? &Helper : nullptr; }
	
	bool Contains(const KeyType& Key) const
	{
		return IsValid() && Helper.FindMapIndexWithKey(&Key) != INDEX_NONE;
	}
	
	const ValueType* Find(const KeyType& Key) const
	{
		if (!IsValid())
		{
			return nullptr;
		}
		
		const int32 Index = Helper.FindMapIndexWithKey(&Key);
		if (Index == INDEX_NONE)
		{
			return nullptr;
		}
		
		return reinterpret_cast<const ValueType*>(const_cast<FScriptMapHelper&>(Helper).GetValuePtr(Index));
	}
	
	const ValueType& operator[](const KeyType& Key) const
	{
		const ValueType* ValuePtr = Find(Key);
		check(ValuePtr);
		return *ValuePtr;
	}
	
	int32 Num() const { return IsValid() ? Helper.Num() : 0;}
	
	explicit operator TMap<KeyType, ValueType>() const
	{
		if (!IsValid())
		{
			return {};
		}
		
		TMap<KeyType, ValueType> Result;
		Result.Reserve(Helper.Num());
		ForEach([&Result](const KeyType& Key, const ValueType& Value) { Result.Emplace(Key, Value); });
		
		return Result;
	}
	
	template <typename FuncType>
	void ForEach(FuncType&& Func) const
	{
		if (!IsValid())
		{
			return;
		}
		
		for (FScriptMapHelper::FIterator It = Helper.CreateIterator(); It; ++It)
		{
			Func(*reinterpret_cast<const KeyType*>(Helper.GetKeyPtr(It)), *reinterpret_cast<const ValueType*>(Helper.GetValuePtr(It)));
		}
	}
	
private:
	friend FPCGMetadataAttributeBase;
	friend IPCGAttributeAccessor;
	FScriptMapHelper Helper;
};
}
