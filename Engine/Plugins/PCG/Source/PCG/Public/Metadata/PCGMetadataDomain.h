// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Metadata/PCGMetadataCommon.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Utils/PCGValueRange.h"

#include "PCGMetadataDomain.generated.h"

#define UE_API PCG_API

class UPCGMetadata;
class FPCGMetadataDomain;

// @todo_pcg: Support blueprint
struct FPCGMetadataDomainInitializeParams
{
	explicit FPCGMetadataDomainInitializeParams(const FPCGMetadataDomain* InParent, const TSet<FName>* InFilteredAttributes = nullptr)
		: Parent(InParent)
	{
		if (InFilteredAttributes)
		{
			FilteredAttributes = *InFilteredAttributes;
		}
	}
	
	/** The parent metadata to use as a template, if any (can be null). */
	const FPCGMetadataDomain* Parent = nullptr;
	
	/** Optional list of attributes to exclude or include when adding the attributes from the parent. */
	TOptional<TSet<FName>> FilteredAttributes;
	
	/** Defines attribute filter operation. */
	EPCGMetadataFilterMode FilterMode = EPCGMetadataFilterMode::ExcludeAttributes;
	
	/** Defines attribute filter operator. */
	EPCGStringMatchingOperator MatchOperator = EPCGStringMatchingOperator::Equal;

	/** Optional keys to copy over, in case of copy operation. Can be constructed from an array. */
	TOptional<TConstPCGValueRange<PCGMetadataEntryKey>> OptionalEntriesToCopy;
};

class FPCGMetadataDomain
{
	friend UPCGMetadata;

public:
	UE_API FPCGMetadataDomain(UPCGMetadata* InTopMetadata, FPCGMetadataDomainID InMetadataDomainID);
	UE_API virtual ~FPCGMetadataDomain();
	
	UE_API virtual void Serialize(FArchive& InArchive);
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const;
	
	/** Initializes the sub metadata from a parent sub metadata, if any (can be null). Copies attributes and values. */
	UE_API void Initialize(const FPCGMetadataDomain* InParent);

	/** Initializes the sub metadata from a parent sub metadata. Copies attributes and values. */
	UE_API void Initialize(const FPCGMetadataDomainInitializeParams& InParams);

	/** Initializes the sub metadata from a parent sub metadata by copying all attributes to it. */
	UE_API void InitializeAsCopy(const FPCGMetadataDomain* InMetadataToCopy);

	/** Initializes the metadata from a parent metadata by copy filtered attributes only to it */
	UE_API void InitializeAsCopy(const FPCGMetadataDomainInitializeParams& InParams);

	/** Creates missing attributes from another metadata if they are not currently present - note that this does not copy values */
	UE_API bool AddAttributes(const FPCGMetadataDomain* InOther);

	/** Creates missing attributes from another metadata if they are not currently present - note that this does not copy values. */
	UE_API bool AddAttributes(const FPCGMetadataDomainInitializeParams& InParams);

	/** Creates missing attribute from another metadata if it is not currently present - note that this does not copy values */
	UE_API bool AddAttribute(const FPCGMetadataDomain* InOther, FName AttributeName);

	/** Copies attributes from another metadata, including entries & values. Warning: this is intended when dealing with the same data set */
	UE_API void CopyAttributes(const FPCGMetadataDomain* InOther);

	/** Copies an attribute from another metadata, including entries & values. Warning: this is intended when dealing with the same data set */
	UE_API void CopyAttribute(const FPCGMetadataDomain* InOther, FName AttributeToCopy, FName NewAttributeName);

	/** Copies another attribute, with options to keep its parent and copy entries/values */
	UE_API FPCGMetadataAttributeBase* CopyAttribute(const FPCGMetadataAttributeBase* OriginalAttribute, FName NewAttributeName, bool bKeepParent, bool bCopyEntries, bool bCopyValues);

	/** Returns this metadata's parent */
	const FPCGMetadataDomain* GetParent() const { return Parent; }
	UE_API const FPCGMetadataDomain* GetRoot() const;
	const UPCGMetadata* GetTopMetadata() const { return TopMetadata; }
	UE_API bool HasParent(const FPCGMetadataDomain* InTentativeParent) const;

	/** Unparents current metadata by flattening the attributes (values, entries, etc.) */
	UE_API void FlattenImpl();

	/** Unparents current metadata, flatten attribute and only keep the entries specified. Return true if something has changed and keys needs be updated. */
	UE_API bool FlattenAndCompress(const TArrayView<const PCGMetadataEntryKey>& InEntryKeysToKeep);

	/** Creates an attribute given a property.
	* @param AttributeName: Target attribute to create
	* @param Object: Object to get the property value from
	* @param Property: The property to set from
	* @returns true if the attribute creation succeeded
	*/
	UE_API bool CreateAttributeFromProperty(FName AttributeName, const UObject* Object, const FProperty* Property);

	/** Creates an attribute given a property.
	* @param AttributeName: Target attribute to create
	* @param Data: Data pointer to get the property value from
	* @param Property: The property to set from
	* @returns true if the attribute creation succeeded
	*/
	UE_API bool CreateAttributeFromDataProperty(FName AttributeName, const void* Data, const FProperty* Property);

	/** Set an attribute given a property and its value.
	* @param AttributeName: Target attribute to set the property's value to
	* @param EntryKey: Metadata entry key to set the value to
	* @param Object: Object to get the property value from
	* @param Property: The property to set from
	* @param bCreate: If true and the attribute doesn't exists, it will create an attribute based on the property type
	* @returns true if the attribute creation (if required) and the value set succeeded
	*/
	UE_API bool SetAttributeFromProperty(FName AttributeName, PCGMetadataEntryKey& EntryKey, const UObject* Object, const FProperty* Property, bool bCreate);

	/** Set an attribute given a property and its value.
	* @param AttributeName: Target attribute to set the property's value to
	* @param EntryKey: Metadata entry key to set the value to
	* @param Data: Data pointer to get the property value from
	* @param Property: The property to set from
	* @param bCreate: If true and the attribute doesn't exists, it will create an attribute based on the property type
	* @returns true if the attribute creation (if required) and the value set succeeded
	*/
	UE_API bool SetAttributeFromDataProperty(FName AttributeName, PCGMetadataEntryKey& EntryKey, const void* Data, const FProperty* Property, bool bCreate);

	/** Get attributes */
	UE_API FPCGMetadataAttributeBase* GetMutableAttribute(FName AttributeName);
	UE_API const FPCGMetadataAttributeBase* GetConstAttribute(FName AttributeName) const;
	UE_API const FPCGMetadataAttributeBase* GetConstAttributeById(int32 AttributeId) const;
	
	UE_API bool HasAttribute(FName AttributeName) const;
	UE_API bool HasCommonAttributes(const FPCGMetadataDomain* InMetadata) const;

	/** Return the number of attributes in this metadata. */
	UE_API int32 GetAttributeCount() const;

	template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
	FPCGMetadataAttribute<T>* GetMutableTypedAttribute(FName AttributeName);

	template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
	UE_DEPRECATED(5.8, "Should not be used anymore.")
	FPCGMetadataAttribute<T>* GetMutableTypedAttribute_Unsafe(FName AttributeName);

	template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
	const FPCGMetadataAttribute<T>* GetConstTypedAttribute(FName AttributeName) const;
	
	UE_API void GetAttributes(TArray<FName>& AttributeNames, TArray<EPCGMetadataTypes>& AttributeTypes) const;

	/** Returns name of the most recently created attribute, or none if no attributes are present. */
	UE_API FName GetLatestAttributeNameOrNone() const;

	/** Delete/Hide attribute */
	// Due to stream inheriting, we might want to consider "hiding" parent stream and deleting local streams only
	UE_API void DeleteAttribute(FName AttributeName);

	/** Copy attribute */
	UE_API bool CopyExistingAttribute(FName AttributeToCopy, FName NewAttributeName, bool bKeepParent = true);

	/** Rename attribute */
	UE_API bool RenameAttribute(FName AttributeToRename, FName NewAttributeName);

	/** Clear/Reinit attribute */
	UE_API void ClearAttribute(FName AttributeToClear);

	/** Change type of an attribute */
	UE_API bool ChangeAttributeType(FName AttributeName, const FPCGMetadataAttributeDesc& AttributeNewDesc);
	
	UE_DEPRECATED(5.8, "Use the version with the Attribute Desc")
	UE_API bool ChangeAttributeType(FName AttributeName, int16 AttributeNewType);

	/** Adds a unique entry key to the metadata */
	UE_API int64 AddEntry(int64 ParentEntryKey = -1);

	/** Adds a unique entry key to the metadata for all the parent entry keys. */
	UE_API TArray<int64> AddEntries(TArrayView<const int64> ParentEntryKeys);

	/** Adds a unique entry key to the metadata for all the parent entry keys, in place. */
	UE_API void AddEntriesInPlace(TArrayView<int64*> ParentEntryKeys);

	/** Advanced method.
	*   In a MT context, we might not want to add the entry directly (because of write lock). Call this to generate an unique index in the MT context
	*   And call AddDelayedEntries at the end when you want to add all the entries.
	*   Make sure to not call AddEntry in a different thread or even in the same thread if you use this function, or it will change all the indexes.
	*/
	UE_API int64 AddEntryPlaceholder();

	/** Advanced method.
	*   If you used AddEntryPlaceholder, call this function at the end of your MT processing to add all the entries in one shot.
	*   Make sure to call this one at the end!!
	*   @param AllEntries Array of pairs. First item is the EntryIndex generated by AddEntryPlaceholder, the second the ParentEntryKey (cf AddEntry)
	*/
	UE_API void AddDelayedEntries(const TArray<TTuple<int64, int64>>& AllEntries);

	/** Initializes the metadata entry key. Returns true if key set from either parent */
	UE_API bool InitializeOnSet(PCGMetadataEntryKey& InOutKey, PCGMetadataEntryKey InParentKeyA = PCGInvalidEntryKey, const FPCGMetadataDomain* InParentMetadataA = nullptr, PCGMetadataEntryKey InParentKeyB = PCGInvalidEntryKey, const FPCGMetadataDomain* InParentMetadataB = nullptr);

	/** Metadata chaining mechanism */
	UE_API PCGMetadataEntryKey GetParentKey(PCGMetadataEntryKey LocalItemKey) const;

	/** Metadata chaining mechanism for bulk version. Can provide a mask to update only a subset of the passed keys. */
	UE_API void GetParentKeys(TArrayView<PCGMetadataEntryKey> LocalItemKeys, const TBitArray<>* Mask = nullptr) const;

	/** Metadata chaining mechanism for bulk version. Can provide a mask to update only a subset of the passed keys. */
	UE_API void GetParentKeysWithRange(TPCGValueRange<PCGMetadataEntryKey> LocalItemKeys, const TBitArray<>* Mask = nullptr) const;

	/** Attributes operations */
	UE_API void MergeAttributes(PCGMetadataEntryKey InKeyA, const FPCGMetadataDomain* InMetadataA, PCGMetadataEntryKey InKeyB, const FPCGMetadataDomain* InMetadataB, PCGMetadataEntryKey& OutKey, EPCGMetadataOp Op);
	UE_API void MergeAttributesSubset(PCGMetadataEntryKey InKeyA, const FPCGMetadataDomain* InMetadataA, const FPCGMetadataDomain* InMetadataSubetA, PCGMetadataEntryKey InKeyB, const FPCGMetadataDomain* InMetadataB, const FPCGMetadataDomain* InMetadataSubsetb, PCGMetadataEntryKey& OutKey, EPCGMetadataOp Op);

	UE_API void ResetWeightedAttributes(PCGMetadataEntryKey& OutKey);
	UE_API void AccumulateWeightedAttributes(PCGMetadataEntryKey InKey, const FPCGMetadataDomain* InMetadata, float Weight, bool bSetNonInterpolableAttributes, PCGMetadataEntryKey& OutKey);

	UE_API void SetAttributes(PCGMetadataEntryKey InKey, const FPCGMetadataDomain* InMetadata, PCGMetadataEntryKey& OutKey);
	UE_API void SetAttributes(const TArrayView<const PCGMetadataEntryKey>& InOriginalKeys, const FPCGMetadataDomain* InMetadata, const TArrayView<PCGMetadataEntryKey>* InOutOptionalKeys = nullptr, FPCGContext* OptionalContext = nullptr);
	UE_API void SetAttributes(const TArrayView<const PCGMetadataEntryKey>& InKeys, const FPCGMetadataDomain* InMetadata, const TArrayView<PCGMetadataEntryKey>& OutKeys, FPCGContext* OptionalContext = nullptr);

	UE_API void ComputeWeightedAttribute(PCGMetadataEntryKey& OutKey, const TArrayView<TPair<PCGMetadataEntryKey, float>>& InWeightedKeys, const FPCGMetadataDomain* InMetadata);

	UE_API int64 GetItemKeyCountForParent() const;
	UE_API int64 GetLocalItemCount() const;

	/** Return the number of entries in metadata including the parent entries. */
	UE_API int64 GetItemCountForChild() const;

	/**
	* Create a new attribute. If the attribute already exists, it will raise a warning (use FindOrCreateAttribute if this usecase can arise)
	* If the attribute already exists but is of the wrong type, it will fail and return nullptr. Same if the name is invalid.
	* Return a typed attribute pointer, of the requested type T.
	*/
	template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
	FPCGMetadataAttribute<T>* CreateAttribute(FName AttributeName, const T& DefaultValue, bool bAllowsInterpolation = true, bool bOverrideParent = true);
	
	/**
	* Create a new attribute, that will be generic and not typed. This should never be static cast to a FPCGMetadataAttribute<T>* as this would
	* be undefined behavior.
	* If the attribute already exists, it will raise a warning (use FindOrCreateAttribute if this usecase can arise)
	* If the attribute already exists but is of the wrong type, it will fail and return nullptr. Same if the name is invalid.
	*/
	template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == false)
	FPCGMetadataAttributeBase* CreateAttribute(FName AttributeName, const T& DefaultValue, bool bAllowsInterpolation = true, bool bOverrideParent = true);
	
	/**
	* Create a new attribute, that will be either generic or typed, depending the type given by the desc. The default value will not be set.
	* If the attribute already exists, it will raise a warning (use FindOrCreateAttribute if this usecase can arise)
	* If the attribute already exists but is of the wrong type, it will fail and return nullptr. Same if the name is invalid.
	*/
	UE_API FPCGMetadataAttributeBase* CreateAttribute(const FPCGMetadataAttributeDesc& AttributeDesc, bool bAllowsInterpolation = true, bool bOverrideParent = true);

	/**
	* Find or create an attribute. Follows CreateAttribute signature.
	* Extra boolean bOverwriteIfTypeMismatch allows to overwrite an existing attribute if the type mismatch.
	* Same as CreateAttribute, it will return nullptr if the attribute name is invalid.
	* Return a typed attribute pointer, of the requested type T.
	*/
	template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
	FPCGMetadataAttribute<T>* FindOrCreateAttribute(FName AttributeName, const T& DefaultValue = T{}, bool bAllowsInterpolation = true, bool bOverrideParent = true, bool bOverwriteIfTypeMismatch = true);
	
	/**
	* Find or create an attribute. Follows CreateAttribute signature.
	* Extra boolean bOverwriteIfTypeMismatch allows to overwrite an existing attribute if the type mismatch.
	* Same as CreateAttribute, it will return nullptr if the attribute name is invalid.
	* Return a generic attribute pointer.
	*/
	template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == false)
	FPCGMetadataAttributeBase* FindOrCreateAttribute(FName AttributeName, const T& DefaultValue = T{}, bool bAllowsInterpolation = true, bool bOverrideParent = true, bool bOverwriteIfTypeMismatch = true);

	/** Computes Crc from all attributes & keys from outer's data. */
	UE_API void AddToCrc(FArchiveCrc32& Ar, const UPCGData* Data, bool bFullDataCrc) const;

	FPCGMetadataDomainID GetDomainID() const { return DomainID; }

	bool SupportsMultiEntries() const { return bSupportMultiEntries; }
	
private:
	UE_API FPCGMetadataAttributeBase* CreateAttributeInternal(const FPCGMetadataAttributeDesc& AttributeDesc, bool bAllowsInterpolation, bool bOverrideParent, TFunctionRef<void(FPCGMetadataAttributeBase*)> SetupDefaultValue, bool bAlreadyLocked = false);
	UE_API FPCGMetadataAttributeBase* FindOrCreateAttributeInternal(const FPCGMetadataAttributeDesc& AttributeDesc, bool bAllowsInterpolation, bool bOverrideParent, bool bOverwriteIfTypeMismatch, TFunctionRef<void(FPCGMetadataAttributeBase*)> SetupDefaultValue);
	
	UE_API FPCGMetadataAttributeBase* GetMutableAttribute_Unsafe(FName AttributeName);
	UE_API const FPCGMetadataAttributeBase* GetConstAttribute_Unsafe(FName AttributeName) const;

protected:
	UE_API FPCGMetadataAttributeBase* CopyAttribute(FName AttributeToCopy, FName NewAttributeName, bool bKeepParent, bool bCopyEntries, bool bCopyValues);

	UE_API bool ParentHasAttribute(FName AttributeName) const;

	UE_API void AddAttributeInternal(FName AttributeName, FPCGMetadataAttributeBase* Attribute);
	UE_API void RemoveAttributeInternal(FName AttributeName);

	UE_API void SetLastCachedSelectorOnOwner(FName AttributeName);
	
	UE_API FPCGMetadataAttributeBase* AllocateAttributeFromDesc(const FPCGMetadataAttributeDesc& InDesc, const FPCGMetadataAttributeBase* InAttributeParent = nullptr);

	UPCGMetadata* TopMetadata = nullptr;
	FPCGMetadataDomainID DomainID;
	const FPCGMetadataDomain* Parent = nullptr;

	// Cached value on construction to know if we support multi entries
	bool bSupportMultiEntries = true;

	TMap<FName, FPCGMetadataAttributeBase*> Attributes;
	PCGMetadataAttributeKey NextAttributeId = 0;

	TArray<PCGMetadataEntryKey> ParentKeys;
	int64 ItemKeyOffset = 0;

	mutable PCG::FSharedLock AttributeLock;
	mutable PCG::FSharedLock ItemLock;

	TAtomic<int64> DelayedEntriesIndex = 0;
};


template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
FPCGMetadataAttribute<T>* FPCGMetadataDomain::CreateAttribute(FName AttributeName, const T& DefaultValue, bool bAllowsInterpolation, bool bOverrideParent)
{
	FPCGMetadataAttributeBase* Attribute = CreateAttributeInternal(PCG::Private::MakeAttributeDesc<T>(AttributeName), bAllowsInterpolation, bOverrideParent, [&DefaultValue](FPCGMetadataAttributeBase* Attribute)
	{
		Attribute->SetDefaultValue(DefaultValue);
	});
	return static_cast<FPCGMetadataAttribute<T>*>(Attribute);
}

template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == false)
FPCGMetadataAttributeBase* FPCGMetadataDomain::CreateAttribute(FName AttributeName, const T& DefaultValue, bool bAllowsInterpolation, bool bOverrideParent)
{
	return CreateAttributeInternal(PCG::Private::MakeAttributeDesc<T>(AttributeName), bAllowsInterpolation, bOverrideParent, [&DefaultValue](FPCGMetadataAttributeBase* Attribute)
	{
		Attribute->SetDefaultValue(DefaultValue);
	});
}

template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
FPCGMetadataAttribute<T>* FPCGMetadataDomain::FindOrCreateAttribute(FName AttributeName, const T& DefaultValue, bool bAllowsInterpolation, bool bOverrideParent, bool bOverwriteIfTypeMismatch)
{
	FPCGMetadataAttributeBase* Attribute = FindOrCreateAttributeInternal(PCG::Private::MakeAttributeDesc<T>(AttributeName), bAllowsInterpolation, bOverrideParent, bOverwriteIfTypeMismatch, [&DefaultValue](FPCGMetadataAttributeBase* Attribute)
	{
		Attribute->SetDefaultValue(DefaultValue);
	});
	
	return static_cast<FPCGMetadataAttribute<T>*>(Attribute);
}

template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == false)
FPCGMetadataAttributeBase* FPCGMetadataDomain::FindOrCreateAttribute(FName AttributeName, const T& DefaultValue, bool bAllowsInterpolation, bool bOverrideParent, bool bOverwriteIfTypeMismatch)
{
	return FindOrCreateAttributeInternal(PCG::Private::MakeAttributeDesc<T>(AttributeName), bAllowsInterpolation, bOverrideParent, bOverwriteIfTypeMismatch, [&DefaultValue](FPCGMetadataAttributeBase* Attribute)
	{
		Attribute->SetDefaultValue(DefaultValue);
	});
}

template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
FPCGMetadataAttribute<T>* FPCGMetadataDomain::GetMutableTypedAttribute_Unsafe(FName AttributeName)
{
	FPCGMetadataAttributeBase* BaseAttribute = GetMutableAttribute_Unsafe(AttributeName);
	return (BaseAttribute && (BaseAttribute->GetAttributeDesc().IsSameType(PCG::Private::GetDefaultAttributeDesc<T>())))
		? static_cast<FPCGMetadataAttribute<T>*>(BaseAttribute)
		: nullptr;
}

template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
FPCGMetadataAttribute<T>* FPCGMetadataDomain::GetMutableTypedAttribute(FName AttributeName)
{
	FPCGMetadataAttributeBase* BaseAttribute = GetMutableAttribute(AttributeName);
	return (BaseAttribute && (BaseAttribute->GetAttributeDesc().IsSameType(PCG::Private::GetDefaultAttributeDesc<T>())))
		? static_cast<FPCGMetadataAttribute<T>*>(BaseAttribute)
		: nullptr;
}

template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
const FPCGMetadataAttribute<T>* FPCGMetadataDomain::GetConstTypedAttribute(FName AttributeName) const
{
	const FPCGMetadataAttributeBase* BaseAttribute = GetConstAttribute(AttributeName);
	return (BaseAttribute && (BaseAttribute->GetAttributeDesc().IsSameType(PCG::Private::GetDefaultAttributeDesc<T>())))
		? static_cast<const FPCGMetadataAttribute<T>*>(BaseAttribute)
		: nullptr;
}

/**
 * Handle wrapping around a Metadata domain to be copied and passed around in BP.
 */
USTRUCT(BlueprintType)
struct FPCGMetadataDomainHandle
{
	GENERATED_BODY()
	
	const FPCGMetadataDomain* GetConstDomain() const;
	FPCGMetadataDomain* GetMutableDomain();

private:
	friend UPCGMetadata;

	bool bConst = false;
	TWeakPtr<FPCGMetadataDomain> DomainHandle;
};

UCLASS()
class UPCGMetadataDomainLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
private:
	/**
	 * Create an attribute of the type given by the DefaultValue.
	 * @param Domain Handle on the Metadata domain to create the attribute on
	 * @param AttributeName Name of the attribute
	 * @param DefaultValue Default value of the attribute, also driving the underlying type
	 * @param bAllowsInterpolation If the attribute can be interpolated
	 * @return true if the creation succeeded.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "PCG|MetadataDomain", meta = (ScriptMethod, CustomStructureParam = "DefaultValue", AllowAbstract = "false"))
	static bool CreateAttributeFromValue(const FPCGMetadataDomainHandle& Domain, const FName AttributeName, const int32& DefaultValue, bool bAllowsInterpolation = true);
	DECLARE_FUNCTION(execCreateAttributeFromValue);
};

#undef UE_API
