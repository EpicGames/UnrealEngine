// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"

struct FPCGAttributePropertySelector;
struct FPCGContext;
class UPCGData;

namespace PCGMetadataPartitionCommon
{
	/**
	* Partition the incoming data on the given attribute.
	* Will return an array of data. Each data will contain the entries (in stable order) for a given partition.
	* @param InData - Data to partition.
	* @param InSelector - Selector on the attribute to partition.
	* @param InOptionalContext - Optional context for logging.
	* @param bSilenceMissingAttributeErrors - Do not log errors to the context or log.
	* @returns Array of data.
	*/
	PCG_API TArray<UPCGData*> AttributePartition(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, FPCGContext* InOptionalContext = nullptr, bool bSilenceMissingAttributeErrors = false);

	/**
	* Partition the incoming data on the given attribute.
	* Will return an array of data. Each data will contain the entries (in stable order) for a given partition.
	* @param InData - Data to partition.
	* @param InSelectorArrayView - ArrayView of selectors on the attribute to partition.
	* @param InOptionalContext - Optional context for logging.
	* @param bSilenceMissingAttributeErrors - Do not log errors to the context or log.
	* @returns Array of data.
	*/
	PCG_API TArray<UPCGData*> AttributePartition(const UPCGData* InData, const TArrayView<const FPCGAttributePropertySelector>& InSelectorArrayView, FPCGContext* InOptionalContext = nullptr, bool bSilenceMissingAttributeErrors = false);

	/**
	* Generic partition for the incoming data on the given attribute.
	* Will return an array of bucket indices. Each bucket will contain the indices (in stable order) for a given partition.
	* @param InData - Data to partition, need to support attributes (spatial data or attribute set).
	* @param InSelector - Selector on the attribute to partition.
	* @param InOptionalContext - Optional context for logging.
	* @param bSilenceMissingAttributeErrors - Do not log errors to the context or log.
	* @returns Array of bucket indices.
	*/
	PCG_API TArray<TArray<int32>> AttributeGenericPartition(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, FPCGContext* InOptionalContext = nullptr, bool bSilenceMissingAttributeErrors = false);

	/**
	* Generic partition for the incoming data on the given array of attributes.
	* Will return an array of bucket indices. Each bucket will contain the indices (in stable order) for a given partition.
	* @param InData - Data to partition, need to support attributes (spatial data or attribute set).
	* @param InSelectorArrayView - ArrayView of selectors on the attribute to partition.
	* @param InOptionalContext - Optional context for logging.
	* @param bSilenceMissingAttributeErrors - Do not log errors to the context or log.
	* @returns Array of bucket indices.
	*/
	PCG_API TArray<TArray<int32>> AttributeGenericPartition(const UPCGData* InData, const TArrayView<const FPCGAttributePropertySelector>& InSelectorArrayView, FPCGContext* InOptionalContext = nullptr, bool bSilenceMissingAttributeErrors = false);

	/**
	* Remove duplicates for the incoming data on the given attribute.
	 *Will return a single data, containing the first occurrence of every unique value
	* @param InData - Data to partition, need to support attributes and multi entries (point data or attribute set).
	* @param InSelector - Selector on the attribute to partition.
	* @param InOptionalContext - Optional context for logging.
	* @param bSilenceMissingAttributeErrors - Do not log errors to the context or log.
	* @returns Resulting data, stripped of duplicates
	*/
	PCG_API UPCGData* RemoveDuplicates(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, FPCGContext* InOptionalContext = nullptr, bool bSilenceMissingAttributeErrors = false);

	/**
	* Remove duplicates for the incoming data on the given array of attributes.
	* Will return a single data, containing the first occurrence of every unique tuple of values.
	* @param InData - Data to partition, need to support attributes and multi entries (point data or attribute set).
	* @param InSelectorArrayView - ArrayView of selectors on the attribute to partition.
	* @param InOptionalContext - Optional context for logging.
	* @param bSilenceMissingAttributeErrors - Do not log errors to the context or log.
	* @returns Resulting data, stripped of duplicates
	*/
	PCG_API UPCGData* RemoveDuplicates(const UPCGData* InData, const TArrayView<const FPCGAttributePropertySelector>& InSelectorArrayView, FPCGContext* InOptionalContext = nullptr, bool bSilenceMissingAttributeErrors = false);
}