// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataAttributeTraits.h"

#include "UObject/SoftObjectPath.h"

#include "PCGMetadataTypesConstantStruct.generated.h"

#define UE_API PCG_API

class UPCGParamData;

// Deprecated: we support the path types natively now.
UENUM()
enum class EPCGMetadataTypesConstantStructStringMode
{
	String,
	SoftObjectPath UMETA(Hidden),
	SoftClassPath UMETA(Hidden),
};

/**
* Struct to be re-used when you need to show constants types for a metadata type
* It will store all our values, and will display nicely depending on the type chosen
*/
USTRUCT(BlueprintType)
struct FPCGMetadataTypesConstantStruct
{
	GENERATED_BODY()
public:
	template <typename Func>
	decltype(auto) Dispatcher(Func Callback) const;

	UE_API FString ToString() const;
	UE_API FString TypeToString() const;

#if WITH_EDITOR
	/** This is not called by the engine because this is a USTRUCT, but it can be called from usages to perform deprecation. */
	UE_API void OnPostLoad();
#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowsTypeChange", EditConditionHides, HideEditConditionToggle, ValidEnumValues = "Float, Double, Integer32, Integer64, Vector2, Vector, Vector4, Quaternion, Transform, String, Boolean, Rotator, Name, SoftObjectPath, SoftClassPath"))
	EPCGMetadataTypes Type = EPCGMetadataTypes::Double;

	UPROPERTY()
	EPCGMetadataTypesConstantStructStringMode StringMode_DEPRECATED = EPCGMetadataTypesConstantStructStringMode::String;

	// All different types
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Float", EditConditionHides))
	float FloatValue = 0.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Integer32", EditConditionHides))
	int32 Int32Value = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Double", EditConditionHides))
	double DoubleValue = 0.0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Integer64", EditConditionHides))
	int64 IntValue = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Vector2", EditConditionHides))
	FVector2D Vector2Value = FVector2D::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Vector", EditConditionHides))
	FVector VectorValue = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Vector4", EditConditionHides))
	FVector4 Vector4Value = FVector4::Zero();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Quaternion", EditConditionHides))
	FQuat QuatValue = FQuat::Identity;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Transform", EditConditionHides))
	FTransform TransformValue = FTransform::Identity;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::String", EditConditionHides))
	FString StringValue = "";

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Boolean", EditConditionHides))
	bool BoolValue = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Rotator", EditConditionHides))
	FRotator RotatorValue = FRotator::ZeroRotator;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::Name", EditConditionHides))
	FName NameValue = NAME_None;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::SoftClassPath", EditConditionHides))
	FSoftClassPath SoftClassPathValue;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Type == EPCGMetadataTypes::SoftObjectPath", EditConditionHides))
	FSoftObjectPath SoftObjectPathValue;

	UPROPERTY()
	bool bAllowsTypeChange = true;
};

template <typename Func>
decltype(auto) FPCGMetadataTypesConstantStruct::Dispatcher(Func Callback) const
{
	using ReturnType = decltype(Callback(double{}));

	switch (Type)
	{
	case EPCGMetadataTypes::Integer64:
		return Callback(IntValue);
	case EPCGMetadataTypes::Integer32:
		return Callback(Int32Value);
	case EPCGMetadataTypes::Float:
		return Callback(FloatValue);
	case EPCGMetadataTypes::Double:
		return Callback(DoubleValue);
	case EPCGMetadataTypes::Vector2:
		return Callback(Vector2Value);
	case EPCGMetadataTypes::Vector:
		return Callback(VectorValue);
	case EPCGMetadataTypes::Vector4:
		return Callback(Vector4Value);
	case EPCGMetadataTypes::Quaternion:
		return Callback(QuatValue);
	case EPCGMetadataTypes::Transform:
		return Callback(TransformValue);
	case EPCGMetadataTypes::String:
		return Callback(StringValue);
	case EPCGMetadataTypes::Boolean:
		return Callback(BoolValue);
	case EPCGMetadataTypes::Rotator:
		return Callback(RotatorValue);
	case EPCGMetadataTypes::Name:
		return Callback(NameValue);
	case EPCGMetadataTypes::SoftObjectPath:
		return Callback(SoftObjectPathValue);
	case EPCGMetadataTypes::SoftClassPath:
		return Callback(SoftClassPathValue);
	default:
		break;
	}

	// ReturnType{} is invalid if ReturnType is void
	if constexpr (std::is_same_v<ReturnType, void>)
	{
		return;
	}
	else
	{
		return ReturnType{};
	}
}

#undef UE_API
