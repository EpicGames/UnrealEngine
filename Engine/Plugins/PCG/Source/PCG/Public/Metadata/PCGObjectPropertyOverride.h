// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "PCGPin.h"
#include "Accessors/PCGAttributeAccessorHelpers.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "Algo/Transform.h"

#include "PCGObjectPropertyOverride.generated.h"

class IPCGAttributeAccessor;
struct FPCGContext;

/**
* Represents the override source (to be read) and the object property (to be written).
*/
USTRUCT(BlueprintType)
struct FPCGObjectPropertyOverrideDescription
{
	GENERATED_BODY()

	FPCGObjectPropertyOverrideDescription() = default;

	FPCGObjectPropertyOverrideDescription(const FPCGAttributePropertyInputSelector& InInputSource, const FString& InPropertyTarget)
		: InputSource(InInputSource)
		, PropertyTarget(InPropertyTarget)
	{}

	/** Provide an attribute or property to read the override value from. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	/**
	* Provide an object property name to be overridden. If you have a property "A" on your object, use "A" as the property target.
	*
	* For example, if you want to override the "Is Editor Only" flag, find it in the details panel, right-click, select 'Copy Internal Name', and paste that as the property target.
	*
	* If you have a component property, such as the static mesh of a static mesh component, use "StaticMeshComponent.StaticMesh".
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	FString PropertyTarget;
};

namespace PCGObjectPropertyOverrideHelpers
{
	/** Create an advanced ParamData pin for capturing property overrides. */
	PCG_API FPCGPinProperties CreateObjectPropertiesOverridePin(FName Label, const FText& Tooltip);

	/** Apply property overrides to the TargetObject directly from the ObjectPropertiesOverride pin. Use CreateObjectPropertiesOverridePin(). */
	PCG_API void ApplyOverridesFromParams(const TArray<FPCGObjectPropertyOverrideDescription>& InObjectPropertyOverrideDescriptions, UObject* TargetObject, FName OverridesPinLabel, FPCGContext* Context, bool bPropagateEditChangeEvent = false);

	struct FApplyOverrideParams
	{
		TConstArrayView<FPCGObjectPropertyOverrideDescription> ObjectPropertyOverrideDescriptions;
		TConstArrayView<TPair<UObject*, int32>> TargetObjectToOverrideIndices;
		TConstArrayView<const UPCGData*> OverridesData;
		FPCGContext* OptionalContext = nullptr;
		bool bPropagateEditChangeEvent = false;
		bool bOneDataSetPerElement = false;
	};

	PCG_API void ApplyOverrides(const FApplyOverrideParams& InParams);

	// @todo_pcg in another pass
	// UE_DEPRECATED(5.8, "Use the Params version")
	PCG_API void ApplyOverrides(const TArray<FPCGObjectPropertyOverrideDescription>& InObjectPropertyOverrideDescriptions, const TArray<TPair<UObject*, int32>>& TargetObjectToOverrideIndices, FName OverridesPinLabel, int32 InputDataIndex, FPCGContext* Context, bool bPropagateEditChangeEvent = false);
}

/**
* Represents a single property override on the provided object. Applies an override function to read the InputAccessor
* and write its value to the OutputAccessor.
* 
* The InputAccessor's InputKeys are created from the given SourceData and InputSelector.
*/
struct FPCGObjectSingleOverride
{
	// Can't copy but can move
	FPCGObjectSingleOverride() = default;
	FPCGObjectSingleOverride(const FPCGObjectSingleOverride&) = delete;
	FPCGObjectSingleOverride(FPCGObjectSingleOverride&&) = default;
	FPCGObjectSingleOverride& operator=(const FPCGObjectSingleOverride&) = delete;
	FPCGObjectSingleOverride& operator=(FPCGObjectSingleOverride&&) = default;

	struct FInitializeParams
	{
		const FPCGAttributePropertySelector* InputSelector = nullptr;
		FStringView OutputProperty;
		const UStruct* TemplateClass = nullptr;
		const UPCGData* SourceData = nullptr;
		FPCGContext* OptionalContext = nullptr;
		bool bComputePropertyEditChain = false;
		bool bAllowedToUseAllSourceDataForContainers = false;
	};

	/** Initialize the single object override. Call before using Apply(InputKeyIndex, OutputKey). Can also compute the edit property chain if it is needed to propagate the change (Editor only) */
	PCG_API void Initialize(const FInitializeParams& InParams);

	// @todo_pcg in another pass
	//UE_DEPRECATED(5.8, "Use Params version")
	PCG_API void Initialize(const FPCGAttributePropertySelector& InputSelector, const FString& OutputProperty, const UStruct* TemplateClass, const UPCGData* SourceData, FPCGContext* Context, bool bComputePropertyEditChain = false);

	/** Returns true if initialization succeeded in creating the accessors and accessor keys. */
	PCG_API bool IsValid() const;

	/** To be called before the Apply, if it is needed to propagate the change to the target object. */
	PCG_API void PreApply(UObject* TargetObject);

	/** Applies a single property override to the object by reading from the InputAccessor at the given KeyIndex, and writing to the OutputKey which represents the object property. */
	PCG_API bool Apply(int32 InputKeyIndex, IPCGAttributeAccessorKeys& OutputKey);

	/** To be called after the Apply, if it is needed to propagate the change to the target object. */
	PCG_API void PostApply(UObject* TargetObject);

	/** Gathers overrides into an array, if they need loading. */
	PCG_API void GatherAllOverridesToLoad(TArray<FSoftObjectPath>& OutObjectsToLoad) const;

	const FPCGAttributePropertySelector& GetInputSelector() const { return ObjectOverrideInputSelector; }

private:
	TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys;
	TUniquePtr<const IPCGAttributeAccessor> ObjectOverrideInputAccessor;
	TUniquePtr<IPCGAttributeAccessor> ObjectOverrideOutputAccessor;
	FPCGAttributePropertySelector ObjectOverrideInputSelector;

#if WITH_EDITOR
	// To be propagated to the object if requested, allow the object to react to those overrides.
	FEditPropertyChain EditPropertyChain;
#endif // WITH_EDITOR

	bool bWillNeedLoading = false;
	bool bAllowedToUseAllSourceDataForContainers = false;
	bool bIsContainer = false;
};

/**
* Represents a set of property overrides for the provided object. Provide a SourceData to read from, and a collection of ObjectPropertyOverrides matching the TemplateObject's class properties.
*/
template <typename T>
struct FPCGObjectOverrides
{
	FPCGObjectOverrides(T* TemplateObject) : OutputKey(TemplateObject)
	{}

	// Can't copy but can move
	FPCGObjectOverrides(const FPCGObjectOverrides&) = delete;
	FPCGObjectOverrides(FPCGObjectOverrides&&) = default;
	FPCGObjectOverrides& operator=(const FPCGObjectOverrides&) = delete;
	FPCGObjectOverrides& operator=(FPCGObjectOverrides&&) = default;

	struct FInitializeParams
	{
		TConstArrayView<FPCGObjectPropertyOverrideDescription> OverrideDescriptions;
		T* TemplateObject = nullptr;
		const UPCGData* SourceData;
		FPCGContext* OptionalContext = nullptr;
		bool bInPropagateEditChangeEvent = false;
		bool bAllowedToUseAllSourceDataForContainers = false;
	};

	// @todo_pcg in another pass
	// UE_DEPRECATED(5.8, "Use the Params version")
	void Initialize(const TArray<FPCGObjectPropertyOverrideDescription>& OverrideDescriptions, T* TemplateObject, const UPCGData* SourceData, FPCGContext* Context, bool bInPropagateEditChangeEvent = false)
	{
		FInitializeParams Params =
		{
			.OverrideDescriptions = OverrideDescriptions,
			.TemplateObject = TemplateObject,
			.SourceData = SourceData,
			.OptionalContext = Context,
			.bInPropagateEditChangeEvent = bInPropagateEditChangeEvent
		};

		Initialize(Params);
	}

	/** Initialize the object overrides. Call before using Apply(InputKeyIndex). */
	void Initialize(const FInitializeParams& InParams)
	{
		bInitialized = false;

		if (!InParams.TemplateObject)
		{
			PCGLog::LogErrorOnGraph(NSLOCTEXT("PCGObjectPropertyOverride", "InitializeOverrideFailedNoObject", "Failed to initialize property overrides. No template object was provided."), InParams.OptionalContext);
			return;
		}

		CachedOutputObject = nullptr; 
		OutputKey = FPCGAttributeAccessorKeysSingleObjectPtr<T>(InParams.TemplateObject);
		bPropagateEditChangeEvent = InParams.bInPropagateEditChangeEvent;

		const UStruct* ClassObject = nullptr;
		if constexpr (TIsDerivedFrom<T, UObject>::Value)
		{
			ClassObject = InParams.TemplateObject->GetClass();
			CachedOutputObject = bPropagateEditChangeEvent ? InParams.TemplateObject : nullptr;
		}
		else
		{
			ClassObject = InParams.TemplateObject->StaticStruct();
		}

		ObjectSingleOverrides.Empty(InParams.OverrideDescriptions.Num());

		for (int32 i = 0; i < InParams.OverrideDescriptions.Num(); ++i)
		{
			const FPCGAttributePropertyInputSelector InputSelector = InParams.OverrideDescriptions[i].InputSource.CopyAndFixLast(InParams.SourceData);
			const FString& OutputProperty = InParams.OverrideDescriptions[i].PropertyTarget;

			FPCGObjectSingleOverride Override{};

			FPCGObjectSingleOverride::FInitializeParams Params =
			{
				.InputSelector = &InputSelector,
				.OutputProperty = OutputProperty,
				.TemplateClass = ClassObject,
				.SourceData = InParams.SourceData,
				.OptionalContext = InParams.OptionalContext,
				.bComputePropertyEditChain = bPropagateEditChangeEvent,
				.bAllowedToUseAllSourceDataForContainers = InParams.bAllowedToUseAllSourceDataForContainers
			};

			Override.Initialize(Params);

			if (Override.IsValid())
			{
				ObjectSingleOverrides.Emplace(MoveTemp(Override));
			}
			else
			{
				PCGLog::LogErrorOnGraph(FText::Format(NSLOCTEXT("PCGObjectPropertyOverride", "InitializeOverrideFailed", "Failed to initialize override '{0}' for property {1} on object '{2}'."), InputSelector.GetDisplayText(), FText::FromString(OutputProperty), FText::FromName(ClassObject->GetFName())), InParams.OptionalContext);
			}
		}

		bInitialized = true;
	}

	/**
	 * Applies each property override to the object by reading from the InputAccessor at the given KeyIndex, and writing to the OutputKey which represents the object property.
	 * Can also propagate an edit change to the target object if we need to propagate the information. Editor only.
	 */
	bool Apply(int32 InputKeyIndex)
	{
		bool bAllSucceeded = true;

		for (FPCGObjectSingleOverride& ObjectSingleOverride : ObjectSingleOverrides)
		{
#if WITH_EDITOR
			if (bPropagateEditChangeEvent && CachedOutputObject)
			{
				ObjectSingleOverride.PreApply(CachedOutputObject);
			}
#endif // WITH_EDITOR

			bAllSucceeded &= ObjectSingleOverride.Apply(InputKeyIndex, OutputKey);

#if WITH_EDITOR
			if (bPropagateEditChangeEvent && CachedOutputObject)
			{
				ObjectSingleOverride.PostApply(CachedOutputObject);
			}
#endif // WITH_EDITOR
		}

		return bAllSucceeded;
	}

	/** Update the template object, only if it was already initialized */
	void UpdateTemplateObject(T* InTemplateObject)
	{
		if (IsValid())
		{
			OutputKey = FPCGAttributeAccessorKeysSingleObjectPtr<T>(InTemplateObject);
			if constexpr (TIsDerivedFrom<T, UObject>::Value)
			{
				CachedOutputObject = bPropagateEditChangeEvent ? InTemplateObject : nullptr;
			}
		}
	}

	/** Returns true if we have any override to apply */
	bool IsValid() const { return bInitialized && !ObjectSingleOverrides.IsEmpty(); }

	/** Gathers overrides into an array, if they need loading. */
	void GatherAllOverridesToLoad(TArray<FSoftObjectPath>& OutObjectsToLoad) const
	{
		for (const FPCGObjectSingleOverride& SingleOverride : ObjectSingleOverrides)
		{
			SingleOverride.GatherAllOverridesToLoad(OutObjectsToLoad);
		}
	}

	/** Utility function to get all the valid input selectors. Useful to do some partitioning after the override is initialized (and validated all selectors are valid). */
	TArray<FPCGAttributePropertySelector> GetAllInputSelectors() const
	{
		TArray<FPCGAttributePropertySelector> Selectors;
		Selectors.Reserve(ObjectSingleOverrides.Num());
		Algo::Transform(ObjectSingleOverrides, Selectors, [](const FPCGObjectSingleOverride& SingleOverride) { return SingleOverride.GetInputSelector(); });
		return Selectors;
	}

private:
	bool bPropagateEditChangeEvent = false;
	UObject* CachedOutputObject = nullptr;
	FPCGAttributeAccessorKeysSingleObjectPtr<T> OutputKey;
	TArray<FPCGObjectSingleOverride> ObjectSingleOverrides;
	bool bInitialized = false;
};

USTRUCT(BlueprintType, meta=(Deprecated = "5.4", DeprecationMessage="Use FPCGObjectPropertyOverrideDescription instead."))
struct FPCGActorPropertyOverride
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	UPROPERTY(EditAnywhere, Category = Settings)
	FString PropertyTarget;
};
