// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataAsset.h"

class UPackage;

#include "PCGAssetExporter.generated.h"

#define UE_API PCG_API

/** Common structure to hold saving options required to export or update PCG assets. */
USTRUCT(BlueprintType)
struct FPCGAssetExporterParameters
{
	GENERATED_BODY()

	/** Controls whether we will open a Save... dialog, works only when a single level is exported. Overrides update anywhere. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bOpenSaveDialog = true;

	/** Target asset path name */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FString AssetName;

	/** Target asset path to write the PCG assets to. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FString AssetPath;

	/** Controls whether the assets will be saved at the end of the process or not. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bSaveOnExportEnded = true;
};

/** Base class for asset exporters. Can be extended either natively or through blueprint. Not intended to be used in non-editor builds. */
UCLASS(MinimalAPI, BlueprintType, Abstract, Blueprintable)
class UPCGAssetExporter : public UObject
{
	GENERATED_BODY()

public:
	/** Drives the asset creation for the data parsed from the level, will return UPCGDataAsset by default. */
	UE_API virtual TSubclassOf<UPCGDataAsset> GetAssetType() const;

	/** Exports data to an asset based on the exporter's metadata */
	UE_API bool Export(const FString& PackageName, UPCGDataAsset* Asset);

	/** Updates the asset from its source data, when possible */
	UE_API UPackage* Update(const FAssetData& PCGAsset);

protected:
	/** Returns the subtype of PCG data asset generated by the Level To Asset process. Override this in the instances where you would not use the default base class. */
	UFUNCTION(BlueprintNativeEvent, meta = (DisplayName = "Get Asset Type", ForceAsFunction))
	UE_API TSubclassOf<UPCGDataAsset> BP_GetAssetType() const;

	UFUNCTION(BlueprintNativeEvent, meta = (DisplayName = "Update Asset", ForceAsFunction))
	UE_API bool BP_ExportToAsset(UPCGDataAsset* Asset);

	/** Loads exporter metadata from the asset */
	UE_API void SerializeMetadataFromAsset(const FAssetData& PCGAsset);

	/** Saves exporter metadata to the asset */
	UE_API void SerializeMetadataToAsset(const FAssetData& PCGAsset);

	/** Exports data to an asset based on the exporter's metadata */
	UE_API virtual bool ExportAsset(const FString& PackageName, UPCGDataAsset* Asset);

	/** Updates the asset from its source data, when possible */
	UE_API virtual UPackage* UpdateAsset(const FAssetData& PCGAsset);

	/** Sets up the exporter prior to performing update */
	virtual void SerializeMetadata(FArchive& Ar) {}

	/** Duplicates data that is outered to the transient package so it is properly saved */
	UE_API void DuplicateAndReOuterData(UPCGDataAsset* Asset);
};

#undef UE_API
