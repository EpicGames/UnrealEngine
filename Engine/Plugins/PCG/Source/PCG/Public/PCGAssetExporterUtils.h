// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGAssetExporter.h"

#include "AssetRegistry/AssetData.h"

#include "PCGAssetExporterUtils.generated.h"

#define UE_API PCG_API

struct FPCGContext;

/**
* Asset export utils - will work only in editor builds. 
*/
UCLASS(MinimalAPI)
class UPCGAssetExporterUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Exports an asset from the given exporter. When calling this function, the exporter should be able to create its own data and as such needs prior information (world, etc.) before calling this method. */
	UFUNCTION(BlueprintCallable, Category = "PCG|IO")
	static UE_API UPackage* CreateAsset(UPCGAssetExporter* Exporter, FPCGAssetExporterParameters Parameters = FPCGAssetExporterParameters());

	/** Updates assets based on their embedded exporter & metadata. */
	UFUNCTION(BlueprintCallable, Category = "PCG|IO")
	static UE_API void UpdateAssets(const TArray<FAssetData>& PCGAssets, FPCGAssetExporterParameters Parameters = FPCGAssetExporterParameters());

	// Native API equivalent
	static UE_API UPackage* CreateAsset(UPCGAssetExporter* Exporter, const FPCGAssetExporterParameters& Parameters, FPCGContext* InOptionalContext);
	static UE_API void UpdateAssets(const TArray<FAssetData>& PCGAssets, const FPCGAssetExporterParameters& Parameters, FPCGContext* InOptionalContext);

	// Generic equivalent
	static UE_API UPackage* CreateAsset(const UClass* AssetClass, const FPCGAssetExporterParameters& Parameters,  TFunctionRef<bool(const FString& PackagePath, UObject* Asset)> ExportFunc, FPCGContext* InOptionalContext);

	template <typename T>
	static UPackage* CreateAsset(const FPCGAssetExporterParameters& Parameters,  TFunctionRef<bool(const FString& PackagePath, UObject* Asset)> ExportFunc, FPCGContext* InOptionalContext)
	{
		static_assert(std::is_base_of_v<UObject, T>, "T needs to be a UObject");
		return CreateAsset(T::StaticClass(), Parameters, std::move(ExportFunc), InOptionalContext);
	}
};

#undef UE_API
