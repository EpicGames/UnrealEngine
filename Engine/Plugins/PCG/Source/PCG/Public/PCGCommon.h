// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPointPropertiesTraits.h"

#include "PerQualityLevelProperties.h"
#include "Async/LockTags.h"
#include "Async/Mutex.h"
#include "Async/RecursiveMutex.h"
#include "Async/SharedMutex.h"
#include "Async/SharedRecursiveMutex.h"
#include "HAL/IConsoleManager.h"

#include "PCGCommon.generated.h"

#define UE_API PCG_API

using FPCGTaskId = uint64;
static const FPCGTaskId InvalidPCGTaskId = (uint64)-1;

using FPCGPinId = uint64;

class IPCGElement;
class IPCGGraphExecutionSource;
class UPCGComponent;
class UPCGGraph;
struct FPCGContext;
struct FPCGStack;
struct FPCGStackContext;

namespace PCG
{
	using FLock = UE::FMutex;
	using FRecursiveLock = UE::FRecursiveMutex;
	using FSharedLock = UE::FSharedMutex;
	using FRecursiveSharedLock = UE::FSharedRecursiveMutex;

	// Similar to existing UE::TScopeLock but allows optional TryLock in which case caller is responsible to 
	// check if Lock succeeded or not with IsLocked(). This allows execution code to be skipped if non main thread
	// tasks fail to get the lock instead of waiting.
	// Also add an option to have an optional lock, if we have a code path that support both lock and lockless.
	template<typename LockType>
	class TScopeLock final
	{
	public:
		UE_NONCOPYABLE(TScopeLock);

		UE_NODISCARD_CTOR inline explicit TScopeLock(LockType* InLock, bool bTryLock = false, bool bInShouldLock = true)
			: Lock(InLock)
			, bLocked(false)
			, bShouldLock(bInShouldLock)
		{
			if (InLock && bShouldLock)
			{
				if (bTryLock)
				{
					bLocked = InLock->TryLock();
				}
				else
				{
					InLock->Lock();
					bLocked = true;
				}
			}
		}


		UE_NODISCARD_CTOR inline explicit TScopeLock(LockType& InLock, bool bTryLock = false, bool bInShouldLock = true)
			: TScopeLock(&InLock, bTryLock, bInShouldLock)
		{
		}

		inline ~TScopeLock()
		{
			if (bLocked && Lock)
			{
				Lock->Unlock();
				Lock = nullptr;
			}
		}

		bool IsLocked() const
		{
			return bLocked;
		}

	private:
		LockType* Lock;
		bool bLocked;
		bool bShouldLock;
	};

	template <typename SharedLockType>
	class TSharedScopeLock final
	{
	public:
		UE_NONCOPYABLE(TSharedScopeLock);

		UE_NODISCARD_CTOR inline explicit TSharedScopeLock(SharedLockType& InSharedLock, bool bInShouldLock = true)
			: SharedLock(InSharedLock, UE::FDeferLock())
			, bShouldLock(bInShouldLock)
		{
			if (bShouldLock)
			{
				SharedLock.Lock();
			}
		}

		inline ~TSharedScopeLock()
		{
			if (bShouldLock)
			{
				SharedLock.Unlock();
			}
		}

	private:
		UE::TDynamicSharedLock<SharedLockType> SharedLock;
		bool bShouldLock;
	};

	template <typename SharedLockType>
	class TUniqueScopeLock final
	{
	public:
		UE_NONCOPYABLE(TUniqueScopeLock);

		UE_NODISCARD_CTOR inline explicit TUniqueScopeLock(SharedLockType& InSharedLock, bool bInShouldLock = true)
			: SharedLock(InSharedLock)
			, bShouldLock(bInShouldLock)
		{
			if (bShouldLock)
			{
				SharedLock.Lock();
			}
		}

		inline ~TUniqueScopeLock()
		{
			if (bShouldLock)
			{
				SharedLock.Unlock();
			}
		}

	private:
		SharedLockType& SharedLock;
		bool bShouldLock;
	};
}

namespace PCGLog
{
	/** Returns the component's owner name. In the case of commandlets, owner full object path will be returned. */
	FString GetExecutionSourceName(const IPCGGraphExecutionSource* InExecutionSource, bool bUseLabel = false, FString InDefaultName = TEXT("MISSINGOWNER"));
}

namespace PCGPinIdHelpers
{
	/** Pin active bitmask stored in uint64, so 64 flags available. */
	constexpr int PinActiveBitmaskSize = 64;

	/** There are 64 pin flags available, however we use flag 63 as a special pin-less ID for task dependencies that don't have associated pins. */
	constexpr int MaxOutputPins = PinActiveBitmaskSize - 1;

	/** Convert node ID and pin index to a unique pin ID. */
	FPCGPinId NodeIdAndPinIndexToPinId(FPCGTaskId NodeId, uint64 PinIndex);

	/** Create a pin ID from a node ID alone. Used for task inputs that don't have associated pins. */
	FPCGPinId NodeIdToPinId(FPCGTaskId NodeId);

	/** Adjust the pin ID to incorporate the given node ID offset. */
	FPCGPinId OffsetNodeIdInPinId(FPCGPinId PinId, uint64 NodeIDOffset);

	/** Extract node ID from the given pin ID. */
	FPCGTaskId GetNodeIdFromPinId(FPCGPinId PinId);

	/** Extract pin index from unique pin ID. */
	uint64 GetPinIndexFromPinId(FPCGPinId PinId);
}

namespace PCGPointCustomPropertyNames
{
	bool IsCustomPropertyName(FName Name);

	extern PCG_API const FName ExtentsName;
	extern PCG_API const FName LocalCenterName;
	extern PCG_API const FName PositionName;
	extern PCG_API const FName RotationName;
	extern PCG_API const FName ScaleName;
	extern PCG_API const FName LocalSizeName;
	extern PCG_API const FName ScaledLocalSizeName;
}

UENUM(meta = (Bitflags))
enum class EPCGChangeType : uint32
{
	None = 0,
	Cosmetic = 1 << 0,
	Settings = 1 << 1,
	Input = 1 << 2,
	Edge = 1 << 3,
	Node = 1 << 4,
	Structural = 1 << 5,
	/** Anything related to generation grids - changing grid size or adding/removing grid size nodes. */
	GenerationGrid = 1 << 6,
	/** Change to any shader source code. */
	ShaderSource = 1 << 7,
	/** Changes in the graph customization that will impact the editor. */
	GraphCustomization = 1 << 8,
	/** Changes in the graph caused by external tools, editor graph should be reconstructed. */
	ExternalModification = 1 << 9,
};
ENUM_CLASS_FLAGS(EPCGChangeType);

enum class ERuntimeGenRefreshReason : uint8
{
	None = 0,
	/** Explicit user/API request. */
	UserRequested      = 1 << 0,
	/** Render state was reset (viewport resize, component re-register, scene materials modified, etc). */
	RenderStateRefresh = 1 << 1,
	/** Engine scalability or PCG quality level changed. */
	QualitySettings    = 1 << 2,
};
ENUM_CLASS_FLAGS(ERuntimeGenRefreshReason);

// Bitmask containing the various data types supported in PCG. Note that this enum cannot be a blueprint type because
// enums have to be uint8 for blueprint, and we already use more than 8 bits in the bitmask.
// This is why we have a parallel enum just below that must match on a name basis 1:1 to allow the make/break functions to work properly
// in blueprint.
// WARNING: Please be mindful that combination of flags that are not explicitly defined there won't be serialized correctly, inducing data loss.
UENUM(meta = (Bitflags))
enum class EPCGDataType : uint32
{
	None = 0 UMETA(Hidden),
	Point = 1 << 1,

	Spline = 1 << 2,
	LandscapeSpline = 1 << 3,
	Polygon2D = 1 << 13,
	PolyLine = Spline | LandscapeSpline | Polygon2D UMETA(DisplayName = "Curve"),

	Landscape = 1 << 4,
	Texture = 1 << 5,
	RenderTarget = 1 << 6,
	// VirtualTexture is not a subtype of BaseTexture because they share no common functionality, particularly with respect to sampling.
	VirtualTexture = 1 << 12,
	BaseTexture = Texture | RenderTarget UMETA(Tooltip = "Common base type for both textures and render targets."),
	Surface = Landscape | BaseTexture | VirtualTexture,

	Volume = 1 << 7,
	Primitive = 1 << 8,
	DynamicMesh = 1 << 10,

	StaticMeshResource = 1 << 11,

	/** Simple concrete data. */
	Concrete = Point | PolyLine | Surface | Volume | Primitive | DynamicMesh,

	/** Boolean operations like union, difference, intersection. */
	Composite = 1 << 9 UMETA(Hidden),

	/** Combinations of concrete data and/or boolean operations. */
	Spatial = Composite | Concrete,

	/** Data that represent resources/assets. */
	Resource = StaticMeshResource UMETA(Hidden),

	/** Proxy for data that was created on the GPU and not necessarily read back to CPU. */
	ProxyForGPU = 1 << 26 UMETA(Hidden),

	Param = 1 << 27 UMETA(DisplayName = "Attribute Set"),

	// Combination of Param and Point, necessary for name-based serialization of enums.
	PointOrParam = Point | Param UMETA(DisplayName = "Point or Attribute Set"),

	VolumeOrPrimitiveOrDynamicMesh = Volume | Primitive | DynamicMesh UMETA(Hidden),
	PointOrSpline = Point | Spline UMETA(Hidden),

	Settings = 1 << 28 UMETA(Hidden),
	Other = 1 << 29,
	Any = (1 << 30) - 1,

	// Used as an indication that a function that returns a EPCGDataType that has gone through a default implementation.
	// If a function returns something else than a DeprecationSentinel, it means the function is overridden, and we want to use the result,
	// even if we have a loss of info. Show deprecation message and indicate how to upgrade the code.
	DeprecationSentinel = (1u << 31) UMETA(Hidden)
};
ENUM_CLASS_FLAGS(EPCGDataType);

// As discussed just before, a parallel version for "exclusive" (as in only type) of the EPCGDataType enum. Needed for blueprint compatibility.
UENUM(BlueprintType, meta=(DisplayName="PCG Data Type"))
enum class EPCGExclusiveDataType : uint8
{
	None = 0 UMETA(Hidden),
	Point,
	Spline,
	LandscapeSpline,
	PolyLine UMETA(DisplayName = "Curve"),
	Landscape,
	Texture,
	RenderTarget,
	VirtualTexture,
	BaseTexture UMETA(Hidden),
	Surface,
	Volume,
	Primitive,
	Concrete,
	Composite UMETA(Hidden),
	Spatial,
	Param UMETA(DisplayName = "Attribute Set"),
	Settings UMETA(Hidden),
	Other,
	Any,
	PointOrParam UMETA(DisplayName = "Point or Attribute Set"),
	DynamicMesh,
	StaticMeshResource,
	Resource UMETA(Hidden),
	Polygon2D
};

UENUM(meta=(DisplayName="PCG Container Type"))
enum class EPCGContainerType : uint8
{
	Element = 0,
	None = Element,
	Array,
	Map,
	Set
};

namespace PCGValueConstants
{
	constexpr int32 DefaultSeed = 42;
}

namespace PCGAttributeNameConstants
{
	extern PCG_API const FLazyName ActorReferenceAttribute;
	extern PCG_API const FLazyName ComponentReferenceAttribute;
}

namespace PCG::Private
{
	extern PCG_API const FString UserParameterTagData;
}

namespace PCGPinConstants
{
	extern PCG_API const FName DefaultInputLabel;
	extern PCG_API const FName DefaultOutputLabel;
	extern PCG_API const FName DefaultParamsLabel;
	UE_DEPRECATED(5.6, "Please use 'DefaultExecutionDependencyLabel' instead.")
	extern PCG_API const FName DefaultDependencyOnlyLabel;
	extern PCG_API const FName DefaultExecutionDependencyLabel;

	extern PCG_API const FName DefaultInFilterLabel;
	extern PCG_API const FName DefaultOutFilterLabel;

namespace Private
{
	extern PCG_API const FName OldDefaultParamsLabel;
}

namespace Icons
{
	extern PCG_API const FName LoopPinIcon;
	extern PCG_API const FName FeedbackPinIcon;
}

#if WITH_EDITOR
namespace Tooltips
{
	extern PCG_API const FText ExecutionDependencyTooltip;
}
#endif // WITH_EDITOR
} // namespace PCGPinConstants

namespace PCGNodeConstants
{
namespace Icons
{
	extern PCG_API const FName CompactNodeConvert;
	extern PCG_API const FName CompactNodeFilter;
}
}

// Metadata used by PCG
namespace PCGObjectMetadata
{
	extern PCG_API const FLazyName Overridable;
	extern PCG_API const FLazyName OverridableCPUAndGPU;
	extern PCG_API const FLazyName OverridableCPUAndGPUWithReadback;
	extern PCG_API const FLazyName NotOverridable;
	extern PCG_API const FLazyName OverridableChildProperties;
	extern PCG_API const FLazyName OverrideAliases;
	extern PCG_API const FLazyName DiscardPropertySelection;
	extern PCG_API const FLazyName DiscardExtraSelection;
	extern PCG_API const FLazyName EnumMetadataDomain;
	extern PCG_API const FLazyName PropertyReadOnly;
	extern PCG_API const FLazyName DataTypeIdentifierSupportsComposition;
	extern PCG_API const FLazyName DataTypeIdentifierFilter;
	extern PCG_API const FLazyName DataTypeDisplayName;

	// Metadata usable in UPROPERTY for customizing the behavior when displaying the property in a property panel or graph node
	enum
	{
		/// [PropertyMetadata] Indicates that the property is overridable by params on the CPU.
		PCG_Overridable,

		/// [PropertyMetadata] Indicates that the property is overridable by params on the CPU and GPU.
		PCG_OverridableCPUAndGPU,

		/// [PropertyMetadata] Indicates that the property is overridable by params on the CPU and GPU and the overridden value will be read back for use in CPU computation during compute graph dispatch.
		PCG_OverridableCPUAndGPUWithReadback,

		/// [PropertyMetadata] Indicates that the property is not-overridable by params. Used in structs to hide some parameters
		PCG_NotOverridable,

		/// [PropertyMetadata] When a struct property is overridable, by default it will expose all the child properties (except the one marked by PCG_NotOverridable)
		/// but for structs that are not defined in a PCG context, we don't want to pollute the metadata of those with PCG.
		/// This extra field can be used to select property paths that should be overridable. The property path should be relative to the struct that is marked
		/// PCG_Overridable, as a comma separated string.
		/// 
		/// Example if we have a struct FBar, that has a field Foo that is a struct with Prop1, Prop2 and Prop3, to expose Prop1 and Prop2:
		/// UPROPERTY(meta=(PCG_Overridable, PCG_OverridableChildProperties = "Foo/Prop1,Foo/Prop2"))
		/// FBar Bar;
		PCG_OverridableChildProperties,

		/// [PropertyMetadata] Extra names to match for a given property.
		PCG_OverrideAliases,

		/// [PropertyMetadata] For FPCGAttributePropertySelector, won't display the point property items in the dropdown
		PCG_DiscardPropertySelection,

		/// [PropertyMetadata] For FPCGAttributePropertySelector, won't display the extra property items in the dropdown
		PCG_DiscardExtraSelection,

		/// [PropertyMetadata] For FPCGDataTypeIdentifier, will enable selecting types like BitFlag, to compose types.
		PCG_SupportsComposition,

		/// [PropertyMetadata] For FPCGDataTypeIdentifier, will enable selecting valid types to show in the dropdown.
		PCG_TypeFilter,
	};

	// Metadata usable in USTRUCT
	enum
	{
		/// [StructMetadata] For FPCGDataTypeInfo, can specify a display name.
		PCG_DataTypeDisplayName
	};

	// Metadata usable in UENUM for customizing the entry in the attribute property selector
	enum
	{
		/// [EnumMetadata] Specify the domain for this entry.
		PCG_MetadataDomain,

		/// [EnumMetadata] Mark the property read-only, won't show in Output selectors
		PCG_PropertyReadOnly
	};
}

namespace PCGFeatureSwitches
{
	extern PCG_API TAutoConsoleVariable<bool> CVarCheckSamplerMemory;
	extern PCG_API TAutoConsoleVariable<float> CVarSamplerMemoryThreshold;

	namespace Helpers
	{
		/** Checks the cvar for allowed physical and virtual memory ratio to be used with samplers. */
		PCG_API uint64 GetAvailableMemoryForSamplers();
	}
}

namespace PCGSystemSwitches
{
#if WITH_EDITOR
	extern PCG_API TAutoConsoleVariable<bool> CVarPausePCGExecution;
	extern TAutoConsoleVariable<bool> CVarGlobalDisableRefresh;
	extern TAutoConsoleVariable<bool> CVarDirtyLoadAsPreviewOnLoad;
	extern TAutoConsoleVariable<bool> CVarForceDynamicGraphDispatch;
#endif

	extern PCG_API TAutoConsoleVariable<bool> CVarPCGDebugDrawGeneratedCells;
	extern TAutoConsoleVariable<bool> CVarPassGPUDataThroughGridLinks;

#if !UE_BUILD_SHIPPING
	extern PCG_API TAutoConsoleVariable<bool> CVarFuzzGPUMemory;
#endif

	UE_DEPRECATED(5.8, "This feature is always enabled now")
	extern PCG_API TAutoConsoleVariable<bool> CVarReleaseTransientResourcesEarly;
}

/** Describes space referential for operations that create data */
UENUM(BlueprintType)
enum class EPCGCoordinateSpace : uint8
{
	World UMETA(DisplayName = "Global"),
	OriginalComponent,
	LocalComponent
};

UENUM(BlueprintType)
enum class EPCGStringMatchingOperator : uint8
{
	/** Will return a match only if the two strings compared are the same */
	Equal,
	/** Will return a match if the first string contains the second */
	Substring,
	/** Will return a match if the first string matches the pattern defined by the second (including wildcards) */
	Matches
};

/** Describes one or more target execution grids. */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EPCGHiGenGrid : uint32
{
	Uninitialized = 0 UMETA(Hidden),

	// NOTE: When adding new grids, increment PCGHiGenGrid::NumGridValues below
	Grid4 = 4 UMETA(DisplayName = "400"),
	Grid8 = 8 UMETA(DisplayName = "800"),
	Grid16 = 16 UMETA(DisplayName = "1600"),
	Grid32 = 32 UMETA(DisplayName = "3200"),
	Grid64 = 64 UMETA(DisplayName = "6400"),
	Grid128 = 128 UMETA(DisplayName = "12800"),
	Grid256 = 256 UMETA(DisplayName = "25600"),
	Grid512 = 512 UMETA(DisplayName = "51200"),
	Grid1024 = 1024 UMETA(DisplayName = "102400"),
	Grid2048 = 2048 UMETA(DisplayName = "204800"),
	Grid4096 = 4096 UMETA(Hidden),
	Grid8192 = 8192 UMETA(Hidden),
	Grid16384 = 16384 UMETA(Hidden),
	Grid32768 = 32768 UMETA(Hidden),
	Grid65536 = 65536 UMETA(Hidden),
	Grid131072 = 131072 UMETA(Hidden),
	Grid262144 = 262144 UMETA(Hidden),
	Grid524288 = 524288 UMETA(Hidden),
	Grid1048576 = 1048576 UMETA(Hidden),
	Grid2097152 = 2097152 UMETA(Hidden),
	Grid4194304 = 4194304 UMETA(Hidden),

	GridMin = Grid4 UMETA(Hidden),
	GridMax = Grid4194304 UMETA(Hidden),

	// Should execute once rather than executing on any grid
	Unbounded = 1u << 31,
};
ENUM_CLASS_FLAGS(EPCGHiGenGrid);

namespace PCGHiGenGrid
{
	// Number of unique values of EPCGHiGenGrid, const so it can be used for the inline allocator below.
	constexpr uint32 NumGridValues = 13;

	// Alias for array which is allocated on the stack (we have a strong idea of the max required elements).
	using FSizeArray = TArray<uint32, TInlineAllocator<PCGHiGenGrid::NumGridValues>>;

	PCG_API bool IsValidGridSize(uint32 InGridSize);
	PCG_API bool IsValidGrid(EPCGHiGenGrid InGrid);
	PCG_API bool IsValidGridOrUninitialized(EPCGHiGenGrid InGrid);
	PCG_API bool IsValidGridOrUninitialized(uint32 InGridSize);
	PCG_API uint32 GridToGridSize(EPCGHiGenGrid InGrid);
	/** Rounds up to the nearest power of 2 grid, clamping between GridMin and GridMax (excepting the unbounded grid, which may exceed GridMax). */
	PCG_API EPCGHiGenGrid GridSizeToGrid(uint32 InGridSize);
	PCG_API EPCGHiGenGrid ScaledGridSizeToGrid(uint32 InScaledGridSize, double InGridScale);

	PCG_API uint32 UninitializedGridSize();
	PCG_API uint32 UnboundedGridSize();
	PCG_API double SanitizeGridSizeMultiplier(double GridSizeMultiplier);
}

UENUM()
enum class EPCGAttachOptions : uint32
{
	NotAttached UMETA(Tooltip="Actor will not be attached to the target actor nor placed in an actor folder"),
	Attached UMETA(Tooltip="Actor will be attached to the target actor in the given node"),
	InFolder UMETA(Tooltip="Actor will be placed in an actor folder containing the name of the target actor."),
	InGraphFolder UMETA(Tooltip="Actor will be placed in a folder named after the top graph it was generated from."),
	InGeneratedFolder UMETA(Tooltip="Actor will be placed in the PCG_Generated folder.")
};

UENUM()
enum class EPCGEditorDirtyMode : uint8
{
	Normal UMETA(Tooltip="Normal editing mode where generation changes (generation, cleanup) dirty the component and its resources."),
	Preview UMETA(Tooltip="Editing mode where generation changes (generation, cleanup, resources) on the component will not trigger any dirty state, but will also not save any of the generated resources. Also represents the state after loading from the Load as Preview edit mode, where this will hold the last saved generation until a new generation is triggered."),
	LoadAsPreview UMETA(Tooltip="Acts as the normal editing mode until the next load of the component, at which state it acts as-if-transient, namely that any further generation changes will not dirty the component.")
};

USTRUCT(BlueprintType)
struct FPCGRuntimeGenerationRadii
{
	GENERATED_BODY()

public:
	UE_API void PostSerialize(const FArchive& Ar);

	/** Get the runtime generation radius for the given grid size. */
	UE_API double GetGenerationRadiusFromGrid(EPCGHiGenGrid Grid) const;

	/** Compute the runtime cleanup radius for the given grid size. */
	UE_API double GetCleanupRadiusFromGrid(EPCGHiGenGrid Grid) const;

	UE_API bool operator==(const FPCGRuntimeGenerationRadii& Other) const;

#if WITH_EDITOR
	UE_API void ApplyDeprecation();
#endif

	/** Recompute the cached hash value. Call this anytime the values are changed. */
	UE_API void ComputeHash();

	friend UE_API uint32 GetTypeHash(const FPCGRuntimeGenerationRadii& In);

	static constexpr double DefaultGenerationRadiusMultiplier = 2.0;
	static constexpr double DefaultCleanupRadiusMultiplier = 1.1;

protected:
	/** The distance (in centimeters) at which the component will be considered for generation by the RuntimeGenerationScheduler. For partitioned components, this also acts as the unbounded generation radius. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation")
	FPerQualityLevelFloat RadiusUnbounded = PCGHiGenGrid::UnboundedGridSize() * DefaultGenerationRadiusMultiplier;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation")
	FPerQualityLevelFloat Radius400 = PCGHiGenGrid::GridToGridSize(EPCGHiGenGrid::Grid4) * DefaultGenerationRadiusMultiplier;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation")
	FPerQualityLevelFloat Radius800 = PCGHiGenGrid::GridToGridSize(EPCGHiGenGrid::Grid8) * DefaultGenerationRadiusMultiplier;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation")
	FPerQualityLevelFloat Radius1600 = PCGHiGenGrid::GridToGridSize(EPCGHiGenGrid::Grid16) * DefaultGenerationRadiusMultiplier;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation")
	FPerQualityLevelFloat Radius3200 = PCGHiGenGrid::GridToGridSize(EPCGHiGenGrid::Grid32) * DefaultGenerationRadiusMultiplier;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation")
	FPerQualityLevelFloat Radius6400 = PCGHiGenGrid::GridToGridSize(EPCGHiGenGrid::Grid64) * DefaultGenerationRadiusMultiplier;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation")
	FPerQualityLevelFloat Radius12800 = PCGHiGenGrid::GridToGridSize(EPCGHiGenGrid::Grid128) * DefaultGenerationRadiusMultiplier;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation")
	FPerQualityLevelFloat Radius25600 = PCGHiGenGrid::GridToGridSize(EPCGHiGenGrid::Grid256) * DefaultGenerationRadiusMultiplier;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation")
	FPerQualityLevelFloat Radius51200 = PCGHiGenGrid::GridToGridSize(EPCGHiGenGrid::Grid512) * DefaultGenerationRadiusMultiplier;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation")
	FPerQualityLevelFloat Radius102400 = PCGHiGenGrid::GridToGridSize(EPCGHiGenGrid::Grid1024) * DefaultGenerationRadiusMultiplier;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation")
	FPerQualityLevelFloat Radius204800 = PCGHiGenGrid::GridToGridSize(EPCGHiGenGrid::Grid2048) * DefaultGenerationRadiusMultiplier;

	/** Multiplier on the GenerationRadius to control the distance at which runtime generated components will be cleaned up. Applied per grid size. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation", meta = (UIMin = "1.0", ClampMin = "1.0"))
	FPerQualityLevelFloat CleanupRadiusScalar = static_cast<float>(DefaultCleanupRadiusMultiplier);

private:
	uint32 Hash = 0;

	// Deprecated section
public:
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use RadiusUnbounded instead"))
	double GenerationRadius = 0.0;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Radius400 instead"))
	double GenerationRadius400 = 0.0;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Radius800 instead"))
	double GenerationRadius800 = 0.0;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Radius1600 instead"))
	double GenerationRadius1600 = 0.0;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Radius3200 instead"))
	double GenerationRadius3200 = 0.0;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Radius6400 instead"))
	double GenerationRadius6400 = 0.0;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Radius12800 instead"))
	double GenerationRadius12800 = 0.0;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Radius25600 instead"))
	double GenerationRadius25600 = 0.0;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Radius51200 instead"))
	double GenerationRadius51200 = 0.0;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Radius102400 instead"))
	double GenerationRadius102400 = 0.0;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Radius204800 instead"))
	double GenerationRadius204800 = 0.0;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use CleanupRadiusScalar instead"))
	double CleanupRadiusMultiplier = 0.0;
};

template<>
struct TStructOpsTypeTraits<FPCGRuntimeGenerationRadii> : public TStructOpsTypeTraitsBase2<FPCGRuntimeGenerationRadii>
{
	enum
	{
		WithPostSerialize = true,
	};
};

struct FInstancedPropertyBag;

namespace PCGDelegates
{
#if WITH_EDITOR
	/** Callback to hook in the UI to detect property bag changes, so the UI is reset and does not try to read in garbage memory. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnInstanceLayoutChanged, const FInstancedPropertyBag& /*Instance*/);
	extern PCG_API FOnInstanceLayoutChanged OnInstancedPropertyBagLayoutChanged;
#endif
}

UENUM()
enum class EPCGNodeTitleType : uint8
{
	/** The full title, may be multiple lines. */
	FullTitle,
	/** More concise, single line title. */
	ListView,
};

#if WITH_EDITORONLY_DATA
UENUM()
enum class EPCGNodeToolStartBehavior : uint8
{
	OnInspect,
	OnToolStartButton,
};
#endif // WITH_EDITORONLY_DATA

namespace PCGQualityHelpers
{
	constexpr int32 NumPins = 6;
	extern PCG_API const FName PinLabelDefault;
	extern PCG_API const FName PinLabelLow;
	extern PCG_API const FName PinLabelMedium;
	extern PCG_API const FName PinLabelHigh;
	extern PCG_API const FName PinLabelEpic;
	extern PCG_API const FName PinLabelCinematic;

	/** Get the pin label associated with the current 'pcg.Quality' value. If the quality level is invalid, it will return the default pin label. */
	PCG_API FName GetQualityPinLabel();
}

USTRUCT(meta=(Deprecated = "5.5"))
struct UE_DEPRECATED(5.5, "FPCGPartitionActorRecord is deprecated.") FPCGPartitionActorRecord
{
	GENERATED_BODY()
		
	/** Unique ID for the grid this actor belongs to. */
	UPROPERTY(VisibleAnywhere, Category = Debug)
	FGuid GridGuid;

	/** The grid size this actor lives on. */
	UPROPERTY(VisibleAnywhere, Category = Debug)
	uint32 GridSize = 0;

	/** The specific grid cell this actor lives in. */
	UPROPERTY(VisibleAnywhere, Category = Debug)
	FIntVector GridCoords = FIntVector::ZeroValue;

	bool operator==(const FPCGPartitionActorRecord& InOther) const;
	friend uint32 GetTypeHash(const FPCGPartitionActorRecord& In);
};

UENUM(BlueprintType)
enum class EPCGDensityMergeOperation : uint8
{
	/** D = B */
	Set,
	/** D = A */
	Ignore,
	/** D = min(A, B) */
	Minimum,
	/** D = max(A, B) */
	Maximum,
	/** D = A + B */
	Add,
	/** D = A - B */
	Subtract,
	/** D = A * B */
	Multiply,
	/** D = A / B */
	Divide
};

UENUM(BlueprintType)
enum class EPCGGenerationStatus : uint8
{
	Completed,
	Aborted
};

using FPCGElementPtr = TSharedPtr<IPCGElement, ESPMode::ThreadSafe>;

struct FPCGScheduleGraphParams
{
	FPCGScheduleGraphParams(
		UPCGGraph* InGraph,
		IPCGGraphExecutionSource* InExecutionSource,
		FPCGElementPtr InPreGraphElement,
		FPCGElementPtr InInputElement,
		TArray<FPCGTaskId> InExternalDependencies,
		const FPCGStack* InFromStack,
		bool bInAllowHierarchicalGeneration)
		: Graph(InGraph)
		, ExecutionSource(InExecutionSource)
		, PreGraphElement(InPreGraphElement)
		, InputElement(InInputElement)
		, ExternalDependencies(MoveTemp(InExternalDependencies))
		, FromStack(InFromStack)
		, bAllowHierarchicalGeneration(bInAllowHierarchicalGeneration)
	{
	}

	// Graph to execute
	UPCGGraph* Graph = nullptr;

	// PCG execution source associated with this task. Can be null.
	IPCGGraphExecutionSource* ExecutionSource = nullptr;

	// First task to run. If null, will fallback on the default FPCGPreGraphElement.
	FPCGElementPtr PreGraphElement;

	// Task to run as an input to the provided graph.
	FPCGElementPtr InputElement;
	
	// PreGraph Task dependencies (will wait on those to finish before executing anything).
	TArray<FPCGTaskId> ExternalDependencies;
	
	// When scheduling sub-graphs, this is the parent execution stack.
	const FPCGStack* FromStack;
	
	// If graph is allowed to use hierarchical generation.
	bool bAllowHierarchicalGeneration = true;
};

struct FPCGScheduleGenericParams
{
	FPCGScheduleGenericParams(TFunction<bool(FPCGContext*)> InOperation, IPCGGraphExecutionSource* InExecutionSource)
		: Operation(MoveTemp(InOperation)), ExecutionSource(InExecutionSource)
	{
	}

	FPCGScheduleGenericParams(
		TFunction<bool(FPCGContext*)> InOperation, 
		IPCGGraphExecutionSource* InExecutionSource, 
		const TArray<FPCGTaskId>& InExecutionDependencies, 
		const TArray<FPCGTaskId>& InDataDependencies, 
		bool bInSupportBasePointDataInput)
		: Operation(MoveTemp(InOperation))
		, ExecutionSource(InExecutionSource)
		, ExecutionDependencies(InExecutionDependencies)
		, DataDependencies(InDataDependencies)
		, bSupportBasePointDataInput(bInSupportBasePointDataInput)
	{
	}
		
	FPCGScheduleGenericParams(
		TFunction<bool(FPCGContext*)> InOperation, 
		TFunction<void(FPCGContext*)> InAbortOperation, 
		IPCGGraphExecutionSource* InExecutionSource,
		const TArray<FPCGTaskId>& InExecutionDependencies, 
		const TArray<FPCGTaskId>& InDataDependencies = {}, 
		bool bInSupportBasePointDataInput = false)
		: Operation(MoveTemp(InOperation))
		, ExecutionSource(InExecutionSource)
		, AbortOperation(MoveTemp(InAbortOperation))
		, ExecutionDependencies(InExecutionDependencies)
		, DataDependencies(InDataDependencies)
		, bSupportBasePointDataInput(bInSupportBasePointDataInput)
	{
	}

	// Callback that takes a Context as argument and returns true if the task is done, false otherwise
	TFunction<bool(FPCGContext*)> Operation;

	// PCG execution source associated with this task. Can be null.
	IPCGGraphExecutionSource* ExecutionSource = nullptr;

	// Callback that is called if the task is aborted (cancelled) before fully executed.
	TFunction<void(FPCGContext*)> AbortOperation;

	// Task will wait on these tasks to execute and won't take their output data as input.
	TArray<FPCGTaskId> ExecutionDependencies;

	// Task will wait on these tasks to execute and will take their output data as input.
	TArray<FPCGTaskId> DataDependencies;

	// When true, generic element will not convert input to UPCGPointData, this is false by default to preserver backward compatibility.
	bool bSupportBasePointDataInput = false;

	// When false, generic element can be executed outside of the game thread.
	bool bCanExecuteOnlyOnMainThread = true;
};

/** A handle to a call stack which holds onto the stack memory. */
struct FPCGStackHandle
{
	FPCGStackHandle() = default;
	UE_API FPCGStackHandle(TSharedPtr<const FPCGStackContext> InStackContext, int32 InStackIndex);

	UE_API bool IsValid() const;

	UE_API const FPCGStack* GetStack() const;

private:
	TSharedPtr<const FPCGStackContext> StackContext;

	int32 StackIndex = INDEX_NONE;
};

// Enable to debug if some PCG Elements are creating some data that should be pre-created by the graph PreGraph element
#define PCG_EXECUTION_CACHE_VALIDATION_ENABLED 0

#if PCG_EXECUTION_CACHE_VALIDATION_ENABLED

#define PCG_EXECUTION_CACHE_VALIDATION_CREATE_SCOPE(Source) TGuardValue<bool> ValidationCreateScope(Source->GetExecutionState().bExecutionCacheWriteEnabled, true);
#define PCG_EXECUTION_CACHE_VALIDATION_CREATE_ORIGINAL_SCOPE(Source) TGuardValue<bool> ValidationCreateOriginalScope(Source->GetExecutionState().GetOriginalSource()->GetExecutionState().bExecutionCacheWriteEnabled, Source->GetExecutionState().bExecutionCacheWriteEnabled);
#define PCG_EXECUTION_CACHE_VALIDATION_CHECK(Source) ensureAlways(Source->GetExecutionState().bExecutionCacheWriteEnabled || Source->GetExecutionState().GetGenerationTaskId() == InvalidPCGTaskId);

#else

#define PCG_EXECUTION_CACHE_VALIDATION_CREATE_SCOPE(Source)
#define PCG_EXECUTION_CACHE_VALIDATION_CREATE_ORIGINAL_SCOPE(Source)
#define PCG_EXECUTION_CACHE_VALIDATION_CHECK(Source)

#endif // PCG_EXECUTION_CACHE_VALIDATION_ENABLED

UENUM()
enum class EPCGTextureResourceType : uint8
{
	TextureObject UMETA(ToolTip = "UObject texture such as UTexture2D or UTextureRenderTarget2D."),
	ExportedTexture UMETA(ToolTip = "Texture handle exported from a texture on the GPU."),
	Invalid UMETA(Hidden)
};

/** Method used to determine the value for a sample based on the value of nearby texels. */
UENUM(BlueprintType)
enum class EPCGTextureFilter : uint8
{
	Point UMETA(Tooltip="Takes the value of whatever texel the sample lands in."),
	Bilinear UMETA(Tooltip="Bilinearly interpolates the values of the four nearest texels to the sample location.")
};

namespace PCGLevelToAssetConstants
{
	/** Attribute names shared between the LevelToAsset workflow and the GetDataFunctionRegistry actor parsing. */
	constexpr FLazyName MaterialAttributeName = TEXT("Material");
	constexpr FLazyName MeshAttributeName = TEXT("Mesh");
	constexpr FLazyName SkeletalMeshAttributeName = TEXT("SkeletalMesh");
	constexpr FLazyName HierarchyDepthAttributeName = TEXT("HierarchyDepth");
	constexpr FLazyName ActorIndexAttributeName = TEXT("ActorIndex");
	constexpr FLazyName ParentIndexAttributeName = TEXT("ParentIndex");
	constexpr FLazyName RelativeTransformAttributeName = TEXT("RelativeTransform");
	constexpr FLazyName IgnoreParentRotationAttributeName = TEXT("IgnoreParentRotation");
	constexpr FLazyName IgnoreParentScaleAttributeName = TEXT("IgnoreParentScale");
	constexpr FLazyName ExcludedActorTag = TEXT("PCGLevelToAssetExclude");
}

#ifndef PCG_PROFILING_ENABLED
#if WITH_EDITOR || !UE_BUILD_SHIPPING
#define PCG_PROFILING_ENABLED 1
#else
#define PCG_PROFILING_ENABLED 0
#endif
#endif

#undef UE_API
