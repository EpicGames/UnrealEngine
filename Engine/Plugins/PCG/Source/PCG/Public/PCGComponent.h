// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGComponentExecutionState.h"
#include "PCGGraphExecutionInspection.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGManagedResourceContainer.h"
#include "PCGNode.h"
#include "PCGSettings.h"
#include "ChangeTracking/PCGChangeTracker.h"
#include "Data/PCGToolData.h"
#include "Graph/PCGSourceDataContainer.h"
#include "Graph/PCGStackContext.h"
#include "Grid/PCGGridDescriptor.h"
#include "Utils/PCGExtraCapture.h"

#include "ComponentInstanceDataCache.h"
#include "Components/ActorComponent.h"
#include "Containers/Map.h"
#include "Misc/ScopeExit.h"
#include "UObject/ObjectKey.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Misc/TransactionallySafeRWLock.h"

#include "PCGComponent.generated.h"

#define UE_API PCG_API

namespace EEndPlayReason { enum Type : int; }

class APCGPartitionActor;
class FPCGActorAndComponentMapping;
class UPCGComponent;
class UPCGData;
class IPCGGenSourceBase;
class UPCGGraph;
class UPCGGraphInstance;
class UPCGGraphInterface;
class UPCGManagedResource;
class UPCGSchedulingPolicyBase;
class UPCGSubsystem;
class ALandscapeProxy;
class FLandscapeProxyComponentDataChangedParams;
class UClass;
struct FPCGContext;
struct FPCGStackContext;
struct FPCGMoveResourceParams;
namespace UE 
{
	class FPCGInternalHelpers;
}

DECLARE_MULTICAST_DELEGATE_OneParam(FOnPCGGraphStartGenerating, UPCGComponent*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPCGGraphCancelled, UPCGComponent*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPCGGraphGenerated, UPCGComponent*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPCGGraphCleaned, UPCGComponent*);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCGGraphStartGeneratingExternal, UPCGComponent*, PCGComponent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCGGraphCancelledExternal, UPCGComponent*, PCGComponent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCGGraphGeneratedExternal, UPCGComponent*, PCGComponent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCGGraphCleanedExternal, UPCGComponent*, PCGComponent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnPCGLinkClearedExternal, UPCGComponent*, PCGComponent, UObject*, NewOwner);

UENUM(Blueprintable)
enum class EPCGComponentInput : uint8
{
	Actor, /** Generates based on owning actor */
	Landscape,
	Other,
	// More?
	EPCGComponentInput_MAX
};

UENUM(Blueprintable)
enum class EPCGComponentGenerationTrigger : uint8
{
	GenerateOnLoad    UMETA(ToolTip = "Generates only when the component is loaded into the level."),
	GenerateOnDemand  UMETA(ToolTip = "Generates only when requested (e.g. via Blueprint)."),
	GenerateAtRuntime UMETA(ToolTip = "Generates only when scheduled by the Runtime Generation Scheduler.")
};

UENUM(meta = (Bitflags))
enum class EPCGComponentDirtyFlag : uint8
{
	None = 0,
	Actor = 1 << 0,
	Landscape = 1 << 1,
	Input = 1 << 2,
	Data = 1 << 3,
	All = Actor | Landscape | Input | Data
};
ENUM_CLASS_FLAGS(EPCGComponentDirtyFlag);

UENUM()
enum class EPCGIgnoreChangeOriginReason : uint8
{
	None = 0,
	ComponentDisabledLandscapeTracking,
	GraphDisabledLandscapeTracking,
	GenerationIgnoredObject,
	GraphIgnoredClass
};

namespace PCGTestsCommon
{
	struct FTestData;
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural), meta = (BlueprintSpawnableComponent, PrioritizeCategories = "PCG"))
class UPCGComponent : public UActorComponent, public IPCGGraphExecutionSource
{
	UE_API UPCGComponent(const FObjectInitializer& InObjectInitializer);

	GENERATED_BODY()

	friend class FPCGComponentExecutionState;
	friend class UPCGManagedActors;
	friend class UPCGSubsystem;
	friend class FPCGGraphExecutor;
	friend class FPCGTrackingManager;
	friend class UE::FPCGInternalHelpers;
	friend struct FPCGWorldPartitionBuilder;

public:
	/** ~Begin IPCGGraphExecutionSource interface */
	virtual IPCGGraphExecutionState& GetExecutionState() override { return ExecutionState; }
	virtual const IPCGGraphExecutionState& GetExecutionState() const override { return ExecutionState; }
	/** ~End IPCGGraphExecutionSource interface */

	/** ~Begin UObject interface */
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual bool IsEditorOnly() const override;
	UE_API virtual bool NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const override;

#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
#endif
	/** ~End UObject interface */

	//~Begin UActorComponent Interface
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;

protected:
	friend struct FPCGComponentInstanceData;
	friend struct PCGTestsCommon::FTestData;

	UE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	//~End UActorComponent Interface

public:
	UE_API UPCGData* GetPCGData() const;
	UE_API UPCGData* GetInputPCGData() const;
	UE_API UPCGData* GetActorPCGData() const;
	UE_API UPCGData* GetLandscapePCGData() const;
	UE_API UPCGData* GetLandscapeHeightPCGData() const;
	UE_API UPCGData* GetOriginalActorPCGData() const;

	/** If this is a local component returns the original component, otherwise returns self. */
	UE_API UPCGComponent* GetOriginalComponent() const;
	UE_API const UPCGComponent* GetConstOriginalComponent() const;

	UPCGSchedulingPolicyBase* GetRuntimeGenSchedulingPolicy() const { return SchedulingPolicy; }

	UE_API bool DoesGridDependOnWorldStreaming(uint32 InGridSize) const;

	UE_API bool CanPartition() const;

	/**
	 * Returns the underlying PCG Graph for this component.
	 * Note that this is not the Graph Instance on the PCG Component, but the PCG Graph asset at the top of the instance hierarchy.
	 * If it is needed to set Graph Parameters on the component, use the GraphInstance.
	 */
	UFUNCTION(BlueprintCallable, Category = PCG)
	UE_API UPCGGraph* GetGraph() const;

	UPCGGraphInstance* GetGraphInstance() const { return GraphInstance; }

	UE_API void SetGraphLocal(UPCGGraphInterface* InGraph);

	UFUNCTION(BlueprintCallable, NetMulticast, Reliable, Category = PCG)
	UE_API void SetGraph(UPCGGraphInterface* InGraph);

	/** Registers some managed resource to the current component */
	UFUNCTION(BlueprintCallable, Category = PCG)
	UE_API void AddToManagedResources(UPCGManagedResource* InResource);

	/** Creates a managed component resource and adds it to the current component. Note: in native code, consider using the explicit creation especially if there are special resource objects involved. */
	UFUNCTION(BlueprintCallable, Category = PCG)
	UE_API void AddComponentsToManagedResources(const TArray<UActorComponent*>& InComponents);

	/** Creates a managed actors resource and adds it to the current component. Note: in native code, consider using the explicit creation especially if there are special resource objects involved. */
	UFUNCTION(BlueprintCallable, Category = PCG)
	UE_API void AddActorsToManagedResources(const TArray<AActor*>& InActors);

public:
	UE_API bool AreManagedResourcesAccessible() const;
	UE_API void ForEachManagedResource(TFunctionRef<void(UPCGManagedResource*)> InFunction);
	UE_API void ForEachConstManagedResource(TFunctionRef<void(const UPCGManagedResource*)> InFunction) const;

	/** Will scan the managed resources to check if any resource manage one of the objects. */
	UE_API bool IsAnyObjectManagedByResource(const TArrayView<const UObject*> InObjects) const;

	UE_API void Generate();
	UE_API void Cleanup();

	/** Starts generation from a local (vs. remote) standpoint. Will not be replicated. Will be delayed. */
	UFUNCTION(BlueprintCallable, Category = PCG)
	UE_API void GenerateLocal(bool bForce);

	/** Requests the component to generate only on the specified grid level (all grid levels if EPCGHiGenGrid::Uninitialized). */
	UE_API void GenerateLocal(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, uint32 Grid = PCGHiGenGrid::UninitializedGridSize(), const TArray<FPCGTaskId>& Dependencies = {});

	UE_API FPCGTaskId GenerateLocalGetTaskId(bool bForce);
	UE_API FPCGTaskId GenerateLocalGetTaskId(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, uint32 Grid = PCGHiGenGrid::UninitializedGridSize());
	UE_API FPCGTaskId GenerateLocalGetTaskId(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, uint32 Grid, const TArray<FPCGTaskId>& Dependencies);

	/** Cleans up the generation from a local (vs. remote) standpoint. Will not be replicated. Will be delayed. */
	UFUNCTION(BlueprintCallable, Category = PCG)
	UE_API void CleanupLocal(bool bRemoveComponents);

	UE_API FPCGTaskId CleanupLocal(bool bRemoveComponents, const TArray<FPCGTaskId>& Dependencies);

	UE_DEPRECATED(5.6, "Use version with no bSave parameter.")
	void CleanupLocal(bool bRemoveComponents, bool bSave) { CleanupLocal(bRemoveComponents); }

	UE_DEPRECATED(5.6, "Use version with no bSave parameter.")
	void CleanupLocal(bool bRemoveComponents, bool bSave, const TArray<FPCGTaskId>& Dependencies) { CleanupLocal(bRemoveComponents, Dependencies); }

	/** Cleanup the generation while purging Actors and Components tagged as generated by PCG but are no longer managed by this or any other actor*/
	UE_API void CleanupLocalDeleteAllGeneratedObjects(const TArray<FPCGTaskId>& Dependencies);

	/**
	 * Same as CleanupLocal, but without any delayed tasks. All is done immediately. If 'bCleanupLocalComponents' is true and the
	 * component is partitioned, we will forward the calls to its registered local components.
	 */
	UE_API void CleanupLocalImmediate(bool bRemoveComponents, bool bCleanupLocalComponents = false);

	/** Networked generation call that also activates the component as needed */
	UFUNCTION(BlueprintCallable, NetMulticast, Reliable, Category = PCG)
	UE_API void Generate(bool bForce);

	/** Networked cleanup call */
	UFUNCTION(BlueprintCallable, NetMulticast, Reliable, Category = PCG)
	UE_API void Cleanup(bool bRemoveComponents);

	/** Cancels in-progress generation */
	UE_API void CancelGeneration();

	/** Notify properties changed, used in runtime cases, will dirty & trigger a regeneration if needed */
	UFUNCTION(BlueprintCallable, Category = PCG)
	UE_API void NotifyPropertiesChangedFromBlueprint();

	/** Retrieves generated data */
	UFUNCTION(BlueprintCallable, Category = PCG)
	const FPCGDataCollection& GetGeneratedGraphOutput() const { return GeneratedGraphOutput; }

	/** Move all generated resources under a new actor, following a template, clearing all link to this PCG component. Returns the newly created actor. */
	UFUNCTION(BlueprintCallable, Category = PCG)
	UE_API AActor* ClearPCGLink(UClass* TemplateActor = nullptr);

	/** Move all generated resources that meet the params requirements, following a template and clearing all link to this PCG component. Returns the newly created actor. */
	UE_API AActor* ClearPCGLink(const FPCGMoveResourceParams& Params);

	uint32 GetGenerationGridSize() const { return GenerationGridSize; }
	void SetGenerationGridSize(uint32 InGenerationGridSize) { GenerationGridSize = InGenerationGridSize; }

	/** Get the grid that this component is executing on without grid scaling applied. */
	EPCGHiGenGrid GetUnscaledGenerationGrid() const;

	/** Store data with a resource key that identifies the pin. */
	UE_API void StoreOutputDataForPin(const FString& InResourceKey, const FPCGDataCollection& InData);

	/** Lookup data using a resource key that identifies the pin. */
	UE_API const FPCGDataCollection* RetrieveOutputDataForPin(const FString& InResourceKey);

	/** Clear any data stored for any pins. */
	UE_API void ClearPerPinGeneratedOutput();

	/** Set the runtime generation scheduling policy type. */
	UE_API void SetSchedulingPolicyClass(TSubclassOf<UPCGSchedulingPolicyBase> InSchedulingPolicyClass);

	/** Get the generation radii that are currently active for this component. */
	UE_API const FPCGRuntimeGenerationRadii& GetGenerationRadii() const;

	/** Get the runtime generation radius for the given (scaled) grid size. */
	UE_API double GetGenerationRadiusFromGrid(uint32 Grid) const;

	/** Compute the runtime cleanup radius for the given (scaled) grid size. */
	UE_API double GetCleanupRadiusFromGrid(uint32 Grid) const;

	/** Allows executing runtime PCG graphs without partition actors and local PCG components. */
	UE_API bool UseActorComponentlessGeneration() const;

	/** Called during execution if one or more procedural ISM components are in use. */
	void NotifyProceduralInstancesInUse() { bProceduralInstancesInUse = true; }

	/** Whether this component created one or more procedural ISM components when last generated. */
	bool AreProceduralInstancesInUse() const { return bProceduralInstancesInUse; }

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (DisplayPriority = 600))
	int Seed = 42;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (DisplayPriority = 100))
	bool bActivated = true;

	/** 
	 * Will partition the component in a grid, dispatching the generation to multiple local components. Grid size is determined by the
	 * PCGWorldActor unless the graph has Hierarchical Generation enabled, in which case grid sizes are determined by the graph.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bIsComponentLocal", DisplayName = "Is Partitioned", DisplayPriority = 500))
	bool bIsComponentPartitioned = false;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (EditCondition = "!bIsComponentLocal", EditConditionHides, DisplayPriority = 200))
	EPCGComponentGenerationTrigger GenerationTrigger = EPCGComponentGenerationTrigger::GenerateOnLoad;

	/** When Generation Trigger is OnDemand, we can still force the component to generate on drop. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Settings|Advanced" , meta = (EditCondition = "GenerationTrigger == EPCGComponentGenerationTrigger::GenerateOnDemand"))
	bool bGenerateOnDropWhenTriggerOnDemand = false;

	/** Manual overrides for the graph generation radii and cleanup radius multiplier. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RuntimeGeneration, meta = (EditCondition = "GenerationTrigger == EPCGComponentGenerationTrigger::GenerateAtRuntime", EditConditionHides))
	bool bOverrideGenerationRadii = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RuntimeGeneration, meta = (EditCondition = "GenerationTrigger == EPCGComponentGenerationTrigger::GenerateAtRuntime && bOverrideGenerationRadii", EditConditionHides))
	FPCGRuntimeGenerationRadii GenerationRadii;

public:
	/** A Scheduling Policy dictates the order in which instances of this component will be scheduled. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, NoClear, Category = RuntimeGeneration, meta = (EditCondition = "GenerationTrigger == EPCGComponentGenerationTrigger::GenerateAtRuntime", EditConditionHides))
	TSubclassOf<UPCGSchedulingPolicyBase> SchedulingPolicyClass;

	/** This is the instanced UPCGSchedulingPolicy object which holds scheduling parameters and calculates priorities. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Instanced, Category = RuntimeGeneration, meta = (EditCondition = "GenerationTrigger == EPCGComponentGenerationTrigger::GenerateAtRuntime", EditConditionHides))
	TObjectPtr<UPCGSchedulingPolicyBase> SchedulingPolicy;

	/** This stores working data of a tool; runtime property as the PCG Component will rely on the tool's working data to generate properly. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Tool Data")
	FPCGInteractiveToolDataContainer ToolDataContainer;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Editing Settings", meta = (DisplayName = "Regenerate PCG Volume In Editor", DisplayPriority = 400))
	bool bRegenerateInEditor = true;

	/** Even if the graph has external dependencies, the component won't react to them. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Editing Settings", meta = (DisplayPriority = 450))
	bool bOnlyTrackItself = false;

	/** Marks the component to be not refreshed automatically when the landscape changes, even if it is used. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Editing Settings", meta = (DisplayPriority = 451))
	bool bIgnoreLandscapeTracking = false;

	/** Tracking priority used to solve tracking dependencies between PCG Components. By default object path is used to create a default priority (order) but overriding this value gives control to the user. (smaller value is higher priority) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Editing Settings", meta = (DisplayPriority = 452, ClampMin="-10000.0", ClampMax="10000.0", UIMin="-10000.0", UIMax="10000.0", Delta="1"))
	double TrackingPriority = 0; 

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Transient, Category = Debug, meta = (NoResetToDefault))
	bool bDirtyGenerated = false;

	// Property that will automatically be set on BP templates, to allow for "Generate on add to world" in editor.
	// Set it as a property to automatically transfer it to its child.
	// Don't use it directly, use ShouldGenerateBPPCGAddedToWorld, as there are other conditions checked.
	UPROPERTY()
	bool bForceGenerateOnBPAddedToWorld = false;

#endif // WITH_EDITORONLY_DATA

	FOnPCGGraphStartGenerating OnPCGGraphStartGeneratingDelegate;
	FOnPCGGraphCancelled OnPCGGraphCancelledDelegate;
	FOnPCGGraphGenerated OnPCGGraphGeneratedDelegate;
	FOnPCGGraphCleaned OnPCGGraphCleanedDelegate;

	/** Event dispatched when a graph begins generation on this component. */
	UPROPERTY(BlueprintAssignable, meta = (DisplayName = "On Graph Started Generating"))
	FOnPCGGraphStartGeneratingExternal OnPCGGraphStartGeneratingExternal;
	/** Event dispatched when a graph cancels generation on this component. */
	UPROPERTY(BlueprintAssignable, meta = (DisplayName = "On Graph Cancelled"))
	FOnPCGGraphCancelledExternal OnPCGGraphCancelledExternal;
	/** Event dispatched when a graph completes its generation on this component. */
	UPROPERTY(BlueprintAssignable, meta = (DisplayName = "On Graph Generated"))
	FOnPCGGraphGeneratedExternal OnPCGGraphGeneratedExternal;
	/** Event dispatched when a graph cleans on this component. */
	UPROPERTY(BlueprintAssignable, meta = (DisplayName = "On Graph Cleaned"))
	FOnPCGGraphCleanedExternal OnPCGGraphCleanedExternal;
	/** Event dispatched when a component has finished to do its Clear PCG Link. */
	UPROPERTY(BlueprintAssignable, meta = (DisplayName = "On PCG Link Cleared"))
	FOnPCGLinkClearedExternal OnPCGLinkClearedExternal;

	/** Flag to indicate whether this component has run in the editor. Note that for partitionable actors, this will always be false. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Debug, NonTransactional, meta = (NoResetToDefault))
	bool bGenerated = false;

	UE_DEPRECATED(5.8, "No longer used.")
	UPROPERTY(NonPIEDuplicateTransient)
	bool bRuntimeGenerated = false;

	/** Can specify a list of functions from the owner of this component to be called when generation is done, in order.
	*   Need to take (and only take) a PCGDataCollection as parameter and with "CallInEditor" flag enabled.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (DisplayPriority = 700))
	TArray<FName> PostGenerateFunctionNames;

	/** Return if we are currently generating the graph for this component */
	bool IsGenerating() const { return CurrentGenerationTask != InvalidPCGTaskId; }
	bool IsCleaningUp() const { return CurrentCleanupTask != InvalidPCGTaskId; }

	/** Return if this component was already generated when it was loaded */
	bool IsGeneratedOffline() const { return bGeneratedOffline; }

	/** Returns task ids to do internal chaining */
	FPCGTaskId GetGenerationTaskId() const { return CurrentGenerationTask; }
	FPCGTaskId GetCleanupTaskId() const { return CurrentCleanupTask; }

#if WITH_EDITOR
	/** Schedules refresh of the component. If bCancelExistingRefresh is true, any existing refresh is cancelled and a new one is scheduled. */
	UE_API void Refresh(EPCGChangeType ChangeType = EPCGChangeType::None, bool bCancelExistingRefresh = false);

	UE_API void OnRefresh(bool bForceRefresh);

	UE_API void StartGenerationInProgress();
	UE_API void StopGenerationInProgress();
	UE_API bool IsGenerationInProgress();

	/** Returns current refresh task ID. */
	bool IsRefreshInProgress() const { return CurrentRefreshTask != InvalidPCGTaskId; }

	/** Dirty generated data depending on the flag. By default the call is forwarded to the local components.
	    We don't forward if the local component has callbacks that would dirty them too.
		For example: When a tracked actor move, we only want to dirty the impacted local components.*/
	UE_API void DirtyGenerated(EPCGComponentDirtyFlag DataToDirtyFlag = EPCGComponentDirtyFlag::None, const bool bDispatchToLocalComponents = true);

	/** Reset last generated bounds to force PCGPartitionActor creation on next refresh */
	UE_API void ResetLastGeneratedBounds();

	/** Functions for managing the node inspection cache */
	bool WasGeneratedThisSession() const { return bWasGeneratedThisSession; }

	/** Retrieve the inactive pin bitmask for the given node and stack in the last execution. */
	UE_API uint64 GetNodeInactivePinMask(const UPCGNode* InNode, const FPCGStack& Stack) const;

	/** Whether the given node was culled by a dynamic branch in the given stack. */
	UE_API void NotifyNodeDynamicInactivePins(const UPCGNode* InNode, const FPCGStack* InStack, uint64 InactivePinBitmask) const;

	/** Did the given node produce one or more data items in the given stack during the last execution. */
	UE_API bool HasNodeProducedData(const UPCGNode* InNode, const FPCGStack& Stack) const;

	UE_DEPRECATED(5.8, "No longer supported")
	UE_API bool IsObjectTracked(const UObject* InObject, bool& bOutIsCulled) const;

	/** Know if we need to force a generation, in case of BP added to the world in editor */
	UE_API bool ShouldGenerateBPPCGAddedToWorld() const;

	/** Changes the transient state (preview, normal, load on preview) - public only because it needs to be accessed by APCGPartitionActor */
	UE_API void ChangeTransientState(EPCGEditorDirtyMode NewEditingMode);

	/** Get execution stack information. */
	UE_API bool GetStackContext(FPCGStackContext& OutStackContext) const;

	/** To be called by an element to notify the component that this settings have a dynamic dependency. */
	UE_API void RegisterDynamicTracking(const UPCGSettings* InSettings, const TArrayView<TPair<FPCGSelectionKey, bool>>& InDynamicKeysAndCulling);

	/** To be called to notify the component that this list of keys have a dynamic dependency. */
	UE_API void RegisterDynamicTracking(const FPCGSelectionKeyToSettingsMap& InKeysToSettings);

	/** For duration of the current/next generation, any change triggers from this change origin will be discarded. */
	UE_API void StartIgnoringChangeOriginDuringGeneration(const UObject* InChangeOriginToIgnore);
	UE_API void StartIgnoringChangeOriginsDuringGeneration(const TArrayView<const UObject*> InChangeOriginsToIgnore);

	UE_API void StopIgnoringChangeOriginDuringGeneration(const UObject* InChangeOriginToIgnore);
	UE_API void StopIgnoringChangeOriginsDuringGeneration(const TArrayView<const UObject*> InChangeOriginsToIgnore);

	UE_API bool IsIgnoringChangeOrigin(const UObject* InChangeOrigin) const;
	UE_API bool IsIgnoringAnyChangeOrigins(const TArrayView<const UObject*> InChangeOrigins, const UObject*& OutFirstObjectFound) const;
	UE_API bool IsIgnoringAnyChangeOrigins(const TArrayView<const UObject*> InChangeOrigins, const UObject*& OutFirstObjectFound, EPCGIgnoreChangeOriginReason& OutReason) const;
	UE_API void ResetIgnoredChangeOrigins(bool bLogIfAnyPresent);
#endif

	// Allow for the function to be defined outside of editor, so it's lighter at the calling site and will just call the InFunc.
	// Note that this will mark the original component.
	template <typename Func>
	void IgnoreChangeOriginsDuringGenerationWithScope(const TArrayView<const UObject*> InChangeOriginsToIgnore, Func InFunc)
	{
#if WITH_EDITOR
		UPCGComponent* OriginalComponent = GetOriginalComponent();
		check(OriginalComponent);
		
		OriginalComponent->StartIgnoringChangeOriginsDuringGeneration(InChangeOriginsToIgnore);

		ON_SCOPE_EXIT
		{
			OriginalComponent->StopIgnoringChangeOriginsDuringGeneration(InChangeOriginsToIgnore);
		};
#endif // WITH_EDITOR
		
		InFunc();
	}

	template <typename Func>
	void IgnoreChangeOriginDuringGenerationWithScope(const UObject* InChangeOriginToIgnore, Func InFunc)
	{
		IgnoreChangeOriginsDuringGenerationWithScope(TArrayView<const UObject*>(&InChangeOriginToIgnore, 1), std::move(InFunc));
	}

	/** Utility function (mostly for tests) to properly set the value of bIsComponentPartitioned.
	*   Will do an immediate cleanup first and then register/unregister the component to the subsystem.
	*   It's your responsibility after to regenerate the graph if you want to.
	*/
	UE_API void SetIsPartitioned(bool bIsNowPartitioned);

	UE_API bool IsPartitioned() const;
	bool IsLocalComponent() const { return bIsComponentLocal; }

	/** Returns true if the component is managed by the runtime generation system. Nothing else should generate or cleanup this component. */
	bool IsManagedByRuntimeGenSystem() const { return GenerationTrigger == EPCGComponentGenerationTrigger::GenerateAtRuntime; }

	/** Returns true if component should output on a 2D Grid */
	UE_API bool Use2DGrid() const;

	/** Returns a GridDescriptor based on this component for the specified grid size */
	UE_API FPCGGridDescriptor GetGridDescriptor(uint32 GridSize) const;

	/* Responsibility of the PCG Partition Actor to mark is local */
	void MarkAsLocalComponent() { bIsComponentLocal = true; }

	/** Updates internal properties from other component, dirties as required but does not trigger Refresh */
	UE_API void SetPropertiesFromOriginal(const UPCGComponent* Original);

	/** Returns whether the component (or resources) should be marked as dirty following interaction/refresh based on the current editing mode */
	bool IsInPreviewMode() const { return CurrentEditingMode == EPCGEditorDirtyMode::Preview; }

	UFUNCTION(BlueprintCallable, Category="PCG|Advanced")
	UE_API void SetEditingMode(EPCGEditorDirtyMode InEditingMode, EPCGEditorDirtyMode InSerializedEditingMode);

	/** Returns the current editing mode */
	UFUNCTION(BlueprintCallable, Category="PCG|Advanced")
	EPCGEditorDirtyMode GetEditingMode() const { return CurrentEditingMode; }

	UFUNCTION(BlueprintCallable, Category = "PCG|Advanced")
	EPCGEditorDirtyMode GetSerializedEditingMode() const { return SerializedEditingMode; }

	UE_API UPCGSubsystem* GetSubsystem() const;

	UE_API FBox GetGridBounds() const;
	UE_API FBox GetOriginalGridBounds() const;
	UE_API FBox GetLocalSpaceBounds() const;
	UE_API FBox GetOriginalLocalSpaceBounds() const;
	UE_API FBox GetTotalBounds() const;

	FBox GetLastGeneratedBounds() const { return LastGeneratedBounds; }

	/** Builds the PCG data from a given actor and its PCG component, and places it in a data collection with appropriate tags */
	static UE_API FPCGDataCollection CreateActorPCGDataCollection(AActor* Actor, const UPCGComponent* Component, EPCGDataType InDataFilter, bool bParseActor = true, bool* bOutOptionalSanitizedTagAttributeName = nullptr);

	/** Builds the canonical PCG data from a given actor and its PCG component if any. */
	static UE_API UPCGData* CreateActorPCGData(AActor* Actor, const UPCGComponent* Component, bool bParseActor = true);

#if WITH_EDITOR

	UE_DEPRECATED(all, "Do not use")
	UE_API bool ShouldTrackLandscape() const;

	UE_DEPRECATED(5.6, "Use FPCGGraphExecutionInspection::IsInspecting() instead")
	UE_API bool IsInspecting() const;

	UE_DEPRECATED(5.6, "Use FPCGGraphExecutionInspection::EnableInspection() instead")
	UE_API void EnableInspection();

	UE_DEPRECATED(5.6, "Use FPCGGraphExecutionInspection::DisableInspection() instead")
	UE_API void DisableInspection();

	UE_DEPRECATED(5.6, "Use FPCGGraphExecutionInspection::StoreInspectionData() instead")
	UE_API void StoreInspectionData(const FPCGStack* InStack, const UPCGNode* InNode, const PCGUtils::FCallTime* InTimer, const FPCGDataCollection& InInputData, const FPCGDataCollection& InOutputData, bool bUsedCache);

	UE_DEPRECATED(5.6, "Use FPCGGraphExecutionInspection::GetInspectionData() instead")
	UE_API const FPCGDataCollection* GetInspectionData(const FPCGStack& InStack) const;

	UE_DEPRECATED(5.6, "Use FPCGGraphExecutionInspection::ClearInspectionData() instead")
	UE_API void ClearInspectionData(bool bClearPerNodeExecutionData = true);

	UE_DEPRECATED(5.6, "Use FPCGGraphExecutionInspection::WasNodeExecuted() instead")
	UE_API bool WasNodeExecuted(const UPCGNode* InNode, const FPCGStack& Stack) const;

	UE_DEPRECATED(5.6, "Use FPCGGraphExecutionInspection::NotifyNodeExecuted() instead")
	UE_API void NotifyNodeExecuted(const UPCGNode* InNode, const FPCGStack* InStack, const PCGUtils::FCallTime* InTimer, bool bNodeUsedCache);
		
	struct UE_DEPRECATED(5.6, "Use FPCGGraphExecutionInspection::FNodeExecutionNotificationData instead") NodeExecutedNotificationData
	{
		NodeExecutedNotificationData(const FPCGStack& InStack, const PCGUtils::FCallTime& InTimer) : Stack(InStack), Timer(InTimer) {}
		// Important implementation note: some logic in WasNodeExecuted relies on the fact we don't use the timer for the operator== and hash functions.
		friend PCG_API uint32 GetTypeHash(const NodeExecutedNotificationData& NotifData) { return GetTypeHash(NotifData.Stack); }
		bool operator==(const NodeExecutedNotificationData& OtherNotifData) const { return Stack == OtherNotifData.Stack; }

		FPCGStack Stack;
		PCGUtils::FCallTime Timer;
	};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Use FPCGGraphExecutionInspection::GetExecutedNodeStacks() instead")
	UE_API TMap<TObjectKey<const UPCGNode>, TSet<NodeExecutedNotificationData>> GetExecutedNodeStacks() const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif // WITH_EDITOR

protected:
	FPCGGridDescriptor GetGridDescriptorInternal(uint32 GridSize, bool bRuntimeHashUpdate) const;
	
	UE_API void RefreshSchedulingPolicy();

	/** Purges Actors and Components generated by PCG but are no longer managed by any PCG Component */
	static UE_API void PurgeUnlinkedResources(const AActor* InActor);

protected:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Settings, Instanced, meta = (NoResetToDefault, DisplayPriority = 100))
	TObjectPtr<UPCGGraphInstance> GraphInstance;

	UPROPERTY(Transient, VisibleAnywhere, Category = Debug)
	uint32 GenerationGridSize = PCGHiGenGrid::UnboundedGridSize();

	/** Current editing mode that depends on the serialized editing mode and loading. If the component is set to GenerateAtRuntime, this will behave as Preview. */
	UPROPERTY(Transient, EditAnywhere, Category = "Editing Settings", meta = (DisplayName = "Editing Mode", EditCondition = "!bIsComponentLocal && GenerationTrigger != EPCGComponentGenerationTrigger::GenerateAtRuntime", DisplayPriority = 300))
	EPCGEditorDirtyMode CurrentEditingMode = EPCGEditorDirtyMode::Normal;

	UPROPERTY(VisibleAnywhere, Category = "Editing Settings", meta = (NoResetToDefault, DisplayPriority = 300))
	EPCGEditorDirtyMode SerializedEditingMode = EPCGEditorDirtyMode::Normal;

	/** Used to store the CurrentEditingMode when it is forcefully changed by another system, such as runtime generation. */
	EPCGEditorDirtyMode PreviousEditingMode = EPCGEditorDirtyMode::Normal;

	/** Used to compare pre/post undo to know if we need to call ChangeTransientState. */
	EPCGEditorDirtyMode LastEditingModePriorToUndo = EPCGEditorDirtyMode::Normal;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Input Node Settings (Deprecated)", meta = (DisplayPriority = 800))
	EPCGComponentInput InputType = EPCGComponentInput::Actor;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Input Node Settings (Deprecated)", meta = (DisplayPriority = 900))
	bool bParseActorComponents = true;

protected:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UPCGGraph> Graph_DEPRECATED;

private:
	/** Track if component should disable 'bIsComponentPartitioned'. Used to deprecate partitioned components in non-WP levels before support for partitioning in non-WP levels. */
	bool bDisableIsComponentPartitionedOnLoad = false;

	/** Track if component was unregistered while in a loading scope. */
	bool bUnregisteredThroughLoading = false;
#endif // WITH_EDITORONLY_DATA
	
	UPROPERTY()
	uint32 RuntimeGridDescriptorHash = 0;

private:
	void ExecutePreGraph(FPCGContext* InContext);

	UE_API UPCGData* CreatePCGData() const;
	UE_API UPCGData* CreateInputPCGData() const;
	UE_API UPCGData* CreateActorPCGData() const;
	UE_API UPCGData* CreateActorPCGData(AActor* Actor, bool bParseActor = true) const;
	UE_API UPCGData* CreateLandscapePCGData(bool bHeightOnly) const;
	UE_API bool IsLandscapeCachedDataDirty(const UPCGData* Data) const;

	UE_API bool ShouldGenerate(bool bForce, EPCGComponentGenerationTrigger RequestedGenerationTrigger) const;

	/* Internal call that allows to delay a Generate/Cleanup call, chain with dependencies and keep track of the task id created. This task id is also returned. */
	UE_API FPCGTaskId GenerateInternal(bool bForce, uint32 Grid, EPCGComponentGenerationTrigger RequestedGenerationTrigger, const TArray<FPCGTaskId>& Dependencies);

	/* Internal call to create tasks to generate the component. If there is nothing to do, an invalid task id will be returned. Should only be used by the subsystem. */
	UE_API FPCGTaskId CreateGenerateTask(bool bForce, const TArray<FPCGTaskId>& Dependencies);

	/* Internal call to create tasks to cleanup the component. If there is nothing to do, an invalid task id will be returned. Should only be used by the subsystem. */
	UE_API FPCGTaskId CreateCleanupTask(bool bRemoveComponents, const TArray<FPCGTaskId>& Dependencies);

	UE_API void PostProcessGraph(const FBox& InNewBounds, bool bInGenerated, FPCGContext* Context);
	UE_API void CallPostGenerateFunctions(FPCGContext* Context) const;
	UE_API void PostCleanupGraph(bool bRemoveComponents);
	UE_API void OnProcessGraphAborted(bool bQuiet = false, bool bCleanupUnusedResources = true);
	UE_API void CleanupUnusedManagedResources();
	UE_API void ClearGraphGeneratedOutput(bool bClearLoadedPreviewData = false);
	UE_API bool MoveResourcesToNewActor(const FPCGMoveResourceParams& Params, bool bCreateChild);

	/** Called as part of the RerunConstructionScript, just takes resources as-is, assumes current state is empty + the resources have been retargeted if needed */
	UE_API void GetManagedResources(TArray<TObjectPtr<UPCGManagedResource>>& Resources) const;
	UE_API void SetManagedResources(const TArray<TObjectPtr<UPCGManagedResource>>& Resources);

	UE_API void RefreshAfterGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType);
	UE_API void OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType);

	UE_API void OnManagedResourceAdded(UPCGManagedResource* InResource);

#if WITH_EDITOR

public:
	UE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;

private:
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation) override;

protected:
	UE_API virtual TSharedPtr<ITransactionObjectAnnotation> FactoryTransactionAnnotation(const ETransactionAnnotationCreationMode InCreationMode) const override;

private:
	/** Sets up actor, tracking, landscape and graph callbacks */
	UE_API void SetupCallbacksOnCreation();

	/** Returns true if something changed in the tracking. */
	UE_API bool UpdateTrackingCache(TArray<FPCGSelectionKey>* OptionalChangedKeys = nullptr);

	/** Apply a function to all settings that track a given key. */
	UE_API void ApplyToEachSettings(const FPCGSelectionKey& InKey, const TFunctionRef<void(const FPCGSelectionKey&, const FPCGSettingsAndCulling&)> InCallback) const;

	/** Return all the keys tracked by the component (statically and dynamically). */
	UE_API TArray<FPCGSelectionKey> GatherTrackingKeys() const;

	/** Return true if the key is tracked, and if so, bOutIsCulled will contains if the key is culled or not. */
	UE_API bool IsKeyTrackedAndCulled(const FPCGSelectionKey& Key, bool& bOutIsCulled) const;

	/** Compare the temp map to the stored map for dynamic tracking and register/unregister accordingly. If it is a local component, it will push the info to the original. */
	UE_API void UpdateDynamicTracking();

	UE_API void MarkResourcesAsTransientOnLoad();
	UE_API bool DeletePreviewResources();

	static UE_API TArray<TSoftObjectPtr<AActor>> GetManagedActorPaths(AActor* InActor);

	UE_API void UpdateRuntimeGridDescriptorHash();
#endif

	UE_API FBox GetGridBounds(const AActor* InActor) const;

	UPROPERTY(Transient, NonPIEDuplicateTransient)
	mutable TObjectPtr<UPCGData> CachedPCGData = nullptr;

	UPROPERTY(Transient, NonPIEDuplicateTransient)
	mutable TObjectPtr<UPCGData> CachedInputData = nullptr;

	UPROPERTY(Transient, NonPIEDuplicateTransient)
	mutable TObjectPtr<UPCGData> CachedActorData = nullptr;

	UPROPERTY(Transient, NonPIEDuplicateTransient)
	mutable TObjectPtr<UPCGData> CachedLandscapeData = nullptr;

	UPROPERTY(Transient, NonPIEDuplicateTransient)
	mutable TObjectPtr<UPCGData> CachedLandscapeHeightData = nullptr;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TSet<TSoftObjectPtr<AActor>> GeneratedActors_DEPRECATED;
#endif

	UPROPERTY()
	FPCGManagedResourceContainer ManagedResourceContainer;

	// NOTE: This should not be made visible or editable because it will change the way the BP actors are
	// duplicated/setup and might trigger an ensure in the resources.
	UE_DEPRECATED(5.8, "Moved to ManagedResourceContainer")
	UPROPERTY()
	TArray<TObjectPtr<UPCGManagedResource>> GeneratedResources_DEPRECATED;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	FPCGDataCollection LoadedPreviewGeneratedGraphOutput;

	UPROPERTY(Transient, VisibleInstanceOnly, Category = Debug)
	bool bGenerationInProgress = false;

	// In order to be more efficient for tracking when a partitioned PCG Component depends on another partitioned PCG component, keep the accumulated
	// bounds of all the local changes.
	UPROPERTY(Transient)
	FBox LocalChangedBounds = FBox(EForceInit::ForceInit);

	mutable PCG::FLock LocalChangedBoundsLock;
#endif // WITH_EDITORONLY_DATA

	// Flag to indicate whether this component was already generated on load.
	bool bGeneratedOffline = false;

	UPROPERTY(VisibleInstanceOnly, Category = Debug)
	FBox LastGeneratedBounds = FBox(EForceInit::ForceInit);

	UPROPERTY(VisibleInstanceOnly, Category = Debug)
	FPCGDataCollection GeneratedGraphOutput;

	/** If any graph edges cross execution grid sizes, data on the edge is stored / retrieved from this map. */
	UPROPERTY(Transient, NonTransactional, VisibleAnywhere, Category = Debug)
	TMap<FString, FPCGDataCollection> PerPinGeneratedOutput;

	mutable FTransactionallySafeRWLock PerPinGeneratedOutputLock;

	FPCGTaskId CurrentGenerationTask = InvalidPCGTaskId;
	FPCGTaskId CurrentCleanupTask = InvalidPCGTaskId;

#if WITH_EDITOR
	FPCGTaskId CurrentRefreshTask = InvalidPCGTaskId;
#endif // WITH_EDITOR

	UPROPERTY(VisibleAnywhere, Transient, Category = Debug, meta = (EditCondition = false, EditConditionHides))
	bool bIsComponentLocal = false;

	/** Whether procedural ISM components were used/generated in the last execution. */
	UPROPERTY()
	bool bProceduralInstancesInUse = false;

#if WITH_EDITOR
	bool bWasGeneratedThisSession = false;
	FBox LastGeneratedBoundsPriorToUndo = FBox(EForceInit::ForceInit);
#endif

#if WITH_EDITORONLY_DATA
	FPCGChangeTracker ChangeTracker;
#endif

#if PCG_PROFILING_ENABLED
	FPCGGraphExecutionInspection ExecutionInspection;
#endif // PCG_PROFILING_ENABLED

#if WITH_EDITOR
	/** The tracking system will not trigger a generation on this component for these change origins. Populated within the scope
	* of an element. Entries removed when counter is decremented to 0, so empty map means no active ignores.
	*/
	TMap<TObjectKey<UObject>, int32> IgnoredChangeOriginsToCounters;
	mutable FTransactionallySafeRWLock IgnoredChangeOriginsLock;
#endif // WITH_EDITOR

	mutable FTransactionallySafeCriticalSection GeneratedResourcesLock;

	/** Manual override for the enabling actor componentless generation. */
	UPROPERTY(EditAnywhere, Category = "RuntimeGeneration|Advanced", meta = (EditCondition = "GenerationTrigger == EPCGComponentGenerationTrigger::GenerateAtRuntime", EditConditionHides))
	bool bOverrideUseActorComponentlessGeneration = false;

	/** [EXPERIMENTAL] Avoids creating partition actors and local PCG components. Any functionality which depends on a target actor/component are not supported in this mode. */
	UPROPERTY(EditAnywhere, Category = "RuntimeGeneration|Advanced", meta = (EditCondition = "GenerationTrigger == EPCGComponentGenerationTrigger::GenerateAtRuntime && bOverrideUseActorComponentlessGeneration", EditConditionHides))
	bool bUseActorComponentlessGeneration = false;

	// Graph instance
private:
	/** Will set the given graph interface into our owned graph instance. Must not be used on local components.*/
	UE_API void SetGraphInterfaceLocal(UPCGGraphInterface* InGraphInterface);

#if PCG_PROFILING_ENABLED
public:
	mutable PCGUtils::FExtraCapture ExtraCapture;
#endif // PCG_PROFILING_ENABLED

private:
	/** Generalized storage for the execution source (ex. tool data, runtime gen data, manual overrides, etc.) */
	UPROPERTY()
	FPCGSourceDataContainer SourceDataContainer;

	FPCGComponentExecutionState ExecutionState;

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "Use the version that takes uint32 grid size instead")
	UE_API void GenerateLocal(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, EPCGHiGenGrid Grid = EPCGHiGenGrid::Uninitialized, const TArray<FPCGTaskId>& Dependencies = {});
	UE_DEPRECATED(5.8, "Use the version that takes uint32 grid size instead")
	UE_API FPCGTaskId GenerateLocalGetTaskId(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, EPCGHiGenGrid Grid);
	UE_DEPRECATED(5.8, "Use the version that takes uint32 grid size instead")
	UE_API FPCGTaskId GenerateLocalGetTaskId(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, EPCGHiGenGrid Grid, const TArray<FPCGTaskId>& Dependencies);
	UE_DEPRECATED(5.8, "Use GetGenerationGridSize")
	EPCGHiGenGrid GetGenerationGrid() const { return PCGHiGenGrid::GridSizeToGrid(GetGenerationGridSize()); }
	UE_DEPRECATED(5.8, "Use version that takes uint32 grid size instead")
	double GetGenerationRadiusFromGrid(EPCGHiGenGrid Grid) const { return GetGenerationRadiusFromGrid(PCGHiGenGrid::GridToGridSize(Grid)); }
	UE_DEPRECATED(5.8, "Use version that takes uint32 grid size instead")
	double GetCleanupRadiusFromGrid(EPCGHiGenGrid Grid) const { return GetCleanupRadiusFromGrid(PCGHiGenGrid::GridToGridSize(Grid)); }
};

USTRUCT()
struct UE_DEPRECATED(5.8, "Use FPCGTrackedKeyInstanceData instead") FDynamicTrackedKeyInstanceData
{
	GENERATED_BODY();

	UPROPERTY()
	FPCGSelectionKey SelectionKey;

	UPROPERTY()
	TSoftObjectPtr<const UPCGSettings> Settings;

	UPROPERTY()
	bool bValue = false;
};

USTRUCT()
struct FPCGTrackedKeyInstanceData
{
	GENERATED_BODY();

	UPROPERTY()
	FPCGSelectionKey SelectionKey;

	UPROPERTY()
	TSoftObjectPtr<const UPCGSettings> Settings;

	UPROPERTY()
	bool bValue = false;
};

/** Used to store generated resources data during RerunConstructionScripts */
USTRUCT()
struct FPCGComponentInstanceData : public FActorComponentInstanceData
{
	GENERATED_BODY()
public:
	FPCGComponentInstanceData() = default;
	explicit FPCGComponentInstanceData(const UPCGComponent* InSourceComponent);

protected:
	virtual bool ContainsData() const override;
	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	UPROPERTY()
	TObjectPtr<const UPCGComponent> SourceComponent;

#if WITH_EDITORONLY_DATA
	// We could rely on the SourceComponent to re-apply the non editable members but to simplify future code we capture
	// the values in the FPCGComponentInstanceData constructor so we do not depend on the state of the SourceComponent anymore.
	// Editable properties like: CurrentEditingMode need to be captures because in the case of user edits, the SourceComponent will actually be modified and we would lose the original value.

	// UPCGComponent::CurrentEditingMode is a Transient property so we need to capture/apply it ourselves
	UPROPERTY()
	EPCGEditorDirtyMode CurrentEditingMode = EPCGEditorDirtyMode::Normal;
	
	// UPCGComponent::PreviousEditingMode is not a property so we need to capture/apply it ourselves
	UPROPERTY()
	EPCGEditorDirtyMode PreviousEditingMode = EPCGEditorDirtyMode::Normal;

	// UPCGComponent::bForceGenerateOnBPAddedToWorld is not a visible property so we need to capture/apply it ourselves
	UPROPERTY()
	bool bForceGenerateOnBPAddedToWorld = false;

	// UPCGComponent::bGeneratedOffline is not a property so we need to capture/apply it ourselves
	UPROPERTY()
	bool bGeneratedOffline = false;

	// UPCGComponent::DynamicallyTrackedKeysToSettings/StaticallyTrackedKeysToSettings are not properties so we need to capture/apply them ourselves.
	// To avoid having to write custom serialization code the map is converted into an array for serialization.
	UPROPERTY()
	TArray<FPCGTrackedKeyInstanceData> DynamicallyTrackedKeysToSettings;

	UPROPERTY()
	TArray<FPCGTrackedKeyInstanceData> StaticallyTrackedKeysToSettings;
#endif
};

#undef UE_API
