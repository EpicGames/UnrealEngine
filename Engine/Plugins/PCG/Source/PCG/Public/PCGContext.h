// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "PCGNode.h" // IWYU pragma: keep
#include "PCGGraphExecutionStateInterface.h"
#include "Helpers/PCGAsyncState.h"
#include "Templates/SubclassOf.h"
#include "Utils/PCGExtraCapture.h"

#include "StructUtils/StructView.h"
#include "UObject/GCObject.h"
#include "UObject/GarbageCollection.h"

#include "PCGContext.generated.h"

#define UE_API PCG_API

class UPCGBasePointData;
class UPCGComponent;
class FPCGGraphExecutor;
class UPCGComputeGraph;
class UPCGGraphInterface;
class UPCGSettingsInterface;
class UPCGSpatialData;
struct FPCGGetFromCacheParams;
struct FPCGInitializeElementParams;
struct FPCGSettingsOverridableParam;
struct FPCGSourceDataContainer;
struct FPCGStack;
struct FPCGStoreInCacheParams;

extern PCG_API TAutoConsoleVariable<bool> CVarPCGEnablePointArrayData;

namespace PCGContextHelpers
{
	template<typename SettingsType>
	const SettingsType* GetInputSettings(const UPCGNode* Node, const FPCGDataCollection& InputData)
	{
		if (Node && Node->GetSettings())
		{
			return Cast<SettingsType>(InputData.GetSettings(Node->GetSettings()));
		}
		else
		{
			return InputData.GetSettings<SettingsType>();
		}
	}
}

UENUM()
enum class EPCGExecutionPhase : uint8
{
		NotExecuted = 0,
		PrepareData,
		Execute,
		PostExecute,
		Done
};

struct FPCGContext;

struct FPCGContextHandle : public TSharedFromThis<FPCGContextHandle>
{
public:
	FPCGContextHandle(FPCGContext* InContext)
		: Context(InContext)
	{
	}

	UE_API ~FPCGContextHandle();

	FPCGContext* GetContext() { return Context; }
private:
	friend struct FPCGContext;
	FPCGContext* Context = nullptr;
};

/**
 * Blueprint specific FPCGContext handle.
 *
 * This handle is safe to copy and pass around in BP.
 * Prior API was using FPCGContext struct references. BP struct by reference can do actual copies causing issues specifically with multi-threaded access of FPCGContext.
 */
USTRUCT(BlueprintType)
struct FPCGBlueprintContextHandle
{
	GENERATED_BODY()

	TWeakPtr<FPCGContextHandle> Handle;
};

USTRUCT(BlueprintType)
struct FPCGContext
{
	GENERATED_BODY()

	friend class FPCGContextBlueprintScope;
	friend class FPCGGraphExecutor;
	friend struct FPCGGraphActiveTask;
	friend struct FPCGGraphTask;

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPCGContext() = default;
	FPCGContext(const FPCGContext&) = default;
	FPCGContext(FPCGContext&&) = default;
	FPCGContext& operator=(const FPCGContext&) = default;
	FPCGContext& operator=(FPCGContext&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_API virtual ~FPCGContext();

	UE_API void InitFromParams(const FPCGInitializeElementParams& InParams);

	FPCGDataCollection InputData;
	FPCGDataCollection OutputData;

	TWeakInterfacePtr<IPCGGraphExecutionSource> ExecutionSource = nullptr;

	UE_DEPRECATED(5.6, "Use ExecutionSource instead")
	TWeakObjectPtr<UPCGComponent> SourceComponent = nullptr;

	/** Used to track when data transformations (such as CPU readback) occurred on the input data collection as part of the element prepare data. */
	bool bInputDataModified = false;

	FPCGAsyncState AsyncState;
	FPCGCrc DependenciesCrc;

	// TODO: replace this by a better identification mechanism
	const UPCGNode* Node = nullptr;
	FPCGTaskId TaskId = InvalidPCGTaskId;
	FPCGTaskId CompiledTaskId = InvalidPCGTaskId;
	bool bIsPaused = false;
	TSet<FPCGTaskId> DynamicDependencies;

	EPCGExecutionPhase CurrentPhase = EPCGExecutionPhase::NotExecuted;
	/** Tracks whether the input conversion step has been done (inside of the PrepareData phase). */
	bool bConvertInputsIfNeededDone : 1 = false;

	TArray<TPair<FPCGDataCollection, FPCGDataCollection>> CachedInputToOutputInternalResults;

	/** Get the current call stack. */
	const FPCGStack* GetStack() const { return StackHandle.GetStack(); }

	UE_DEPRECATED(5.6, "Access the stack via GetStack()")
	const FPCGStack* Stack = nullptr;

	bool CanExecuteOnlyOnMainThread() const { return bMustRunOnMainThread; }
	void RequestToRunOnMainThread() { bMustRunOnMainThread = true; }

	UE_API const UPCGSettingsInterface* GetInputSettingsInterface() const;
	
	// After initializing the context, we can call this method to prepare for parameter override
	// It will create a copy of the original settings if there is indeed a possible override.
	UE_API void InitializeSettings(bool bSkipPostLoad = false);

	static UE_API bool IsInitializingSettings();

	// If we any any parameter override, it will read from the params and override matching values
	// in the settings copy.
	UE_API void OverrideSettings();

	// Returns true if the given property has been overriden
	UE_API bool IsValueOverriden(const FName PropertyName) const;
	UE_API bool IsValueOverriden(const TArrayView<const FName>& PropertyNameChain) const;

	// Return the seed, possibly overriden by params, and combined with the source component (if any).
	UE_API int GetSeed() const;

	// Allows creating a new object safely inside the execution of a PCG Element, this object will also get tracked properly by the context
	template<class T, typename... Args>
	static T* NewObject_AnyThread(FPCGContext* Context, Args&&... InArgs)
	{
		ensure(Context != nullptr || IsInGameThread());
		return Context ? Context->NewObject_AnyThread_Impl<T>(std::forward<Args>(InArgs)...) : ::NewObject<T>(std::forward<Args>(InArgs)...);
	}

	// Temporary so that we can toggle between UPCGPointData and UPCGPointArrayData with a CVar
	static UE_API UPCGBasePointData* NewPointData_AnyThread(FPCGContext* Context);
	// Like NewPointData_AnyThread, gets the class used for point data depending on the same CVar
	static UE_API TSubclassOf<UPCGBasePointData> GetDefaultPointDataClass();

	bool ContainsAsyncObject(const UObject* InAsyncObject) { return AsyncObjects.Contains(InAsyncObject); }

	// Return the settings casted in the wanted type.
	// If there is any override, those settings will already contains all the overriden values.
	template<typename SettingsType>
	const SettingsType* GetInputSettings() const
	{
		return SettingsWithOverride ? Cast<SettingsType>(SettingsWithOverride) : GetOriginalSettings<SettingsType>();
	}

	// Return the mutable settings casted in the wanted type.
	// If there is no override, it returns null.
	template<typename SettingsType>
	SettingsType* GetMutableInputSettings()
	{
		return SettingsWithOverride ? Cast<SettingsType>(SettingsWithOverride) : nullptr;
	}

	UE_API FPCGTaskId GetGraphExecutionTaskId() const;

	UE_API FString GetTaskName() const;

	UE_DEPRECATED(5.6, "Use GetExecutionSourceName() instead")
	UE_API FString GetComponentName() const;

	UE_API FString GetExecutionSourceName() const;

	bool ShouldStop() const { return AsyncState.ShouldStop(); }

	// Will eventually be deprecated in favor of GetExecutionTarget/GetTypedExecutionTarget
	UE_API AActor* GetTargetActor(const UPCGSpatialData* InSpatialData) const;

	UE_API UObject* GetExecutionTarget() const;

	template<class TargetType>
	TargetType* GetTypedExecutionTarget() const
	{
		return Cast<TargetType>(GetExecutionTarget());
	}

	/** Time slicing is not enabled by default. */
	virtual bool TimeSliceIsEnabled() const { return false; }

	/** Is this a context for the compute graph element. */
	virtual bool IsComputeContext() const { return false; }

#if WITH_EDITOR
	/** Log warnings and errors to be displayed on node in graph editor. */
	UE_API void LogVisual(ELogVerbosity::Type InVerbosity, const FText& InMessage) const;

	/** True if any issues were logged during last execution. */
	UE_API bool HasVisualLogs() const;

	/** Whether the graph for the current frame in the execution stack is selected for debugging in the graph editor. */
	UE_API bool IsExecutingGraphInspected() const;

	/** Returns the proper actor to do debug visualization. */
	UE_API AActor* GetDebugDisplayActor() const;
#endif // WITH_EDITOR

	/** Gathers references to objects to prevent them from being garbage collected. Extend resources to be collected with AddExtractStructReferencedObjects. */
	// Implementation note: this is NOT the same as a struct using WithAddStructReferencedObjects since we are not holding contexts
	//  in properties in any case. This will be called from the graph executor when needed and is implemented to look like normal reference traversal.
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector);

	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) {}

	/** Caution: most use cases should use GetInputSettings, because they contain the overridden values. Use this one if you really need to get the original pointer. */
	template<typename SettingsType>
	const SettingsType* GetOriginalSettings() const 
	{
		return PCGContextHelpers::GetInputSettings<SettingsType>(Node, InputData);
	}

	// This is not thread safe, make sure it is not called concurrently on the same context
	TWeakPtr<FPCGContextHandle> GetOrCreateHandle()
	{
		if (!Handle)
		{
			Handle = MakeShared<FPCGContextHandle>(this);
		}

		return Handle.ToWeakPtr();
	}

	static UE_API void Release(FPCGContext* InContext);

	template <typename ContextType>
	UE_DEPRECATED(5.6, "Use FSharedContext instead")
	static ContextType* GetContextFromHandle(TWeakPtr<FPCGContextHandle> WeakHandle)
	{
		if (TSharedPtr<FPCGContextHandle> SharedHandle = WeakHandle.Pin())
		{
			return (ContextType*)SharedHandle->GetContext();
		}

		return nullptr;
	}

	template <typename ContextType>
	class FSharedContext
	{
	public:
		FSharedContext(TWeakPtr<FPCGContextHandle> WeakHandle)
		{
			SharedHandle = WeakHandle.Pin();
		}
		ContextType* Get() const { return SharedHandle.IsValid() ? (ContextType*)SharedHandle->GetContext() : nullptr; }

	private:
		TSharedPtr<FPCGContextHandle> SharedHandle;
	};

	UE_API FPCGTaskId ScheduleGraph(const FPCGScheduleGraphParams& InParams);
	UE_API FPCGTaskId ScheduleGeneric(const FPCGScheduleGenericParams& InParams);

	UE_API bool GetOutputData(FPCGTaskId InTaskId, FPCGDataCollection& OutData);
	UE_API void ClearOutputData(FPCGTaskId InTaskId);

	UE_API bool IsCacheEnabled();
	UE_API void StoreInCache(const FPCGStoreInCacheParams& Params, const FPCGDataCollection& InOutput);
	UE_API bool GetFromCache(const FPCGGetFromCacheParams& Params, FPCGDataCollection& OutOutput) const;

	/**
	 * Get the context setup for this thread, can be null. It is set within an element execution, but it is the responsibility of the
	 * implementer to set this context (with FPCGContextScope) when starting new async tasks.
	 */
	UE_API static FPCGContext* GetContextForThread();

	UE_API FPCGSourceDataContainer* GetSourceDataContainer();
	UE_API const FPCGSourceDataContainer* GetSourceDataContainer() const;

protected:
	virtual UObject* GetExternalContainerForOverridableParam(const FPCGSettingsOverridableParam& InParam) { return nullptr; }
	virtual FStructView GetExternalStructContainerForOverridableParam(const FPCGSettingsOverridableParam& InParam) { return {}; }

	UE_DEPRECATED(5.8, "You should use GetExternalStructContainerForOverridableParam for structs overrides.")
	virtual void* GetUnsafeExternalContainerForOverridableParam(const FPCGSettingsOverridableParam& InParam) { return nullptr; }
	
	UE_API void InitializeGraphExecutor(FPCGContext* InContext);

	TWeakPtr<FPCGGraphExecutor> GetGraphExecutor() { return GraphExecutor; }

private:
	template<class T, typename... Args>
	T* NewObject_AnyThread_Impl(Args&&... InArgs)
	{
		if (!IsInGameThread())
		{
			ensure(!AsyncState.bIsRunningOnMainThread);
			T* Object = nullptr;
			{
				FGCScopeGuard Scope;
				Object = ::NewObject<T>(std::forward<Args>(InArgs)...);
			}
			check(Object);
			AsyncObjects.Add(Object);
			return Object;
		}

		return ::NewObject<T>(std::forward<Args>(InArgs)...);
	}

	// Copy of the settings that will be used to apply overrides.
	TObjectPtr<UPCGSettings> SettingsWithOverride = nullptr;

	// List of params that were in effect overriden
	TArray<const FPCGSettingsOverridableParam*> OverriddenParams;

	// If the element must run on the main thread (like if the settings need to be overridden on the main thread, because we have to load objects))
	bool bMustRunOnMainThread = false;

	// List of objects created by the PCG Elements, we need to track them so we can remove their Async flags when storing results on main thread
	// so that they can be considered as existing on the main thread (and get properly GCed)
	TSet<TObjectPtr<UObject>> AsyncObjects;

	// Lazy initialized shared handle pointer that can be used in lambda captures to test if Context is still valid before accessing it
	TSharedPtr<FPCGContextHandle> Handle;

	// Graph Executor that scheduled the task
	TWeakPtr<FPCGGraphExecutor> GraphExecutor;

	/** Handle for the current call stack. */
	FPCGStackHandle StackHandle;

#if PCG_PROFILING_ENABLED
	friend class PCGUtils::FExtraCapture;
	PCGUtils::FCallTime Timer;
#endif
};

/**
 * Allow to set the current context for the current thread. The context can then be queryable with "FPCGContext::GetContextForThread".
 * Scopes can be nested, as we store the previous context to restore it on scope destruction.
 * Be very careful in the scope nesting, scopes need to be destroyed in order (ScopeA ctor -> ScopeB ctor -> ScopeB dtor -> ScopeA dtor).
 */
struct FPCGContextScope
{
	PCG_API FPCGContextScope(FPCGContext* InContext);
	PCG_API ~FPCGContextScope();

	// Non-copyable, non-moveable
	FPCGContextScope(const FPCGContextScope&) = delete;
	FPCGContextScope(FPCGContextScope&&) = delete;
	FPCGContextScope& operator=(const FPCGContextScope&) = delete;
	FPCGContextScope& operator=(FPCGContextScope&&) = delete;
	
	// Keep this context to have some verification check that scopes are not destroyed in the wrong order.
	FPCGContext* ThisContext = nullptr;
	FPCGContext* PreviousContext = nullptr;
};

#undef UE_API
