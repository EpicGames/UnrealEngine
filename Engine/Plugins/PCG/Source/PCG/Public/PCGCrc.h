// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Crc.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"

#include "PCGCrc.generated.h"

/** Crc with valid flag and helper functionality. */
USTRUCT()
struct FPCGCrc
{
	GENERATED_BODY()

	/** Creates an invalid Crc. */
	FPCGCrc() = default;

	/** Initializes a valid Crc set to InValue. */
	explicit FPCGCrc(uint32 InValue)
		: Value(InValue)
		, bValid(true)
	{
	}

	bool IsValid() const { return bValid; }

	uint32 GetValue() const { return Value; }

	/** Combines another Crc into this Crc to chain them. */
	void Combine(const FPCGCrc& InOtherCrc)
	{
#if WITH_EDITOR
		ensure(IsValid() && InOtherCrc.IsValid());
#endif

		Value = HashCombineFast(Value, InOtherCrc.Value);
	}

	/** Combines another Crc value into this Crc to chain them. */
	void Combine(uint32 InOtherCrcValue)
	{
#if WITH_EDITOR
		ensure(IsValid());
#endif

		Value = HashCombineFast(Value, InOtherCrcValue);
	}

	/** Compares Crc. This and other Crc must be valid. */
	bool operator==(const FPCGCrc& InOtherCrc) const
	{
#if WITH_EDITOR
		ensure(IsValid() && InOtherCrc.IsValid());
#endif

		return IsValid() && InOtherCrc.IsValid() && Value == InOtherCrc.Value;
	}

private:
	/** Crc32 value. */
	UPROPERTY(VisibleAnywhere, Category = Crc)
	uint32 Value = 0;

	UPROPERTY(VisibleAnywhere, Category = Crc)
	bool bValid = false;
};
