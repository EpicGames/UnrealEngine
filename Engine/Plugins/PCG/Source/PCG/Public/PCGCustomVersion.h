// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

struct FGuid;

// Custom serialization version for assets/classes in the PCG plugin
struct FPCGCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,

		// Split projection nodes inputs to separate source edges and target edge
		SplitProjectionNodeInputs = 1,
		
		MoveSelfPruningParamsOffFirstPin = 2,

		MoveParamsOffFirstPinDensityNodes = 3,

		// Split samplers to give a sampling shape and a bounding shape inputs
		SplitSamplerNodesInputs = 4,

		MovePointFilterParamsOffFirstPin = 5,

		// Add param pin for all nodes that have override and were using the default input pin.
		AddParamPinToOverridableNodes = 6,

		// Sampling shape and bounding shape inputs.
		SplitVolumeSamplerNodeInputs = 7,

		// Renamed spline input and added bounding shape input.
		SplineSamplerUpdatedNodeInputs = 8,

		// Renamed params to override.
		RenameDefaultParamsToOverride = 9,

		// Behavior change for SplineSampler which now defaults to being bounded
		SplineSamplerBoundedByDefault = 10,

		// StaticMeshSpawner now defaults to modify point bounds based on StaticMesh bounds
		StaticMeshSpawnerApplyMeshBoundsToPointsByDefault = 11,

		// Update of Input Selectors. Previous versions should default on @LastCreated
		UpdateAttributePropertyInputSelector = 12,

		// Difference node now iterates on the source pin and unions the differences pin
		DifferenceNodeIterateOnSourceAndUnionDifferences = 13,

		// Update AddAttribute with selectors
		UpdateAddAttributeWithSelectors = 14,

		// Update TransferAttribute with selectors
		UpdateTransferAttributeWithSelectors = 15,

		// Removed by-default pins on input node. Note, this breaks cooked binary compatibility
		UpdateInputOutputNodesDefaults = 16,

		// Introduced the concept of pin usage and graph defaults around loops, which changed the default behavior otherwise
		UpdateGraphSettingsLoopPins = 17,

		// Added 'out' filter pins on filter by tag & by type
		UpdateFilterNodeOutputPins = 18,

		// Added 'bComponentsMustOverlapSelf' to GetActorData when the mode collects PCG component data
		GetPCGComponentDataMustOverlapSourceComponentByDefault = 19,

		// Added dynamic tracking to the PCG component serialization
		DynamicTrackingKeysSerializedInComponent = 20,

		// Supporting partitioned components in non-partitioned levels
		SupportPartitionedComponentsInNonPartitionedLevels = 21,

		// New gate for new data, so any node that has a non Point pin don't do any ToPointData by default.
		NoMoreSpatialDataConversionToPointDataByDefaultOnNonPointPins = 22,

		// Attributes and tags can now contain spaces and will no longer be parsed by spaces.
		AttributesAndTagsCanContainSpaces = 23,

		// Multi Domain Metadata
		MultiLevelMetadata = 24,

		// Refactor of the Attribute Property selector to deprecate point properties and support any property.
		AttributePropertySelectorDeprecatePointProperties = 25,

		// Changed Attribute rename to support selectors
		AttributeRenameSupportToSelectors = 26,

		// Removed spaces from some polygon 2d node pin labels
		RemovedSpacesInPolygonPinLabels = 27,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FPCGCustomVersion() {}
};
