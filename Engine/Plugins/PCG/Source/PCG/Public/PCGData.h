// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGCrc.h"
#include "Data/PCGDataPtrWrapper.h"
#include "Data/Registry/PCGDataType.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataCommon.h"

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SubclassOf.h"

#include "PCGData.generated.h"

#define UE_API PCG_API

class FArchiveCrc32;
class UPCGMetadata;
class UPCGNode;
class UPCGParamData;
class UPCGSettings;
class UPCGSettingsInterface;
class UPCGSpatialData;
struct FPCGContext;

UENUM(meta = (Bitflags))
enum class EPCGDataUsage : uint8
{
	None = 0,
	GraphExecutorTaskOutput = 1 << 0,
	ComponentOutputData = 1 << 1,
	ComponentPerPinOutputData = 1 << 2,
	ComponentInspectionData = 1 << 3,
};
ENUM_CLASS_FLAGS(EPCGDataUsage);

namespace PCGDataConstants
{
	extern PCG_API const FName DataDomainName;
	extern PCG_API const FName DefaultDomainName;
}

/**
* Base class for any "data" class in the PCG framework.
* This is an intentionally vague base class so we can have the required
* flexibility to pass in various concrete data types, settings, and more.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGData : public UObject
{
	GENERATED_BODY()

public:
	UE_API UPCGData(const FObjectInitializer& ObjectInitializer);

	//~Begin UObject Interface
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~End UObject Interface

	// @todo_pcg To be deprecated when we switch to the new type
	//UE_DEPRECATED(5.7, "Use the new GetDataTypeId")
	virtual EPCGDataType GetDataType() const { return EPCGDataType::Any; }

	using TypeInfo = FPCGDataTypeInfo;
	virtual const FPCGDataTypeBaseId& GetDataTypeId() const { return FPCGDataTypeInfo::AsId(); }

	/** Returns a Crc for this and any connected data. */
	UE_API FPCGCrc GetOrComputeCrc(bool bFullDataCrc) const;

	/** Executes a lambda over all connected data objects. */
	UE_API virtual void VisitDataNetwork(TFunctionRef<void(const UPCGData*)> Action) const;

	/** Whether this data can be serialized. */
	virtual bool CanBeSerialized() const { return true; }

	/** Whether the data can be placed in the graph cache. */
	virtual bool IsCacheable() const { return true; }

	virtual void ReleaseTransientResources(const TCHAR* InReason = nullptr) {}

	/** If this data is a proxy, returns the underlying data type, otherwise returns this data type. */
	// @todo_pcg To be deprecated when we switch to the new type
	// UE_DEPRECATED(5.7, "Use the new GetUnderlyingDataTypeId")
	virtual EPCGDataType GetUnderlyingDataType() const
	{
		return GetUnderlyingDataTypeId().AsLegacyType();
	}

	virtual FPCGDataTypeBaseId GetUnderlyingDataTypeId() const { return GetDataTypeId(); }

	/** Unique ID for this object instance. */
	UPROPERTY(Transient)
	uint64 UID = 0;

	/** CRC for this object instance. */
	mutable FPCGCrc Crc;

	/** Returns true if the data has a cached last selector. Used to know how to convert `@Last` in an attribute selector. */
	virtual bool HasCachedLastSelector() const { return false; }

	/** Returns the cached last selector. Used to know how to convert `@Last` in an attribute selector. */
	virtual FPCGAttributePropertyInputSelector GetCachedLastSelector() const { return FPCGAttributePropertyInputSelector{}; }

	/** Set the last selector used to modify an attribute. */
	virtual void SetLastSelector(const FPCGAttributePropertySelector& InSelector) {};

	UE_DEPRECATED(5.5, "Call/Implement version with FPCGContext parameter")
	UE_API virtual UPCGData* DuplicateData(bool bInitializeMetadata = true) const;

	/** Return a copy of the data, with Metadata inheritence for spatial data. */
	UE_API virtual UPCGData* DuplicateData(FPCGContext* Context, bool bInitializeMetadata = true) const;

	// ~Begin UObject interface
	virtual void PostDuplicate(bool bDuplicateForPIE) override { InitUID(); }
	UE_API virtual void PostEditImport() override;
	// ~End UObject interface

	// Metadata ops, to be implemented if data supports Metadata
	UFUNCTION(BlueprintCallable, Category = Metadata)
	virtual const UPCGMetadata* ConstMetadata() const { return Metadata; }

	UFUNCTION(BlueprintCallable, Category = Metadata)
	virtual UPCGMetadata* MutableMetadata() { return Metadata; }

	UE_API virtual void Flatten();

	/************** Metadata domain specific functions **************/

	/** Returns the default domain for this data. Any data that has other default domain should override this method. */
	virtual FPCGMetadataDomainID GetDefaultMetadataDomainID() const { return PCGMetadataDomainID::Data; }

	/** Returns all the supported domain for this data. Any data that has more than one domain should override this method. */
	virtual TArray<FPCGMetadataDomainID> GetAllSupportedMetadataDomainIDs() const { return {PCGMetadataDomainID::Data}; }

	/** Returns true if the domain is supported by this data's metadata. */
	UE_API bool IsSupportedMetadataDomainID(const FPCGMetadataDomainID& InDomainID) const;

	/** Return the associated domain ID for a given domain name in the selector. Must return "Invalid" if it is not supported. Each data is responsible to override this for every supported domain.*/
	UE_API virtual FPCGMetadataDomainID GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const;

	/** Write the domain name into the selector associated with this domain ID. Each data is responsible to override this for every supported domain. */
	UE_API virtual bool SetDomainFromDomainID(const FPCGMetadataDomainID& InDomainID, FPCGAttributePropertySelector& InOutSelector) const;

	/** Check if the domain supports multi entries. If not, an add entry won't work. */
	UE_API virtual bool MetadataDomainSupportsMultiEntries(const FPCGMetadataDomainID& InDomainID) const;

	/** Check if parenting is possible for this domain, if not, any initialization will result to a copy. */
	UE_API virtual bool MetadataDomainSupportsParenting(const FPCGMetadataDomainID& InDomainID) const;

	// Not accessible through blueprint to make sure the constness is preserved.
	UPROPERTY()
	TObjectPtr<UPCGMetadata> Metadata = nullptr;

	UE_API void MarkUsage(EPCGDataUsage InUsage) const;
	UE_API void ClearUsage(EPCGDataUsage InUsage) const;
	UE_API bool HasUsage(EPCGDataUsage InUsage) const;
	UE_API void IncCollectionRefCount() const;
	UE_API void DecCollectionRefCount() const;
	int32 GetCollectionRefCount() const { return CollectionRefCount; }

protected:
	/** Computes Crc for this and any connected data. */
	UE_API virtual FPCGCrc ComputeCrc(bool bFullDataCrc) const;

	/** Adds this data to Crc. Derived classes should override this and add all required data to CRC, or fall back to adding the unique object instance UID. */
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const;

	/** Whether the data distinguishes between regular crc and full crc */
	virtual bool SupportsFullDataCrc() const { return false; }

	UE_API void AddUIDToCrc(FArchiveCrc32& Ar) const;

private:
	UE_API void InitUID();
	
	/** If the Crc cache contains a full data Crc, if data type supports it */
	mutable bool bIsFullDataCrc = false;

	/** Serves unique ID values to instances of this object. */
	static inline std::atomic<uint64> UIDCounter{ 1 };

	/** Usage flags to record any current usages of this data. Used to help manage lifetimes of any internal transient resources. */
	mutable std::atomic<__underlying_type(EPCGDataUsage)> Usage{ 0 };
	mutable std::atomic<int32> CollectionRefCount{ 0 };

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "No longer required")
	virtual bool HoldsTransientResources() const { return false; }
};

USTRUCT(BlueprintType, meta = (HasNativeBreak = "/Script/PCG.PCGDataFunctionLibrary.BreakTaggedData", HasNativeMake = "/Script/PCG.PCGDataFunctionLibrary.MakeTaggedData"))
struct FPCGTaggedData
{
	GENERATED_BODY()

	/** Wraps a TObjectPtr<const UPCGData> so we can track ref counting easier. */
	UPROPERTY(VisibleAnywhere, Category = Data, meta = (ShowOnlyInnerProperties))
	FPCGDataPtrWrapper Data;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data)
	TSet<FString> Tags;

	/** The label of the pin that this data was either emitted from or received on. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data)
	FName Pin = NAME_None;

	// Special flag for data that are forwarded to other nodes, but without a pin. Useful for internal data.
	UPROPERTY()
	bool bPinlessData = false;
	
	/**
	* Special flag to be modified by execution when a data is used multiple times (in this node or other nodes),
	* to enable optimization when they are not. Always assume that it is true by default.
	*/
	UPROPERTY(BlueprintReadOnly, Transient, Category = Data)
	bool bIsUsedMultipleTimes = true;

#if !UE_BUILD_SHIPPING
	// For debug tracking.
	TWeakObjectPtr<const UPCGNode> OriginatingNode;
#endif // !UE_BUILD_SHIPPING

	// Used to re-order elements when using per-data caching
	int32 OriginalIndex = INDEX_NONE;

	UE_API bool operator==(const FPCGTaggedData& Other) const;
	UE_API bool operator!=(const FPCGTaggedData& Other) const;

	UE_API FPCGCrc ComputeCrc(bool bFullDataCrc) const;
};

USTRUCT(BlueprintType)
struct FPCGDataCollection
{
	GENERATED_BODY()

	/** Returns all spatial data in the collection */
	UE_DEPRECATED(5.6, "Use GetAllSpatialInputs for only spatial inputs, or use GetAllInputs.")
	UE_API TArray<FPCGTaggedData> GetInputs() const;

	/** Returns all spatial data in the collection. */
	UE_API TArray<FPCGTaggedData> GetAllSpatialInputs() const;

	/** Returns all inputs in the collection. */
	const TArray<FPCGTaggedData>& GetAllInputs() const { return TaggedData; }

	/** Returns all data on a given pin. */
	UE_API TArray<FPCGTaggedData> GetInputsByPin(const FName& InPinLabel) const;
	/** Returns all spatial data on a given pin */
	UE_API TArray<FPCGTaggedData> GetSpatialInputsByPin(const FName& InPinLabel) const;

	/** Returns all data and corresponding cached data CRCs for a given pin. */
	template<typename AllocatorType1, typename AllocatorType2>
	void GetInputsAndCrcsByPin(const FName& InPinLabel, TArray<FPCGTaggedData, AllocatorType1>& OutData, TArray<FPCGCrc, AllocatorType2>& OutDataCrcs) const
	{
		if (!DataCrcs.IsEmpty() && !ensure(TaggedData.Num() == DataCrcs.Num()))
		{
			// CRCs are not up to date. Error recovery - add 0 CRCs.
			const_cast<TArray<FPCGCrc>&>(DataCrcs).SetNumZeroed(TaggedData.Num());
		}

		for (int I = 0; I < TaggedData.Num(); ++I)
		{
			if (ensure(TaggedData[I].Data) && TaggedData[I].Pin == InPinLabel)
			{
				OutData.Add(TaggedData[I]);

				if (!DataCrcs.IsEmpty())
				{
					OutDataCrcs.Add(DataCrcs[I]);
				}
			}
		}
	}

	/** Gets number of data items on a given pin */
	UE_API int32 GetInputCountByPin(const FName& InPinLabel) const;
	/** Gets number of spatial data items on a given pin */
	UE_API int32 GetSpatialInputCountByPin(const FName& InPinLabel) const;
	/** Returns spatial union of all data on a given pin, returns null if no such data exists. bOutUnionDataCreated indicates if new data created that may need rooting. */
	UE_API const UPCGSpatialData* GetSpatialUnionOfInputsByPin(FPCGContext* InContext, const FName& InPinLabel, bool& bOutUnionDataCreated) const;

	UE_DEPRECATED(5.5, "Use version with FPCGContext param")
	UE_API const UPCGSpatialData* GetSpatialUnionOfInputsByPin(const FName& InPinLabel, bool& bOutUnionDataCreated) const;

	/** Returns all spatial data in the collection with the given tag */
	UE_API TArray<FPCGTaggedData> GetTaggedInputs(const FString& InTag) const;
	/** Returns all settings in the collection */
	UE_API TArray<FPCGTaggedData> GetAllSettings() const;
	/** Returns all params in the collection */
	UE_API TArray<FPCGTaggedData> GetAllParams() const;
	/** Returns all params in the collection with a given tag */
	UE_API TArray<FPCGTaggedData> GetTaggedParams(const FString& InTag) const;
	/** Returns all params on a given pin */
	UE_API TArray<FPCGTaggedData> GetParamsByPin(const FName& InPinLabel) const;

	/** Returns all data in the collection with the given tag and given type */
	template <typename PCGDataType>
	TArray<FPCGTaggedData> GetTaggedTypedInputs(const FString& InTag) const;

	// Only used as a temporary solution for old graph with nodes that didn't have params pins.
	// Should NOT be used with new nodes.
	UE_DEPRECATED(5.4, "Was not supposed to be used anyway, you should query the data per pin using GetParamsByPin")
	UE_API UPCGParamData* GetParamsWithDeprecation(const UPCGNode* Node) const;

	/** Returns the first/only param found on the default params pin */
	UE_API UPCGParamData* GetFirstParamsOnParamsPin() const;

	UE_API const UPCGSettingsInterface* GetSettingsInterface() const;
	UE_API const UPCGSettingsInterface* GetSettingsInterface(const UPCGSettingsInterface* InDefaultSettingsInterface) const;

	/** Memory size calculation. Forward call to the data objects in the collection. */
	UE_API void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const;

	template<typename SettingsType>
	const SettingsType* GetSettings() const;

	UE_API const UPCGSettings* GetSettings(const UPCGSettings* InDefaultSettings) const;

	UE_API bool operator==(const FPCGDataCollection& Other) const;
	UE_API bool operator!=(const FPCGDataCollection& Other) const;
	UE_API FPCGDataCollection& operator+=(const FPCGDataCollection& Other);
	UE_API void AddReferences(FReferenceCollector& Collector);

	/** Computes CRCs for all data items. */
	UE_API void ComputeCrcs(bool bFullDataCrc);

	/** Add data and CRC to collection. */
	UE_API void AddData(const FPCGTaggedData& InData, const FPCGCrc& InDataCrc);
	/** Add data and CRCs to collection. */
	UE_API void AddData(const TConstArrayView<FPCGTaggedData>& InData, const TConstArrayView<FPCGCrc>& InDataCrcs);
	/** Add data and CRCs to collection with pin label combined into the CRC. */
	UE_API void AddDataForPin(const TConstArrayView<FPCGTaggedData>& InData, const TConstArrayView<FPCGCrc>& InDataCrcs, uint32 InputPinLabelCrc);

	/** Cleans up the collection, but does not unroot any previously rooted data. */
	UE_API void Reset();

	/** Strips all empty point data from the collection. */
	UE_API int32 StripEmptyPointData();

	void MarkUsage(EPCGDataUsage InUsage) const
	{
		for (const FPCGTaggedData& Data : TaggedData)
		{
			if (Data.Data)
			{
				Data.Data->MarkUsage(InUsage);
			}
		}
	}

	void ClearUsage(EPCGDataUsage InUsage) const
	{
		for (const FPCGTaggedData& Data : TaggedData)
		{
			if (Data.Data)
			{
				Data.Data->ClearUsage(InUsage);
			}
		}
	}

	bool HasUsage(EPCGDataUsage InUsage) const
	{
		for (const FPCGTaggedData& Data : TaggedData)
		{
			if (Data.Data && Data.Data->HasUsage(InUsage))
			{
				return true;
			}
		}

		return false;
	}

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data)
	TArray<FPCGTaggedData> TaggedData;

	/** Deprecated - Will be removed in 5.4 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data)
	bool bCancelExecutionOnEmpty = false;

	/** This flag is used to cancel further computation or for the debug/isolate feature */
	bool bCancelExecution = false;

	/** Per-data CRC which will capture tags, data, output pin and in some cases input pin too. */
	TArray<FPCGCrc> DataCrcs;

	/** After the task is complete, bit j is set if output pin index j is deactivated. Stored here so that it can be retrieved from the cache. */
	uint64 InactiveOutputPinBitmask = 0;
};

template<typename SettingsType>
inline const SettingsType* FPCGDataCollection::GetSettings() const
{
	const FPCGTaggedData* MatchingData = TaggedData.FindByPredicate([](const FPCGTaggedData& Data) {
		return Cast<const SettingsType>(Data.Data) != nullptr;
		});

	return MatchingData ? Cast<const SettingsType>(MatchingData->Data) : nullptr;
}

template <typename PCGDataType>
inline TArray<FPCGTaggedData> FPCGDataCollection::GetTaggedTypedInputs(const FString& InTag) const
{
	return TaggedData.FilterByPredicate([&InTag](const FPCGTaggedData& Data) {
		return Data.Tags.Contains(InTag) && Cast<PCGDataType>(Data.Data);
	});
}

UCLASS(MinimalAPI)
class UPCGDataFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Gets all inputs of the given class type, returning matching tagged data in the OutTaggedData value too */
	UFUNCTION(BlueprintCallable, Category = Data, meta = (ScriptMethod, DeterminesOutputType = "InDataTypeClass"))
	static UE_API TArray<UPCGData*> GetTypedInputs(const FPCGDataCollection& InCollection, TArray<FPCGTaggedData>& OutTaggedData, TSubclassOf<UPCGData> InDataTypeClass = {});

	/** Gets all inputs of the given class type and on the given pin, returning matching tagged data in the OutTaggedData value too */
	UFUNCTION(BlueprintCallable, Category = Data, meta = (ScriptMethod, DeterminesOutputType = "InDataTypeClass"))
	static UE_API TArray<UPCGData*> GetTypedInputsByPin(const FPCGDataCollection& InCollection, const FPCGPinProperties& InPin, TArray<FPCGTaggedData>& OutTaggedData, TSubclassOf<UPCGData> InDataTypeClass = {});

	/** Gets all inputs of the given class type and on the given pin label, returning matching tagged data in the OutTaggedData value too */
	UFUNCTION(BlueprintCallable, Category = Data, meta = (ScriptMethod, DeterminesOutputType = "InDataTypeClass"))
	static UE_API TArray<UPCGData*> GetTypedInputsByPinLabel(const FPCGDataCollection& InCollection, FName InPinLabel, TArray<FPCGTaggedData>& OutTaggedData, TSubclassOf<UPCGData> InDataTypeClass = {});

	/** Gets all inputs of the given class type and having the provided tag, returning matching tagged data in the OutTaggedData value too */
	UFUNCTION(BlueprintCallable, Category = Data, meta = (ScriptMethod, DeterminesOutputType = "InDataTypeClass"))
	static UE_API TArray<UPCGData*> GetTypedInputsByTag(const FPCGDataCollection& InCollection, const FString& InTag, TArray<FPCGTaggedData>& OutTaggedData, TSubclassOf<UPCGData> InDataTypeClass = {});

	/** Adds a data object to a given collection, simpler usage than making a PCGTaggedData object. InTags can be empty. */
	UFUNCTION(BlueprintCallable, Category = Data, meta = (ScriptMethod, AutoCreateRefTerm = "InTags"))
	static UE_API void AddToCollection(UPARAM(ref) FPCGDataCollection& InCollection, const UPCGData* InData, FName InPinLabel, TArray<FString> InTags);

	// Blueprint methods to support interaction with FPCGDataCollection
	UFUNCTION(BlueprintCallable, Category = Data, meta = (DisplayName="Get All Spatial Inputs"))
	static UE_API TArray<FPCGTaggedData> GetInputs(const FPCGDataCollection& InCollection);

	UFUNCTION(BlueprintCallable, Category = Data)
	static UE_API TArray<FPCGTaggedData> GetInputsByPinLabel(const FPCGDataCollection& InCollection, const FName InPinLabel);

	UFUNCTION(BlueprintCallable, Category = Data)
	static UE_API TArray<FPCGTaggedData> GetInputsByTag(const FPCGDataCollection& InCollection, const FString& InTag);

	UFUNCTION(BlueprintCallable, Category = Data)
	static UE_API TArray<FPCGTaggedData> GetParams(const FPCGDataCollection& InCollection);

	UFUNCTION(BlueprintCallable, Category = Data)
	static UE_API TArray<FPCGTaggedData> GetParamsByPinLabel(const FPCGDataCollection& InCollection, const FName InPinLabel);

	UFUNCTION(BlueprintCallable, Category = Data)
	static UE_API TArray<FPCGTaggedData> GetParamsByTag(const FPCGDataCollection& InCollection, const FString& InTag);

	UFUNCTION(BlueprintCallable, Category = Data)
	static UE_API TArray<FPCGTaggedData> GetAllSettings(const FPCGDataCollection& InCollection);

	UFUNCTION(BlueprintPure, Category = "PCG|TaggedData", meta = (NativeBreakFunc))
	static void BreakTaggedData(const FPCGTaggedData& TaggedData, UPCGData*& Data, TSet<FString>& Tags, FName& Pin, bool& bIsUsedMultipleTimes)
	{
		Data = const_cast<UPCGData*>(TaggedData.Data.Get());
		Tags = TaggedData.Tags;
		Pin = TaggedData.Pin;
		bIsUsedMultipleTimes = TaggedData.bIsUsedMultipleTimes;
	}

	UFUNCTION(BlueprintPure, Category = "PCG|TaggedData", meta = (NativeMakeFunc, AutoCreateRefTerm = "Tags"))
	static FPCGTaggedData MakeTaggedData(UPCGData* Data, const TSet<FString>& Tags, FName Pin)
	{
		FPCGTaggedData TaggedData;
		TaggedData.Data = Data;
		TaggedData.Tags = Tags;
		TaggedData.Pin = Pin;

		return TaggedData;
	}

protected:
	static UE_API TArray<UPCGData*> GetInputsByPredicate(const FPCGDataCollection& InCollection, TArray<FPCGTaggedData>& OutTaggedData, TFunctionRef<bool(const FPCGTaggedData&)> InPredicate);
};

#undef UE_API
