// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "PCGSettings.h"

#include "PCGDataAsset.generated.h"

class UPCGDataAsset;

USTRUCT(BlueprintInternalUseOnly)
struct FPCGDataAssetCachedPins
{
	GENERATED_BODY();
	
	/** Cached pins to be able to update the Load PCG Data Asset node, without loading the asset. */
	UPROPERTY()
	TArray<FPCGPinProperties> CachedPins;

	/** For deprecation, to know if the asset has cached pins. */
	UPROPERTY()
	bool bHasCachedPins = false;
};

namespace PCGDataAsset
{
	struct FGetAssetDataOutput
	{
		// Can be null if the asset is not loaded.
		UPCGDataAsset* Asset = nullptr;
		FString Name;
		FPCGDataAssetCachedPins CachedPins;
		
#if WITH_EDITOR
		FText Description;
		TOptional<FLinearColor> Color;
#endif // WITH_EDITOR
	};
}

/** Container for PCG data exported as standalone objects */
UCLASS(MinimalAPI, hidecategories=Object, BlueprintType, ClassGroup=(Procedural))
class UPCGDataAsset : public UObject
{
	GENERATED_BODY()

public:
	// TODO: crc?
	// TODO: add compression on serialization
	// TODO: add array of references to other assets for recursive parsing
	// TODO: additional embedded graph?

	//~ Begin UObject interface
	PCG_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	//~ End UObject interface

	PCG_API TArray<FPCGPinProperties> GetPins() const;

	static PCG_API PCGDataAsset::FGetAssetDataOutput GetAssetRegistryData(const TSoftObjectPtr<UPCGDataAsset>& InSoftAsset, const bool bShouldLoadAsset);
#if WITH_EDITOR
	static PCG_API PCGDataAsset::FGetAssetDataOutput GetAssetRegistryData(const FAssetData& AssetData);
#endif // WITH_EDITOR

	/** Contains direct data owned by this data asset */
	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category = Data, meta = (NoResetToDefault))
	FPCGDataCollection Data;

	/** Alternative name (instead of asset name), can be left empty */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, AssetRegistrySearchable)
	FString Name;

#if WITH_EDITORONLY_DATA
	/** Custom exporter class that was used to create this PCG asset. Should derive from UPCGAssetExporter. Can be left empty. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Data, AssetRegistrySearchable, meta = (NoResetToDefault))
	TSoftClassPtr<UObject> ExporterClass;

	/** Custom exporter metadata to be able to update data without user intervention */
	UPROPERTY(AssetRegistrySearchable)
	FString ExporterMetadata;

	/** Custom class to create settings/node in the graph when dragged in the editor. Can be left empty. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Data, AssetRegistrySearchable, meta = (NoResetToDefault))
	TSubclassOf<UPCGSettings> SettingsClass;

	/** Reference to originating object (often will be a level) */
	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category = Data, AssetRegistrySearchable, meta = (NoResetToDefault))
	FSoftObjectPath ObjectPath;

	/** Custom node color to use for this specific asset */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, AssetRegistrySearchable)
	FLinearColor Color = FLinearColor::White;

	/** Controls whether the asset will be visible from the palette & contextual menus. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable)
	bool bExposeToLibrary = false;

	/** Controls in what category the asset will appear in the palette & contextual menus. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable)
	FText Category = NSLOCTEXT("PCGDataAsset", "DefaultCategory", "PCGDataAssets");

	/** Additional description (tooltip) shown on the asset/loader node. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable)
	FText Description;

private:
	/** Cached data to be able to update the pins without loading the asset. */
	UPROPERTY(AssetRegistrySearchable)
	FPCGDataAssetCachedPins CachedPins;
#endif // WITH_EDITORONLY_DATA
};