// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataCommon.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"

#include "EngineDefines.h"

#if WITH_EDITOR
	#include "Editor/UnrealEdTypes.h"
#endif // WITH_EDITOR

#if WITH_EDITOR
class AActor;
class FPCGMetadataAttributeBase;
class UPCGData;
class UPCGSettingsInterface;
struct FPCGContext;

class FAdvancedPreviewScene;
class FEditorViewportClient;
class SWidget;
struct FStreamableHandle;
enum ELevelViewportType: int;

struct FPCGSceneSetupParams
{
	FAdvancedPreviewScene* Scene = nullptr;
	FEditorViewportClient* EditorViewportClient = nullptr;

	/** Resources which are kept loaded for scene setup. */
	TConstArrayView<UObject*> Resources;

	/** Any UObjects created during scene setup should be tracked here to be kept visible for GC. */
	TArray<TObjectPtr<UObject>> ManagedResources;

	/** Used when focusing the data viewport to the visualization. */
	FBoxSphereBounds FocusBounds = FBoxSphereBounds(EForceInit::ForceInit);

	/** Used when focusing the data viewport to the visualization for orthographic views. If not set then FocusBounds is used to determin zoom. */
	TOptional<float> FocusOrthoZoom;

	/** Preferred viewport type for this data (e.g. orthographic for 2D data). Applied on first inspection of a node. */
	TOptional<ELevelViewportType> PreferredViewportType = ELevelViewportType::LVT_Perspective;

	/** Optional widget placed in the viewport toolbar for the duration of this scene. The visualization owns any state the widget needs to update its scene resources. Cleared by the viewport on the next scene reset. */
	TSharedPtr<SWidget> ViewportToolbarWidget;
};

using FPCGSetupSceneFunc = TFunction<void(FPCGSceneSetupParams& InOutParams)>;

enum class EPCGTableVisualizerColumnSortingMode
{
	None,
	Ascending,
	Descending
};

enum class EPCGTableVisualizerCellAlignment
{
	Fill,
	Left,
	Center,
	Right,
};

struct FPCGTableVisualizerColumnInfo
{
	FName Id;
	FText Label;
	FText Tooltip;
	float Width = -1.0f; // Will be calculated automatically if < 0
	EPCGTableVisualizerCellAlignment CellAlignment = EPCGTableVisualizerCellAlignment::Right;
	TSharedPtr<const IPCGAttributeAccessor> Accessor;
	TSharedPtr<const IPCGAttributeAccessorKeys> AccessorKeys;
	const FPCGMetadataAttributeBase* UnderlyingAttribute = nullptr;
};

struct FPCGTableVisualizerInfo
{
	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPCGTableVisualizerInfo() = default;
	~FPCGTableVisualizerInfo() = default;
	FPCGTableVisualizerInfo(const FPCGTableVisualizerInfo&) = default;
	FPCGTableVisualizerInfo(FPCGTableVisualizerInfo&&) = default;
	FPCGTableVisualizerInfo& operator=(const FPCGTableVisualizerInfo&) = default;
	FPCGTableVisualizerInfo& operator=(FPCGTableVisualizerInfo&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	const UPCGData* Data = nullptr;
	TArray<FPCGTableVisualizerColumnInfo> ColumnInfos;
	EPCGTableVisualizerColumnSortingMode SortingMode = EPCGTableVisualizerColumnSortingMode::Ascending;
	FName SortingColumn = NAME_None;
	TFunction<void(const UPCGData*, TArrayView<const int>)> FocusOnDataCallback = nullptr;
	
	UE_DEPRECATED(5.6, "Set the keys in the ColumnInfo")
	TSharedPtr<const IPCGAttributeAccessorKeys> AccessorKeys;
};

/** Implement this interface to provide custom PCGData visualizations. Register your implementation to FPCGModule::FPCGDataVisualizationRegistry to be used automatically. */
class IPCGDataVisualization
{
public:
	virtual ~IPCGDataVisualization() = default;
	virtual void ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const = 0;
	
	UE_DEPRECATED(5.6, "Use and implement GetTableVisualizerInfoWithDomain instead.")
	virtual FPCGTableVisualizerInfo GetTableVisualizerInfo(const UPCGData* Data) const
	{
		ensureMsgf(false, TEXT("Should never be called, child class need to override GetTableVisualizerInfoWithDomain"));
		return {};
	}

	/** To be overriden by child classes. For deprecation, call the simple version one by default. */
	virtual FPCGTableVisualizerInfo GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GetTableVisualizerInfo(Data);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	virtual FPCGMetadataDomainID GetDefaultDomainForInspection(const UPCGData* Data) const { return Data ? Data->GetDefaultMetadataDomainID() : PCGMetadataDomainID::Invalid; }
	virtual TArray<FPCGMetadataDomainID> GetAllSupportedDomainsForInspection(const UPCGData* Data) const { return Data ? Data->GetAllSupportedMetadataDomainIDs() : TArray<FPCGMetadataDomainID>(); }
	virtual FString GetDomainDisplayNameForInspection(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
	{
		if (!Data)
		{
			return {};
		}

		FPCGAttributePropertySelector Selector;
		Data->SetDomainFromDomainID(DomainID, Selector);
		return Selector.GetDomainName().ToString();
	}

	/** Initiates an async load on any resources this data needs in order to be visualized. */
	virtual TArray<TSharedPtr<FStreamableHandle>> LoadRequiredResources(const UPCGData* Data) const { return {}; }

	UE_DEPRECATED(5.7, "Use the version taking the settings and data.")
	virtual FPCGSetupSceneFunc GetViewportSetupFunc(const UPCGData* Data) const { return FPCGSetupSceneFunc(); }

	/** Optionally provide a function to setup the data viewport. */
	virtual FPCGSetupSceneFunc GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// For deprecation, call the old virtual by default.
		return GetViewportSetupFunc(Data);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
};
#endif
