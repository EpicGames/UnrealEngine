// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "PCGData.h"
#include "PCGDataVisualization.h"

/** Register a PCGData type with an IPCGDataVisualization to implement custom visualization behavior. */
struct FPCGDataVisualizationRegistry
{
public:
	FPCGDataVisualizationRegistry() = default;
	FPCGDataVisualizationRegistry(const FPCGDataVisualizationRegistry&) = delete;
	FPCGDataVisualizationRegistry(FPCGDataVisualizationRegistry&&) = default;
	FPCGDataVisualizationRegistry& operator=(const FPCGDataVisualizationRegistry&) = delete;
	FPCGDataVisualizationRegistry& operator=(FPCGDataVisualizationRegistry&&) = default;
	virtual ~FPCGDataVisualizationRegistry() = default;

	/** Register an external PCGData visualization. */
	PCG_API void RegisterPCGDataVisualization(const TSubclassOf<UPCGData> DataClass, TUniquePtr<const IPCGDataVisualization> Visualization);

	/** Unregister an external PCGData visualization. */
	PCG_API void UnregisterPCGDataVisualization(const TSubclassOf<UPCGData> DataClass);

	template <typename PCGDataType, typename = typename std::enable_if_t<std::is_base_of_v<UPCGData, PCGDataType>>>
	const IPCGDataVisualization* GetDataVisualization() const { return GetDataVisualization(PCGDataType::StaticClass()); }
	PCG_API const IPCGDataVisualization* GetDataVisualization(const TSubclassOf<UPCGData> PCGDataTypeClass) const;

private:
	/** Registry for PCGDataVisualizations defined inside the PCG Plugin. */
	TMap<const TSubclassOf<UPCGData>, TUniquePtr<const IPCGDataVisualization>> InternalRegistry;

	/** Registry for PCGDataVisualizations defined outside the PCG Plugin. These take priority, allowing the user to override default visualization behavior for internal types. */
	TMap<const TSubclassOf<UPCGData>, TUniquePtr<const IPCGDataVisualization>> ExternalRegistry;

	friend class FPCGEditorModule;
};
#endif
