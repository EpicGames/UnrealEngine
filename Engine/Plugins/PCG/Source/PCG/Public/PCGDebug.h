// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "PCGDebug.generated.h"

#define UE_API PCG_API

class UStaticMesh;
class UMaterialInterface;

UENUM()
enum class EPCGDebugVisScaleMethod : uint8
{
	Relative,
	Absolute,
	Extents
};

USTRUCT(BlueprintType)
struct FPCGDebugVisualizationSettings
{
	GENERATED_BODY()

public:
	UE_API FPCGDebugVisualizationSettings();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Debug, meta = (ClampMin="0", EditCondition = "ScaleMethod != EPCGDebugVisScaleMethod::Extents && bDisplayProperties", EditConditionHides, HideEditConditionToggle))
	float PointScale = 1.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Debug, meta = (EditCondition = "bDisplayProperties", EditConditionHides, HideEditConditionToggle))
	EPCGDebugVisScaleMethod ScaleMethod = EPCGDebugVisScaleMethod::Extents;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Debug, meta = (EditCondition = "bDisplayProperties", EditConditionHides, HideEditConditionToggle))
	TSoftObjectPtr<UStaticMesh> PointMesh;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Debug, meta = (EditCondition = "bDisplayProperties", EditConditionHides, HideEditConditionToggle))
	TSoftObjectPtr<UMaterialInterface> MaterialOverride;

	UE_API TSoftObjectPtr<UMaterialInterface> GetMaterial() const;

#if WITH_EDITORONLY_DATA
	// This can be set false to hide the debugging properties.
	UPROPERTY(Transient, meta = (EditCondition = false, EditConditionHides))
	bool bDisplayProperties = true;
#endif
};

namespace PCGDebugVisConstants
{
	/** Default mesh used for PCG debug point visualization. */
	UE_API extern const FSoftObjectPath DefaultPointMesh;
}

#undef UE_API
