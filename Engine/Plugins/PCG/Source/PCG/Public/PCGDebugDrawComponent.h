// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGManagedResource.h"

#include "DebugRenderSceneProxy.h"
#include "Debug/DebugDrawComponent.h"

#include "PCGDebugDrawComponent.generated.h"

/** 
* Manages the editor DebugDrawComponent for displaying debug information within SIE and PIE.
* Note: Must be a managed component to ensure proper resource management during generation and cleanup.
*/
UCLASS(ClassGroup = (Procedural))
class UPCGManagedDebugDrawComponent : public UPCGManagedComponent
{
	GENERATED_BODY()

#if WITH_EDITOR
	// Debug Draw should always be transient, regardless of editing mode
	virtual void ChangeTransientState(EPCGEditorDirtyMode NewEditingMode) override {}
#endif // WITH_EDITOR
};

/**
 * A transient component intended to visualize attribute information of PCG Point Data, such as printing the string value in World Space. Future support may include: vectors, flow fields, heatmaps, etc.
 * Note: Uses a built in timer for users that may want debug information for a limited timeframe during generation.
 */
UCLASS(Transient, ClassGroup = (Procedural))
class UPCGDebugDrawComponent : public UDebugDrawComponent
{
	GENERATED_BODY()

public:
	void AddDebugString(const FDebugRenderSceneProxy::FText3d& TextIn3D);
	void AddDebugString(const FString& InString, const FVector& InLocation, const FLinearColor& InColor);
	void AddDebugStrings(const TArrayView<FDebugRenderSceneProxy::FText3d> TextIn3DArray);

	void StartTimer(float DurationInMilliseconds);

protected:
	//~Begin UDebugDrawComponent interface
	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
	//~End UDebugDrawComponent interface

	//~Begin UPrimitiveComponent interface
	virtual void OnUnregister() override;
	//~End UPrimitiveComponent interface

	virtual void OnTimerElapsed();

	FDebugRenderSceneProxy* SceneProxy = nullptr;
	FTimerHandle TimerHandle;

private:
	void ClearTimer();
};