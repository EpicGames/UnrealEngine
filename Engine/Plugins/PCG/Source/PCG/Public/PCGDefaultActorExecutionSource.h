// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDefaultWorldObjectExecutionSource.h"

#include "PCGDefaultActorExecutionSource.generated.h"

class AActor;
class FPCGDefaultActorExecutionState;

struct FPCGDefaultActorExecutionSourceParams : public FPCGDefaultExecutionSourceParams
{
	AActor* Actor = nullptr;
};

UCLASS(MinimalAPI)
class UPCGDefaultActorExecutionSource : public UPCGDefaultWorldObjectExecutionSource
{
	friend FPCGDefaultActorExecutionState;

	GENERATED_BODY()

public:
	using ParamsType = FPCGDefaultActorExecutionSourceParams;

	PCG_API UPCGDefaultActorExecutionSource();
	PCG_API void Initialize(const ParamsType& InParams);
	PCG_API AActor* GetActor();
	PCG_API void SetActor(AActor* InActor);
};

class FPCGDefaultActorExecutionState : public FPCGDefaultWorldObjectExecutionState
{
public:
	FPCGDefaultActorExecutionState() = default;
	explicit FPCGDefaultActorExecutionState(UPCGDefaultActorExecutionSource* InSource);

	PCG_API virtual FString GetDebugName() const override;
	PCG_API virtual UPCGData* GetSelfData() const override;
	PCG_API virtual FTransform GetTransform() const override;
	PCG_API virtual UObject* GetTarget() const override;
	PCG_API virtual bool HasAuthority() const override;
	PCG_API virtual FBox GetBounds() const override;
	PCG_API virtual FBox GetLocalSpaceBounds() const override;

	PCG_API virtual void ExecutePreGraph(FPCGContext* InContext) override;

protected:
	UPCGDefaultActorExecutionSource* GetTypedSource() const { return CastChecked<UPCGDefaultActorExecutionSource>(GetSource()); }

private:
	UPCGData* GetSelfDataInternal() const;
	FBox GetBoundsInternal() const;
	FBox GetLocalSpaceBoundsInternal() const;

	AActor* GetActor() const { return GetTypedSource()->GetActor(); }
};