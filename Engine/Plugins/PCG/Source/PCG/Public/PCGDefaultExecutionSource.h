// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGraphExecutionInspection.h"
#include "PCGGraphExecutionStateInterface.h"
#include "Subsystems/PCGEngineSubsystem.h"

#include "PCGDefaultExecutionSource.generated.h"

class UPCGDefaultExecutionSource;
class UPCGGraph;
class UPCGGraphInstance;
class UPCGGraphInterface;
struct FPCGDataCollection;

DECLARE_DELEGATE_OneParam(FPCGGenerationPostProcessCallback, const FPCGDataCollection&);

class FPCGDefaultExecutionState : public IPCGGraphExecutionState
{
public:
	FPCGDefaultExecutionState() = default;
	explicit FPCGDefaultExecutionState(UPCGDefaultExecutionSource* InSource) : Source(InSource){}
	
	PCG_API virtual UPCGData* GetSelfData() const override;
	PCG_API virtual int32 GetSeed() const override;
	PCG_API virtual FString GetDebugName() const override;
	PCG_API virtual FTransform GetTransform() const override;
	PCG_API virtual UWorld* GetWorld() const override;
	PCG_API virtual bool HasAuthority() const override;
	PCG_API virtual FBox GetBounds() const override;
	PCG_API virtual UPCGGraph* GetGraph() const override;
	PCG_API virtual UPCGGraphInstance* GetGraphInstance() const override;
	PCG_API virtual void OnGraphExecutionAborted(bool bQuiet, bool bCleanupUnusedResources) override;
	PCG_API virtual void Cancel() override;
	PCG_API virtual bool IsGenerating() const override;
	PCG_API virtual IPCGGraphExecutionSource* GetOriginalSource() const override;

	void SetPostProcessCallback(FPCGGenerationPostProcessCallback InCallback) { PostProcessCallback = MoveTemp(InCallback); }
	PCG_API void OnPostProcess(const FPCGDataCollection& InDataCollection);

#if PCG_PROFILING_ENABLED
	PCG_API virtual const PCGUtils::FExtraCapture& GetExtraCapture() const override;
	PCG_API virtual PCGUtils::FExtraCapture& GetExtraCapture() override;

	PCG_API virtual const FPCGGraphExecutionInspection& GetInspection() const override;
	PCG_API virtual FPCGGraphExecutionInspection& GetInspection() override;
#endif // PCG_PROFILING_ENABLED

#if WITH_EDITOR
	// No dynamic tracking for editor resources (at the moment)
	virtual void RegisterDynamicTracking(const UPCGSettings* InSettings, const TArrayView<TPair<FPCGSelectionKey, bool>>& InDynamicKeysAndCulling) override {};
	virtual void RegisterDynamicTracking(const FPCGSelectionKeyToSettingsMap& InKeysToSettings) override {};
	
	virtual bool IsRefreshInProgress() const override {return IsGenerating();}

	virtual FPCGDynamicTrackingPriority GetDynamicTrackingPriority() const override { return FPCGDynamicTrackingPriority(); }
#endif // WITH_EDITOR

protected:
	UPCGDefaultExecutionSource* GetSource() const { return Source; }

private:
	UPCGDefaultExecutionSource* Source = nullptr;
	FPCGGenerationPostProcessCallback PostProcessCallback;
};

struct FPCGDefaultExecutionSourceParams
{
	UPCGGraphInterface* GraphInterface = nullptr;
	int32 Seed = 42;
	bool bFireAndForgetExecution = false;

	/** Optional callback when generation completes. */
	FPCGOnEditorGenerationDone GenerationCallback;
	/** Optional callback to process graph execution results, before the Generation Done callback. */
	FPCGGenerationPostProcessCallback PostProcessCallback;
};

UCLASS(MinimalAPI)
class UPCGDefaultExecutionSource : public UObject, public IPCGGraphExecutionSource
{
	friend FPCGDefaultExecutionState;
	
	GENERATED_BODY()
	
public:
	using ParamsType = FPCGDefaultExecutionSourceParams;

	PCG_API UPCGDefaultExecutionSource();
	PCG_API virtual void BeginDestroy() override;

	PCG_API void Initialize(const ParamsType& InParams);

	PCG_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	
	virtual IPCGGraphExecutionState& GetExecutionState() override { return *State; }
	virtual const IPCGGraphExecutionState& GetExecutionState() const override { return *State; }

	PCG_API void SetGraphInterface(UPCGGraphInterface* InGraphInterface);

	UPCGGraphInterface* GetGraphInterface() { return GraphInterface.Get(); }
	PCG_API UPCGGraphInstance* GetGraphInstance();
	PCG_API UPCGGraph* GetGraph();

	PCG_API void SetSeed(int32 InSeed);

#if WITH_EDITOR
	PCG_API void OnGraphChanged(UPCGGraphInterface* GraphInterface, EPCGChangeType ChangeType);
#endif // WITH_EDITOR
	
	PCG_API void Generate();

	PCG_API void Sunset();

	FPCGTaskId GetCurrentGenerationTask() const { return CurrentGenerationTask; }

protected:
	FPCGTaskId CurrentGenerationTask = InvalidPCGTaskId;
	
	UPROPERTY()
	TObjectPtr<UPCGGraphInterface> GraphInterface;
	
	TUniquePtr<FPCGDefaultExecutionState> State;
	int32 Seed = 42;

#if PCG_PROFILING_ENABLED
	PCGUtils::FExtraCapture ExtraCapture;
	FPCGGraphExecutionInspection Inspection;
#endif // PCG_PROFILING_ENABLED
};