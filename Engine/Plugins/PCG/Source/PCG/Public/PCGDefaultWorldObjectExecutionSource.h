// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDefaultExecutionSource.h"
#include "PCGManagedResourceContainer.h"

#include "PCGDefaultWorldObjectExecutionSource.generated.h"

class FPCGDefaultWorldObjectExecutionState;

struct FPCGDefaultWorldObjectExecutionSourceParams : public FPCGDefaultExecutionSourceParams
{
	UObject* WorldObject = nullptr;
};

UCLASS(MinimalAPI)
class UPCGDefaultWorldObjectExecutionSource : public UPCGDefaultExecutionSource
{
	friend FPCGDefaultWorldObjectExecutionState;

	GENERATED_BODY()
	
public:
	using ParamsType = FPCGDefaultWorldObjectExecutionSourceParams;

	PCG_API UPCGDefaultWorldObjectExecutionSource();
	PCG_API void Initialize(const FPCGDefaultWorldObjectExecutionSourceParams& InParams);
	UObject* GetWorldObject() { return WorldObject.Get(); }
	void SetWorldObject(UObject* InWorldObject) { WorldObject = InWorldObject; }

private:
	TWeakObjectPtr<UObject> WorldObject = nullptr;

	/** Managed resources need while the execution occurs. They will be forgotten when execution is completed*/
	UPROPERTY()
	FPCGManagedResourceContainer ManagedResourceContainer;
	mutable FTransactionallySafeCriticalSection ManagedResourcesLock;
};

class FPCGDefaultWorldObjectExecutionState : public FPCGDefaultExecutionState
{
public:
	PCG_API FPCGDefaultWorldObjectExecutionState();
	explicit PCG_API FPCGDefaultWorldObjectExecutionState(UPCGDefaultWorldObjectExecutionSource* InSource);

	PCG_API virtual FString GetDebugName() const override;
	PCG_API virtual UWorld* GetWorld() const override;
	PCG_API virtual UObject* GetTarget() const override;
	virtual FPCGManagedResourceContainer* GetManagedResourceContainer() override { return GetTypedSource() ? &GetTypedSource()->ManagedResourceContainer : nullptr; }
	virtual FTransactionallySafeCriticalSection* GetManagedResourceContainerLock() override { return GetTypedSource() ? &GetTypedSource()->ManagedResourcesLock : nullptr; }

	virtual bool IsCacheEnabled() const override { return false; }

#if WITH_EDITOR
	virtual bool UseTransactions() const override { return bUseTransactions; }
#endif // WITH_EDITOR

protected:
	UPCGDefaultWorldObjectExecutionSource* GetTypedSource() const { return CastChecked<UPCGDefaultWorldObjectExecutionSource>(GetSource()); }
	UObject* GetWorldObject() const { return GetTypedSource()->GetWorldObject(); }

#if WITH_EDITOR
private:
	bool bUseTransactions = true;
#endif
};