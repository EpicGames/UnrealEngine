// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPin.h"

#include "UObject/Interface.h"

#include "PCGDynamicPins.generated.h"

class UPCGNode;
enum class EPCGChangeType : uint32;

/**
 * Composable state struct for dynamic pin management.
 * Any UPCGSettings implementing IPCGDynamicPinsProvider can embed this as a UPROPERTY member (one per direction) and
 * override GetDynamicPinContainer()/GetMutableDynamicPinContainer() to get full dynamic pin support with minimal boilerplate.
 */
USTRUCT()
struct FPCGDynamicPinContainer
{
	GENERATED_BODY()

	/** The dynamic pin properties managed by this container. */
	UPROPERTY()
	TArray<FPCGPinProperties> PinProperties;

	/** Returns the number of dynamic pins. */
	int32 Num() const { return PinProperties.Num(); }

	/** Returns the labels of all dynamic pins. */
	PCG_API TArray<FName> GetPinLabels() const;

	/** Returns the index of the pin with the given label, or INDEX_NONE if no such pin exists. */
	int32 FindPinIndex(const FName Label) const
	{
		return PinProperties.IndexOfByPredicate([Label](const FPCGPinProperties& Props) { return Props.Label == Label; });
	}

	/** Returns true if a pin with the given label exists in the container. */
	bool ContainsPin(const FName Label) const { return FindPinIndex(Label) != INDEX_NONE; }

#if WITH_EDITOR
	/** Appends a new dynamic pin. The label will be made unique if it collides	with existing dynamic pin labels. */
	PCG_API void AddPin(FPCGPinProperties&& Properties);

	/** Returns true if the pin at the given index can be removed. */
	bool CanRemovePin(const int32 Index) const { return Index >= 0 && Index < PinProperties.Num(); }

	/**
	 * Removes the dynamic pin at the given index. Breaks edges on the corresponding node pin and renames it to a placeholder
	 * to avoid label conflicts during UpdatePins.
	 * Returns the change type flags for the caller to broadcast.
	 */
	PCG_API EPCGChangeType RemovePin(int32 Index, UPCGNode* Node, EPCGPinDirection Direction);

	/**
	 * Renames the dynamic pin at the given index. Updates both the stored properties and the live node pin. The new label
	 * will be made unique if it collides.
	 * Returns the change type flags for the caller to broadcast.
	 */
	PCG_API EPCGChangeType RenamePin(int32 Index, FName NewLabel, UPCGNode* Node, EPCGPinDirection Direction);

private:
	/** Returns the unique label, with potentially an appended numeric suffix. */
	FName MakeUniqueLabel(FName BaseName, int32 ExcludeIndex = INDEX_NONE) const;
#endif // WITH_EDITOR
};

/**
 * Interface for settings that support dynamic pins. Implementers embed one or more FPCGDynamicPinContainer as UPROPERTY
 * members and override the accessors to expose them.
 *
 * Implementers must also override UPCGSettings::HasDynamicPins() to return true so that output pin type inference
 * in GetOutputType() uses the declared pin type rather than inferring from input edges.
 */
UINTERFACE(MinimalAPI)
class UPCGDynamicPinsProvider : public UInterface
{
	GENERATED_BODY()
};

class IPCGDynamicPinsProvider
{
	GENERATED_BODY()

public:
	/** Returns the dynamic pin container for the given direction, or nullptr if that direction is not supported. */
	virtual const FPCGDynamicPinContainer* GetDynamicPinContainer(EPCGPinDirection Direction) const = 0;

	/** Returns the mutable dynamic pin container for the given direction, or nullptr if not supported. */
	virtual FPCGDynamicPinContainer* GetMutableDynamicPinContainer(EPCGPinDirection Direction) = 0;

	/** Returns the static pin properties for the given direction. */
	virtual TArray<FPCGPinProperties> GetStaticPinProperties(EPCGPinDirection Direction) const = 0;

	/** Returns static + dynamic pin properties concatenated. Suitable for use in InputPinProperties()/OutputPinProperties() overrides. */
	PCG_API virtual TArray<FPCGPinProperties> GetCombinedPinProperties(EPCGPinDirection Direction) const;

	/** Returns the labels of all dynamic pins for the given direction. */
	PCG_API virtual TArray<FName> GetAllDynamicPinLabels(EPCGPinDirection Direction) const;

#if WITH_EDITOR
	/** Returns default properties for a newly created dynamic pin. */
	virtual FPCGPinProperties CreateDefaultDynamicPin(EPCGPinDirection Direction) const = 0;

	/** Whether the user can add dynamic pins in the given direction. */
	virtual bool CanUserAddDynamicPins(const EPCGPinDirection Direction) const { return GetDynamicPinContainer(Direction) != nullptr; }

	/** Whether the user can remove dynamic pins in the given direction. */
	virtual bool CanUserRemoveDynamicPins(const EPCGPinDirection Direction) const { return GetDynamicPinContainer(Direction) != nullptr; }

	/** Whether the user can rename dynamic pins in the given direction. */
	virtual bool CanUserRenameDynamicPin(EPCGPinDirection Direction) const { return false; }

	/** Called after a dynamic pin has been renamed. */
	virtual void OnPinRenamed(FName OldLabel, FName NewLabel, EPCGPinDirection Direction) {}
#endif // WITH_EDITOR
};
