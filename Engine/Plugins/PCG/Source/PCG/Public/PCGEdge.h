// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "PCGEdge.generated.h"

#define UE_API PCG_API

class UPCGNode;
class UPCGPin;

UCLASS(MinimalAPI, ClassGroup = (Procedural))
class UPCGEdge : public UObject
{
	GENERATED_BODY()
public:
	UE_API UPCGEdge(const FObjectInitializer& ObjectInitializer);

	// ~Begin UObject interface
	UE_API virtual void PostLoad() override;
	// ~End UObject interface

	UPROPERTY(meta=(PCGNoHash))
	FName InboundLabel_DEPRECATED = NAME_None;

	UPROPERTY(meta = (PCGNoHash))
	TObjectPtr<UPCGNode> InboundNode_DEPRECATED;

	UPROPERTY(meta = (PCGNoHash))
	FName OutboundLabel_DEPRECATED = NAME_None;

	UPROPERTY(meta = (PCGNoHash))
	TObjectPtr<UPCGNode> OutboundNode_DEPRECATED;

	/** Pin at upstream end of edge. */
	UPROPERTY(BlueprintReadOnly, Category = Properties)
	TObjectPtr<UPCGPin> InputPin;

	/** Pin at downstream end of edge. */
	UPROPERTY(BlueprintReadOnly, Category = Properties)
	TObjectPtr<UPCGPin> OutputPin;

	UE_API bool IsValid() const;
	UE_API UPCGPin* GetOtherPin(const UPCGPin* Pin);
	UE_API const UPCGPin* GetOtherPin(const UPCGPin* Pin) const;

	UFUNCTION(BlueprintCallable, Category = Properties)
	UE_API const UPCGNode* GetInputNode() const;

	UFUNCTION(BlueprintCallable, Category = Properties)
	UE_API const UPCGNode* GetOutputNode() const;

	UFUNCTION(BlueprintCallable, Category = Properties)
	UE_API FName GetInputPinLabel() const;

	UFUNCTION(BlueprintCallable, Category = Properties)
	UE_API FName GetOutputPinLabel() const;

};

#undef UE_API
