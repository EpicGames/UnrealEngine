// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "Graph/DataOverride/PCGDataOverride.h"

#include "Logging/LogVerbosity.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakInterfacePtr.h"

#define UE_API PCG_API

class IPCGElement;
class IPCGGraphCache;
class IPCGGraphExecutionSource;
class UPCGComponent;
class UPCGNode;
class UPCGSettings;
class UPCGSettingsInterface;

struct FPCGContext;
struct FPCGCrc;
struct FPCGGetFromCacheParams;

enum class EPCGDataOverridePhase : uint8;

typedef TSharedPtr<IPCGElement, ESPMode::ThreadSafe> FPCGElementPtr;

namespace EPCGElementLogMode
{
	enum Type : uint8
	{
		/** Output to log. */
		LogOnly = 0,
		/** Display errors/warnings on graph as well as writing to log. */
		GraphAndLog,

		NumLogModes,

		// Used below in log macros to silence PVS by making the log mode comparison 'look' non-trivial by adding a trivial mask op (an identical
		// mechanism as the one employed in the macro UE_ASYNC_PACKAGE_LOG in AsyncLoading2.cpp).
		// Warning V501: There are identical sub-expressions 'EPCGElementLogMode::GraphAndLog' to the left and to the right of the '==' operator.
		// The warning disable comment can can't be used in a macro: //-V501 
		LogModeMask = 0xff
	};
};

#define PCGE_LOG_BASE(Verbosity, CustomContext, Message) \
	UE_LOG(LogPCG, \
		Verbosity, \
		TEXT("[%s - %s]: %s"), \
		(CustomContext) ? *((CustomContext)->GetExecutionSourceName()) : TEXT("UnknownSource"), \
		(CustomContext) ? *((CustomContext)->GetTaskName()) : TEXT("UnknownTask"), \
		*Message.ToString())

#if WITH_EDITOR
// Output to PCG log and optionally also display warnings/errors on graph.
#define PCGE_LOG(Verbosity, LogMode, Message) do { \
		if (((EPCGElementLogMode::LogMode) & EPCGElementLogMode::LogModeMask) == EPCGElementLogMode::GraphAndLog && (Context)) { (Context)->LogVisual(ELogVerbosity::Verbosity, Message); } \
		if (ShouldLog()) { PCGE_LOG_BASE(Verbosity, Context, Message); } \
	} while (0)
// Output to PCG log and optionally also display warnings/errors on graph. Takes context as argument.
#define PCGE_LOG_C(Verbosity, LogMode, CustomContext, Message) do { \
		if (((EPCGElementLogMode::LogMode) & EPCGElementLogMode::LogModeMask) == EPCGElementLogMode::GraphAndLog && (CustomContext)) { (CustomContext)->LogVisual(ELogVerbosity::Verbosity, Message); } \
		PCGE_LOG_BASE(Verbosity, CustomContext, Message); \
	} while (0)
#else
#define PCGE_LOG(Verbosity, LogMode, Message) PCGE_LOG_BASE(Verbosity, Context, Message)
#define PCGE_LOG_C(Verbosity, LogMode, CustomContext, Message) PCGE_LOG_BASE(Verbosity, CustomContext, Message)
#endif

enum class EPCGElementExecutionLoopMode : uint8
{
	/** Not a trivial input -> output mapping, with respect to caching. */
	NotALoop,
	/** Loops on (singular) required pin. */
	SinglePrimaryPin,
	/** Loops on matching indices on required pin(s). */
	MatchingPrimaryPins,
	/** Loops on matching indices on required pin(s), but the second+ pin can have a single entry. E.g. 1:1, N:N or N:1 */
	PrimaryPinAndBroadcastablePins,
	/** Cartesian loop on required pins. */
	// CartesianPins // TODO
};

enum class EPCGCachingStatus : uint8
{
	NotCacheable,
	NotInCache,
	Cached
};

namespace PCGElementHelpers
{
	/** Breaks down the input data collection (InCollection) in a set of primary inputs (OutPrimaryCollections) and a set of fixed data per-itereation (OutCommonCollection), based on the provided mode. 
	* This is needed to perform per-data caching in nodes that support it, either in single-primary-pin-loops or in matching-pins loops.
	* Note that this uses the settings to drive the selection of the "primary" pins by testing if they are required.
	*/
	bool PCG_API SplitDataPerPrimaryPin(const UPCGSettings* Settings, const FPCGDataCollection& InCollection, EPCGElementExecutionLoopMode Mode, TArray<FPCGDataCollection>& OutPrimaryCollections, FPCGDataCollection& OutCommonCollection);
};

/**
 * Struct containing params to initialize IPCGElement
 */
struct FPCGInitializeElementParams
{
	FPCGInitializeElementParams(const FPCGDataCollection* InInputData, TWeakInterfacePtr<IPCGGraphExecutionSource> InExecutionSource, const UPCGNode* InNode)
		: InputData(InInputData)
		, ExecutionSource(InExecutionSource)
		, Node(InNode)
	{
		check(InputData);
	}
	
	const FPCGDataCollection* InputData;
	TWeakInterfacePtr<IPCGGraphExecutionSource> ExecutionSource;
	const UPCGNode* Node;
};

struct FPCGGetDependenciesCrcParams
{
	FPCGGetDependenciesCrcParams(const FPCGDataCollection* InInputData, const UPCGSettings* InSettings, IPCGGraphExecutionSource* InExecutionSource)
		: InputData(InInputData)
		, Settings(InSettings)
		, ExecutionSource(InExecutionSource)
	{
		check(InputData);
	}

	const FPCGDataCollection* InputData;
	const UPCGSettings* Settings;
	IPCGGraphExecutionSource* ExecutionSource;
};

/**
* Base class for the processing bit of a PCG node/settings
*/
class IPCGElement
{
public:
	virtual ~IPCGElement() = default;

	/** Creates a custom context object paired to this element */
	UE_API virtual FPCGContext* Initialize(const FPCGInitializeElementParams& InParams);

	UE_DEPRECATED(5.6, "Use/Implement version with FPCGInitializeElementParams parameter instead")
	UE_API virtual FPCGContext* Initialize(const FPCGDataCollection& InputData, TWeakObjectPtr<UPCGComponent> SourceComponent, const UPCGNode* Node);

	/** Returns true if the element, in its current phase can be executed only from the main thread */
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const { return false; }

	/** Returns true if the node can be cached - also checks for instance flags, if any. */
	UE_API bool IsCacheableInstance(const UPCGSettingsInterface* InSettingsInterface) const;

	/** Returns true if the node can be cached (e.g. does not create artifacts & does not depend on untracked data */
	UE_API virtual bool IsCacheable(const UPCGSettings* InSettings) const;
	
	/** Returns true if the node outputs requires to update the output tagged data to detect if the data is used multiple times. */
	UE_API virtual bool ShouldVerifyIfOutputsAreUsedMultipleTimes(const UPCGSettings* InSettings) const;

	/** Whether to do a 'deep' fine-grained CRC of the output data to pass to downstream nodes. Can be expensive so should be used sparingly. */
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const { return false; }

	/**
	 * Calculate a Crc that provides a receipt for the input data that can be paired with output data from the cache. If any dependency (setting, node input or
	 * external data) changes then this value should change. For some elements it is inefficient or not possible to output a Crc here. These can return an invalid
	 * Crc and the Crc can either be computed during execution, or afterwards based on output data.
	 */
	UE_API virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const;

	UE_DEPRECATED(5.6, "Use/Implement version with FPCGGetDependenciesCrcParams parameter instead")
	UE_API virtual void GetDependenciesCrc(const FPCGDataCollection& InInput, const UPCGSettings* InSettings, UPCGComponent* InComponent, FPCGCrc& OutCrc) const;

	/** Build a query structure to call RetrieveResultsFromCache */
	UE_API bool BuildCacheParams(IPCGGraphCache* Cache, const UPCGNode* Node, const FPCGDataCollection& Input, IPCGGraphExecutionSource* ExecutionSource, FPCGGetFromCacheParams& OutCacheParams) const;

	/** Gather input data (pre-context creation) and tries to retrieve matching data from the cache if the element is cacheable. */
	UE_API EPCGCachingStatus RetrieveResultsFromCache(IPCGGraphCache* Cache, const FPCGGetFromCacheParams& InCacheParams, FPCGDataCollection& Output) const;

	UE_DEPRECATED(5.8, "Use BuildCacheParam/RetrieveResultsFromCache combination instead")
	UE_API EPCGCachingStatus RetrieveResultsFromCache(IPCGGraphCache* Cache, const UPCGNode* Node, const FPCGDataCollection& Input, IPCGGraphExecutionSource* ExecutionSource, FPCGDataCollection& Output, FPCGCrc* OutCrc = nullptr) const;

	/** Public function that executes the element on the appropriately created context.
	* The caller should call the Execute function until it returns true.
	*/
	UE_API bool Execute(FPCGContext* Context) const;

	/** Public function called when an element is cancelled, passing its current context if any. */
	UE_API void Abort(FPCGContext* Context) const;

	/** Is this element used for marshaling data across higen grids. Used as rudimentary RTTI for element object comparisons (editor only). */
	virtual bool IsGridLinkage() const { return false; }

#if WITH_EDITOR
	/** Is this element used for dispatching compute graphs. Used as rudimentary RTTI for element object comparisons (editor only). */
	virtual bool IsComputeGraphElement() const { return false; }

	/** Performs debug display. Returns true when complete. Note: must be called from the main thread. */
	UE_API bool DebugDisplay(FPCGContext* Context) const;
#endif

	/** Performs some input conversions if element doesn't support it. Called in PrepareData but available for proxy nodes */
	UE_API bool ConvertInputsIfNeeded(FPCGContext* Context) const;

	/** Which phase to apply the data overrides to the internal data. */
	UE_API virtual EPCGDataOverridePhase GetDataOverridePhase() const { return EPCGDataOverridePhase::PostExecute; }

protected:
	/** This function is called at the beginning of an execution. Will be called until it returns true. */
	UE_API bool PreExecute(FPCGContext* Context) const;
	/** The prepare data phase is one where it is more likely to be able to multithread */
	UE_API bool PrepareData(FPCGContext* Context) const;
	UE_API virtual bool PrepareDataInternal(FPCGContext* Context) const;
	/** Core execution method for the given element. Will be called until it returns true. */
	virtual bool ExecuteInternal(FPCGContext* Context) const = 0;
	/** This function will be called once and once only, at the end of an execution */
	UE_API void PostExecute(FPCGContext* Context) const;
	/** Core post execute method for the given element. */
	virtual void PostExecuteInternal(FPCGContext* Context) const {}
	/** This function will be called once and only once if the element is aborted. The Context can be used to retrieve the current phase if needed. */
	virtual void AbortInternal(FPCGContext* Context) const {};

	/** Controls whether an element can skip its execution wholly when the input data has the cancelled tag */
	virtual bool IsCancellable() const { return true; }
	/** Used to specify that the element passes through the data without any manipulation - used to correct target pins, etc. */
	virtual bool IsPassthrough(const UPCGSettings* InSettings) const { return false; }

	/** Passes through data when the element is Disabled. Can be implemented to override what gets passed through. */
	UE_API virtual void DisabledPassThroughData(FPCGContext* Context) const;

	/** Describes internal execution behavior, which is used to break down inputs/outputs for caching purposes. */
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const { return EPCGElementExecutionLoopMode::NotALoop; }

	/** Implements input breakdown for caching purposes, if the ExecutionLoopMode is set to something else than not a loop. */
	UE_API virtual void PreExecutePrimaryLoopElement(FPCGContext* Context, const UPCGSettings* Settings) const;

	/** Implements output breakdown for caching purposes, if the ExecutionLoopMode is set to something else than not a loop. */
	UE_API virtual void PostExecutePrimaryLoopElement(FPCGContext* Context, const UPCGSettings* Settings) const;

	/** Let each element optionally act as a concrete factory for its own context */
	UE_API virtual FPCGContext* CreateContext();

	/** Returns true if node supports data objects that are only resident on GPU (not CPU). */
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const { return false; }

	/** Returns true if node supports any point data format. If not a conversion (costly) will happen on the node inputs. */
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return false; }

#if WITH_EDITOR
	virtual bool ShouldLog() const { return true; }
#endif

private:
	UE_API void CleanupAndValidateOutput(FPCGContext* Context) const;

	/** Poll GPU overridden attributes for readback. Replaces the GPU data with a CPU param data upon successful readback. Returns false until readback is complete. */
	bool ReadbackGPUDataForOverrides(FPCGContext* Context) const;
};

/**
* Convenience class for element with custom context that can be default constructed (need no initialization)
*/
template <typename ContextType, typename = typename std::enable_if_t<std::is_base_of_v<FPCGContext, ContextType> && std::is_default_constructible_v<ContextType>>>
class IPCGElementWithCustomContext : public IPCGElement
{
public:
	virtual FPCGContext* CreateContext() { return new ContextType(); }
};

class UE_DEPRECATED(5.4, "This class has been deprecated. Please inherit from IPCGElement directly.") PCG_API FSimplePCGElement : public IPCGElement
{};

#undef UE_API
