// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Engine/DeveloperSettingsBackedByCVars.h"

#include "PCGEngineSettings.generated.h"

#define UE_API PCG_API

UCLASS(MinimalAPI, config = Engine, defaultconfig)
class UPCGEngineSettings : public UDeveloperSettingsBackedByCVars
{
	GENERATED_BODY()

public:
	/** Specifies the scale of the volume created on PCG graph drag/drop */
	UPROPERTY(EditAnywhere, Config, Category = Workflow)
	FVector VolumeScale = FVector(25, 25, 10);

	/** Whether we want to generate PCG graph/BP with PCG after drag/drop or not */
	UPROPERTY(EditAnywhere, Config, Category = Workflow)
	bool bGenerateOnDrop = true;

	/** Display culling state in graph editor when a debug object is selected (requires regeneration to apply). */
	UPROPERTY(EditAnywhere, Config, Category = Workflow)
	bool bDisplayCullingStateWhenDebugging = true;

#if WITH_EDITORONLY_DATA
	// Console variables defined in PCGActorAndComponentMapping.cpp
	// Use the variables to get the values, not GetDefault<UPCGEngineSettings>

	/** Globally disable refresh. */
	UPROPERTY(EditAnywhere, Config, Category = Tracking, meta = (ConsoleVariable = "pcg.GlobalDisableRefresh"))
	bool bGlobalDisableRefresh = false;

	/** Completely disable landscape refresh when it changes. */
	UPROPERTY(EditAnywhere, Config, Category = Tracking, meta = (ConsoleVariable = "pcg.LandscapeDisableRefreshTracking"))
	bool bLandscapeDisableRefreshTracking = false;

	/** Completely disable landscape refresh when it changes in edit mode. Will force a refresh when landscape edit mode is exited. */
	UPROPERTY(EditAnywhere, Config, Category = Tracking, meta = (ConsoleVariable = "pcg.LandscapeDisableRefreshTrackingInLandscapeEditingMode"))
	bool bLandscapeDisableRefreshTrackingInLandscapeEditingMode = false;

	/** Time in MS between a landscape change and PCG refresh. Set it to 0 or negative value to disable the delay. */
	UPROPERTY(EditAnywhere, Config, Category = "Tracking|Advanced", meta = (ConsoleVariable = "pcg.LandscapeRefreshTimeDelayMS"))
	int32 LandscapeRefreshTimeDelayMS = 1000;
#endif // WITH_EDITORONLY_DATA

	// Begin UDeveloperSettings Interface
	UE_API virtual FName GetCategoryName() const override;
#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
#endif // WITH_EDITOR
	// End UDeveloperSettings Interface

	//~ Begin UObject Interface
	UE_API virtual void PostInitProperties() override;
	//~ End UObject Interface
};

#undef UE_API
