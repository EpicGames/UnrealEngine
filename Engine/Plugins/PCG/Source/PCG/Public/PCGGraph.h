// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "Concepts/UEnum.h"
#include "Data/PCGToolData.h"
#include "Editor/PCGGraphCustomization.h"
#include "Graph/PCGStackContext.h"
#include "Helpers/PCGAssetHelpers.h"
#include "Helpers/PCGGraphParameterExtension.h"

#if WITH_EDITOR
#include "Editor/PCGGraphComment.h"
#endif // WITH_EDITOR

#include "ComputeFramework/ComputeGraphInstance.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/ObjectPtr.h"
#include "UObject/PerPlatformProperties.h"

#include "PCGGraph.generated.h"

#define UE_API PCG_API

struct FAssetData;
struct FPCGCompilerDiagnostics;
class UPCGGraphInterface;
class UPCGGraphCompilationData;
class UPCGUserParametersData;
#if WITH_EDITOR
class UPCGEditorGraph;
struct FEdGraphPinType;
#endif // WITH_EDITOR

enum class EPCGGraphParameterEvent
{
	GraphChanged,
	GraphPostLoad,
	Added,
	RemovedUnused,
	RemovedUsed,
	PropertyMoved,
	PropertyRenamed,
	PropertyTypeModified,
	ValueModifiedLocally,
	ValueModifiedByParent,
	MultiplePropertiesAdded,
	UndoRedo,
	CategoryChanged,
	MetadataModified,
	None
};

#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPCGGraphChanged, UPCGGraphInterface* /*Graph*/, EPCGChangeType /*ChangeType*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPCGGraphStructureChanged, UPCGGraphInterface* /*Graph*/);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnPCGGraphParametersChanged, UPCGGraphInterface* /*Graph*/, EPCGGraphParameterEvent /*ChangeType*/, FName /*ChangedPropertyName*/);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPCGNodeSourceCompiled, const UPCGNode*, const FPCGCompilerDiagnostics&);
#endif // WITH_EDITOR

UENUM()
enum class EPCGGraphUsage
{
	/** Used for graphs running on pcg components. */
	Standard,
	/** Used for standalone graphs that generate asset(s) and can be run from the content browser. */
	Asset,
	/** Used for standalone graphs that affect the currently loaded level and can be run from the content browser. */
	Level
};

/**
* Extended version of FInstancedPropertyBag, to support overrides and have a custom UI for it
* Must only be used with PCGGraphInstances.
* TODO: Should be made generic and moved to ScriptUtils.
*/
USTRUCT()
struct FPCGOverrideInstancedPropertyBag
{
	GENERATED_BODY()

public:
	/** Add/Remove given property from overrides, and reset its value if it is removed. Returns true if the value was changed. */
	UE_API bool UpdatePropertyOverride(const FProperty* InProperty, bool bMarkAsOverridden, const FInstancedPropertyBag* ParentUserParameters);

	/** Reset overridden property to its parent value. Return true if the value was different. */
	UE_API bool ResetPropertyToDefault(const FProperty* InProperty, const FInstancedPropertyBag* ParentUserParameters);

#if WITH_EDITOR
	/** Get parent value of overridden property for re-importing. Will set bIsDifferent true if the value for the property in this struct is different from the one in the parent struct. */
	UE_API FString GetDefaultPropertyValueForEditor(const FProperty* InProperty, const FInstancedPropertyBag* ParentUserParameters, bool& bIsDifferent) const;
#endif // WITH_EDITOR

	/** Return if the property is currently marked overridden. */
	UE_API bool IsPropertyOverridden(const FProperty* InProperty) const;

	/** Return if the property is currently marked overridden and has a different value than its default value. */
	UE_API bool IsPropertyOverriddenAndNotDefault(const FProperty* InProperty, const FInstancedPropertyBag* ParentUserParameters) const;

	/** Reset the struct. */
	UE_API void Reset();

	/** Return if the parameters are valid */
	bool IsValid() { return Parameters.IsValid(); }

	/** Will migrate to a new property bag instanced, and will remove porperties that doesn't exists anymore or have changed types. */
	UE_API void MigrateToNewBagInstance(const FInstancedPropertyBag& NewBagInstance);

	/** Handle the sync between the parent parameters and the overrides. Will return true if something changed. */
	UE_API bool RefreshParameters(const FInstancedPropertyBag* ParentUserParameters, EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName);

	UPROPERTY(EditAnywhere, Category = "", meta = (ShowOnlyInnerProperties))
	FInstancedPropertyBag Parameters;

	UPROPERTY(VisibleAnywhere, Category = "", meta = (Hidden))
	TSet<FGuid> PropertiesIDsOverridden;
};

/**
 * Struct used to restore graph editor per document settings
 */
USTRUCT()
struct FPCGGraphDocumentInfo
{
	GENERATED_BODY()

	/** Edited object */
	UPROPERTY()
	FSoftObjectPath EditedObjectPath;

	/** Saved view position */
	UPROPERTY()
	FVector2f SavedViewOffset = FVector2f::ZeroVector;

	/** Saved zoom amount */
	UPROPERTY()
	float SavedZoomAmount = -1.0f;

	FPCGGraphDocumentInfo() = default;

	FPCGGraphDocumentInfo(UObject* InEditedObject, const FVector2f& InSavedViewOffset, float InSavedZoomAmount)
		: EditedObjectPath(InEditedObject)
		, SavedViewOffset(InSavedViewOffset)
		, SavedZoomAmount(InSavedZoomAmount)
	{
	}

	friend bool operator==(const FPCGGraphDocumentInfo& LHS, const FPCGGraphDocumentInfo& RHS)
	{
		return LHS.EditedObjectPath == RHS.EditedObjectPath && LHS.SavedViewOffset == RHS.SavedViewOffset && LHS.SavedZoomAmount == RHS.SavedZoomAmount;
	}
};

UCLASS(MinimalAPI, BlueprintType, Abstract, ClassGroup = (Procedural))
class UPCGGraphInterface : public UObject
{
	GENERATED_BODY()

public:
	/** Return the underlying PCG Graph for this interface. */
	UFUNCTION(BlueprintCallable, Category = Graph)
	UPCGGraph* GetMutablePCGGraph() { return GetGraph(); }

	/** Return the underlying PCG Graph for this interface. */
	UFUNCTION(BlueprintCallable, Category = Graph)
	const UPCGGraph* GetConstPCGGraph() const { return GetGraph(); }

	virtual UPCGGraph* GetGraph() PURE_VIRTUAL(UPCGGraphInterface::GetGraph, return nullptr;)
	virtual const UPCGGraph* GetGraph() const PURE_VIRTUAL(UPCGGraphInterface::GetGraph, return nullptr;)

	virtual const FInstancedPropertyBag* GetUserParametersStruct() const PURE_VIRTUAL(UPCGGraphInterface::GetUserParametersStruct, return nullptr;)

	// Mutable version - should not be used outside of testing. Use UpdateUserParametersStruct on PCGGraph,
	// or the different get/set/update functions in PCGGraphInterface instead to have proper callbacks.
	FInstancedPropertyBag* GetMutableUserParametersStruct_Unsafe() const { return const_cast<FInstancedPropertyBag*>(GetUserParametersStruct()); }

	UE_API bool IsInstance() const;

	/** A graph interface is equivalent to another graph interface if they are the same (same ptr), or if they have the same graph. Will be overridden when graph instance supports overrides. */
	UE_API virtual bool IsEquivalent(const UPCGGraphInterface* Other) const;

	/** Construct a data that will contain a copy of the UserParameters, in order to freeze them while executing a graph. */
	UE_API UPCGUserParametersData* ConstructUserParametersData(FPCGContext* InContext) const;

	UE_API virtual FText GetDisplayName() const;

#if WITH_EDITOR
	FOnPCGGraphChanged OnGraphChangedDelegate;
	FOnPCGGraphParametersChanged OnGraphParametersChangedDelegate;
	FOnPCGNodeSourceCompiled OnNodeSourceCompiledDelegate;

	virtual TOptional<FText> GetTitleOverride() const { return bOverrideTitle ? Title : TOptional<FText>(); }
	virtual TOptional<FLinearColor> GetColorOverride() const { return bOverrideColor ? Color : TOptional<FLinearColor>(); }
	virtual TOptional<FPCGGraphToolData> GetGraphToolData() const { return {}; }
	
	UE_API static bool IsStandaloneGraphAsset(const FAssetData& InAssetData);
	UE_API static EPCGGraphUsage GetGraphUsage(const FAssetData& InAssetData);

	UFUNCTION()
	UE_API bool IsStandaloneGraph() const;

	UE_API EPCGGraphUsage GetGraphUsage() const;

	UFUNCTION()
	UE_API virtual UObject* GetUserParameterHierarchyRoot() const; 
protected:
	/** By default export to library is visible only for graph that are assets, but can be enabled/disabled if needed. */
	UFUNCTION()
	virtual bool IsExportToLibraryEnabled() const { return IsAsset(); }

	UFUNCTION()
	virtual bool AreOverridesEnabled() const { return IsExportToLibraryEnabled() && bExposeToLibrary; }

	UFUNCTION()
	virtual bool IsTemplatePropertyEnabled() const { return false; }

	UE_API bool VerifyAndUpdateIfGraphParameterValueChanged(const FPropertyChangedChainEvent& PropertyChangedEvent);
	UE_API bool VerifyIfGraphCustomizationChanged(const FPropertyChangedChainEvent& PropertyChangedEvent);
#endif // WITH_EDITOR

public:

#if WITH_EDITORONLY_DATA
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (EditCondition = "IsExportToLibraryEnabled", EditConditionHides, PCGNoHash))
	bool bOverrideTitle = false;

	/** Override of the title for the subgraph node for this graph. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (EditCondition = "IsExportToLibraryEnabled && bOverrideTitle", EditConditionHides, PCGNoHash))
	FText Title;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, meta = (EditCondition = "IsExportToLibraryEnabled", EditConditionHides, PCGNoHash))
	bool bOverrideColor = false;

	/** Override of the color for the subgraph node for this graph. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, meta = (EditCondition = "IsExportToLibraryEnabled && bOverrideColor", EditConditionHides, PCGNoHash))
	FLinearColor Color = FLinearColor::White;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (EditCondition = "IsTemplatePropertyEnabled", EditConditionHides, PCGNoHash))
	bool bIsTemplate = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (EditCondition = "IsExportToLibraryEnabled", EditConditionHides, PCGNoHash))
	bool bExposeToLibrary = false;

	/** When enabled and standalone, it adds the action in the asset explorer to trigger a generation. Invisible, to be set in the CDO of child class to disable. */
	UPROPERTY(AssetRegistrySearchable, meta = (PCGNoHash))
	bool bExposeGenerationInAssetExplorer = true;

	/** Set of documents that were being edited in this pcg graph, so we can open them right away */
	UPROPERTY()
	TArray<FPCGGraphDocumentInfo> LastEditedDocuments;
#endif

	template <typename T>
	TValueOrError<T, EPropertyBagResult> GetGraphParameter(const FName PropertyName) const
	{
		const FInstancedPropertyBag* UserParameters = GetUserParametersStruct();
		check(UserParameters);

		if constexpr (UE::CUEnum<T>)
		{
			return FPCGGraphParameterExtension::GetGraphParameter<T>(*UserParameters, PropertyName, StaticEnum<T>());
		}
		else
		{
			return FPCGGraphParameterExtension::GetGraphParameter<T>(*UserParameters, PropertyName);
		}
	}

	virtual bool IsGraphParameterOverridden(const FName PropertyName) const { return false; }

	template <typename T>
	EPropertyBagResult SetGraphParameter(const FName PropertyName, const T& Value)
	{
		FInstancedPropertyBag* UserParameters = GetMutableUserParametersStruct();
		check(UserParameters);

		EPropertyBagResult Result;
		if constexpr (UE::CUEnum<T>)
		{
			Result = FPCGGraphParameterExtension::SetGraphParameter(*UserParameters, PropertyName, Value, StaticEnum<T>());
		}
		else
		{
			Result = FPCGGraphParameterExtension::SetGraphParameter<T>(*UserParameters, PropertyName, Value);
		}

		if (Result == EPropertyBagResult::Success)
		{
			OnGraphParametersChanged(EPCGGraphParameterEvent::ValueModifiedLocally, PropertyName);
		}

		return Result;
	}

	UE_API EPropertyBagResult SetGraphParameter(const FName PropertyName, const uint64 Value, const UEnum* Enum);

#if WITH_EDITOR
	/** Attempts a rename of a user parameter. */
	UE_API EPropertyBagAlterationResult RenameUserParameter(FName CurrentName, FName NewName);
#endif // WITH_EDITOR

	/**
	 * Allows to manipulate directly FPropertyBagArrayRef, while propagating changes to child instances. 
	 * @param PropertyName Name of the property to access. Must be an Array.
	 * @param Callback Callback to call with the FPropertyBagArrayRef. Returns a bool telling if there was a change.
	 * @returns True if the change succeeded.
	 */
	UE_API bool UpdateArrayGraphParameter(const FName PropertyName, TFunctionRef<bool(FPropertyBagArrayRef& PropertyBagArrayRef)> Callback);

	/**
	 * Allows to manipulate directly FPropertyBagSetRef, while propagating changes to child instances. 
	 * @param PropertyName Name of the property to access. Must be a Set.
	 * @param Callback Callback to call with the FPropertyBagSetRef. Returns a bool telling if there was a change.
	 * @returns True if the change succeeded.
	 */
	UE_API bool UpdateSetGraphParameter(const FName PropertyName, TFunctionRef<bool(FPropertyBagSetRef& PropertyBagSetRef)> Callback);

	virtual void OnGraphParametersChanged(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName) PURE_VIRTUAL(UPCGGraphInterface::OnGraphParametersChanged, )

protected:
	virtual FInstancedPropertyBag* GetMutableUserParametersStruct() PURE_VIRTUAL(UPCGGraphInterface::GetMutableUserParametersStruct, return nullptr;)

	/** Detecting if we need to refresh the graph depending on the type of change in the Graph Parameter. */
	UE_API EPCGChangeType GetChangeTypeForGraphParameterChange(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName);
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural), hidecategories=(Object))
class UPCGGraph : public UPCGGraphInterface
{
#if WITH_EDITOR
	friend class FPCGEditor;
	friend class FPCGEditorModule;
	friend class FPCGSubgraphHelpers;
	friend class FPCGAssetHelpers;
#endif // WITH_EDITOR
	friend class UPCGSubgraphSettings;

	GENERATED_BODY()

public:
	UE_API UPCGGraph(const FObjectInitializer& ObjectInitializer);
	/** ~Begin UObject interface */
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual bool IsEditorOnly() const override;
	UE_API virtual bool NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const override;
	UE_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	UE_API void SetupEditorData();
	
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	UE_API virtual void BeginDestroy() override;
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

#if WITH_EDITOR
	UE_API virtual void PreEditChange(FProperty* InProperty) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent);
	UE_API virtual void PostEditUndo() override;
	
	virtual TOptional<FPCGGraphToolData> GetGraphToolData() const override { return ToolData; }

	virtual TSubclassOf<UPCGGraph> GetEmbeddedSubgraphClass() const { return UPCGGraph::StaticClass(); }

	UE_API UPCGGraph* AddNewEmbeddedSubgraph();
	UE_API void DeleteEmbeddedSubgraph(UPCGGraph* InEmbeddedSubgraph);
#endif
	UE_API bool ContainsEmbeddedSubgraph(UPCGGraph* InEmbeddedSubgraph) const;
	UE_API void DuplicateEmbeddedSubgraphs(const TArray<UPCGGraph*>& InEmbeddedSubgraphs, TMap<UPCGGraph*, UPCGGraph*>& OutDuplicatedGraphs);

	virtual bool SupportsEmbeddedSubgraphs() const { return !IsEmbeddedSubgraph(); }
	UPCGGraph* GetEmbeddedParentGraph() const { return GetTypedOuter<UPCGGraph>(); }

	UFUNCTION()
	bool IsEmbeddedSubgraph() const { return !!GetEmbeddedParentGraph(); }
	
	const TArray<TObjectPtr<UPCGGraph>> GetEmbeddedSubgraphs() { return EmbeddedSubgraphs; }

	/** ~End UObject interface */

	/** ~Begin UPCGGraphInterface interface */
	virtual UPCGGraph* GetGraph() override { return this; }
	virtual const UPCGGraph* GetGraph() const override { return this; }
#if WITH_EDITOR
	virtual UObject* GetUserParameterHierarchyRoot() const override { return UserParameterHierarchyRoot; }
#endif
	/** ~End UPCGGraphInterface interface */

	/** Default grid for generation. For hierarchical generation, nodes outside of grid size graph ranges will generate on this grid. */
	UE_API EPCGHiGenGrid GetDefaultGrid() const;

	/** Default grid size for generation. For hierarchical generation, nodes outside of grid size graph ranges will generate on this grid. */
	UE_API uint32 GetDefaultGridSize() const;

	UE_API bool IsHierarchicalGenerationEnabled() const;
	UE_API bool Use2DGrid() const;

	bool HasDefaultConstructedInputs() const { return bHasDefaultConstructedInputs; }

	using FComputeGraphInstanceKey = TPair</*GridSize*/uint32, /*Compute graph index*/int32>;
	using FComputeGraphInstancePool = TMap<FComputeGraphInstanceKey, TArray<TSharedPtr<FComputeGraphInstance>>>;

	/** Attempt to retrieve a pooled compute graph instance for the given key. Creates a new instance if one could not be found. */
	UE_API TSharedPtr<FComputeGraphInstance> RetrieveComputeGraphInstanceFromPool(const FComputeGraphInstanceKey& InKey, bool& bOutNewInstance) const;

	/** Places given compute graph instance into pool. No-ops if pooling is disabled. */
	UE_API void ReturnComputeGraphInstanceToPool(const FComputeGraphInstanceKey& InKey, TSharedPtr<FComputeGraphInstance> InInstance) const;

#if WITH_EDITORONLY_DATA
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta=(PCGNoHash))
	FText Category;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta=(Multiline, PCGNoHash))
	FText Description;

	/** Marks the graph to be not refreshed automatically when the landscape changes, even if it is used. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Advanced", meta=(PCGNoHash, EditCondition="!IsStandaloneGraph() && !IsEmbeddedSubgraph()", EditConditionHides))
	bool bIgnoreLandscapeTracking = false;

	UPROPERTY(EditAnywhere, Category = Customization, meta=(PCGNoHash, EditCondition="ShowGraphCustomization()", EditConditionHides))
	FPCGGraphEditorCustomization GraphCustomization;

	/** Contains the data relevant for PCG Editor Mode usage. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta=(PCGNoHash, EditCondition="ShowToolData()", EditConditionHides))
	FPCGGraphToolData ToolData;

	/** Controls whether a graph can be executed outside of the context of a PCG actor/entity, e.g. run on for asset or level workflows. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (PCGNoHash, EditCondition = "CanToggleStandaloneGraph()", EditConditionHides))
	EPCGGraphUsage GraphUsageContext = EPCGGraphUsage::Standard;

	/** When enabled, this graph can be executed outside of the world using an editor execution source. */
	UE_DEPRECATED(5.8, "Use GraphUsageContext instead.")
	UPROPERTY(BlueprintReadWrite, Category = AssetInfo, meta = (PCGNoHash, DeprecatedProperty, DeprecationMessage = "Deprecated in favor of GraphUsageContext"))
	bool bIsStandaloneGraph = false;
#endif

	UPROPERTY(EditAnywhere, Category = "Settings|Advanced", meta=(EditCondition="!IsStandaloneGraph() && !IsEmbeddedSubgraph()", EditConditionHides))
	bool bLandscapeUsesMetadata = true;

	/** Creates a node using the given settings interface. Does not manage ownership - done outside of this method. */
	UE_API UPCGNode* AddNode(UPCGSettingsInterface* InSettings);

	/** Creates a default node based on the settings class wanted. Returns the newly created node. */
	UFUNCTION(BlueprintCallable, Category = Graph, meta=(DeterminesOutputType = "InSettingsClass", DynamicOutputParam = "DefaultNodeSettings"))
	UE_API UPCGNode* AddNodeOfType(TSubclassOf<class UPCGSettings> InSettingsClass, UPCGSettings*& DefaultNodeSettings);

	template <typename T, typename = typename std::enable_if_t<std::is_base_of_v<UPCGSettings, T>>>
	UPCGNode* AddNodeOfType(T*& DefaultNodeSettings);

	/** Creates a node containing an instance to the given settings. Returns the created node. */
	UFUNCTION(BlueprintCallable, Category = Graph)
	UE_API UPCGNode* AddNodeInstance(UPCGSettings* InSettings);

	/** Creates a node and copies the input settings. Returns the created node. */
	UFUNCTION(BlueprintCallable, Category = Graph, meta = (DeterminesOutputType = "InSettings", DynamicOutputParam = "OutCopiedSettings"))
	UE_API UPCGNode* AddNodeCopy(const UPCGSettings* InSettings, UPCGSettings*& DefaultNodeSettings);

	/** Removes a node from the graph. */
	UFUNCTION(BlueprintCallable, Category = Graph)
	UE_API void RemoveNode(UPCGNode* InNode);

	/** Bulk removal of nodes, to avoid notifying the world everytime. */
	UFUNCTION(BlueprintCallable, Category = Graph)
	UE_API void RemoveNodes(TArray<UPCGNode*>& InNodes);

	/** Adds a directed edge in the graph. Returns the "To" node for easy chaining */
	UFUNCTION(BlueprintCallable, Category = Graph)
	UE_API UPCGNode* AddEdge(UPCGNode* From, const FName& FromPinLabel, UPCGNode* To, const FName& ToPinLabel);

	/** Removes an edge in the graph. Returns true if an edge was removed. */
	UFUNCTION(BlueprintCallable, Category = Graph)
	UE_API bool RemoveEdge(UPCGNode* From, const FName& FromLabel, UPCGNode* To, const FName& ToLabel);

	/** Returns the graph input node */
	UFUNCTION(BlueprintCallable, Category = Graph)
	UPCGNode* GetInputNode() const { return InputNode; }

	/** Returns the graph output node */
	UFUNCTION(BlueprintCallable, Category = Graph)
	UPCGNode* GetOutputNode() const { return OutputNode; }

	/** Returns the default graph input pins (from the input node), which exclude the parameter pins, execution dependency pin, etc. */
	UE_API TArray<FPCGPinProperties> DefaultInputPinProperties() const;

	/** Returns the default graph output pins (from the input node), which exclude the parameter pins, execution dependency pin, etc. */
	UE_API TArray<FPCGPinProperties> DefaultOutputPinProperties() const;

	/** Returns all the edges of the graph */
	UFUNCTION(BlueprintCallable, Category = Graph)
	UE_API TArray<UPCGEdge*> GetAllEdges() const;

	/** Duplicate a given node by creating a new node with the same settings and properties, but without any edges and add it to the graph */
	UE_API TObjectPtr<UPCGNode> ReconstructNewNode(const UPCGNode* InNode);

	/** Creates an edge between two nodes/pins based on the labels. Returns true if the To node has removed other edges (happens with single pins) */
	UE_API bool AddLabeledEdge(UPCGNode* From, const FName& InboundLabel, UPCGNode* To, const FName& OutboundLabel);

	/** Returns true if the current graph contains directly the specified node. This does not query recursively (through subgraphs). */
	UE_API bool Contains(UPCGNode* Node) const;

	/** Returns true if the current graph contains a subgraph node using statically the specified graph, recursively. */
	UE_API bool Contains(const UPCGGraph* InGraph) const;

	/** Returns the node with the given settings in the graph, if any */
	UE_API UPCGNode* FindNodeWithSettings(const UPCGSettingsInterface* InSettings, bool bRecursive = false) const;
	
	UE_API TArray<UPCGNode*> FindNodesWithSettings(const TSubclassOf<UPCGSettingsInterface> InSettingsClass, bool bRecursive = false) const;

	/** Returns the first node that matches the given name in the graph, if any.*/
	UE_API UPCGNode* FindNodeByTitleName(FName NodeTitle, bool bRecursive = false, TSubclassOf<const UPCGSettings> OptionalClass = {}) const;

	const TArray<UPCGNode*>& GetNodes() const { return Nodes; }
	UE_API void AddNode(UPCGNode* InNode);
	UE_API void AddNodes(TArray<UPCGNode*>& InNodes);

	/** Calls the lambda on every node in the graph or until the Action call returns false */
	UE_API bool ForEachNode(TFunctionRef<bool(UPCGNode*)> Action) const;

	/** Calls the lambda on every node (going through subgraphs too) or until the Action call returns false */
	UE_API bool ForEachNodeRecursively(TFunctionRef<bool(UPCGNode*)> Action) const;

	UE_API bool RemoveInboundEdges(UPCGNode* InNode, const FName& InboundLabel);
	UE_API bool RemoveOutboundEdges(UPCGNode* InNode, const FName& OutboundLabel);

	/** Determine the relevant grid sizes by inspecting all HiGenGridSize nodes. */
	UE_API void GetGridSizes(PCGHiGenGrid::FSizeArray& OutGridSizes, bool& bOutHasUnbounded) const;

	/** Returns all parent grid sizes for the given child grid size, calculated by inspecting nodes. */
	UE_API void GetParentGridSizes(const uint32 InChildGridSize, PCGHiGenGrid::FSizeArray& OutParentGridSizes) const;

	/** Returns multiplier on grid size. */
	double GetGridSizeMultiplier() const { return PCGHiGenGrid::SanitizeGridSizeMultiplier(HiGenGridSizeMultiplier); }

	/** Gets generation radius from grid, considering grid size multiplier. */
	UE_API double GetGridGenerationRadiusFromGrid(uint32 Grid) const;

	/** Gets cleanup radius from grid, considering grid size multiplier. */
	UE_API double GetGridCleanupRadiusFromGrid(uint32 Grid) const;

	/** Allows executing runtime PCG graphs without partition actors and local PCG components. */
	bool UseActorComponentlessGeneration() const;

#if WITH_EDITOR
	UPCGEditorGraph* GetEditorGraph() const { return PCGEditorGraph; }

	/** Can be overriden by child class to disable debug globally on all settings. */
	virtual bool ShouldDisplayDebuggingProperties() const { return true; };
	
	UE_API void DisableNotificationsForEditor();
	UE_API void EnableNotificationsForEditor();
	UE_API void ToggleUserPausedNotificationsForEditor();
	bool NotificationsForEditorArePausedByUser() const { return bUserPausedNotificationsInGraphEditor; }

	UFUNCTION(BlueprintCallable, Category = "Graph|Advanced")
	UE_API void ForceNotificationForEditor(EPCGChangeType ChangeType = EPCGChangeType::Structural);

	UE_API void PreNodeUndo(UPCGNode* InPCGNode);
	UE_API void PostNodeUndo(UPCGNode* InPCGNode);

	const TArray<TObjectPtr<UObject>>& GetExtraEditorNodes() const { return ExtraEditorNodes; }
	UE_API void SetExtraEditorNodes(const TArray<TObjectPtr<const UObject>>& InNodes);

	const TArray<FPCGGraphCommentNodeData>& GetCommentNodes() const { return CommentNodes; }
	void SetCommentNodes(TArray<FPCGGraphCommentNodeData> InNodes) { CommentNodes = std::move(InNodes); }
	UE_API void RemoveCommentNode(const FGuid& InNodeGUID);

	UE_API void RemoveExtraEditorNode(const UObject* InNode);

	bool IsInspecting() const { return bIsInspecting; }
	UE_API void EnableInspection(const FPCGStack& InInspectedStack);
	UE_API void DisableInspection();
	bool DebugFlagAppliesToIndividualComponents() const { return bDebugFlagAppliesToIndividualComponents; }
	UE_API bool IsSameInspectedStack(const FPCGStack& InOtherStack) const;

	UE_DEPRECATED(5.8, "Compare using the IsSameInspectedStack method.")
	UE_API FPCGStack GetInspectedStack() const;
	
	/** Instruct the graph compiler to cache the relevant permutations of this graph. */
	UE_API bool PrimeGraphCompilationCache();

	/** Trigger a recompilation of the relevant permutations of this graph and check for change in the compiled tasks. */
	UE_API bool Recompile();

	UE_API void OnPCGQualityLevelChanged();

	/** Overridable function for child classes to have a graph-wide node title sanitization when the title of a node changes. Returns true if the title have changed. */
	virtual bool SanitizeNodeTitle(FName& InOutTitle, const UPCGNode* InNode) const { return false; }

	UE_API const TArray<TSubclassOf<UObject>>& GetChangeTrackingIgnoredClasses() const;
#endif

#if WITH_EDITOR
	UE_API FPCGSelectionKeyToSettingsMap GetTrackedSelectionKeysToSettings() const;
	UE_API void GetTrackedSelectionKeysToSettings(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const;
	UE_API bool UpdateTrackedSelectionKeys(FPCGSelectionKeyToSettingsMap& InExistingKeys, FPCGSelectionKeyToSettingsMap& InOutUpdatedKeys, TArray<FPCGSelectionKey>* OptionalChangedKeys) const;

	UE_DEPRECATED(5.8, "Use GetTrackedSelectionKeysToSettings instead")
	FPCGSelectionKeyToSettingsMap GetTrackedActorKeysToSettings() const { return GetTrackedSelectionKeysToSettings(); }

	UE_DEPRECATED(5.8, "Use GetTrackedSelectionKeysToSettings instead")
	void GetTrackedActorKeysToSettings(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const { GetTrackedSelectionKeysToSettings(OutKeysToSettings, OutVisitedGraphs); }
#endif

	/** Size of grid on which this node should be executed. Nodes execute at the minimum of all input grid sizes. */
	UE_API uint32 GetNodeGenerationGridSize(const UPCGNode* InNode, uint32 InDefaultGridSize) const;

	TObjectPtr<UPCGGraphCompilationData> GetCookedCompilationData() { return CookedCompilationData; }
	const TObjectPtr<UPCGGraphCompilationData> GetCookedCompilationData() const { return CookedCompilationData; }

	/** Returns true if ShouldCook is enabled for the default or any per-platform override. When false, the graph is effectively editor-only and will be excluded from all cooked builds. */
	bool AnyPlatformShouldCook() const;

	/** Per-target cook decision for this graph. Embedded subgraphs inherit from their parent. */
	UE_API bool ShouldCookForPlatform(const ITargetPlatform* TargetPlatform) const;

protected:
#if WITH_EDITOR
	UFUNCTION()
	virtual bool CanToggleStandaloneGraph() const { return !IsEmbeddedSubgraph(); }

	UFUNCTION()
	virtual bool SupportHierarchicalGeneration() const { return !IsStandaloneGraph(); }

	UFUNCTION()
	virtual bool CanEditHierarchicalGeneration() const { return SupportHierarchicalGeneration() && !IsEmbeddedSubgraph(); }

	UFUNCTION()
	virtual bool ShowGraphCustomization() const { return !IsEmbeddedSubgraph(); }

	UFUNCTION()
	virtual bool ShowToolData() const { return !IsEmbeddedSubgraph(); }

	virtual bool IsTemplatePropertyEnabled() const override { return IsAsset(); }

	/** Mark the input node hidden/not hidden. */
	PCG_API void SetHiddenFlagInputNode(bool bHidden);
	
	/** Mark the output node hidden/not hidden. */
	PCG_API void SetHiddenFlagOutputNode(bool bHidden);
#endif

	/** Internal function to react to add/remove nodes. bNotify can be set to false to not notify the world. */
	UE_API void OnNodeAdded(UPCGNode* InNode, bool bNotify = true);
	UE_API void OnNodesAdded(TArrayView<UPCGNode*> InNodes, bool bNotify = true);
	UE_API void OnNodeRemoved(UPCGNode* InNode, bool bNotify = true);
	UE_API void OnNodesRemoved(TArrayView<UPCGNode*> InNodes, bool bNotify = true);

	UE_API void RemoveNodes_Internal(TArrayView<UPCGNode*> InNodes);
	UE_API void AddNodes_Internal(TArrayView<UPCGNode*> InNodes);

	UE_API bool IsEditorOnly_Internal() const;

	UE_API bool ForEachNodeRecursively_Internal(TFunctionRef<bool(UPCGNode*)> Action, TSet<const UPCGGraph*>& VisitedGraphs) const;

	void CacheGridSizesInternalNoLock() const;

	/** Calculates node grid size. Not thread safe, must be called within write lock. */
	UE_API uint32 CalculateNodeGridSizeRecursive_Unsafe(const UPCGNode* InNode, uint32 InDefaultGridSize) const;

	/** Calculates all parent grid sizes that a given node depends on. Not thread safe, must be called within write lock. */
	UE_API PCGHiGenGrid::FSizeArray CalculateNodeGridSizesRecursiveNoLock(const UPCGNode* InNode, uint32 InDefaultGridSize) const;

	UPROPERTY(BlueprintReadOnly, Category = Graph, meta = (NoResetToDefault))
	TArray<TObjectPtr<UPCGNode>> Nodes;

	// Add input/output nodes
	UPROPERTY(BlueprintReadOnly, Category = Graph, meta = (NoResetToDefault, PCGNoHash))
	TObjectPtr<UPCGNode> InputNode;

	UPROPERTY(BlueprintReadOnly, Category = Graph, meta = (NoResetToDefault, PCGNoHash))
	TObjectPtr<UPCGNode> OutputNode;

#if WITH_EDITORONLY_DATA
	// Extra data to hold information that is useful only in editor
	UPROPERTY(meta=(PCGNoHash))
	TArray<TObjectPtr<UObject>> ExtraEditorNodes;

	// Extra data to hold information for comments
	UPROPERTY(meta=(PCGNoHash))
	TArray<FPCGGraphCommentNodeData> CommentNodes;

	// Editor graph created from PCG Editor but owned by this, reference is collected using AddReferencedObjects
	TObjectPtr<UPCGEditorGraph> PCGEditorGraph = nullptr;
	
	UPROPERTY(meta=(PCGNoHash))
	TObjectPtr<UObject> UserParameterHierarchyRoot;

	UE_API void OnUserParameterHierarchyModified();
#endif // WITH_EDITORONLY_DATA

	// Parameters
	UPROPERTY(EditAnywhere, Category = "Graph Parameters", DisplayName = "Parameters", meta = (NoResetToDefault, DefaultType = "EPropertyBagPropertyType::Double", IsPinTypeAccepted = "UserParametersIsPinTypeAccepted", CanRemoveProperty = "UserParametersCanRemoveProperty", ChildRowFeatures = "Extended", PropertyBagSchemaClass="/Script/PCGEditor.PCGGraphParametersSchema"))
	FInstancedPropertyBag UserParameters;

	UPROPERTY()
	TArray<TObjectPtr<UPCGGraph>> EmbeddedSubgraphs;

#if WITH_EDITOR
	UFUNCTION(BlueprintInternalUseOnly)
	UE_API bool UserParametersIsPinTypeAccepted(FEdGraphPinType InPinType, bool bIsChild);

	UFUNCTION(BlueprintInternalUseOnly)
	UE_API bool UserParametersCanRemoveProperty(FGuid InPropertyID, FName InPropertyName);
#endif // WITH_EDITOR

	UE_API virtual FInstancedPropertyBag* GetMutableUserParametersStruct() override;

	UPROPERTY(EditAnywhere, Category = Settings, meta=(EditCondition="CanEditHierarchicalGeneration()", EditConditionHides))
	bool bUseHierarchicalGeneration = false;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "HiGen Default Grid Size", EditCondition = "CanEditHierarchicalGeneration() && bUseHierarchicalGeneration", EditConditionHides))
	EPCGHiGenGrid HiGenGridSize = EPCGHiGenGrid::Grid256;

	/** Scales the grid sizes allowing to use larger or smaller grids than those provided out of the box. Can be useful to align generation to other grids in the world. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "HiGen Grid Size Multiplier", EditCondition = "SupportHierarchicalGeneration && bUseHierarchicalGeneration", EditConditionHides))
	double HiGenGridSizeMultiplier = 1.0;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "2D Grid", EditCondition = "CanEditHierarchicalGeneration() && bUseHierarchicalGeneration", EditConditionHides))
	bool bUse2DGrid = true;

	/** Controls whether the default pins (In, ...) are provided data on top-level graphs automatically. Note this will eventually change to be turned off by default. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(EditCondition="!IsStandaloneGraph() && !IsEmbeddedSubgraph()", EditConditionHides))
	bool bHasDefaultConstructedInputs = true;

	/** Execution grid size for nodes. */
	mutable TMap<const UPCGNode*, uint32> NodeToGridSize;

	/** All execution grid sizes for nodes (node grid size and all parent grid sizes that it depends on). */
	mutable TMap<const UPCGNode*, PCGHiGenGrid::FSizeArray> NodeToAllGridSizes;

	mutable PCG::FSharedLock NodeToGridSizeLock;

	struct FGridInfo
	{
		PCGHiGenGrid::FSizeArray GridSizes;
		bool bHasUnbounded = false;
	};

	/** Cached information about higen grids present in graph. */
	mutable TOptional<FGridInfo> CachedGridInfo;
	mutable TMap<uint32, PCGHiGenGrid::FSizeArray> ChildGridSizeToParentGridSizes;
	mutable PCG::FLock CachedGridInfoLock;

	/**
	 * Controls whether this PCG graph is included when cooking for each platform.
	 * If disabled, the graph and all of its compiled data (compute graphs, kernel sources, pins, nodes, edges)
	 * will be stripped from cooked builds. Any PCG components or graph instances that reference this graph
	 * will also be excluded from cooking.
	 *
	 * Not exposed on embedded subgraphs: the cook decision is inherited from the parent graph.
	 */
	UPROPERTY(EditAnywhere, Category = Cooking, meta=(EditCondition="!IsStandaloneGraph() && !IsEmbeddedSubgraph()", EditConditionHides))
	FPerPlatformBool ShouldCook = true;

#if WITH_EDITORONLY_DATA
	/** When true the Debug flag in the graph editor will display debug information contextually for the selected debug object. Otherwise
	* debug information is displayed for all components using a graph (requires regenerate).
	*/
	UPROPERTY(EditAnywhere, Category = Debug, meta=(EditCondition="!IsStandaloneGraph() && !IsEmbeddedSubgraph()", EditConditionHides))
	bool bDebugFlagAppliesToIndividualComponents = true;

	UPROPERTY(EditAnywhere, Category = ChangeTracking, DisplayName = "Ignored Classes", meta = (EditCondition = "!IsEmbeddedSubgraph()", EditConditionHides))
	TArray<TSubclassOf<UObject>> ChangeTrackingIgnoredClasses;
#endif // WITH_EDITORONLY_DATA

	/**
	 * Populated during cook to prewarm graph compiler cache in standalone builds. Also necessary for GPU execution because compiling
	 * compute graphs is not supported outside of editor.
	 */
	UPROPERTY()
	TObjectPtr<UPCGGraphCompilationData> CookedCompilationData = nullptr;

private:
#if WITH_EDITORONLY_DATA
	/** Cached pin data to get type information without loading. */
	UPROPERTY(AssetRegistrySearchable)
	FPCGAssetCachedPins CachedPins;

	/** Builds and returns list of pins as visible for graph creation purposes. */
	FPCGAssetCachedPins BuildCachedPins() const;
#endif // WITH_EDITORONLY_DATA

public:
	virtual const FInstancedPropertyBag* GetUserParametersStruct() const override { return &UserParameters; }

	// Add new user parameters using an array of descriptors. Can also provide an original graph to copy the values.
	// Original Graph needs to have the properties.
	// Be careful if there is any overlap between existing parameters, that also exists in the original graph, they will be overridden by the original.
	// Best used on a brand new PCG Graph.
	UE_API EPropertyBagAlterationResult AddUserParameters(const TArray<FPropertyBagPropertyDesc>& InDescs, const UPCGGraph* InOptionalOriginalGraph = nullptr);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation", meta=(EditCondition = "!IsStandaloneGraph() && !IsEmbeddedSubgraph()", EditConditionHides))
	FPCGRuntimeGenerationRadii GenerationRadii;

	// @todo_pcg: This could be on by default/hidden and graphs which need actors/actor-components could create a target actor on demand when necessary.
	/** [EXPERIMENTAL] Avoids creating partition actors and local PCG components. Any functionality which depends on a target actor/component are not supported in this mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation|Advanced", meta=(EditCondition = "!IsStandaloneGraph() && !IsEmbeddedSubgraph()", EditConditionHides))
	bool bUseActorComponentlessGeneration = false;

	/** Console variable names that trigger a runtime PCG refresh of this graph when any of their values change. Disabled in shipping game builds. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation", meta=(EditCondition = "!IsStandaloneGraph() && !IsEmbeddedSubgraph()", EditConditionHides))
	TArray<FString> ConsoleVariablesTriggeringRefresh;

	/** Auto-refresh on render-state events: viewport resize, component re-register, scene material change, preview platform change, fast-geo recreate.
	 *  Disable for slow CPU-generated graphs that should survive incidental render-state churn. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation", meta=(EditCondition = "!IsStandaloneGraph() && !IsEmbeddedSubgraph()", EditConditionHides))
	bool bAutoRefreshOnRenderStateRefresh = true;

	/** Auto-refresh when engine scalability settings or the pcg.Quality CVar change. Disable for slow CPU-generated graphs that shouldn't tear down on quality toggles. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Runtime Generation", meta=(EditCondition = "!IsStandaloneGraph() && !IsEmbeddedSubgraph()", EditConditionHides))
	bool bAutoRefreshOnQualitySettingsChanged = true;

	UE_API virtual void OnGraphParametersChanged(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName) override;

	// Will call the callback function with a mutable property bag and will trigger the updates when it's done.
	UE_API void UpdateUserParametersStruct(TFunctionRef<void(FInstancedPropertyBag&)> Callback);

private:
	/** Pool of compute graph instances available for use. Grows when new instances are requested and when instances are returned to the pool. */
	mutable FComputeGraphInstancePool AvailableComputeGraphInstances;

	/** Used to track all valid compute graph instances that are alive for this graph. Grows monotonically until flushed. */
	mutable FComputeGraphInstancePool AllComputeGraphInstances;

#if WITH_EDITOR
	/** Sends a change notification. Demotes change if the compiled tasks are not significantly changed. */
	UE_API void NotifyGraphStructureChanged(EPCGChangeType ChangeType, bool bForce = false);

	UE_API void NotifyGraphChanged(EPCGChangeType ChangeType);

	UE_API void OnNodeChanged(UPCGNode* InNode, EPCGChangeType ChangeType);

	UE_API void NotifyGraphParametersChanged(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName);

	/** Remove invalid edges and edges to nodes that are not present in the node array. */
	UE_API void FixInvalidEdges();

	// Keep track of the previous PropertyBag, to see if we had a change in the number of properties, or if it is a rename/move.
	TObjectPtr<const UPropertyBag> PreviousPropertyBag;

	int32 GraphChangeNotificationsDisableCounter = 0;
	EPCGChangeType DelayedChangeType = EPCGChangeType::None;
	bool bDelayedChangeNotification = false;
	bool bIsNotifying = false;
	bool bUserPausedNotificationsInGraphEditor = false;
	bool bIsInspecting = false;
	FPCGStack InspectedStack;
	mutable PCG::FSharedLock InspectionStateLock;
#endif // WITH_EDITOR

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "Use version that takes uint32 grid size instead")
	double GetGridGenerationRadiusFromGrid(EPCGHiGenGrid Grid) const { return GetGridGenerationRadiusFromGrid(PCGHiGenGrid::GridToGridSize(Grid)); }
	UE_DEPRECATED(5.8, "Use version that takes uint32 grid size instead")
	double GetGridCleanupRadiusFromGrid(EPCGHiGenGrid Grid) const { return GetGridCleanupRadiusFromGrid(PCGHiGenGrid::GridToGridSize(Grid)); }
	UE_DEPRECATED(5.8, "Use GetGridSizeMultiplier instead")
	uint32 GetGridExponential() const { return FMath::RoundToInt(FMath::Log2(FMath::Max(HiGenGridSizeMultiplier, UE_KINDA_SMALL_NUMBER))); }
	UE_DEPRECATED(5.8, "Use ShouldCook instead")
	UPROPERTY()
	bool bIsEditorOnly = false;
protected:
	UE_DEPRECATED(5.8, "Use HiGenGridSizeMultiplier instead")
	UPROPERTY()
	uint32 HiGenExponential = 0;

};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural), hidecategories = (Object))
class UPCGGraphInstance : public UPCGGraphInterface
{
	GENERATED_BODY()
public:
	/** ~Begin UPCGGraphInterface interface */
	virtual UPCGGraph* GetGraph() override { return Graph ? Graph->GetGraph() : nullptr; }
	virtual const UPCGGraph* GetGraph() const override { return Graph ? Graph->GetGraph() : nullptr; }
	/** ~End UPCGGraphInterface interface */

	// ~Begin UObject interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual void BeginDestroy() override;
	virtual bool IsEditorOnly() const override { return GetGraph() && GetGraph()->IsEditorOnly(); }
	UE_API virtual bool NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const override;

#if WITH_EDITOR
	UE_API virtual void PreEditChange(FProperty* InProperty) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent);
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	// ~End UObject interface

	UE_API void SetupCallbacks();
	UE_API void TeardownCallbacks();

	UE_API virtual TOptional<FPCGGraphToolData> GetGraphToolData() const override;
#endif

	UE_API virtual void OnGraphParametersChanged(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName) override;

protected:
#if WITH_EDITOR
	UE_API void OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType);
	UE_API void NotifyGraphParametersChanged(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName);
#endif
	UE_API void OnGraphParametersChanged(UPCGGraphInterface* InGraph, EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName);
	UE_API void RefreshParameters(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName = NAME_None);
	UE_API virtual FInstancedPropertyBag* GetMutableUserParametersStruct() override;

public:
	static UE_API TObjectPtr<UPCGGraphInterface> CreateInstance(UObject* InOwner, UPCGGraphInterface* InGraph);

	UE_API void SetGraph(UPCGGraphInterface* InGraph);
	UE_API void CopyParameterOverrides(UPCGGraphInterface* InGraph);
	UE_API void UpdatePropertyOverride(const FProperty* InProperty, bool bMarkAsOverridden);
	UE_API void ResetPropertyToDefault(const FProperty* InProperty);
	bool IsPropertyOverridden(const FProperty* InProperty) const { return ParametersOverrides.IsPropertyOverridden(InProperty); }
	UE_API bool IsPropertyOverriddenAndNotDefault(const FProperty* InProperty) const;

#if WITH_EDITOR
	/** For properties to be propagated correctly, changes need to happen with ImportText.
	 * This function will make a copy of the currently overridden property ids, add/remove the InProperty (depending on bMarkAsOverridden)
	 * and return the ExportText of the set of overridden property ids, for it to be re-imported by the Editor to effectively make the change.
	 */
	UE_API FString ExportOverriddenPropertyIdsChangeForEditor(const FProperty* InProperty, bool bMarkAsOverridden, bool& bIsDifferent) const;

	 /** Same thing for the default value reset, for properties to be propagated correctly, changes need to happen with ImportText.
	  * This function will return the default value as string, to be re-imported with ImportText. bHasChanged will be set to true if
	  * the value for this instance is different than the value from the parent.
	  */
	UE_API FString GetDefaultPropertyValueForEditor(const FProperty* InProperty, bool& bIsDifferent) const;

	UFUNCTION()
	bool ShowToolDataOverrides() const { return IsAsset(); }

	UE_API virtual UObject* GetUserParameterHierarchyRoot() const override;

	UFUNCTION()
	virtual bool ShouldShowGraphProperty() const { return true; }
	virtual bool CanSaveAsAsset() const { return true; }
#endif // WITH_EDITOR

	/** Because subgraph nodes hold graph instances, delegation of the selection can't be done elsewhere, so this function gets called when the dialog is opened and will forward to the graph customization if filtering is applied. */
	UFUNCTION()
	UE_API bool GraphAssetFilter(const FAssetData& AssetData) const;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Instance, AssetRegistrySearchable, meta = (GetAssetFilter = "GraphAssetFilter", EditCondition = "ShouldShowGraphProperty", EditConditionHides))
	TObjectPtr<UPCGGraphInterface> Graph;

	UPROPERTY(EditAnywhere, Category = Instance, meta = (NoResetToDefault, PropertyBagSchemaClass="/Script/PCGEditor.PCGGraphParametersSchema"))
	FPCGOverrideInstancedPropertyBag ParametersOverrides;

	virtual const FInstancedPropertyBag* GetUserParametersStruct() const override { return &ParametersOverrides.Parameters; }
	UE_API virtual bool IsGraphParameterOverridden(const FName PropertyName) const override;

	/** 
	* When setting a graph instance as a base to another graph instance, we need to make sure we don't find this graph in the graph hierarchy.
	* Otherwise it would cause infinite recursion (like A is an instance of B and B is an instance of A).
	* This function will go up the graph hierarchy and returning false if `this` is ever encountered.
	*/
	UE_API bool CanGraphInterfaceBeSet(const UPCGGraphInterface* GraphInterface) const;

#if WITH_EDITOR
	UE_API virtual TOptional<FText> GetTitleOverride() const override;
	UE_API virtual TOptional<FLinearColor> GetColorOverride() const override;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
private:
	// Transient, to keep track of the previous graph when it changed.
	TWeakObjectPtr<UPCGGraphInterface> PreGraphCache = nullptr;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (EditCondition = "AreOverridesEnabled", EditConditionHides, DisplayAfter = ColorOverride, PCGNoHash))
	bool bOverrideDescription = false;

	/** Can override the description of this instance. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (EditCondition = "AreOverridesEnabled && bOverrideDescription", EditConditionHides, DisplayAfter = ColorOverride, PCGNoHash))
	FText Description;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (EditCondition = "AreOverridesEnabled", EditConditionHides, DisplayAfter = ColorOverride, PCGNoHash))
	bool bOverrideCategory = false;

	/** Can override the category of this instance. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (EditCondition = "AreOverridesEnabled && bOverrideCategory", EditConditionHides, DisplayAfter = ColorOverride, PCGNoHash))
	FText Category;

	/** Override information for tool usage. */
	UPROPERTY(EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (PCGNoHash, EditCondition = "ShowToolDataOverrides()", EditConditionHides))
	FPCGGraphInstanceToolDataOverrides ToolDataOverrides;
#endif // WITH_EDITORONLY_DATA
};

template <typename T, typename>
UPCGNode* UPCGGraph::AddNodeOfType(T*& DefaultNodeSettings)
{
	UPCGSettings* TempSettings = DefaultNodeSettings;
	UPCGNode* Node = AddNodeOfType(T::StaticClass(), TempSettings);
	DefaultNodeSettings = Cast<T>(TempSettings);
	return Node;
}

#undef UE_API
