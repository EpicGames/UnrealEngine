// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#if PCG_PROFILING_ENABLED

#include "PCGData.h"
#include "PCGNode.h"
#include "Graph/PCGSourceDataContainer.h"
#include "Graph/PCGStackContext.h"
#include "Graph/DataOverride/PCGDataOverride.h"
#include "Utils/PCGExtraCapture.h"

#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"

#if WITH_EDITOR
/* Helper struct to keep references on data collection UPCGData visible to GC even if inspection data source gets cleared */
struct FPCGInspectionData : public FGCObject
{
	//~ GCObject Interface
	PCG_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	PCG_API virtual FString GetReferencerName() const override;
	//~ GCObject Interface

	TSharedPtr<FPCGDataCollection> Data;
};
#endif // WITH_EDITOR

class FPCGGraphExecutionInspection
{
public:
	friend struct FPCGComponentInstanceData;

	/* Retrieves the executed nodes information */
	struct FNodeExecutedNotificationData
	{
		FNodeExecutedNotificationData(const FPCGStack& InStack, const PCGUtils::FCallTime& InTimer) : Stack(InStack), Timer(InTimer) {}
		// Important implementation note: some logic in WasNodeExecuted relies on the fact we don't use the timer for the operator== and hash functions.
		friend uint32 GetTypeHash(const FNodeExecutedNotificationData& NotifData) { return GetTypeHash(NotifData.Stack); }
		bool operator==(const FNodeExecutedNotificationData& OtherNotifData) const { return Stack == OtherNotifData.Stack; }

		FPCGStack Stack;
		PCGUtils::FCallTime Timer;
	};

	/** Called at execution time each time a node has been executed. */
	PCG_API void NotifyNodeExecuted(const UPCGNode* InNode, const FPCGStack* InStack, const PCGUtils::FCallTime* InTimer, bool bNodeUsedCache);

	/** Returns all executed node stacks. Note that this is a heavy operation. */
	PCG_API TMap<TObjectKey<const UPCGNode>, TSet<FNodeExecutedNotificationData>> GetExecutedNodeStacks() const;

	/** Returns all executed node stacks associated to the given node. */
	PCG_API TSet<FNodeExecutedNotificationData> GetExecutedNodeStacks(const UPCGNode* InNode) const;

	/** Iterates executed node stacks.  Callback returns false to stop iteration. */
	PCG_API void ForEachExecutedNodeStack(TFunctionRef<bool(const UPCGNode*, const TSet<FNodeExecutedNotificationData>&)> Callback) const;

	/** Iterates executed node stacks set for a specific node. */
	PCG_API void ForEachExecutedNodeStack(const UPCGNode* InNode, TFunctionRef<void(const TSet<FNodeExecutedNotificationData>&)> Callback) const;

	/** Monotonic counter bumped whenever the execution stacks map mutates. */
	PCG_API uint64 GetExecutedStacksGeneration() const;

	/** Clears per-node execution data. Called by the profiling log after reading in non-editor builds. */
	PCG_API void ClearExecutedNodeData();

	PCG_API void StoreInspectionData(const FPCGStack* InStack, const UPCGNode* InNode, const PCGUtils::FCallTime* InTimer, const FPCGDataCollection& InInputData, const FPCGDataCollection& InOutputData, bool bUsedCache);

#if WITH_EDITOR
	PCG_API bool IsInspecting() const;
	PCG_API void EnableInspection();
	PCG_API void DisableInspection();

	/** Data is only valid inside the function call. If you want the data to outlive the call, use GetInspectionDataPtr instead. */
	PCG_API bool InspectData(const FPCGStack& InStack, TFunctionRef<void(const FPCGDataCollection&)> InspectFunc) const;
	
	/** If found, the inspection data is returned as a garbage collection visible object (FGCObject). Prefer using InspectData if you can manage UObject references yourself. */
	PCG_API TSharedPtr<FPCGInspectionData> GetInspectionDataPtr(const FPCGStack& InStack) const;
	
	PCG_API void ClearInspectionData(bool bClearPerNodeExecutionData = true);

	/** Whether a task for the given node and stack was executed during the last execution. */
	PCG_API bool WasNodeExecuted(const UPCGNode* InNode, const FPCGStack& Stack) const;

	/** Retrieve the inactive pin bitmask for the given node and stack in the last execution. */
	PCG_API uint64 GetNodeInactivePinMask(const UPCGNode* InNode, const FPCGStack& Stack) const;

	/** Whether the given node was culled by a dynamic branch in the given stack. */
	PCG_API void NotifyNodeDynamicInactivePins(const UPCGNode* InNode, const FPCGStack* InStack, uint64 InactivePinBitmask) const;

	/** Did the given node produce one or more data items in the given stack during the last execution. */
	PCG_API bool HasNodeProducedData(const UPCGNode* InNode, const FPCGStack& Stack) const;

	PCG_API void NotifyGPUToCPUReadback(const UPCGNode* InNode, const FPCGStack* InStack) const;
	PCG_API void NotifyCPUToGPUUpload(const UPCGNode* InNode, const FPCGStack* InStack) const;

	PCG_API bool DidNodeTriggerGPUToCPUReadback(const UPCGNode* InNode, const FPCGStack& Stack) const;
	PCG_API bool DidNodeTriggerCPUToGPUUpload(const UPCGNode* InNode, const FPCGStack& Stack) const;

	/** Called at execution time when data overrides have been applied to a node's data. */
	PCG_API void NotifyDataOverridesApplied(const UPCGNode* InNode, const FPCGStack* InStack) const;

	/** Returns true if the given node had data overrides applied during the last execution. */
	PCG_API bool NodeAppliedDataOverrides(const UPCGNode* InNode, const FPCGStack& Stack) const;

	/** Replaces the set of resolved deltas by storage key. */
	PCG_API void NotifyDeltasResolved(const FPCGSourceDataStorageKey& InStorageKey, TArray<FPCGDeltaKey>&& InResolvedDeltas) const;

	/** Returns true if the given delta resolved during the last execution. */
	PCG_API bool IsDeltaResolved(const FPCGSourceDataStorageKey& InStorageKey, const FPCGDeltaKey& InDeltaKey) const;

	PCG_API void AddReferencedObjects(FReferenceCollector& Collector);

	UE_DEPRECATED(5.8, "Use GetInspectionDataPtr/InspectData instead")
	const FPCGDataCollection* GetInspectionData(const FPCGStack& InStack) const { return nullptr;}
#endif // WITH_EDITOR

private:
    /** Map from nodes to all stacks for which a task for the node was executed. */
    TMap<TObjectKey<const UPCGNode>, TSet<FNodeExecutedNotificationData>> NodeToStacksInWhichNodeExecuted;

    uint64 ExecutedStacksGeneration = 0;
    mutable PCG::FSharedLock NodeToStacksInWhichNodeExecutedLock;

#if WITH_EDITOR
	int32 InspectionCounter = 0;

	TMap<FPCGStack, TSharedPtr<FPCGDataCollection>> InspectionCache;

	mutable PCG::FSharedLock InspectionCacheLock;

	/** Map from nodes to all stacks for which the node produced at least one data item. */
	TMap<TObjectKey<const UPCGNode>, TSet<FPCGStack>> NodeToStacksThatProducedData;
	mutable PCG::FSharedLock NodeToStacksThatProducedDataLock;

	/** Map from nodes to stacks to mask of output pins that were deactivated during execution. */
	mutable TMap<TObjectKey<const UPCGNode>, TMap<const FPCGStack, uint64>> NodeToStackToInactivePinMask;
	mutable PCG::FSharedLock NodeToStackToInactivePinMaskLock;

	/** Map from nodes to all stacks for which GPU data transfers occurred. */
	mutable TMap<TObjectKey<const UPCGNode>, TSet<FPCGStack>> NodeToStacksTriggeringGPUUploads;
	mutable TMap<TObjectKey<const UPCGNode>, TSet<FPCGStack>> NodeToStacksTriggeringGPUReadbacks;
	mutable PCG::FSharedLock NodeToStacksTriggeringGPUTransfersLock;

	/** Map from nodes to all stacks for which data overrides were applied. */
	mutable TMap<TObjectKey<const UPCGNode>, TSet<FPCGStack>> NodeToStacksWithOverridesApplied;
	mutable PCG::FSharedLock NodeToStacksWithOverridesAppliedLock;

	/** Resolved deltas tracked during ApplyDataOverrides. Note: TSet could be explored if common use case grows past ~50. */
	mutable TMap<FPCGSourceDataStorageKey, TArray<FPCGDeltaKey>> ResolvedDeltas;

	mutable PCG::FSharedLock ResolvedDeltasLock;
#endif // WITH_EDITOR
};

#endif // PCG_PROFILING_ENABLED
