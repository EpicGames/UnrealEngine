// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

DECLARE_MULTICAST_DELEGATE(FPCGGraphExecutionSourcesChanged);

class FPCGGraphExecutionRegistry;

/**
 * Execution source descriptor which points to an execution source and some extra parameters, returned by the provider class.
 */
struct FPCGGraphExecutionSourceDescriptor
{
	IPCGGraphExecutionSource* ExecutionSource = nullptr;
#if WITH_EDITOR
	bool bShowInDebugger = true;
#endif
};

using FPCGGraphExecutionSourceProviderHandle = int32;

/** 
 * Registered providers are responsible for providing a list of execution sources.
 * Instead of having a catch all getting classes implementing IPCGGraphExecutionSource this gives a bit more control to implementers.
 */
class IPCGGraphExecutionSourceProvider
{
public:	
	virtual ~IPCGGraphExecutionSourceProvider() = default;

	virtual TArray<FPCGGraphExecutionSourceDescriptor> GatherExecutionSources() const = 0;
	
	void BroadcastExecutionSourcesChanged();

private:
	friend class FPCGGraphExecutionRegistry;
	FPCGGraphExecutionSourceProviderHandle Handle = INDEX_NONE;
};

/**
 * Registry for graph execution source providers.
 */
class FPCGGraphExecutionRegistry
{
public:
	PCG_API TArray<FPCGGraphExecutionSourceDescriptor> GatherExecutionSources() const;
	void Shutdown();

	PCG_API FPCGGraphExecutionSourceProviderHandle RegisterExecutionSourceProvider(TSharedRef<IPCGGraphExecutionSourceProvider> Provider);
	PCG_API void UnregisterExecutionSourceProvider(FPCGGraphExecutionSourceProviderHandle Handle);

	FPCGGraphExecutionSourcesChanged& OnGraphExecutionSourcesChanged() { return GraphExecutionSourcesChanged; }

	void BroadcastGraphExecutionSourcesChanged() { GraphExecutionSourcesChanged.Broadcast(); }
private:
	
	TArray<TSharedRef<IPCGGraphExecutionSourceProvider>> RegisteredExecutionSourceProviders;
	FPCGGraphExecutionSourcesChanged GraphExecutionSourcesChanged;
	static FPCGGraphExecutionSourceProviderHandle NextHandle;
};