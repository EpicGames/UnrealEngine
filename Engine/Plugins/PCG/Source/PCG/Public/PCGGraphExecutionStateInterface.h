// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Utils/PCGExtraCapture.h"

#include "CoreMinimal.h"
#include "Misc/SpinLock.h"
#include "UObject/Interface.h"

#include "PCGGraphExecutionStateInterface.generated.h"

struct FPCGSourceDataContainer;
class IPCGBaseSubsystem;
class ITransactionObjectAnnotation;
class UPCGNode;
class FPCGGraphExecutionInspection;
class IPCGGraphExecutionSource;
class UPCGData;
class UPCGGraph;
class UPCGGraphInstance;
class UPCGManagedResource;
class UPCGSchedulingPolicyBase;
struct FPCGGridDescriptor;
struct FPCGManagedResourceContainer;
struct FPCGRuntimeGenerationRadii;

namespace PCGUtils
{
	class FExtraCapture;
}

using FPCGDynamicTrackingPriority = double;

/**
* Interface returned by a IPCGGraphExecutionSource that is queried / updated during execution of a PCG Graph.
*/
class IPCGGraphExecutionState
{
public:
	struct FGenerateParams
	{
		/** If we should force generation even when the source is already generated. */
		bool bEvenIfAlreadyGenerated = false;

		/** If the execution source is partitioned, will forward the generate call to its registered local execution sources. */
		bool bGenerateLocalSources = true;

		/** Dependencies to wait on before the generation can execute. */
		TArray<FPCGTaskId> Dependencies = {};
	};

	struct FCleanupParams
	{
		/** If the execution source is partitioned, will forward the clean up call to its registered local execution sources. */
		bool bCleanupLocalSources = true;

		/** Performs a hard release (no re-use) of generated managed resources. */
		bool bReleaseManagedResources = true;

		/** Performs the cleanup immediately without scheduling. Ignores dependencies. */
		bool bImmediate = false;

		/** Dependencies to wait on before the cleanup can execute. */
		TArray<FPCGTaskId> Dependencies = {};
	};

	virtual ~IPCGGraphExecutionState() = default;

	/** Returns a UPCGData representation of the ExecutionState. */
	virtual UPCGData* GetSelfData() const = 0;

	/** Returns a Seed for graph execution. */
	virtual int32 GetSeed() const = 0;

	/** Returns a Debug name that can be used for logging. */
	virtual FString GetDebugName() const = 0;

	/** Returns a World, can be null */
	virtual UWorld* GetWorld() const = 0;

	/** Returns the execution target for managed resources */
	virtual UObject* GetTarget() const { return nullptr; }

	template<class TargetType>
	TargetType* GetTypedTarget() const
	{
		return Cast<TargetType>(GetTarget());
	}

	/** True if the ExecutionState is active. */
	virtual bool IsActive() const { return true; }

	/** Returns true if the ExecutionState has network authority */
	virtual bool HasAuthority() const = 0;

	/** Returns a Transform if the ExecutionState is a spatial one. */
	virtual FTransform GetTransform() const = 0;

	/** Returns a the original source Transform if the ExecutionState is a local source. */
	virtual FTransform GetOriginalTransform() const { return GetTransform(); }

	/** Returns the ExecutionState bounds if the ExecutionState is a spatial one. */
	virtual FBox GetBounds() const = 0;

	/** Returns the ExecutionState's original source bounds if this ExecutionState is a local source. */
	virtual FBox GetOriginalBounds() const { return GetBounds(); }

	/** Returns the ExecutionState's local bounds if this ExecutionState is a spatial one. */
	virtual FBox GetLocalSpaceBounds() const { return GetBounds().InverseTransformBy(GetTransform()); }

	/** Returns the ExecutionState's original source local space bounds if this ExecutionState is a local source. */
	virtual FBox GetOriginalLocalSpaceBounds() const { return GetLocalSpaceBounds(); }

	/** Returns the ExecutionState's generated bounds which includes the artifact bounds. */
	virtual FBox GetTotalBounds() const { return GetBounds(); }

	/** Returns the UPCGGraph this ExecutionState is executing. */
	virtual UPCGGraph* GetGraph() const = 0;

	/** Returns the UPCGGrahInstance this ExecutionState is executing. */
	virtual UPCGGraphInstance* GetGraphInstance() const = 0;

	/** Cancel execution of this ExecutionState. */
	virtual void Cancel() = 0;

	/** Notify ExecutionState that its execution is being aborted. */
	virtual void OnGraphExecutionAborted(bool bQuiet = false, bool bCleanupUnusedResources = true) = 0;

	/** Indicates if the current source is generating. */
	virtual bool IsGenerating() const = 0;

	/** Indicates if the current source is generated */
	virtual bool IsGenerated() const { return false; }

	/** If this source is generated, returns the generated bounds */
	virtual FBox GetGeneratedBounds() const { return FBox(EForceInit::ForceInit); }

	/** Gets the task ID for the current generation task if it exists. */
	virtual FPCGTaskId GetGenerationTaskId() const { return InvalidPCGTaskId; }

	/** Requests the execution source to generate. */
	virtual FPCGTaskId Generate(const FGenerateParams& InGenerateParams) { return InvalidPCGTaskId; }
	FPCGTaskId Generate() { return Generate(FGenerateParams()); }

	UE_DEPRECATED(5.8, "Call/Implement Generate instead.")
	virtual FPCGTaskId GenerateLocalGetTaskId(EPCGHiGenGrid Grid = EPCGHiGenGrid::Uninitialized) { return InvalidPCGTaskId; }

	/** Requests the execution source to clean up. */
	virtual FPCGTaskId Cleanup(const FCleanupParams& InCleanupParams) { return InvalidPCGTaskId; }
	FPCGTaskId Cleanup() { return Cleanup(FCleanupParams()); }

	/** Store data with a resource key that identifies the pin. */
	virtual void StoreOutputDataForPin(const FString& InResourceKey, const FPCGDataCollection& InData) const {}

	/** Lookup data using a resource key that identifies the pin. */
	virtual const FPCGDataCollection* RetrieveOutputDataForPin(const FString& InResourceKey) const { return nullptr; }

	UE_DEPRECATED(5.8, "Use GetManagedResourceContainer instead.")
	PCG_API virtual void AddToManagedResources(UPCGManagedResource* InResource);

	virtual FPCGManagedResourceContainer* GetManagedResourceContainer() { return nullptr; }
	virtual FTransactionallySafeCriticalSection* GetManagedResourceContainerLock() { return nullptr; }

	virtual void OnManagedResourceAdded(UPCGManagedResource* InResource) {}

	/** Returns the subsystem associated with this source. By default, it will return the UPCGSubsystem if the source has a world, otherwise, the EditorSubsystem (in editor only, obviously). */
	PCG_API virtual IPCGBaseSubsystem* GetSubsystem() const;

	/** Cache any data that is not thread safe or that needs to be computed only once for the whole execution */
	virtual void ExecutePreGraph(FPCGContext* InContext) {}

	/** Get the data container from the execution source. */
	virtual FPCGSourceDataContainer* GetSourceDataContainer() { return nullptr; }

	/** Get the immutable data container from the execution source. */
	virtual const FPCGSourceDataContainer* GetSourceDataContainer() const { return nullptr; }

	/** Whether this execution source uses the graph cache. */
	virtual bool IsCacheEnabled() const { return true; }

#if WITH_EDITOR
	/** Returns the FExtraCapture object for this ExecutionState */
	virtual const PCGUtils::FExtraCapture& GetExtraCapture() const = 0;
	virtual PCGUtils::FExtraCapture& GetExtraCapture() = 0;

	/** Returns the FPCGGraphExecutionInspection object for this ExecutionState */
	virtual const FPCGGraphExecutionInspection& GetInspection() const = 0;
	virtual FPCGGraphExecutionInspection& GetInspection() = 0;
#else
#if !UE_BUILD_SHIPPING // Deprecation (5.8): fallback implementations for any implementors that have not implemented these methods in !UE_BUILD_SHIPPING
	PCG_API virtual const PCGUtils::FExtraCapture& GetExtraCapture() const;
	PCG_API virtual PCGUtils::FExtraCapture& GetExtraCapture();

	PCG_API virtual const FPCGGraphExecutionInspection& GetInspection() const;
	PCG_API virtual FPCGGraphExecutionInspection& GetInspection();
#endif // !UE_BUILD_SHIPPING
#endif // WITH_EDITOR

#if WITH_EDITOR
	/** Get an execution priority */
	PCG_API virtual FPCGDynamicTrackingPriority GetDynamicTrackingPriority() const;

	/** Register tracking dependencies, so ExecutionState can be updated when they change */
	virtual void RegisterDynamicTracking(const UPCGSettings* InSettings, const TArrayView<TPair<FPCGSelectionKey, bool>>& InDynamicKeysAndCulling) = 0;

	/** Register multiple tracking dependencies, so ExecutionState can be updated when they change */
	virtual void RegisterDynamicTracking(const FPCGSelectionKeyToSettingsMap& InKeysToSettings) = 0;

	/** Returns the tracked keys */
	virtual TArray<FPCGSelectionKey> GatherTrackingKeys() const { return {}; }

	/** Return true if the key is tracked, and if so, bOutIsCulled will contains if the key is culled or not. */
	virtual bool IsKeyTrackedAndCulled(const FPCGSelectionKey& Key, bool& bOutIsCulled) const { return false; }

	/** Apply a function to all settings that track a given key. */
	virtual void ForEachSettingTrackingKey(const FPCGSelectionKey& InKey, const TFunctionRef<void(const FPCGSelectionKey&, const FPCGSettingsAndCulling&)> InCallback) const {}

	/** Add dynamic tracking keys from local source */
	virtual void AddLocalSourceDynamicTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceDynamicTrackingKeys) {}

	/** Add current tracking keys from local source */
	virtual void AddLocalSourceCurrentTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceCurrentExecutionTrackingKeys, const TSet<const UPCGSettings*>& LocalSourceCurrentExecutionTrackingSettings) {}

	/** Indicates if the current source is waiting for a refresh */
	virtual bool IsRefreshInProgress() const = 0;

	/** Whether this execution source has been generated at any point during this editor session. */
	virtual bool WasGeneratedThisSession() const { return false; }

	/** Whether this execution source will create scoped transaction around operations that impact the world/data. */
	virtual bool UseTransactions() const { return false; }
#endif // WITH_EDITOR

	/** If this is a local execution source returns the original execution source, otherwise returns self. */
	virtual IPCGGraphExecutionSource* GetOriginalSource() const = 0;

	/** Retrieves a local execution source using grid descriptor and grid coordinates, returns nullptr if no such execution source is found. */
	virtual IPCGGraphExecutionSource* GetLocalSource(const FPCGGridDescriptor& GridDescriptor, const FIntVector& CellCoords) const { return nullptr; }

	/** Returns true if this execution source uses partitioned generation. */
	virtual bool IsPartitioned() const { return false; }

	/** Returns true if this execution source is a local source of a partitioned execution source. */
	virtual bool IsLocalSource() const { return false; }

	/** Returns true if execution source should output on a 2D grid. */
	virtual bool Use2DGrid() const { return false; }

	/** Returns a GridDescriptor based on this execution source for the specified grid size. */
	PCG_API virtual FPCGGridDescriptor GetGridDescriptor(uint32 InGridSize) const;

	/** Get the size for the HiGen grid this execution source executes on. */
	virtual uint32 GetGenerationGridSize() const { return PCGHiGenGrid::UninitializedGridSize(); }

	/** Get the grid cell coordinates this execution source executes on. */
	virtual FIntVector GetGenerationGridCoords() const { return FIntVector::ZeroValue; }

	// @todo_pcg: The execution source should probably just have an execution domain instead of directly querying runtime gen vs etc.
	/** Returns true if the execution source is managed by the runtime generation system. Nothing else should generate or cleanup this execution source. */
	virtual bool IsManagedByRuntimeGenSystem() const { return false; }

	/** Gets the policy which dictates behavior when using runtime generation. */
	virtual const UPCGSchedulingPolicyBase* GetRuntimeGenSchedulingPolicy() const { return nullptr; }

	/** Gets the radii inside of which this execution source will be generated when using runtime generation. */
	PCG_API virtual const FPCGRuntimeGenerationRadii& GetGenerationRadii() const;

	/** Gets the radius inside of which this execution source will be generated when using runtime generation. */
	virtual double GetGenerationRadiusFromGrid(uint32 InGrid) const { return 0.0f; }

	/** Gets the radius outside of which this execution source will be cleaned up when using runtime generation. */
	virtual double GetCleanupRadiusFromGrid(uint32 InGrid) const { return 0.0f; }

	/** Returns true if the generated artifacts are transient */
	virtual bool IsInPreviewMode() const { return false; }

#if PCG_EXECUTION_CACHE_VALIDATION_ENABLED
	bool bExecutionCacheWriteEnabled = false;
#endif
};

UINTERFACE(BlueprintType, MinimalAPI)
class UPCGGraphExecutionSource : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/**
* Interface used by the FPCGGraphExecutor to get an IPCGGraphExecutionState used to query/update execution.
*/
class IPCGGraphExecutionSource
{
	GENERATED_IINTERFACE_BODY()

public:
	virtual IPCGGraphExecutionState& GetExecutionState() = 0;
	virtual const IPCGGraphExecutionState& GetExecutionState() const = 0;
};

USTRUCT()
struct FPCGSoftGraphExecutionSource
{
	GENERATED_BODY()

	FPCGSoftGraphExecutionSource() = default;
	PCG_API explicit FPCGSoftGraphExecutionSource(const TSoftObjectPtr<UObject>& InSoftObjectPtr);
	PCG_API FPCGSoftGraphExecutionSource(IPCGGraphExecutionSource* InSource);
	PCG_API FPCGSoftGraphExecutionSource(const IPCGGraphExecutionSource* InSource);

	// Need the copy/move constructor/assignment because of the lock.
	PCG_API FPCGSoftGraphExecutionSource(const FPCGSoftGraphExecutionSource& Other);
	PCG_API FPCGSoftGraphExecutionSource(FPCGSoftGraphExecutionSource&& Other);
	PCG_API FPCGSoftGraphExecutionSource& operator=(const FPCGSoftGraphExecutionSource& Other);
	PCG_API FPCGSoftGraphExecutionSource& operator=(FPCGSoftGraphExecutionSource&& Other);

	/** Can assign an execution source. Will reset the cached weak pointer. */
	PCG_API FPCGSoftGraphExecutionSource& operator=(const IPCGGraphExecutionSource* InSource);

	/** De-reference the weak pointer if it is valid, otherwise will resolve the soft pointer and update the cached weak. Threadsafe. */
	PCG_API IPCGGraphExecutionSource* Get() const;

	/** Resolve the soft pointer as a UObject. */
	PCG_API UObject* GetObject() const;

	/** Reset the soft pointer to null and invalidate the cached weak. */
	PCG_API void Reset();
	
	bool IsValid() const { return Get() != nullptr; }
	
	IPCGGraphExecutionSource* operator->() const { return Get(); }

	PCG_API bool operator==(const FPCGSoftGraphExecutionSource& Other) const;

	PCG_API friend int32 GetTypeHash(const FPCGSoftGraphExecutionSource& This);
	
private:
	UPROPERTY()
	TSoftObjectPtr<UObject> SoftObjectPtr;

	mutable PCG::FLock Lock;
	mutable TWeakInterfacePtr<IPCGGraphExecutionSource> CachedWeakPtr;
};

#if WITH_EDITOR
// Source UObject must have RF_Transactional for the engine to invoke FactoryTransactionAnnotation/PostEditUndo on it.
namespace PCG::Transaction
{
	/** Populated by Serialize on undo. */
	PCG_API TSharedPtr<ITransactionObjectAnnotation> CreateAnnotation();

	/** Snapshots for transactional undo/redo. Returns nullptr if Source has no container. */
	PCG_API TSharedPtr<ITransactionObjectAnnotation> CreateAnnotation(const IPCGGraphExecutionSource& Source);

	/** Restores Annotation's snapshot directly into the Source's container.  No-op if either is missing. */
	PCG_API void RestoreFromAnnotation(IPCGGraphExecutionSource& Source, const TSharedPtr<ITransactionObjectAnnotation>& Annotation);
} // namespace PCG::Transaction
#endif // WITH_EDITOR
