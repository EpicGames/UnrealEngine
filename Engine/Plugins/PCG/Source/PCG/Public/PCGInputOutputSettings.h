// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGInputOutputSettings.generated.h"

namespace PCGInputOutputConstants
{
	const FName DefaultInputLabel = TEXT("Input");
	const FName DefaultActorLabel = TEXT("Actor");
	const FName DefaultOriginalActorLabel = TEXT("OriginalActor");
	const FName DefaultLandscapeLabel = TEXT("Landscape");
	const FName DefaultLandscapeHeightLabel = TEXT("Landscape Height");
	const FName DefaultNewCustomPinName = TEXT("NewPin");
}

class FPCGInputOutputElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};

UCLASS(MinimalAPI, NotBlueprintable, Hidden, ClassGroup = (Procedural))
class UPCGGraphInputOutputSettings : public UPCGSettings
{
	GENERATED_BODY()

	// @todo_pcg: This is needed because we need the pin properties when caching them in the graph. However, if we remove the override on the Default versions
	// this wouldn't be needed anymore.
	friend class UPCGGraph; 

public:
	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface

	// ~Begin UPCGSettings interface
	virtual TArray<FPCGPinProperties> DefaultInputPinProperties() const override;
	virtual TArray<FPCGPinProperties> DefaultOutputPinProperties() const override;

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return bIsInput ? FName(TEXT("InputNode")) : FName(TEXT("OutputNode")); }
	virtual FText GetDefaultNodeTitle() const override { return bIsInput ? NSLOCTEXT("PCGGraphInputOutputSettings", "InputNodeTitle", "Input Node") : NSLOCTEXT("PCGGraphInputOutputSettings", "OutputNodeTitle", "Output Node"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
	virtual bool CanUserEditTitle() const override { return false; }
#endif // WITH_EDITOR

protected:
	virtual FPCGElementPtr CreateElement() const override { return MakeShared<FPCGInputOutputElement>(); }
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual bool HasExecutionDependencyPin() const override { return false; }

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const override;
#endif // WITH_EDITOR
	// ~End UPCGSettings interface

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TArray<FPCGPinProperties> Pins;

public:
	bool IsInput() const { return bIsInput; }
	void SetInput(bool bInIsInput);

	// Add a new custom pin
	// Note that you should use the return value of this function, since it can be different from
	// the one passed as argument. It will change if its label collides with existing pins.
	[[nodiscard]] PCG_API const FPCGPinProperties& AddPin(const FPCGPinProperties& NewCustomPinProperties);

#if WITH_EDITOR
	UFUNCTION()
	static bool SupportsComposition() { return true; }
#endif // WITH_EDITOR

protected:
	TArray<FPCGPinProperties> DefaultPinProperties(bool bInvisiblePin) const;
	void FixPinProperties();

	UPROPERTY()
	TSet<FName> PinLabels_DEPRECATED;

	UPROPERTY()
	TArray<FPCGPinProperties> CustomPins_DEPRECATED;

	UPROPERTY()
	bool bHasAddedDefaultPin = false;

	bool bIsInput = false;
};
