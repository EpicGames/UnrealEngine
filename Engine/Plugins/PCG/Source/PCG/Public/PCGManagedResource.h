// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGCrc.h"
#include "Elements/PCGSplineMeshParams.h"

#include "Engine/SplineMeshComponentDescriptor.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "Animation/AnimBank.h"

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionHandle.h"
#endif

#include "PCGManagedResource.generated.h"

#define UE_API PCG_API

class UActorComponent;
class UInstancedSkinnedMeshComponent;
class UInstancedStaticMeshComponent;
class USplineMeshComponent;
struct FPCGDataCollection;
struct FPCGTaggedData;

struct FPCGMoveResourceParams
{
	/** Class of the object that needs to be created as part of moving the resources. */
	UClass* TemplateTargetClass = nullptr;
	/** Target object to move resources to. Can be null, in which case we'll create one from the TemplateTargetClass. */
	UObject* Target = nullptr;
	/** Additional validation object owner. */
	const UObject* ExpectedPreviousOwner = nullptr;
	/** Conjunction of tags required to move a resource to the target. */
	TArray<FName> RequiredTags;
	/** Disjunction of tags that would disqualify moving a resource to the target. */
	TArray<FName> ExcludedTags;
	/** If there is generated data on the source objects, it will be moved using the same requirements, into the target collection. */
	FPCGDataCollection* TargetDataCollection = nullptr;
	/** Controls whether the created target should be 'attached' to the original object. */
	bool bAttachToParent = false;
	/** Name Override, can be left empty. */
	FName DefaultName = NAME_None;

	template<typename ObjectType>
	TSubclassOf<ObjectType> GetTemplateClass() const
	{
		if (TemplateTargetClass && TemplateTargetClass->IsChildOf(ObjectType::StaticClass()))
		{
			return TSubclassOf<ObjectType>(TemplateTargetClass);
		}
		else
		{
			return nullptr;
		}
	}

	template<typename ObjectType>
	ObjectType* GetTarget() const
	{
		return Cast<ObjectType>(Target);
	}

	template<typename ObjectType>
	const ObjectType* GetExpectedPreviousOwner() const
	{
		return Cast<ObjectType>(ExpectedPreviousOwner);
	}

	bool Accepts(const UObject* InObject) const;
	bool Accepts(const FPCGTaggedData& InData) const;
};

/** 
* This class is used to hold resources and their mechanism to delete them on demand.
* In order to allow for some reuse (e.g. components), the Release call supports a "soft"
* release by marking them unused in order to be potentially re-used down the line.
* At the end of the generate, a call to ReleaseIfUnused will serve to finally cleanup
* what is not needed anymore.
*/
UCLASS(MinimalAPI, BlueprintType)
class UPCGManagedResource : public UObject
{
	GENERATED_BODY()
public:
	/** Called when after a PCG component is applied to (such as after a RerunConstructionScript) */
	UE_API virtual void PostApplyToComponent();

	/** Releases/Mark Unused the resource depending on the bHardRelease flag. Returns true if resource can be removed from the PCG component */
	UE_API virtual bool Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete);
	/** Releases resource if empty or unused. Returns true if the resource can be removed from the PCG component */
	UE_API virtual bool ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete);

	/** Returns whether a resource can be used - generally true except for resources marked as transient (from loading) */
	UE_API virtual bool CanBeUsed() const;

	/** Whether resource is transient and should be released on tear down. */
	virtual bool ReleaseOnTeardown() const { return false; }

	/** Marks the resources as being kept and changed through generation */
	UE_API virtual void MarkAsUsed();
	/** Marks the resource as being reused as-is during the generation */
	virtual void MarkAsReused() { bIsMarkedUnused = false; }
	bool IsMarkedUnused() const { return bIsMarkedUnused; }

	/** Moves a part or the full resource contents to another target as dictated by the params. Returns true if it has succeeded. */
	UE_API virtual bool MoveResource(const FPCGMoveResourceParams& Params);

	static UE_API bool DebugForcePurgeAllResourcesOnGenerate();

	const FPCGCrc& GetCrc() const { return Crc; }
	void SetCrc(const FPCGCrc& InCrc) { Crc = InCrc; }

	/** Returns true if this resource manages this object. */
	virtual bool IsManaging(const UObject* InObject) const { return false; }

#if WITH_EDITOR
	UE_API virtual void ChangeTransientState(EPCGEditorDirtyMode NewEditingMode);
	virtual void MarkTransientOnLoad() { bMarkedTransientOnLoad = true; }

	bool IsMarkedTransientOnLoad() const { return bMarkedTransientOnLoad; }

	bool IsPreview() const { return bIsPreview; }
	void SetIsPreview(bool bInIsPreview) { bIsPreview = bInIsPreview; }
#endif // WITH_EDITOR

protected:
	UPROPERTY(VisibleAnywhere, Category = GeneratedData)
	FPCGCrc Crc;

	UPROPERTY(Transient, VisibleAnywhere, Category = GeneratedData)
	bool bIsMarkedUnused = false;

#if WITH_EDITORONLY_DATA
	// Resources on a Load-as-preview component are marked as 'transient on load'; these resources must not be affected in any
	//  permanent way in order to make sure they are not serialized in a different state if their outer is saved.
	// These resources will generally have a different Release path, and will be managed differently from the PCG component as well.
	// Note that this flag will be reset if there is a transient state change originating from the component, which might trigger resource deletion, flags change, etc.
	UPROPERTY(VisibleAnywhere, Category = GeneratedData, meta = (NoResetToDefault))
	bool bMarkedTransientOnLoad = false;

	UPROPERTY(Transient)
	bool bIsPreview = false;
#endif

	// Deprecated calls section
public:
	/** Move the given resource to a new actor. Return true if it has succeeded */
	UE_DEPRECATED(5.8, "Implement or use MoveResource instead.")
	virtual bool MoveResourceToNewActor(AActor* NewActor) { return false; };

	UE_DEPRECATED(5.8, "Implement or use MoveResource instead.")
	UE_API virtual bool MoveResourceToNewActor(AActor* NewActor, const AActor* ExpectedPreviousOwner);
};

struct FPCGManagedActorLoadingScope
{
	FPCGManagedActorLoadingScope();
	FPCGManagedActorLoadingScope(bool bInShouldLoadActors);
	virtual ~FPCGManagedActorLoadingScope() = default;

	void AddResources(IPCGGraphExecutionSource* InExecutionSource, const TArray<UPCGManagedResource*>& InManagedResources);

#if WITH_EDITOR
	TSet<FWorldPartitionReference> LoadedActors;
	bool bShouldLoadActors;
#endif
};

UCLASS(MinimalAPI, BlueprintType)
class UPCGManagedActors : public UPCGManagedResource
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGManagedResource interface
	UE_API virtual bool Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	UE_API virtual bool ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	UE_API virtual bool MoveResource(const FPCGMoveResourceParams& Params) override;
	UE_API virtual void MarkAsUsed() override;
	UE_API virtual void MarkAsReused() override;

	UE_API virtual bool IsManaging(const UObject* InObject) const override;

#if WITH_EDITOR
	UE_API virtual void ChangeTransientState(EPCGEditorDirtyMode NewEditingMode) override;
#endif

	UE_DEPRECATED(5.8, "Use MoveResource instead.")
	UE_API virtual bool MoveResourceToNewActor(AActor* NewActor) override;
	//~End UPCGManagedResource interface

	const TArray<TSoftObjectPtr<AActor>>& GetConstGeneratedActors() const { return GeneratedActorsArray; }
	TArray<TSoftObjectPtr<AActor>>& GetMutableGeneratedActors() { return GeneratedActorsArray; }

	/** Controls whether the resource will be removed at the beginning of the generation instead of being kept until the end, in the eventuality it is reused. 
	* In practice, this means that the actors will be deleted as part of the initial cleanup in the generation instead of being removed at the end.
	* In some instances, this might be required if some components on the actors interact negatively with the PCG processing in the graph.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = GeneratedData)
	bool bSupportsReset = true;

	UE_DEPRECATED(5.6, "Generated Actors is now an array, you must use the Getter/Setter, as this set won't be used anymore.")
	UPROPERTY(meta = (DeprecatedProperty))
	TSet<TSoftObjectPtr<AActor>> GeneratedActors;
	
private:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = GeneratedData, meta = (AllowPrivateAccess = "true", DisplayName = "GeneratedActors"))
	TArray<TSoftObjectPtr<AActor>> GeneratedActorsArray;
};

UCLASS(MinimalAPI, Abstract)
class UPCGManagedComponentBase : public UPCGManagedResource
{
	GENERATED_BODY()

	friend class UPCGComponent;
public:
	//~Begin UObject interface
	UE_API virtual void PostEditImport() override;
	//~End UObject interface

	//~Begin UPCGManagedResource interface
	UE_API virtual void MarkAsUsed() override;
	UE_API virtual void MarkAsReused() override;
	UE_API virtual bool MoveResource(const FPCGMoveResourceParams& Params) override;
	UE_API virtual bool IsManaging(const UObject* InObject) const override;

#if WITH_EDITOR
	UE_API virtual void ChangeTransientState(EPCGEditorDirtyMode NewEditingMode) override;
#endif

	UE_DEPRECATED(5.8, "Implement or use MoveResource instead.")
	UE_API virtual bool MoveResourceToNewActor(AActor* NewActor, const AActor* ExpectedPreviousOwner) override;
	//~End UPCGManagedResource interface

public:
#if WITH_EDITOR
	/** Hides the content of the component in a transient way (such as unregistering) */
	UE_API void HideComponents();
	UE_API virtual void HideComponent(int32 ComponentIndex);
	virtual void HideComponent() {};
#endif

	UE_API void ForgetComponents();
	UE_API virtual void ForgetComponent(int32 ComponentIndex);
	virtual void ForgetComponent() {}

	UE_API void ResetComponents();
	UE_API virtual void ResetComponent(int32 ComponentIndex);
	virtual void ResetComponent() { check(0); }
	virtual bool SupportsComponentReset() const { return false; }

protected:
	virtual TArrayView<TSoftObjectPtr<UActorComponent>> GetComponentsArray() PURE_VIRTUAL(UPCGManagedComponentBase::GetComponentsArray, return TArrayView<TSoftObjectPtr<UActorComponent>>{};);
	virtual int32 GetComponentsCount() const PURE_VIRTUAL(UPCGManagedComponentBase::GetComponentsCount, return 0;);
	UE_API void SetupGeneratedComponentFromBP(TSoftObjectPtr<UActorComponent> InGeneratedComponent);
};

UCLASS(MinimalAPI, BlueprintType)
class UPCGManagedComponent : public UPCGManagedComponentBase
{
	GENERATED_BODY()

	friend class UPCGComponent;

public:
	//~Begin UPCGManagedResource interface
	UE_API virtual bool Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	UE_API virtual bool ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	//~End UPCGManagedResource interface

	//~Begin UPCGManagedComponentBase interface
#if WITH_EDITOR
	/** Hides the content of the component in a transient way (such as unregistering) */
	UE_API virtual void HideComponent();
#endif
	
	virtual void ForgetComponent() { GeneratedComponent.Reset(); }

protected:
	virtual TArrayView<TSoftObjectPtr<UActorComponent>> GetComponentsArray() override { return TArrayView<TSoftObjectPtr<UActorComponent>>(&GeneratedComponent, 1); }
	virtual int32 GetComponentsCount() const override { return 1; }
	//~End UPCGManagedComponentBase interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = GeneratedData, BlueprintSetter = SetGeneratedComponentFromBP)
	TSoftObjectPtr<UActorComponent> GeneratedComponent;

private:
	// When creating components from BP they will be tagged automatically as created from construction script, which makes them transient and isn't compatible with the PCG workflow.
	UFUNCTION(BlueprintSetter, meta = (BlueprintInternalUseOnly = "true"))
	UE_API void SetGeneratedComponentFromBP(TSoftObjectPtr<UActorComponent> InGeneratedComponent);
};

/** This managed resource class is used to tie multiple components in the same resource so that they are cleaned up all at the same time. */
UCLASS(MinimalAPI, BlueprintType)
class UPCGManagedComponentList : public UPCGManagedComponentBase
{
	GENERATED_BODY()
	friend class UPCGComponent;

public:
	//~Begin UPCGManagedResource interface
	UE_API virtual bool Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	UE_API virtual bool ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	//~End UPCGManagedResource interface

	//~Begin UPCGManagedComponentBase interface
#if WITH_EDITOR
	/** Hides the content of the component in a transient way (such as unregistering) */
	UE_API virtual void HideComponent(int32 ComponentIndex) override;
#endif

	UE_API virtual void ForgetComponent(int32 ComponentIndex) override;

protected:
	virtual TArrayView<TSoftObjectPtr<UActorComponent>> GetComponentsArray() override { return GeneratedComponents; }
	virtual int32 GetComponentsCount() const override { return GeneratedComponents.Num(); }
	//~End UPCGManagedComponentBase interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = GeneratedData, BlueprintSetter = SetGeneratedComponentsFromBP)
	TArray<TSoftObjectPtr<UActorComponent>> GeneratedComponents;

private:
	// When creating components from BP they will be tagged automatically as created from construction script, which makes them transient and isn't compatible with the PCG workflow.
	UFUNCTION(BlueprintSetter, meta = (BlueprintInternalUseOnly = "true"))
	UE_API void SetGeneratedComponentsFromBP(const TArray<TSoftObjectPtr<UActorComponent>>& InGeneratedComponent);
};

/** Stub default list to hold resources pushed from BP in a single place on the PCG component. */
UCLASS()
class UPCGManagedComponentDefaultList final : public UPCGManagedComponentList
{
	GENERATED_BODY()
	friend class UPCGComponent;

private:
	void AddGeneratedComponentsFromBP(const TArray<TSoftObjectPtr<UActorComponent>>& InGeneratedComponents);
};

UCLASS(MinimalAPI, BlueprintType)
class UPCGManagedISMComponent : public UPCGManagedComponent
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGManagedResource interface
	UE_API virtual bool ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	//~End UPCGManagedResource interface

	//~Begin UPCGManagedComponents interface
	UE_API virtual void ResetComponent() override;
	virtual bool SupportsComponentReset() const override{ return true; }
	UE_API virtual void MarkAsUsed() override;
	UE_API virtual void MarkAsReused() override;
	//~End UPCGManagedComponents interface

	UE_API UInstancedStaticMeshComponent* GetComponent() const;
	UE_API void SetComponent(UInstancedStaticMeshComponent* InComponent);

	UE_API void SetDescriptor(const FISMComponentDescriptor& InDescriptor);
	const FISMComponentDescriptor& GetDescriptor() const { return Descriptor; }

	UE_API void SetRootLocation(const FVector& InRootLocation);

	FPCGCrc GetSettingsCrc() const { return SettingsCrc; }
	void SetSettingsCrc(const FPCGCrc& InSettingsCrc) { SettingsCrc = InSettingsCrc; }

	FPCGCrc GetDataCrc() const { return DataCrc; }
	void SetDataCrc(const FPCGCrc& InDataCrc) { DataCrc = InDataCrc; }

	void SetCrcReuseCount(int32 InCount) { CrcReuseCount = InCount; }
	int32 GetCrcReuseCount() const { return CrcReuseCount; }

	UE_API void SetDataCrcReuseCount(int32 InCount) { DataCrcReuseCount = InCount; }
	UE_API int32 GetDataCrcReuseCount() const { return DataCrcReuseCount; }

	UE_DEPRECATED(5.6, "Use GetSettingsCrc instead")
	uint64 GetSettingsUID() const { return -1; }
	
	UE_DEPRECATED(5.6, "Use SetSettingsCrc instead")
	void SetSettingsUID(uint64 InSettingsUID) { }

protected:
	UPROPERTY()
	bool bHasDescriptor = false;

	UPROPERTY()
	FISMComponentDescriptor Descriptor;

	UPROPERTY()
	bool bHasRootLocation = false;

	UPROPERTY()
	FVector RootLocation = FVector::ZeroVector;

	UPROPERTY()
	FPCGCrc SettingsCrc;
	
	UPROPERTY()
	FPCGCrc DataCrc;

	UPROPERTY()
	int32 CrcReuseCount = INDEX_NONE;

	UPROPERTY()
	int32 DataCrcReuseCount = INDEX_NONE;
};

UCLASS(MinimalAPI, BlueprintType)
class UPCGManagedISKMComponent : public UPCGManagedComponent
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGManagedResource interface
	UE_API virtual bool ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	//~End UPCGManagedResource interface

	//~Begin UPCGManagedComponents interface
	UE_API virtual void ResetComponent() override;
	virtual bool SupportsComponentReset() const override{ return true; }
	UE_API virtual void MarkAsUsed() override;
	UE_API virtual void MarkAsReused() override;
	//~End UPCGManagedComponents interface

	UE_API UInstancedSkinnedMeshComponent* GetComponent() const;
	UE_API void SetComponent(UInstancedSkinnedMeshComponent* InComponent);

	UE_API void SetDescriptor(const FSkinnedMeshComponentDescriptor& InDescriptor);
	const FSkinnedMeshComponentDescriptor& GetDescriptor() const { return Descriptor; }

	UE_API void SetRootLocation(const FVector& InRootLocation);

	FPCGCrc GetSettingsCrc() const { return SettingsCrc; }
	void SetSettingsCrc(const FPCGCrc& InSettingsCrc) { SettingsCrc = InSettingsCrc; }

protected:
	UPROPERTY()
	bool bHasDescriptor = false;

	UPROPERTY()
	FSkinnedMeshComponentDescriptor Descriptor;

	UPROPERTY()
	bool bHasRootLocation = false;

	UPROPERTY()
	FVector RootLocation = FVector::ZeroVector;

	UPROPERTY()
	FPCGCrc SettingsCrc;
};

UCLASS(MinimalAPI, BlueprintType)
class UPCGManagedSplineMeshComponent : public UPCGManagedComponent
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGManagedComponents interface
	UE_API virtual void ForgetComponent() override;
	virtual void ResetComponent() override { /* Does nothing, but implementation is required to support reuse. */ }
	virtual bool SupportsComponentReset() const override { return true; }
	//~End UPCGManagedComponents interface

	UE_API USplineMeshComponent* GetComponent() const;
	UE_API void SetComponent(USplineMeshComponent* InComponent);

	void SetDescriptor(const FSplineMeshComponentDescriptor& InDescriptor) { Descriptor = InDescriptor; }
	const FSplineMeshComponentDescriptor& GetDescriptor() const { return Descriptor; }

	void SetSplineMeshParams(const FPCGSplineMeshParams& InSplineMeshParams) { SplineMeshParams = InSplineMeshParams; }
	const FPCGSplineMeshParams& GetSplineMeshParams() const { return SplineMeshParams; }

	FPCGCrc GetSettingsCrc() const { return SettingsCrc; }
	void SetSettingsCrc(const FPCGCrc& InSettingsCrc) { SettingsCrc = InSettingsCrc; }

	UE_DEPRECATED(5.6, "Use GetSettingsCrc instead")
	uint64 GetSettingsUID() const { return -1; }

	UE_DEPRECATED(5.6, "Use SetSettingsCrc instead")
	void SetSettingsUID(uint64 InSettingsUID) { }

protected:
	UPROPERTY()
	FSplineMeshComponentDescriptor Descriptor;

	UPROPERTY()
	FPCGSplineMeshParams SplineMeshParams;

	UPROPERTY()
	FPCGCrc SettingsCrc;
	
private:
	USplineMeshComponent* CacheAndGetComponent() const;
	
	UPROPERTY(Transient)
	mutable TWeakObjectPtr<USplineMeshComponent> CachedGeneratedComponent;

	mutable PCG::FSharedLock CachedLock;
};

#undef UE_API
