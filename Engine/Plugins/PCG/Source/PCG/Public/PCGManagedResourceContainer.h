// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGManagedResource.h"

#include "UObject/UObjectGlobals.h"
#include "UObject/WeakInterfacePtr.h"

#include "PCGManagedResourceContainer.generated.h"

class UPCGManagedResource;

USTRUCT()
struct FPCGManagedResourceContainer
{
	GENERATED_BODY()

private:
	friend struct FPCGManagedResourceConstContainerHelper;
	friend struct FPCGManagedResourceContainerHelper;

	bool bAreResourcesInaccessible = false;

	UPROPERTY()
	TArray<TObjectPtr<UPCGManagedResource>> GeneratedResources;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<TObjectPtr<UPCGManagedResource>> LoadedPreviewResources;
#endif
};

// Const and Non-const API to interface with FPCGManagedResourceContainer
//
// The helper is not tracking the execution source becoming invalid so avoid storing a copy.
// Instead store a weak ptr to the execution source which allows creating a helper on the stack when needed.
struct FPCGManagedResourceConstContainerHelper
{
	FPCGManagedResourceConstContainerHelper() = delete;

	// const_cast here to store a single pointer for the const and non const versions of the helper.
	explicit FPCGManagedResourceConstContainerHelper(const IPCGGraphExecutionSource* InExecutionSource)
		: Owner(const_cast<IPCGGraphExecutionSource*>(InExecutionSource))
	{
		Container = Owner ? Owner->GetExecutionState().GetManagedResourceContainer() : nullptr;
	}

	bool IsValid() const { return Owner != nullptr && Container != nullptr; }
	bool AreResourcesAccessible() const { return IsValid() && !Container->bAreResourcesInaccessible; }
	bool IsEmpty() const { return !Container || Container->GeneratedResources.IsEmpty(); }

	PCG_API TArray<TObjectPtr<UPCGManagedResource>> GetManagedResourcesCopy() const;

	const TArray<TObjectPtr<UPCGManagedResource>>& GetConstManagedResourcesNoLock() const { check(IsValid()); return Container->GeneratedResources; }

	PCG_API void ForEachConstManagedResource(TFunctionRef<void(const UPCGManagedResource*)> InFunction) const;
	PCG_API bool IsAnyObjectManagedByResource(const TArrayView<const UObject*> InObjects) const;

#if WITH_EDITOR
	const TArray<TObjectPtr<UPCGManagedResource>>& GetConstLoadedPreviewResourcesNoLock() const { check(IsValid()); return Container->LoadedPreviewResources; }
#endif

protected:
	IPCGGraphExecutionSource* Owner = nullptr;
	FPCGManagedResourceContainer* Container = nullptr;
};

struct FPCGManagedResourceContainerHelper final : public FPCGManagedResourceConstContainerHelper
{
	explicit FPCGManagedResourceContainerHelper(IPCGGraphExecutionSource* InExecutionSource) 
		: FPCGManagedResourceConstContainerHelper(InExecutionSource)
	{
	}

	PCG_API void ForgetAll();

	PCG_API void AddManagedResource(UPCGManagedResource* InManagedResource);
	PCG_API void CleanupUnusedManagedResources();
	PCG_API void SetManagedResources(TArray<TObjectPtr<UPCGManagedResource>> InManagedResources);
	
	PCG_API void ForEachManagedResource(TFunctionRef<void(UPCGManagedResource*)> InFunction);
	PCG_API void EndPlay();

	struct FCleanupTask : public FPCGManagedActorLoadingScope
	{
		TWeakInterfacePtr<IPCGGraphExecutionSource> WeakExecutionSource;
		bool bIsFirstIteration = true;
		int32 ResourceIndex = -1;
		TSet<TSoftObjectPtr<AActor>> ActorsToDelete;

		PCG_API void Abort();
		PCG_API bool Execute(bool bReleaseManagedResources);
	};

	PCG_API TSharedPtr<FCleanupTask> CreateCleanupTask();
	PCG_API void Cleanup(bool bReleaseManagedResources);

#if WITH_EDITOR
	void SetLoadedPreviewResourcesNoLock(TArray<TObjectPtr<UPCGManagedResource>> InManagedResources) { check(IsValid()); Container->LoadedPreviewResources = MoveTemp(InManagedResources); }

	PCG_API void MarkResourcesAsTransientOnLoad();
	PCG_API bool ChangeTransientState(EPCGEditorDirtyMode NewEditingMode);
	PCG_API bool DeletePreviewResources();

	PCG_API void BeginPreviewSave(TArray<TObjectPtr<UPCGManagedResource>>& OutGeneratedCopy);
	PCG_API void EndPreviewSave(const TArray<TObjectPtr<UPCGManagedResource>>& InGeneratedCopy);
#endif	

private:
	friend class UPCGComponent;

	void AddManagedResourceNoLock(UPCGManagedResource* InManagedResource);
	void RemoveManagedResourceNoLock(UPCGManagedResource* InManagedResource);
	void RemoveManagedResourceAtNoLock(int32 ResourceIndex);

	void AddManagedResourceInternal(UPCGManagedResource* InManagedResource, FTransactionallySafeCriticalSection* InLock);
};