// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataVisualizationRegistry.h"
#include "PCGGraphExecutionRegistry.h"
#include "ChangeTracking/PCGChangeTrackingRegistry.h"
#include "Data/Registry/PCGDataTypeRegistry.h"
#include "Data/Registry/PCGDataViewRegistry.h"
#include "Data/Registry/PCGGetDataFunctionRegistry.h"
#include "Hash/PCGObjectHash.h"
#include "Metadata/Accessors/PCGAttributeAccessorFactory.h"
#include "Physics/PCGPhysicsRegistry.h"
#include "Utils/PCGLogErrors.h"

#include "Containers/Ticker.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"

// Logs
PCG_API DECLARE_LOG_CATEGORY_EXTERN(LogPCG, Log, All);

struct FPCGContext;
class IPCGDataVisualization;

namespace PCGEngineShowFlags
{
	static constexpr TCHAR Debug[] = TEXT("PCGDebug");
}

// Stats
DECLARE_STATS_GROUP(TEXT("PCG"), STATGROUP_PCG, STATCAT_Advanced);

LLM_DECLARE_TAG(PCG);

// Delegates
DECLARE_MULTICAST_DELEGATE_TwoParams(FPCGGraphChangedDelegate, UPCGGraph* /*InGraph*/, EPCGChangeType /*ChangeType*/);

class FPCGModule final : public IModuleInterface
{
public:
	//~ IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override { return true; }
	//~ End IModuleInterface implementation

	void PreExit();
	void OnPostInitEngine();

#if WITH_EDITOR
	PCG_API static void SetCPUOnlyExecution(bool bEnabled);
	PCG_API static bool IsCPUOnlyExecution();

	PCG_API static void SetEditorOnlyGeneration(bool bEnabled);
	PCG_API static bool IsEditorOnlyGeneration();
#endif

	PCG_API static FPCGModule& GetPCGModuleChecked();
	PCG_API static FPCGModule* GetPCGModule();
	static const FPCGGetDataFunctionRegistry& ConstGetDataFunctionRegistry() { return GetPCGModuleChecked().GetDataFunctionRegistry; }
	static FPCGGetDataFunctionRegistry& MutableGetDataFunctionRegistry() { return GetPCGModuleChecked().GetDataFunctionRegistry; }

	PCG_API static const FPCGDataTypeRegistry& GetConstDataTypeRegistry();
	PCG_API static FPCGDataTypeRegistry& GetMutableDataTypeRegistry();

	static const FPCGAttributeAccessorFactory& GetConstAttributeAccessorFactory() { return GetPCGModuleChecked().AttributeAccessorFactory; }
	static FPCGAttributeAccessorFactory& GetMutableAttributeAccessorFactory() { return GetPCGModuleChecked().AttributeAccessorFactory; }
	
	static const FPCGGraphExecutionRegistry& GetConstGraphExecutionRegistry() { return GetPCGModuleChecked().GraphExecutionRegistry; }
	static FPCGGraphExecutionRegistry& GetGraphExecutionRegistry() { return GetPCGModuleChecked().GraphExecutionRegistry; }

	static const FPCGDataViewRegistry& GetConstPCGDataViewRegistry() { return GetPCGModuleChecked().DataViewRegistry; }
	static FPCGDataViewRegistry& GetMutablePCGDataViewRegistry() { return GetPCGModuleChecked().DataViewRegistry; }

	static const FPCGChangeTrackingRegistry& GetConstChangeTrackingRegistry() { return GetPCGModuleChecked().ChangeTrackingRegistry; }
	static FPCGChangeTrackingRegistry& GetMutableChangeTrackingRegistry() { return GetPCGModuleChecked().ChangeTrackingRegistry; }

	static const FPCGPhysicsRegistry& GetConstPhysicsRegistry() { return GetPCGModuleChecked().PhysicsRegistry; }
	static FPCGPhysicsRegistry& GetMutablePhysicsRegistry() { return GetPCGModuleChecked().PhysicsRegistry; }

#if WITH_EDITOR
	static const FPCGObjectHashFactory& GetConstObjectHashFactory() { return GetPCGModuleChecked().ObjectHashFactory; }
	static FPCGObjectHashFactory& GetMutableObjectHashFactory() { return GetPCGModuleChecked().ObjectHashFactory; }
#endif

	PCG_API static bool IsPCGModuleLoaded();
	
	PCG_API void ExecuteNextTick(TFunction<void()> TickFunction);

#if WITH_EDITOR
	static FPCGGraphChangedDelegate& OnGraphChanged() { return OnGraphChangedDelegate; }
#endif

private:
	bool Tick(float DeltaTime);
	
#if WITH_EDITOR
	/** Load additional source assets used by all kernels in the PCG plugin, so that they are ready to be consumed during compilation. */
	void LoadAdditionalKernelSources();
#endif

	FPCGGetDataFunctionRegistry GetDataFunctionRegistry;
	FPCGAttributeAccessorFactory AttributeAccessorFactory;
	FPCGDataTypeRegistry DataTypeRegistry;
	FPCGGraphExecutionRegistry GraphExecutionRegistry;
	FPCGDataViewRegistry DataViewRegistry;
	FPCGChangeTrackingRegistry ChangeTrackingRegistry;
	FPCGPhysicsRegistry PhysicsRegistry;

#if WITH_EDITOR
	FPCGObjectHashFactory ObjectHashFactory;
#endif

	FTSTicker::FDelegateHandle	TickDelegateHandle;

	PCG::FLock ExecuteNextTickLock;
	TArray<TFunction<void()>> ExecuteNextTicks;

#if WITH_EDITOR
private:
	void RegisterNativeElementDeterminismTests();
	void DeregisterNativeElementDeterminismTests();

public:
	static const FPCGDataVisualizationRegistry& GetConstPCGDataVisualizationRegistry() { return GetPCGModuleChecked().PCGDataVisualizationRegistry; }
	static FPCGDataVisualizationRegistry& GetMutablePCGDataVisualizationRegistry() { return GetPCGModuleChecked().PCGDataVisualizationRegistry; }

private:
	FPCGDataVisualizationRegistry PCGDataVisualizationRegistry;

	friend class UPCGGraph;
	void NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType) { OnGraphChangedDelegate.Broadcast(InGraph, ChangeType); }

	static FPCGGraphChangedDelegate OnGraphChangedDelegate;

	bool bIsCPUOnlyExecution = false;
	bool bIsEditorOnlyGeneration = false;
#endif // WITH_EDITOR
};
