// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGNode.generated.h"

#define UE_API PCG_API

class UPCGNode;
class UPCGPin;
enum class EPCGChangeType : uint32;
enum class EPCGNodeTitleType : uint8;
struct FPCGPinProperties;

class UPCGSettings;
class UPCGSettingsInterface;
class UPCGGraph;
class UPCGEdge;
class IPCGElement;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPCGNodeChanged, UPCGNode*, EPCGChangeType);
#endif

UCLASS(MinimalAPI, ClassGroup = (Procedural))
class UPCGNode : public UObject
{
	GENERATED_BODY()

	friend class UPCGGraph;
	friend class UPCGEdge;
	friend class FPCGGraphCompiler;

public:
	UE_API UPCGNode(const FObjectInitializer& ObjectInitializer);
	
	/** ~Begin UObject interface */
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	/** ~End UObject interface */

	/** UpdatePins will kick off invalid edges, so this is useful for making pin changes graceful. */
	UE_API void ApplyDeprecationBeforeUpdatePins();

	/** Used to be able to force deprecation when things need to be deprecated at the graph level. */
	UE_API void ApplyDeprecation();

	/** If a node does require structural changes, this will apply them */
	UE_API virtual void ApplyStructuralDeprecation();

	UE_API virtual void RebuildAfterPaste();
#endif

	/** Returns the owning graph */
	UFUNCTION(BlueprintCallable, Category = Node)
	UE_API UPCGGraph* GetGraph() const;

	/** Adds an edge in the owning graph to the given "To" node. */
	UFUNCTION(BlueprintCallable, Category = Node)
	UE_API UPCGNode* AddEdgeTo(FName FromPinLabel, UPCGNode* To, FName ToPinLabel);

	/** Removes an edge originating from this node */
	UFUNCTION(BlueprintCallable, Category = Node)
	UE_API bool RemoveEdgeTo(FName FromPinLable, UPCGNode* To, FName ToPinLabel);

	/** Get title for node of specified type. */
	UE_API FText GetNodeTitle(EPCGNodeTitleType TitleType) const;

	/** Whether user has renamed the node. */
	bool HasAuthoredTitle() const { return NodeTitle != NAME_None; }

	/** Title to use if no title is authored. */
	UE_API FText GetDefaultTitle() const;

	/** Authored part of node title (like "Create Constant 1"). */
	UE_API FText GetAuthoredTitleLine() const;

	/** Authored node title as raw name or none if no title authored. */
	FName GetAuthoredTitleName() const { return NodeTitle; }

	/** Whether to flip the order of the title lines - display generated title first and authored second. */
	UE_API bool HasFlippedTitleLines() const;

	/** Generated part of node title, not user editable (like "MyValue = 5.0"). */
	UE_API FText GetGeneratedTitleLine() const;

#if WITH_EDITOR
	/** Tooltip that describes node functionality and other information. */
	UE_API FText GetNodeTooltipText() const;
	
	/** Set the node title. Can be sanitized by the graph/settings if asked. Returns true if the title changed. */
	UE_API bool SetNodeTitle(FName NewTitle, bool bApplySanitization = true);
#endif

	/** Returns all the input pin properties */
	UE_API TArray<FPCGPinProperties> InputPinProperties() const;

	/** Returns all the output pin properties */
	UE_API TArray<FPCGPinProperties> OutputPinProperties() const;

	/** Returns true if the input pin is connected */
	UE_API bool IsInputPinConnected(const FName& Label) const;

	/** Returns true if the output pin is connected */
	UE_API bool IsOutputPinConnected(const FName& Label) const;

	/** Returns true if the node has an instance of the settings (e.g. does not own the settings) */
	UE_API bool IsInstance() const;

	/** Returns the settings interface (settings or instance of settings) on this node */
	UPCGSettingsInterface* GetSettingsInterface() const { return SettingsInterface.Get(); }

	/** Changes the default settings in the node */
	UE_API void SetSettingsInterface(UPCGSettingsInterface* InSettingsInterface, bool bUpdatePins = true);

	/** Returns the settings this node holds (either directly or through an instance) */
	UFUNCTION(BlueprintCallable, Category = Node)
	UE_API UPCGSettings* GetSettings() const;

	/** Triggers some uppdates after creating a new node and changing its settings */
	UE_API void UpdateAfterSettingsChangeDuringCreation();

	UE_API UPCGPin* GetInputPin(const FName& Label);
	UE_API const UPCGPin* GetInputPin(const FName& Label) const;
	UE_API UPCGPin* GetOutputPin(const FName& Label);
	UE_API const UPCGPin* GetOutputPin(const FName& Label) const;
	UE_API bool HasInboundEdges() const;
	UE_API int32 GetInboundEdgesNum() const;

	/** Allow to change the name of a pin, to keep edges connected. You need to make sure that the underlying settings are also updated, otherwise, it will be overwritten next time the settings are updated */
	UE_API void RenameInputPin(const FName& InOldLabel, const FName& InNewLabel, bool bInBroadcastUpdate = true);
	UE_API void RenameOutputPin(const FName& InOldLabel, const FName& InNewLabel, bool bInBroadcastUpdate = true);

	/** Pin from which data is passed through when this node is disabled. */
	UE_API virtual const UPCGPin* GetPassThroughInputPin() const;

	/** Pin to which data is passed through when this node is disabled. */
	UE_API virtual const UPCGPin* GetPassThroughOutputPin() const;

	/** A node will be executed (not culled) if at least one required-pin is connected to at least one active upstream pin. */
	UE_API virtual bool IsInputPinRequiredByExecution(const UPCGPin* InPin) const;

	/** True if the pin is being used by the node. UI will gray out unused pins. */
	UE_API virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const;

	/** True if the edge is being used by the node. UI will gray out unused pins. */
	UE_API virtual bool IsEdgeUsedByNodeExecution(const UPCGEdge* InEdge) const;

	/** Returns the first connected input pin on the node. */
	UE_API const UPCGPin* GetFirstConnectedInputPin() const;

	/** Returns the first connected output pin on the node. */
	UE_API const UPCGPin* GetFirstConnectedOutputPin() const;

	const TArray<TObjectPtr<UPCGPin>>& GetInputPins() const { return InputPins; }
	const TArray<TObjectPtr<UPCGPin>>& GetOutputPins() const { return OutputPins; }

	/** Recursively follow downstream edges and call UpdatePins on each node that has dynamic pins. */
	UE_API EPCGChangeType PropagateDynamicPinTypes(TSet<UPCGNode*>& TouchedNodes, const UPCGNode* FromNode = nullptr);

#if WITH_EDITOR
	/** Transfer all editor only properties to the other node */
	UE_API void TransferEditorProperties(UPCGNode* OtherNode) const;
#endif // WITH_EDITOR

#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, Category = Node)
	UE_API void GetNodePosition(int32& OutPositionX, int32& OutPositionY) const;

	UFUNCTION(BlueprintCallable, Category = Node)
	UE_API void SetNodePosition(int32 InPositionX, int32 InPositionY);
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (PCGNoHash))
	TObjectPtr<UPCGSettings> DefaultSettings_DEPRECATED; 
#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Node, meta = (PCGNoHash))
	FName NodeTitle = NAME_None;

#if WITH_EDITORONLY_DATA
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Node, meta = (PCGNoHash))
	FLinearColor NodeTitleColor = FLinearColor::White;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	FOnPCGNodeChanged OnNodeChangedDelegate;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (PCGNoHash))
	int32 PositionX;

	UPROPERTY(meta = (PCGNoHash))
	int32 PositionY;

	UPROPERTY(meta = (PCGNoHash))
	FString NodeComment;

	UPROPERTY(meta = (PCGNoHash))
	uint8 bCommentBubblePinned : 1;

	UPROPERTY(meta = (PCGNoHash))
	uint8 bCommentBubbleVisible : 1;

private:
	/** Will be hidden in the Editor, but will still exist in the backend. Not exposed and mainly used to hide the input/output node. */
	UPROPERTY(meta = (PCGNoHash))
	uint8 bHidden : 1;
#endif // WITH_EDITORONLY_DATA

public:
#if WITH_EDITOR
	bool IsHidden() const { return bHidden;}
#endif // WITH_EDITOR
	
protected:
	/** Updates pins based on node settings. Attempts to migrate pins via matching. Broadcasts node change events for affected nodes. Can provide warning messages when something destructive happens. */
	UE_API EPCGChangeType UpdatePins(bool bQuiet = true);
	/** Updates pins based on node settings PinAllocator creates new pin objects. Attempts to migrate pins via matching. Broadcasts node change events for affected nodes. Can provide warning messages when something destructive happens. */
	UE_API EPCGChangeType UpdatePins(TFunctionRef<UPCGPin* (UPCGNode*)> PinAllocator, bool bQuiet = true);

	// When we create a new graph, we initialize the input/output nodes as default, with default pins.
	// Those default pins are not serialized, therefore if we change the default pins, combined with the use
	// of recycling objects in Unreal, can lead to pins that are garbage or even worse: valid pins but not the right
	// one, potentially making the edges connecting wrong pins together!
	// That is why we have a specific function to create default pins, and we have to make sure that those
	// default pins are always created the same way.
	UE_API void CreateDefaultPins(TFunctionRef<UPCGPin* (UPCGNode*)> PinAllocator);

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API void OnSettingsChanged(UPCGSettings* InSettings, EPCGChangeType ChangeType);
#endif

	/** Note: do not set this property directly from code, use SetSettingsInterface instead */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Node)
	TObjectPtr<UPCGSettingsInterface> SettingsInterface;

	UPROPERTY(meta = (PCGNoHash))
	TArray<TObjectPtr<UPCGNode>> OutboundNodes_DEPRECATED;

	UPROPERTY(TextExportTransient, meta = (PCGNoHash))
	TArray<TObjectPtr<UPCGEdge>> InboundEdges_DEPRECATED;

	UPROPERTY(TextExportTransient, meta = (PCGNoHash))
	TArray<TObjectPtr<UPCGEdge>> OutboundEdges_DEPRECATED;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Node)
	TArray<TObjectPtr<UPCGPin>> InputPins;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Node)
	TArray<TObjectPtr<UPCGPin>> OutputPins;

	// TODO: add this information:
	// - Ability to run on non-game threads (here or element)
	// - Ability to be multithreaded (here or element)
	// - Generates artifacts (here or element)
	// - Priority
};

#undef UE_API
