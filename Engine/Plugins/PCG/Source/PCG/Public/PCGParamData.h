// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "Metadata/PCGMetadata.h"

#include "PCGParamData.generated.h"

#define UE_API PCG_API

namespace PCGParamDataConstants
{
	const FName ElementsDomainName = "Elements";
}

USTRUCT(meta = (PCG_DataTypeDisplayName="Attribute Set"))
struct FPCGDataTypeInfoParam : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Param)

	PCG_API virtual EPCGDataTypeCompatibilityResult IsCompatibleForSubtype(const FPCGDataTypeIdentifier& InType, const FPCGDataTypeIdentifier& OutType, FText* OptionalOutCompatibilityMessage) const override;
	PCG_API virtual bool SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const override;

#if WITH_EDITOR
	PCG_API virtual TOptional<FText> GetSubtypeTooltip(const FPCGDataTypeIdentifier& ThisType) const override;
#endif // WITH_EDITOR

	PCG_API static bool HasValidSubtype(const FPCGDataTypeIdentifier& Id);
};

/**
* Class to hold execution parameters that will be consumed in nodes of the graph
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural), DisplayName = "PCGAttribute Set")
class UPCGParamData : public UPCGData
{
	GENERATED_BODY()

public:
	UE_API UPCGParamData(const FObjectInitializer& ObjectInitializer);

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoParam)
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;

	UE_API virtual bool HasCachedLastSelector() const override;
	UE_API virtual FPCGAttributePropertyInputSelector GetCachedLastSelector() const override;
	UE_API virtual void SetLastSelector(const FPCGAttributePropertySelector& InSelector) override;

	UE_API virtual UPCGParamData* DuplicateData(FPCGContext* Context, bool bInitializeMetadata = true) const override;

	virtual FPCGMetadataDomainID GetDefaultMetadataDomainID() const override { return PCGMetadataDomainID::Elements; }
	virtual TArray<FPCGMetadataDomainID> GetAllSupportedMetadataDomainIDs() const override { return {PCGMetadataDomainID::Data, PCGMetadataDomainID::Elements}; }
	UE_API virtual bool SetDomainFromDomainID(const FPCGMetadataDomainID& InDomainID, FPCGAttributePropertySelector& InOutSelector) const override;
	UE_API virtual FPCGMetadataDomainID GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const override;
	UE_API virtual bool MetadataDomainSupportsParenting(const FPCGMetadataDomainID& InDomainID) const override;
	// ~End UPCGData interface

	/** Returns the entry for the given name */
	UFUNCTION(BlueprintCallable, Category = Params)
	UE_API int64 FindMetadataKey(const FName& InName) const;

	/** Creates an entry for the given name, if not already added */
	UFUNCTION(BlueprintCallable, Category = Params)
	UE_API int64 FindOrAddMetadataKey(const FName& InName);

	/** Creates a new params that keeps only a given key/name */
	UFUNCTION(BlueprintCallable, Category = Params, meta = (DisplayName = "Filter Params By Name"))
	UE_API UPCGParamData* K2_FilterParamsByName(const FName& InName) const;

	UE_DEPRECATED(5.5, "Call version with FPCGContext parameter")
	UPCGParamData* FilterParamsByName(const FName& InName) const { return FilterParamsByName(nullptr, InName); }

	UE_API UPCGParamData* FilterParamsByName(FPCGContext* Context, const FName& InName) const;

	UFUNCTION(BlueprintCallable, Category = Params, meta = (DisplayName = "Filter Params By Key"))
	UE_API UPCGParamData* K2_FilterParamsByKey(int64 InKey) const;

	UE_DEPRECATED(5.5, "Call version with FPCGContext parameter")
	UPCGParamData* FilterParamsByKey(int64 InKey) const { return FilterParamsByKey(nullptr, InKey); }

	UE_API UPCGParamData* FilterParamsByKey(FPCGContext* Context, int64 InKey) const;

protected:
	UPROPERTY()
	TMap<FName, int64> NameMap;

private:
	/** Cache to keep track of the latest attribute manipulated on this data. */
	UPROPERTY()
	bool bHasCachedLastSelector = false;

	UPROPERTY()
	FPCGAttributePropertyInputSelector CachedLastSelector;
};

#undef UE_API
