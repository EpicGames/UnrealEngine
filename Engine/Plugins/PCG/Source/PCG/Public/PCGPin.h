// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Data/Registry/PCGDataTypeIdentifier.h"
#include "Data/Registry/PCGDataTypeRegistry.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "PCGPin.generated.h"

#define UE_API PCG_API

class UPCGNode;
class UPCGEdge;

UENUM(BlueprintType)
enum class EPCGPinUsage : uint8
{
	/** Normal usage pin, will pass all data as is. */
	Normal = 0,
	/** When used in a loop subgraph node, will separate each data from that pin into separate subgraph executions. */
	Loop,
	/** When used in a loop subgraph node, will pass data on the feedback pins to the next iteration only if the data is passed from a previous iteration (or the original subgraph call). */
	Feedback,
	DependencyOnly UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EPCGPinStatus : uint8
{
	/** Normal usage pin. */
	Normal = 0,
	/** Only for input pins, mark this pin as required.If set on an output pin, behave as Normal. */
	Required,
	/** Advanced pin will be hidden by default in the UIand will be shown only if the user extends the node(in the UI) to see advanced pins.Pins can't be required and advanced at the same time. */
	Advanced,
	/** Pins generated by overridable parameters (normal nodes) or user parameters (subgraph nodes) - note that these are considered advanced pins. */
	OverrideOrUserParam UMETA(Hidden)
};

// Alternative to EEdGraphPinDirection which is an EdGraph dependency, which Runtime PCG avoids.
UENUM()
enum class EPCGPinDirection : uint8
{
	Input,
	Output
};

USTRUCT(BlueprintType, meta=(HasNativeBreak="/Script/PCG.PCGBlueprintPinHelpers.BreakPinProperty", HasNativeMake="/Script/PCG.PCGBlueprintPinHelpers.MakePinProperty"))
struct FPCGPinProperties
{
	GENERATED_BODY()

	FPCGPinProperties();

	UE_API explicit FPCGPinProperties(
		const FName& InLabel,
		FPCGDataTypeIdentifier InAllowedTypes = FPCGDataTypeIdentifier{EPCGDataType::Any},
		bool bInAllowMultipleConnections = true,
		bool bAllowMultipleData = true,
		const FText& InTooltip = FText::GetEmpty());

	UE_API bool operator==(const FPCGPinProperties& Other) const;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FName Label = NAME_None;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGPinUsage Usage = EPCGPinUsage::Normal;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_SupportsComposition = "SupportsComposition()"))
	FPCGDataTypeIdentifier AllowedTypes;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bAllowMultipleData = true;

	/** Define the status of a given pin (Normal, Required or Advanced) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGPinStatus PinStatus = EPCGPinStatus::Normal;

	UPROPERTY(BlueprintReadWrite, Category = Settings)
	bool bInvisiblePin = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = Settings)
	FText Tooltip;
#endif

	// Multiple connections are only possible if we support multi data.
	bool AllowsMultipleConnections() const { return bAllowMultipleData && bAllowMultipleConnections; }

	// Allowing multiple connections will automatically enable multi data.
	UE_API void SetAllowMultipleConnections(bool bInAllowMultipleConnectons);

	bool IsAdvancedPin() const { return PinStatus == EPCGPinStatus::Advanced || PinStatus == EPCGPinStatus::OverrideOrUserParam; }
	void SetAdvancedPin() { PinStatus = EPCGPinStatus::Advanced; }

	bool IsOverrideOrUserParamPin() const { return PinStatus == EPCGPinStatus::OverrideOrUserParam; }
	void SetOverrideOrUserParamPin() { PinStatus = EPCGPinStatus::OverrideOrUserParam; }

	bool IsRequiredPin() const { return PinStatus == EPCGPinStatus::Required; }
	void SetRequiredPin() { PinStatus = EPCGPinStatus::Required; }

	bool IsNormalPin() const { return PinStatus == EPCGPinStatus::Normal; }
	void SetNormalPin() { PinStatus = EPCGPinStatus::Normal; }

	bool IsDatalessPin() const { return Usage == EPCGPinUsage::DependencyOnly; }

	// Convert the bIsAdvanced boolean to PinStatus for deprecation purposes.
	UE_API void PostSerialize(const FArchive& Ar);

	friend uint32 GetTypeHash(const FPCGPinProperties& Value);

#if WITH_EDITOR
	UE_API bool CanEditChange(const FEditPropertyChain& PropertyChain) const;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	bool bAllowEditMultipleData = true;
	
	UPROPERTY(Transient)
	bool bAllowEditMultipleConnections = true;
#endif // WITH_EDITORONLY_DATA

private:
#if WITH_EDITORONLY_DATA
	/* Advanced pin will be hidden by default in the UI and will be shown only if the user extend the node (in the UI) to see advanced pins. */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage = "Use IsAdvancedPin function or PinStatus property."))
	bool bAdvancedPin_DEPRECATED = false;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (AllowPrivateAccess = "true", DisplayAfter = "bAllowMultipleData"))
	bool bAllowMultipleConnections = true;
};

template<>
struct TStructOpsTypeTraits<FPCGPinProperties> : public TStructOpsTypeTraitsBase2<FPCGPinProperties>
{
	enum
	{
		WithPostSerialize = true,
		WithCanEditChange = true,
	};
};

UENUM()
enum class UE_DEPRECATED(5.7, "Not used anymore. Use the result from GetCompatibilityWithOtherPin and query the conversion function from FPCGDataTypeInfo form the FPCGDataTypeRegistry.") EPCGTypeConversion : uint8
{
	NoConversionRequired,
	CollapseToPoint,
	Filter,
	MakeConcrete,
	SplineToSurface,
	Failed,
};

UCLASS(MinimalAPI)
class UPCGBlueprintPinHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "PCG|Pins")
	static int32 GetCorrespondingDataType(EPCGExclusiveDataType InExclusiveDataType)
	{
		return static_cast<int32>(GetCorrespondingDataType_Internal(InExclusiveDataType));
	}

	UFUNCTION(BlueprintPure, Category="PCG|Pins", meta = (NativeBreakFunc))
	static void BreakPinProperty(const FPCGPinProperties& PinProperty, FName& Label, bool& bAllowMultipleData, bool& bAllowMultipleConnections, bool& bIsAdvancedPin, EPCGExclusiveDataType& AllowedType)
	{
		Label = PinProperty.Label;

		const UEnum* DataTypeEnum = StaticEnum<EPCGDataType>();
		const UEnum* ExclusiveDataTypeEnum = StaticEnum<EPCGExclusiveDataType>();

		AllowedType = EPCGExclusiveDataType::Other;

		if (DataTypeEnum && ExclusiveDataTypeEnum)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FName DataTypeName = DataTypeEnum->GetNameByValue(static_cast<__underlying_type(EPCGDataType)>(PinProperty.AllowedTypes));
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (DataTypeName != NAME_None)
			{
				const int64 MatchingType = ExclusiveDataTypeEnum->GetValueByName(DataTypeName);

				if (MatchingType != INDEX_NONE)
				{
					AllowedType = static_cast<EPCGExclusiveDataType>(MatchingType);
				}
			}
		}

		bAllowMultipleData = PinProperty.bAllowMultipleData;
		bAllowMultipleConnections = PinProperty.AllowsMultipleConnections();
		bIsAdvancedPin = PinProperty.IsAdvancedPin();
	}

	UFUNCTION(BlueprintPure, Category="PCG|Pins", meta = (NativeMakeFunc))
	static FPCGPinProperties MakePinProperty(FName Label, bool bAllowMultipleData, bool bAllowMultipleConnections, bool bIsAdvancedPin, EPCGExclusiveDataType AllowedType = EPCGExclusiveDataType::Any)
	{
		EPCGDataType DataType = GetCorrespondingDataType_Internal(AllowedType);

		FPCGPinProperties PinProperties = FPCGPinProperties(Label, FPCGDataTypeIdentifier{DataType}, bAllowMultipleConnections, bAllowMultipleData);
		if (bIsAdvancedPin)
		{
			PinProperties.SetAdvancedPin();
		}

		return PinProperties;
	}

	UFUNCTION(BlueprintCallable, Category = "PCG|Pins")
	static bool IsOfType(int32 AllowedTypes, EPCGExclusiveDataType TypeToCheck)
	{
		int32 DataType = GetCorrespondingDataType(TypeToCheck);
		return !!(AllowedTypes & DataType);
	}

	UFUNCTION(BlueprintCallable, Category = "PCG|Pins")
	static bool IsExactlySameType(int32 AllowedTypes, EPCGExclusiveDataType TypeToCheck)
	{
		int32 DataType = GetCorrespondingDataType(TypeToCheck);
		return AllowedTypes == DataType;
	}

private:
	static EPCGDataType GetCorrespondingDataType_Internal(EPCGExclusiveDataType InExclusiveDataType)
	{
		const UEnum* DataTypeEnum = StaticEnum<EPCGDataType>();
		const UEnum* ExclusiveDataTypeEnum = StaticEnum<EPCGExclusiveDataType>();

		EPCGDataType DataType = EPCGDataType::Other;

		if (DataTypeEnum && ExclusiveDataTypeEnum)
		{
			FName ExclusiveDataTypeName = ExclusiveDataTypeEnum->GetNameByValue(static_cast<__underlying_type(EPCGExclusiveDataType)>(InExclusiveDataType));
			if (ensure(ExclusiveDataTypeName != NAME_None))
			{
				const int64 MatchingType = DataTypeEnum->GetValueByName(ExclusiveDataTypeName);
				if (ensure(MatchingType != INDEX_NONE))
				{
					DataType = static_cast<EPCGDataType>(MatchingType);
				}
			}
		}

		return DataType;
	}
};

UCLASS(MinimalAPI, ClassGroup = (Procedural))
class UPCGPin : public UObject
{
	GENERATED_BODY()

public:
	UE_API UPCGPin(const FObjectInitializer& ObjectInitializer);

	// ~Begin UObject interface
	UE_API virtual void PostLoad() override;
	// ~End UObject interface

	UPROPERTY(BlueprintReadOnly, Category = Properties)
	TObjectPtr<UPCGNode> Node = nullptr;

	UPROPERTY(meta=(PCGNoHash))
	FName Label_DEPRECATED = NAME_None;

	UPROPERTY(BlueprintReadOnly, TextExportTransient, Category = Properties)
	TArray<TObjectPtr<UPCGEdge>> Edges;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties))
	FPCGPinProperties Properties;

	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FText GetTooltip() const;
	
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetTooltip(const FText& InTooltip);

	UE_API bool AllowsMultipleConnections() const;
	UE_API bool AllowsMultipleData() const;
	UE_API bool IsCompatible(const UPCGPin* OtherPin) const;

	/** Query the data type registry to check if this pin is compatible with the other pin. Can also provide an optional text to have custom compatibility messages. */
	UE_API EPCGDataTypeCompatibilityResult GetCompatibilityWithOtherPin(const UPCGPin* OtherPin, FText* OptionalOutCompatibilityMessage = nullptr) const;

	/** For checking when the new node doesn't exist yet. */
	UE_API bool IsDownstreamPinTypeCompatible(EPCGDataType UpstreamTypes) const;
	UE_API bool CanConnect(const UPCGPin* OtherPin) const;

	/** Adds edge to pin if edge does not exist. Optionally returns list of affected nodes (including the pin owner). */
	UE_API bool AddEdgeTo(UPCGPin* OtherPin, TSet<UPCGNode*>* InTouchedNodes = nullptr);

	/** Breaks edge to pin if edge exists. Optionally returns list of affected nodes (including the pin owner). */
	UE_API bool BreakEdgeTo(UPCGPin* OtherPin, TSet<UPCGNode*>* InTouchedNodes = nullptr);

	/** Breaks all connected edges. Optionally returns list of affected nodes (including the pin owner). */
	UE_API bool BreakAllEdges(TSet<UPCGNode*>* InTouchedNodes = nullptr, TSet<UPCGPin*>* OptionalOtherPinsTouched = nullptr);

	/** Break all edges the connect pins of invalid type. Optionally returns list of affected nodes (including the pin owner). */
	UE_API bool BreakAllIncompatibleEdges(TSet<UPCGNode*>* InTouchedNodes = nullptr, FString* OptionalReason = nullptr);

	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API bool IsConnected() const;

	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API bool IsOutputPin() const;

	UE_API int32 EdgeCount() const;

	UE_DEPRECATED(5.7, "Use GetCurrentTypesID")
	UE_API EPCGDataType GetCurrentTypes() const;

	/** Returns the current pin types, which can either be the static types from the pin properties, or a dynamic type based on connected edges. */
	UE_API FPCGDataTypeIdentifier GetCurrentTypesID() const;

	UE_DEPRECATED(5.7, "Not used anymore. Use the result from GetCompatibilityWithOtherPin and query the conversion function from FPCGDataTypeInfo form the FPCGDataTypeRegistry.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	EPCGTypeConversion GetRequiredTypeConversion(const UPCGPin* InOtherPin) const { return EPCGTypeConversion::Failed; };
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:
	friend UPCGNode;

	// Keep track if a pin had their properties changed while loading, so UpdatePins will try to break incompatible edges.
	bool bPropertiesHaveChangedOnDeprecation = false;
};

#if WITH_EDITOR
namespace PCGPinPropertiesHelpers
{
	bool GetDefaultPinExtraIcon(const UPCGPin* InPin, FName& OutExtraIcon, FText& OutTooltip);
	bool GetDefaultPinExtraIcon(const FPCGPinProperties& InPinProperties, FName& OutExtraIcon, FText& OutTooltip);
}
#endif // WITH_EDITOR

/**
* Helper class to allow the BP to call the custom functions on FPCGPinProperties.
*/
UCLASS(MinimalAPI)
class UPCGPinPropertiesBlueprintHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "PCG|PinProperties", meta = (ScriptMethod))
	static bool AllowsMultipleConnections(UPARAM(ref) const FPCGPinProperties& PinProperties) { return PinProperties.AllowsMultipleConnections(); }

	UFUNCTION(BlueprintCallable, Category = "PCG|PinProperties", meta = (ScriptMethod))
	static void SetAllowMultipleConnections(UPARAM(ref) FPCGPinProperties& PinProperties, bool bAllowMultipleConnections) { PinProperties.SetAllowMultipleConnections(bAllowMultipleConnections); }

	UFUNCTION(BlueprintCallable, Category = "PCG|PinProperties", meta = (ScriptMethod))
	static bool IsAdvancedPin(UPARAM(ref) const FPCGPinProperties& PinProperties) { return PinProperties.IsAdvancedPin(); }

	UFUNCTION(BlueprintCallable, Category = "PCG|PinProperties", meta = (ScriptMethod))
	static void SetAdvancedPin(UPARAM(ref) FPCGPinProperties& PinProperties) { PinProperties.SetAdvancedPin(); }

	UFUNCTION(BlueprintCallable, Category = "PCG|PinProperties", meta = (ScriptMethod))
	static bool IsRequiredPin(UPARAM(ref) const FPCGPinProperties& PinProperties) { return PinProperties.IsRequiredPin(); }

	UFUNCTION(BlueprintCallable, Category = "PCG|PinProperties", meta = (ScriptMethod))
	static void SetRequiredPin(UPARAM(ref) FPCGPinProperties& PinProperties) { PinProperties.SetRequiredPin(); }

	UFUNCTION(BlueprintCallable, Category = "PCG|PinProperties", meta = (ScriptMethod))
	static bool IsNormalPin(UPARAM(ref) const FPCGPinProperties& PinProperties) { return PinProperties.IsNormalPin(); }

	UFUNCTION(BlueprintCallable, Category = "PCG|PinProperties", meta = (ScriptMethod))
	static void SetNormalPin(UPARAM(ref) FPCGPinProperties& PinProperties) { PinProperties.SetNormalPin(); }
};

#undef UE_API
