// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers/PCGPointHelpers.h"

#include "Math/Box.h"
#include "PCGPoint.generated.h"

#define UE_API PCG_API

class IPCGAttributeAccessor;

UENUM()
enum class EPCGPointProperties : uint8
{
	Density UMETA(Tooltip = "When points are sampled, this density value represents the highest value of the density function within that point's volume. It is also used as a weighted value, for example, when testing points against a threshold in filtering operations."),
	BoundsMin UMETA(Tooltip = "Minimum corner of the point's bounds in local space."),
	BoundsMax UMETA(Tooltip = "Maximum corner of the point's bounds in local space."),
	Extents UMETA(Tooltip = "Half the local space difference between the maximum and minimum bounds of the point's volume. Can be used with the point's position to represent the volume."),
	Color UMETA(Tooltip = "An RGBA (four channel) color value."),
	Position UMETA(Tooltip = "Location component of the point's transform."),
	Rotation UMETA(Tooltip = "Rotation component of the point's transform."),
	Scale UMETA(Tooltip = "Scale component of the point's transform."),
	Transform UMETA(Tooltip = "The point's transform."),
	Steepness UMETA(Tooltip = "A normalized value that establishes how 'hard' or 'soft' that volume will be represented. From 0, it will ramp up linearly increasing its influence over the density from the point's center up to two times the bounds. At 1, it will represent a binary box function with the size of the point's bounds."),
	LocalCenter UMETA(Tooltip = "The local center location of the point's volume, halfway between the minimum and maximum bounds."),
	Seed UMETA(Tooltip = "Used to seed random processes during various operations."),
	LocalSize UMETA(Tooltip = "The difference between the maximum and minimum bounds of the point."),
	ScaledLocalSize UMETA(Tooltip = "The difference between the maximum and minimum bounds of the point, after only the scale has been applied."),

	Invalid = 255 UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FPCGPoint
{
	GENERATED_BODY()
public:
	FPCGPoint() = default;
	UE_API FPCGPoint(const FTransform& InTransform, float InDensity, int32 InSeed);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Properties)
	FTransform Transform;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Properties)
	float Density = 1.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Properties)
	FVector BoundsMin = -FVector::One();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Properties)
	FVector BoundsMax = FVector::One();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Properties)
	FVector4 Color = FVector4::One();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Properties, meta = (ClampMin = "0", ClampMax = "1"))
	float Steepness = 0.5f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Properties)
	int32 Seed = 0;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Properties|Metadata")
	int64 MetadataEntry = -1;

	FBox GetLocalBounds() const
	{
		return PCGPointHelpers::GetLocalBounds(BoundsMin, BoundsMax);
	}

	FBox GetLocalDensityBounds() const
	{
		return PCGPointHelpers::GetLocalDensityBounds(Steepness, BoundsMin, BoundsMax);
	}

	void SetLocalBounds(const FBox& InBounds)
	{
		PCGPointHelpers::SetLocalBounds(InBounds, BoundsMin, BoundsMax);
	}

	FBoxSphereBounds GetDensityBounds() const
	{
		return PCGPointHelpers::GetDensityBounds(Transform, Steepness, BoundsMin, BoundsMax);
	}

	FVector GetExtents() const 
	{ 
		return PCGPointHelpers::GetExtents(BoundsMin, BoundsMax);
	}

	void SetExtents(const FVector& InExtents)
	{
		PCGPointHelpers::SetExtents(InExtents, BoundsMin, BoundsMax);
	}

	FVector GetScaledExtents() const 
	{ 
		return PCGPointHelpers::GetScaledExtents(Transform, BoundsMin, BoundsMax);
	}

	FVector GetLocalCenter() const 
	{ 
		return PCGPointHelpers::GetLocalCenter(BoundsMin, BoundsMax);
	}
	
	void SetLocalCenter(const FVector& InCenter)
	{
		PCGPointHelpers::SetLocalCenter(InCenter, BoundsMin, BoundsMax);
	}

	FVector GetLocalSize() const 
	{ 
		return PCGPointHelpers::GetLocalSize(BoundsMin, BoundsMax);
	}
	
	FVector GetScaledLocalSize() const 
	{ 
		return PCGPointHelpers::GetScaledLocalSize(Transform, BoundsMin, BoundsMax);
	}

	void ApplyScaleToBounds()
	{
		PCGPointHelpers::ApplyScaleToBounds(Transform, BoundsMin, BoundsMax);
	}

	void ResetPointCenter(const FVector& BoundsRatio)
	{
		PCGPointHelpers::ResetPointCenter(BoundsRatio, Transform, BoundsMin, BoundsMax);
	}

	using PointCustomPropertyGetter = TFunction<bool(const FPCGPoint&, void*)>;
	using PointCustomPropertySetter = TFunction<bool(FPCGPoint&, const void*)>;

	static UE_API bool HasCustomPropertyGetterSetter(FName Name);
	static UE_API TUniquePtr<IPCGAttributeAccessor> CreateCustomPropertyAccessor(FName Name);

	UE_API bool Serialize(FStructuredArchive::FSlot Slot);
};

template<>
struct TStructOpsTypeTraits<FPCGPoint> : public TStructOpsTypeTraitsBase2<FPCGPoint>
{
	enum
	{
		WithStructuredSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

#undef UE_API
