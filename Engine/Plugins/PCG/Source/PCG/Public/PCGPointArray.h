// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGPoint.h"

#include "Helpers/PCGPointHelpers.h"
#include "Utils/PCGValueRange.h"

#include "CoreTypes.h"
#include "Containers/StridedView.h"
#include "Math/MathFwd.h"
#include "Serialization/Archive.h"

#include "PCGPointArray.generated.h"

#define UE_API PCG_API

/**
 * Templated struct used to store NumValues of the same type.
 * If all values are equal then they are stored into the Value field and Values is not allocated
 * If some values differ then Values is allocated and stores the different values
 * 
 * It provides a GetValueRange()/GetConstValueRange() implementation so that those NumValues can be indexed without user knowing of the internal allocations.
 */
template<typename T>
struct FPCGPointArrayProperty
{
public:
	FPCGPointArrayProperty()
	{
		static_assert(std::is_trivially_copyable_v<T>, "FPCGPointArrayProperty type must be trivially copyable");
	}

	/** Defined for serialization purposes(FPCGPointArray trait WithIdenticalViaEquality) */
	[[nodiscard]] inline bool operator==(const FPCGPointArrayProperty<T>& Other) const
	{
		if (NumValues != Other.NumValues || Values.Num() != Other.Values.Num())
		{
			return false;
		}

		auto Equals = [](const T& ValueA, const T& ValueB)
		{
			if constexpr (std::is_same_v<FVector,T> || std::is_same_v<FTransform,T> || std::is_same_v<FVector4,T>)
			{
				return ValueA.Equals(ValueB);
			}
			else
			{
				return ValueA == ValueB;
			}
		};

		if(!Equals(Value, Other.Value))
		{
			return false;
		}

		for (int32 Index = 0; Index < Values.Num(); ++Index)
		{
			if (!Equals(Values[Index], Other.Values[Index]))
			{
				return false;
			}
		}

		return true;
	}
	
	/** If 'Values' isn't currently allocated, allocate it and copy 'Value' at every index of 'Values[0, NumValues - 1]' */
	inline void Allocate(bool bInitializeValues)
	{
		SetNum(NumValues, /*bAllocate=*/true, bInitializeValues);
	}

	/** Set the 'NumValues', optionally allocating 'Values' */
	inline void SetNum(int32 NewNum, bool bAllocate, bool bInitializeValues)
	{
		NumValues = NewNum;

		const int32 OldNum = Values.Num();
		if (OldNum == NewNum)
		{
			return;
		}

		if (OldNum > 0 || bAllocate)
		{
			Values.SetNumUninitialized(NewNum);
			if (bInitializeValues)
			{
				for (int32 i = OldNum; i < NewNum; ++i)
				{
					Values[i] = Value;
				}
			}
		}
	};

	inline int32 Num() const
	{
		return NumValues;
	}

	/** Free 'Values', effectively resets all values to 'Value' */
	inline void Free()
	{
		Values.Reset();
	}

	/** Set all values('Value/Values') to InValue */
	inline void SetValue(const T& InValue)
	{
		for (int32 i = 0; i < Values.Num(); ++i)
		{
			Values[i] = InValue;
		}
		Value = InValue;
	}

	/** Return value at Index */
	[[nodiscard]] inline const T& GetValue(int32 Index) const
	{
		ensure(Index >= 0 && Index < NumValues);
		if (Values.Num() == 0)
		{
			return Value;
		}

		return Values[Index];
	}

	inline void RemoveAt(int32 Index)
	{
		ensure(Index >= 0 && Index < NumValues);

		if (IsAllocated())
		{
			Values.RemoveAt(Index);
		}

		NumValues -= 1;
	}
	
	/** Move ranges of values(moves them back into the array).Used by the PCGAsync api */
	inline void MoveRange(int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
	{
		if (IsAllocated())
		{
			check(Values.IsValidIndex(MoveToIndex) && Values.IsValidIndex(MoveToIndex + NumElements - 1) && Values.IsValidIndex(RangeStartIndex) && Values.IsValidIndex(RangeStartIndex + NumElements - 1));
			std::memmove(&Values[MoveToIndex], &Values[RangeStartIndex], sizeof(T) * NumElements);
		}
	}

	/* Returns true if 'Values' is allocated */
	[[nodiscard]] inline bool IsAllocated() const
	{
		ensure(Values.Num() == 0 || Values.Num() == NumValues);
		return Values.Num() != 0;
	}

	/** Returns a TArray copy of size NumValues */
	[[nodiscard]] inline TArray<T> GetCopy() const
	{
		if (IsAllocated())
		{
			return Values;
		}

		TArray<T> Copy;
		Copy.SetNumUninitialized(NumValues);
		for (int32 i = 0; i < NumValues; ++i)
		{
			Copy[i] = Value;
		}

		return Copy;
	}

	/** Returns a TPCGValueRange<T> of NumValues and optionally allocate 'Values' */
	[[nodiscard]] inline TPCGValueRange<T> GetValueRange(bool bAllocate, bool bInitializeValues)
	{
		if (bAllocate)
		{
			// Allocate when getting non-const version
			SetNum(NumValues, /*bForce=*/true, bInitializeValues);
		}

		return TPCGValueRange<T>(IsAllocated() ? MakeStridedView(Values) : MakeStridedView(sizeof(Value), &Value, 1), NumValues);
	}

	/** Returns a TConstPCGValueRange<T> of NumValues */
	[[nodiscard]] inline TConstPCGValueRange<T> GetConstValueRange() const
	{
		return TConstPCGValueRange<T>(IsAllocated() ? MakeConstStridedView(Values) : MakeConstStridedView(sizeof(Value), &Value, 1), NumValues);
	}
		
	[[nodiscard]] inline SIZE_T GetSizeBytes() const
	{
		return Values.GetAllocatedSize() + sizeof(Value) + sizeof(NumValues) + sizeof(Values);
	}

	inline void Serialize(FArchive& Ar)
	{
		Ar << NumValues;
		Ar << Value;
		Ar << Values;
	}

	inline void CopyTo(FPCGPointArrayProperty<T>& OutProperty, int32 StartReadIndex, int32 StartWriteIndex, int32 Count) const
	{
		check(StartReadIndex + Count <= NumValues && StartWriteIndex + Count <= OutProperty.NumValues);
		if (!IsAllocated())
		{
			if (!OutProperty.IsAllocated())
			{
				OutProperty.Value = Value;
			}
			else
			{
				for (int32 Index = StartWriteIndex; Index < StartWriteIndex + Count; ++Index)
				{
					OutProperty.Values[Index] = Value;
				}
			}
		}
		else
		{
			check(OutProperty.IsAllocated());
			std::memcpy(&OutProperty.Values[StartWriteIndex], &Values[StartReadIndex], sizeof(T) * Count);
		}
	}

	inline void CopyUnallocatedProperty(FPCGPointArrayProperty<T>& OutProperty) const
	{
		if (!IsAllocated() && !OutProperty.IsAllocated())
		{
			OutProperty.Value = Value;
		}
	}

private:
	// Array containing values if allocated
	TArray<T> Values;
	// Value representing all values if array is unallocated
	T Value;
	// Number of values represented by this FPCGPointArrayProperty
	int32 NumValues;
};

USTRUCT()
struct FPCGPointArray
{
	GENERATED_BODY()
public:
	UE_API FPCGPointArray();

	friend bool operator==(const FPCGPointArray&, const FPCGPointArray&) = default;

	UE_API void SetNumPoints(int32 InNumPoints, bool bInitializeValues);
	int32 GetNumPoints() const { return NumPoints; }
	UE_API void RemoveAt(int32 Index);
		
	UE_API EPCGPointNativeProperties GetAllocatedProperties() const;
	UE_API void Allocate(EPCGPointNativeProperties InProperties);
	UE_API void Free(EPCGPointNativeProperties InProperties);
	UE_API void MoveRange(int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements);

	SIZE_T GetSizeBytes() const
	{
		return Transform.GetSizeBytes() 
			+ Density.GetSizeBytes() 
			+ BoundsMin.GetSizeBytes() 
			+ BoundsMax.GetSizeBytes() 
			+ Color.GetSizeBytes() 
			+ Steepness.GetSizeBytes() 
			+ Seed.GetSizeBytes() 
			+ MetadataEntry.GetSizeBytes();
	}

	UE_API TArray<FTransform> GetTransformCopy() const;

	TPCGValueRange<FTransform> GetTransformValueRange(bool bAllocate) { return Transform.GetValueRange(bAllocate, bInitializedValues); }
	TPCGValueRange<float> GetDensityValueRange(bool bAllocate) { return Density.GetValueRange(bAllocate, bInitializedValues); }
	TPCGValueRange<FVector> GetBoundsMinValueRange(bool bAllocate) { return BoundsMin.GetValueRange(bAllocate, bInitializedValues); }
	TPCGValueRange<FVector> GetBoundsMaxValueRange(bool bAllocate) { return BoundsMax.GetValueRange(bAllocate, bInitializedValues); }
	TPCGValueRange<FVector4> GetColorValueRange(bool bAllocate) { return Color.GetValueRange(bAllocate, bInitializedValues); }
	TPCGValueRange<float> GetSteepnessValueRange(bool bAllocate) { return Steepness.GetValueRange(bAllocate, bInitializedValues); }
	TPCGValueRange<int32> GetSeedValueRange(bool bAllocate) { return Seed.GetValueRange(bAllocate, bInitializedValues); }
	TPCGValueRange<int64> GetMetadataEntryValueRange(bool bAllocate) { return MetadataEntry.GetValueRange(bAllocate, bInitializedValues); }

	TConstPCGValueRange<FTransform> GetConstTransformValueRange() const { return Transform.GetConstValueRange(); }
	TConstPCGValueRange<float> GetConstDensityValueRange() const { return Density.GetConstValueRange(); }
	TConstPCGValueRange<FVector> GetConstBoundsMinValueRange() const { return BoundsMin.GetConstValueRange(); }
	TConstPCGValueRange<FVector> GetConstBoundsMaxValueRange() const { return BoundsMax.GetConstValueRange(); }
	TConstPCGValueRange<FVector4> GetConstColorValueRange() const { return Color.GetConstValueRange(); }
	TConstPCGValueRange<float> GetConstSteepnessValueRange() const { return Steepness.GetConstValueRange(); }
	TConstPCGValueRange<int32> GetConstSeedValueRange() const { return Seed.GetConstValueRange(); }
	TConstPCGValueRange<int64> GetConstMetadataEntryValueRange() const { return MetadataEntry.GetConstValueRange(); }

	void SetTransform(const FTransform& InTransform) { Transform.SetValue(InTransform); }
	void SetDensity(float InDensity) { Density.SetValue(InDensity); }
	void SetBoundsMin(const FVector& InBoundsMin) { BoundsMin.SetValue(InBoundsMin); }
	void SetBoundsMax(const FVector& InBoundsMax) { BoundsMax.SetValue(InBoundsMax); }
	void SetColor(const FVector4& InColor) { Color.SetValue(InColor); }
	void SetSteepness(float InSteepness) { Steepness.SetValue(InSteepness); }
	void SetSeed(int32 InSeed) { Seed.SetValue(InSeed); }
	void SetMetadataEntry(int64 InMetadataEntry) { MetadataEntry.SetValue(InMetadataEntry); }
		
	const FTransform& GetTransform(int32 InPointIndex) const { return Transform.GetValue(InPointIndex); }
	float GetDensity(int32 InPointIndex) const { return Density.GetValue(InPointIndex); }
	const FVector& GetBoundsMin(int32 InPointIndex) const { return BoundsMin.GetValue(InPointIndex); }
	const FVector& GetBoundsMax(int32 InPointIndex) const { return BoundsMax.GetValue(InPointIndex); }
	const FVector4& GetColor(int32 InPointIndex) const { return Color.GetValue(InPointIndex); }
	float GetSteepness(int32 InPointIndex) const { return Steepness.GetValue(InPointIndex); }
	int32 GetSeed(int32 InPointIndex) const { return Seed.GetValue(InPointIndex); }
	int64 GetMetadataEntry(int32 InPointIndex) const { return MetadataEntry.GetValue(InPointIndex); }

	FBoxSphereBounds GetDensityBounds(int32 InPointIndex) const
	{
		return PCGPointHelpers::GetDensityBounds(GetTransform(InPointIndex), GetSteepness(InPointIndex), GetBoundsMin(InPointIndex), GetBoundsMax(InPointIndex));
	}

	FBox GetLocalDensityBounds(int32 InPointIndex) const
	{
		return PCGPointHelpers::GetLocalDensityBounds(GetSteepness(InPointIndex), GetBoundsMin(InPointIndex), GetBoundsMax(InPointIndex));
	}

	FBox GetLocalBounds(int32 InPointIndex) const
	{
		return PCGPointHelpers::GetLocalBounds(GetBoundsMin(InPointIndex), GetBoundsMax(InPointIndex));
	}

	FVector GetLocalCenter(int32 InPointIndex) const
	{
		return PCGPointHelpers::GetLocalCenter(GetBoundsMin(InPointIndex), GetBoundsMax(InPointIndex));
	}

	FVector GetExtents(int32 InPointIndex) const
	{
		return PCGPointHelpers::GetExtents(GetBoundsMin(InPointIndex), GetBoundsMax(InPointIndex));
	}

	FVector GetScaledExtents(int32 InPointIndex) const
	{
		return PCGPointHelpers::GetScaledExtents(GetTransform(InPointIndex), GetBoundsMin(InPointIndex), GetBoundsMax(InPointIndex));
	}
	
	FVector GetLocalSize(int32 InPointIndex) const
	{
		return PCGPointHelpers::GetLocalSize(GetBoundsMin(InPointIndex), GetBoundsMax(InPointIndex));
	}

	FVector GetScaledLocalSize(int32 InPointIndex) const
	{
		return PCGPointHelpers::GetScaledLocalSize(GetTransform(InPointIndex), GetBoundsMin(InPointIndex), GetBoundsMax(InPointIndex));
	}

	// WithSerializer & WithIdenticalViaEquality needed for this struct to properly serialize
	UE_API bool Serialize(FArchive& Ar);

	UE_API void CopyPropertiesTo(FPCGPointArray& OutPointArray, int32 InStartReadIndex, int32 InStartWriteIndex, int32 Count, EPCGPointNativeProperties Properties) const;

private:
	friend class UPCGPointArrayData;
	
	FPCGPointArrayProperty<FTransform>	Transform;
	FPCGPointArrayProperty<float>		Density;
	FPCGPointArrayProperty<FVector>		BoundsMin;
	FPCGPointArrayProperty<FVector>		BoundsMax;
	FPCGPointArrayProperty<FVector4>	Color;
	FPCGPointArrayProperty<float>		Steepness;
	FPCGPointArrayProperty<int32>		Seed;
	FPCGPointArrayProperty<int64>		MetadataEntry;

	int32 NumPoints = 0;

	// Last call to SetNumPoints param value
	bool bInitializedValues = false;
};

template<>
struct TStructOpsTypeTraits<FPCGPointArray> : public TStructOpsTypeTraitsBase2<FPCGPointArray>
{
	enum
	{
		WithSerializer = true,
		WithIdenticalViaEquality = true,
	};
};

#undef UE_API
