// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Utils/PCGValueRange.h"

#include "PCGPointPropertiesTraits.generated.h"

// Value names need to match EPCGPointProperties
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EPCGPointNativeProperties : uint32
{
	None = 0 UMETA(Hidden),
	Transform = 1 << 0,
	Density = 1 << 1,
	BoundsMin = 1 << 2,
	BoundsMax = 1 << 3,
	Color = 1 << 4,
	Steepness = 1 << 5,
	Seed = 1 << 6,
	MetadataEntry = 1 << 7,
	All = Transform | Density | BoundsMin | BoundsMax | Color | Steepness | Seed | MetadataEntry,
	AllProperties = All & (~MetadataEntry),
};
ENUM_CLASS_FLAGS(EPCGPointNativeProperties);

template<EPCGPointNativeProperties Property>
struct TPCGBasePointNativeProperty
{
	static constexpr EPCGPointNativeProperties EnumValue = Property;
};

template<EPCGPointNativeProperties Property>
struct TPCGPointUnsupportedProperty
{
	static constexpr bool Value = false;
};

template<EPCGPointNativeProperties Property>
struct TPCGPointNativeProperty : public TPCGBasePointNativeProperty<Property>
{
	using Type = void;
	using AccessorType = Type;
	using ValueRange = TPCGValueRange<Type>;
	using ConstValueRange = TConstPCGValueRange<Type>;
};

template<>
struct TPCGPointNativeProperty<EPCGPointNativeProperties::Transform> : public TPCGBasePointNativeProperty<EPCGPointNativeProperties::Transform>
{
	using Type = FTransform;
	using AccessorType = Type;
	using ValueRange = TPCGValueRange<Type>;
	using ConstValueRange = TConstPCGValueRange<Type>;
};

using FPCGPointTransform = TPCGPointNativeProperty<EPCGPointNativeProperties::Transform>;

template<>
struct TPCGPointNativeProperty<EPCGPointNativeProperties::Density> : public TPCGBasePointNativeProperty<EPCGPointNativeProperties::Density>
{
	using Type = float;
	using AccessorType = double;
	using ValueRange = TPCGValueRange<Type>;
	using ConstValueRange = TConstPCGValueRange<Type>;
};

using FPCGPointDensity = TPCGPointNativeProperty<EPCGPointNativeProperties::Density>;

template<>
struct TPCGPointNativeProperty<EPCGPointNativeProperties::BoundsMin> : public TPCGBasePointNativeProperty<EPCGPointNativeProperties::BoundsMin>
{
	using Type = FVector;
	using AccessorType = Type;
	using ValueRange = TPCGValueRange<Type>;
	using ConstValueRange = TConstPCGValueRange<Type>;
};

using FPCGPointBoundsMin = TPCGPointNativeProperty<EPCGPointNativeProperties::BoundsMin>;

template<>
struct TPCGPointNativeProperty<EPCGPointNativeProperties::BoundsMax> : public TPCGBasePointNativeProperty<EPCGPointNativeProperties::BoundsMax>
{
	using Type = FVector;
	using AccessorType = Type;
	using ValueRange = TPCGValueRange<Type>;
	using ConstValueRange = TConstPCGValueRange<Type>;
};

using FPCGPointBoundsMax = TPCGPointNativeProperty<EPCGPointNativeProperties::BoundsMax>;

template<>
struct TPCGPointNativeProperty<EPCGPointNativeProperties::Color> : public TPCGBasePointNativeProperty<EPCGPointNativeProperties::Color>
{
	using Type = FVector4;
	using AccessorType = Type;
	using ValueRange = TPCGValueRange<Type>;
	using ConstValueRange = TConstPCGValueRange<Type>;
};

using FPCGPointColor = TPCGPointNativeProperty<EPCGPointNativeProperties::Color>;

template<>
struct TPCGPointNativeProperty<EPCGPointNativeProperties::Steepness> : public TPCGBasePointNativeProperty<EPCGPointNativeProperties::Steepness>
{
	using Type = float;
	using AccessorType = double;
	using ValueRange = TPCGValueRange<Type>;
	using ConstValueRange = TConstPCGValueRange<Type>;
};

using FPCGPointSteepness = TPCGPointNativeProperty<EPCGPointNativeProperties::Steepness>;

template<>
struct TPCGPointNativeProperty<EPCGPointNativeProperties::Seed> : public TPCGBasePointNativeProperty<EPCGPointNativeProperties::Seed>
{
	using Type = int32;
	using AccessorType = Type;
	using ValueRange = TPCGValueRange<Type>;
	using ConstValueRange = TConstPCGValueRange<Type>;
};

using FPCGPointSeed = TPCGPointNativeProperty<EPCGPointNativeProperties::Seed>;

template<>
struct TPCGPointNativeProperty<EPCGPointNativeProperties::MetadataEntry> : public TPCGBasePointNativeProperty<EPCGPointNativeProperties::MetadataEntry>
{
	using Type = int64;
	using AccessorType = Type;
	using ValueRange = TPCGValueRange<Type>;
	using ConstValueRange = TConstPCGValueRange<Type>;
};

using FPCGPointMetadataEntry = TPCGPointNativeProperty<EPCGPointNativeProperties::MetadataEntry>;