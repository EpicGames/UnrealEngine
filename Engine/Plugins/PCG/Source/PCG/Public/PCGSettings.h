// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "PCGDebug.h"
#include "PCGElement.h"
#include "PCGPin.h"
#include "Data/Registry/PCGDataType.h"
#include "Elements/PCGActorSelector.h"
#include "Helpers/PCGAssetHelpers.h"
#include "Utils/PCGPreconfiguration.h"
#include "Tests/Determinism/PCGDeterminismSettings.h"

#include "Algo/AllOf.h"
#include "Misc/TVariant.h"

#include "PCGSettings.generated.h"

#define UE_API PCG_API

class UPCGComponent;
class UPCGPin;
class UPCGComputeKernel;
class UPCGGraph;
class UPCGNode;
class UPCGSettings;
class UPropertyBag;
class UTexture;
struct FPCGGPUCompilationContext;
struct FPCGKernelEdge;
struct FPCGPinProperties;
struct FPCGPinReference;
struct FPropertyChangedEvent;

using FPCGSettingsAndCulling = TPair<TSoftObjectPtr<const UPCGSettings>, bool>;
using FPCGSelectionKeyToSettingsMap = TMap<FPCGSelectionKey, TArray<FPCGSettingsAndCulling>>;

namespace PCGSettings
{
	// A key is culled if and only if all the settings are culled.
	inline bool IsKeyCulled(const TArray<FPCGSettingsAndCulling>& SettingsAndCulling)
	{
		return Algo::AllOf(SettingsAndCulling, [](const FPCGSettingsAndCulling& SettingsAndCullingPair) { return SettingsAndCullingPair.Value; });
	}
}

UENUM()
enum class EPCGSettingsExecutionMode : uint8
{
	Enabled,
	Debug,
	Isolated,
	Disabled
};

UENUM()
enum class EPCGSettingsType : uint8
{
	InputOutput,
	Spatial,
	Density,
	Blueprint,
	Metadata,
	Filter,
	Sampler,
	Spawner,
	Subgraph,
	Debug,
	Generic,
	Param,
	HierarchicalGeneration,
	ControlFlow,
	PointOps,
	GraphParameters,
	Reroute,
	GPU,
	DynamicMesh,
	DataLayers,
	Resource,
};

#if WITH_EDITOR
struct FPCGNodeThumbnailProxy
{
	using FTextureType = TVariant<const UTexture*, FSoftObjectPath>;

	PCG_API static FPCGNodeThumbnailProxy FromAssetPath(const FSoftObjectPath& InAssetPath);
	PCG_API static FPCGNodeThumbnailProxy FromTexture(const UTexture* InTexture);

	FTextureType TextureSource;
};

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPCGSettingsChanged, UPCGSettings*, EPCGChangeType);
#endif

// Dummy struct to bypass the UHT limitation for array of arrays.
USTRUCT(meta=(Hidden))
struct FPCGPropertyAliases
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FName> Aliases;
};

USTRUCT()
struct FPCGSettingsOverridableParam
{
	GENERATED_BODY()

	UPROPERTY()
	FName Label = NAME_None;

	UPROPERTY()
	TArray<FName> PropertiesNames;

	UPROPERTY()
	TObjectPtr<const UStruct> PropertyClass;

	// Map of all aliases for a given property, using its Index (to avoid name clashes within the same path)
	UPROPERTY()
	TMap<int32, FPCGPropertyAliases> MapOfAliases;

	// If this flag is true, Label will be the full property path.
	UPROPERTY()
	bool bHasNameClash = false;

	/** Whether this overridable param is supported on the GPU. */
	UPROPERTY()
	bool bSupportsGPU = false;

	/** Whether this overridable param requires a readback to CPU when overridden on the GPU. Necessary for params which can drive CPU logic during
	 * ComputeGraph dispatch, for example any params that affect thread counts, buffer sizes, data descriptions, etc.
	 */
	UPROPERTY()
	bool bRequiresGPUReadback = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	EPCGMetadataTypes UnderlyingType = EPCGMetadataTypes::Unknown;
#endif // WITH_EDITORONLY_DATA

	/** Indicates the number of containers in the property chain. */
	UPROPERTY()
	int32 NumContainers = 0;

	bool HasAliases() const { return !MapOfAliases.IsEmpty(); }

	// Transient
	TArray<const FProperty*> Properties;

	FString GetPropertyPath() const;

	TArray<FName> GenerateAllPossibleAliases() const;

	/** Returns true if the last property is an ObjectProperty. */
	PCG_API bool IsHardReferenceOverride() const;

#if WITH_EDITOR
	FString GetDisplayPropertyPath() const;
	PCG_API FText GetDisplayPropertyPathText() const;
#endif // WITH_EDITOR
};

USTRUCT()
struct FPCGDataTypeInfoSettings : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Settings);

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; }
#endif // WITH_EDITOR
};

UCLASS(MinimalAPI, Abstract)
class UPCGSettingsInterface : public UPCGData
{
	GENERATED_BODY()

public:
	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoSettings)
	// ~End UPCGData interface
	
	virtual UPCGSettings* GetSettings() PURE_VIRTUAL(UPCGSettingsInterface::GetSettings, return nullptr;);
	virtual const UPCGSettings* GetSettings() const PURE_VIRTUAL(UPCGSettingsInterface::GetSettings, return nullptr;);

	UE_API bool IsInstance() const;
	/** Dedicated method to change enable state because some nodes have more complex behavior on enable/disable (such as subgraphs) */
	UE_API void SetEnabled(bool bInEnabled);

	/**
	 * Dedicated method to change debug state because some nodes have more complex behavior on enable/disable (such as reroutes).
	 * Returns true if the flag actually changed.
	 */
	UE_API bool SetDebugged(bool bInDebugged);

#if WITH_EDITOR
	/** Whether this node has manual edit access in the viewport. */
	bool IsTemporaryManualEditingEnabled() const { return bTemporaryManualEditingEnabled; }

	/** Whether this node is persistently marked for manual editing. */
	bool IsMarkedForManualEditing() const { return bMarkedForManualEditing; }

	/** Marks the node for a one-shot pass of manual edit in the viewport. */
	UE_API void SetTemporaryManualEditingEnabled(bool bInEnabled);

	/** Marks the node for permanent access to manual edit in the viewport. */
	UE_API void SetMarkedForManualEditing(bool bInMarked);
#endif // WITH_EDITOR

	/** Whether this element can be disabled. */
	UFUNCTION()
	virtual bool CanBeDisabled() const { return true; }

	/** Whether this element supports Debug and Inspect features. */
	virtual bool CanBeDebugged() const { return true; }

#if WITH_EDITOR
	FOnPCGSettingsChanged OnSettingsChangedDelegate;

	/** Whether the Inspect feature is active on the node corresponding to this settings. */
	bool bIsInspecting = false;
	
	/** Whether the respective viewport tool (if any) is actively running in the viewport. */
	bool bNodeToolActive = false;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug, meta = (EditCondition = "bDisplayDebuggingProperties && CanBeDisabled()", EditConditionHides, HideEditConditionToggle))
	bool bEnabled = true;

	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category = Debug, meta = (EditCondition = bDisplayDebuggingProperties, EditConditionHides, HideEditConditionToggle))
	bool bDebug = false;

#if WITH_EDITORONLY_DATA

private:
	/** Set while the user is actively viewport-editing this node. */
	UPROPERTY(Transient, DuplicateTransient, meta = (PCGNoHash))
	bool bTemporaryManualEditingEnabled = false;

	/** Saved to the graph asset so viewport editing markers survive across sessions. */
	UPROPERTY(meta = (PCGNoHash))
	bool bMarkedForManualEditing = false;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug, meta = (ShowOnlyInnerProperties, PCGNoHash))
	FPCGDebugVisualizationSettings DebugSettings;

	/** If a debugger is attached, a breakpoint will be triggered in the execution code to enable debugging. Only applies when inspecting a debug object. Transient and Editor-only. */
	UPROPERTY(Transient, DuplicateTransient, EditAnywhere, BlueprintReadWrite, Category = Debug, AdvancedDisplay, meta = (DisplayName = "Break In Debugger", EditCondition = bDisplayDebuggingProperties, EditConditionHides, HideEditConditionToggle))
	bool bBreakDebugger = false;

	// This can be set false by inheriting nodes to hide the debugging properties.
	UPROPERTY(Transient, meta = (EditCondition = false, EditConditionHides))
	bool bDisplayDebuggingProperties = true;
#endif
};

/**
* Base class for settings-as-data in the PCG framework
*/
UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = (Procedural))
class UPCGSettings : public UPCGSettingsInterface
{
	GENERATED_BODY()

	friend class FPCGSettingsHashPolicy;
	friend class UPCGSettingsInterface;
	friend struct FPCGContext;

#if WITH_EDITOR
	friend class FPCGAssetHelpers;
#endif

public:
	// ~Begin UPCGData interface
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	// ~Begin UPCGSettingsInterface interface
	virtual UPCGSettings* GetSettings() { return this; }
	virtual const UPCGSettings* GetSettings() const { return this; }
	// ~End UPCGSettingsInterface interface

	//~Begin UObject interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext) override;
	UE_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif // WITH_EDITOR
	//~End UObject interface

	UE_API void OnOverrideSettingsDuplicated(bool bSkippedPostLoad);

	// TODO: check if we need this to be virtual, we don't really need if we're always caching
	UE_API /*virtual*/ FPCGElementPtr GetElement() const;
	UE_API virtual UPCGNode* CreateNode() const;
	
#if WITH_EDITOR
	bool IsNodeToolActive() const { return bNodeToolActive; }
	void SetNodeToolActive(bool InActive) { bNodeToolActive = InActive; }

	virtual EPCGNodeToolStartBehavior GetNodeToolStartBehaviour() const { return EPCGNodeToolStartBehavior::OnToolStartButton; }
#endif // WITH_EDITOR
	
	/** Return the concatenation of InputPinProperties, ExecutionDependencyPin, and FillOverridableParamsPins */
	UE_API TArray<FPCGPinProperties> AllInputPinProperties() const;

	/** For symmetry reason, do the same with output pins. For now forward just the call to OutputPinProperties */
	UE_API TArray<FPCGPinProperties> AllOutputPinProperties() const;

	/** If the node has any dynamic pins that can change type based on input types or settings. If true the default behavior 
	* for the output pin types will be to take the union of types incident on primary input pin. */
	virtual bool HasDynamicPins() const { return false; }

	/** Whether this node can deactivate its output pins during execution, which can dynamically cull downstream nodes. */
	virtual bool OutputPinsCanBeDeactivated() const { return false; }

	UE_DEPRECATED(5.7, "Use GetTypeUnionIDOfIncidentEdges version")
	UE_API EPCGDataType GetTypeUnionOfIncidentEdges(const FName& PinLabel) const;

	/** Bitwise union of the allowed types of each incident edge on pin. Returns None if no common bits, or no edges. */
	UFUNCTION(BlueprintCallable, Category="Settings|DynamicPins")
	UE_API FPCGDataTypeIdentifier GetTypeUnionIDOfIncidentEdges(const FName& PinLabel) const;

	/** Bitwise union of the allowed types of each incident edge on pin. Returns None if no common bits, or no edges. */
	UE_API FPCGDataTypeIdentifier GetTypeUnionIDOfAllIncidentEdges(TConstArrayView<FName> PinLabels) const;

	// Internal functions, should not be used by any user.
	// Return a different subset for input/output pin properties, in case of a default object.
	UE_API virtual TArray<FPCGPinProperties> DefaultInputPinProperties() const;
	UE_API virtual TArray<FPCGPinProperties> DefaultOutputPinProperties() const;

	UE_API bool operator==(const UPCGSettings& Other) const;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** By default, settings do not use a seed. Override this in the settings subclass to enable usage of the seed. UFUNCTION to be used by EditCondition. */
	UFUNCTION()
	virtual bool UseSeed() const { return bUseSeed; }
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Get the seed, combined with optional ExecutionSource seed
	int GetSeed(const IPCGGraphExecutionSource* InExecutionSource = nullptr) const;

	UE_DEPRECATED(5.6, "Use GetSettingsCrc instead")
	UE_API const FPCGCrc& GetCachedCrc() const;

	UE_API FPCGCrc GetSettingsCrc() const;

#if WITH_EDITOR
	/** Puts node title on node body, reducing overall node size */
	virtual bool ShouldDrawNodeCompact() const { return false; }
	/** Whether to show/hide the title text/icon on the compact node. */
	virtual bool ShouldShowCompactNodeTitle() const { return true; }
	/** Returns the icon to use instead of text in compact node form */
	virtual bool GetCompactNodeIcon(FName& OutCompactNodeIcon) const { return false; }
	/** Returns whether the user can directly interact with the node name */
	UE_API virtual bool CanUserEditTitle() const;

	/** UpdatePins will kick off invalid edges, so this is useful for moving edges around in case of pin changes. */
	UE_API virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins);
	/** Any final migration/recovery that can be done after pins are finalized. This function should also set DataVersion to LatestVersion. */
	UE_API virtual void ApplyDeprecation(UPCGNode* InOutNode);

	/** If settings require structural changes, this will apply them */
	UE_API virtual void ApplyStructuralDeprecation(UPCGNode* InOutNode);

	virtual FName GetDefaultNodeName() const { return NAME_None; }
	virtual FText GetDefaultNodeTitle() const { return FText::FromName(GetDefaultNodeName()); }

	/** List of extra aliases that will be added to the node list in the Editor.Useful when we rename a node, but we still want the user to find the old one. */
	virtual TArray<FText> GetNodeTitleAliases() const { return {}; }

	virtual FText GetNodeTooltipText() const { return FText::GetEmpty(); }
	virtual FLinearColor GetNodeTitleColor() const { return FLinearColor::White; }
	virtual TOptional<FPCGNodeThumbnailProxy> GetNodeThumbnail() const { return {}; }
	virtual EPCGSettingsType GetType() const { return EPCGSettingsType::Generic; }

	/** Can override the label style for a pin. Return false if no override is available. */
	virtual bool GetPinLabelStyle(const UPCGPin* InPin, FName& OutLabelStyle) const { return false; }

	/** Can override to add a custom icon next to the pin label (and an optional tooltip). Return false if no override is available. */
	UE_API virtual bool GetPinExtraIcon(const UPCGPin* InPin, FName& OutExtraIcon, FText& OutTooltip) const;

	/** Derived classes must implement this to communicate dependencies that are known statically. */
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const {}

	/** Derived classes must implement this to communicate that they might have dynamic dependencies. */
	virtual bool CanDynamicallyTrackKeys() const { return false; }

	/** Override this class to provide an UObject to jump to in case of double click on node
	 *  ie. returning a blueprint instance will open the given blueprint in its editor.
	 *  By default, it will return the underlying class, to try to jump to its header in code
     */
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const;

	/** Return preconfigured info that will be filled in the editor palette action, allowing to create pre-configured settings */
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const { return {}; }

	/** If there are preconfigured info, we can skip the default settings and only expose pre-configured actions in the editor palette */
	virtual bool OnlyExposePreconfiguredSettings() const { return false; }

	/** If there are preconfigured info, decide if they are grouped in the palette in a folder with the node name, or if they are not grouped. */
	virtual bool GroupPreconfiguredSettings() const { return true; }

	/** The predefined parameters defining the conversion. */
	virtual TArray<FPCGPreconfiguredInfo> GetConversionInfo() const { return {}; }

	/** Perform post-operations when an editor node is copied */
	UE_API virtual void PostPaste();

	/** Returns whether the execution dependency pin can be configured to be used for culling. */
	UFUNCTION()
	bool CanUseExecutionDependencyPinForCulling() const;

	/** Overridable function for child classes to have a node title sanitization when the title of a node changes. Return true if the name has changed. */
	virtual bool SanitizeNodeTitle(FName& InOutName) const { return false; }
#endif // WITH_EDITOR

	/** Derived classes can implement this to expose additional information or context, such as an asset in use by the node. */
	virtual FString GetAdditionalTitleInformation() const { return FString(); }

	/** Display generated title line as primary title (example: BP nodes display the blueprint name as the primary title). */
	virtual bool HasFlippedTitleLines() const { return false; }

	/** Returns true if the given property is overridden by graph parameters */
	UE_API virtual bool IsPropertyOverriddenByPin(const FProperty* InProperty) const;

	/** Returns true if the base property, given by name, is overridden by graph parameters */
	UE_API virtual bool IsPropertyOverriddenByPin(FName PropertyName) const;

	/** Returns true if the property, given by chain of property names, is overridden by graph parameters */
	UE_API virtual bool IsPropertyOverriddenByPin(const TArrayView<const FName>& PropertyNameChain) const;

	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) {}

	UE_DEPRECATED(5.4, "AdditionalTaskName is deprecated and is replaced with GetAdditionalTitleInformation.")
	virtual FName AdditionalTaskName() const { return FName(GetAdditionalTitleInformation()); }

	/** By default a node does not specify any pin requirements, and will execute if it has no non-advanced pins or if it has any active
	* connection to any pin. Set the Required status on pin properties to specify which pins *must* have active connections to avoid getting culled.
	*/
	virtual bool IsInputPinRequiredByExecution(const UPCGPin* InPin) const { return InPin && InPin->Properties.IsRequiredPin(); }

	/** Returns true if InPin is in use by node (assuming node enabled). Can be used to communicate when a pin is not in use to user. */
	UE_API virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const;

	/** True if we know prior to execution that the given pin will be active (not on inactive branch). */
	virtual bool IsPinStaticallyActive(const FName& OutputPinLabel) const { return true; }

	/** True if we can safely cull this node & task if it has unwired non-advanced inputs. Counter-example: BP nodes may have side effects and always execute
	* if not on an inactive branch.
	*/
	virtual bool CanCullTaskIfUnwired() const { return true; }

	/** Returns true if only the first input edge is used from the primary pin when the node is disabled. */
	virtual bool OnlyPassThroughOneEdgeWhenDisabled() const { return false; }

	/** By default, the first non-Advanced pin that supports pass-through will be selected. */
	UE_API virtual bool DoesPinSupportPassThrough(UPCGPin* InPin) const;

	UE_DEPRECATED(5.7, "Use GetCurrentPinTypesID version")
	UE_API virtual EPCGDataType GetCurrentPinTypes(const UPCGPin* InPin) const;
	
	/** Returns the current pin types, which can either be the static types from the pin properties, or a dynamic type based on connected edges.
	* By default we set output pin types to the union of the default input pin incident edge types, if it is dynamic and the default input exists.
	*/
	UE_API virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const;

	/** Convert this owning node into the corresponding target. Returns true upon success. */
	virtual bool ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo) { return false; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta=(EditCondition="UseSeed()", EditConditionHides, PCG_Overridable))
	int Seed = 0xC35A9631; // Default seed is a random prime number, but will be overriden for new settings based on the class type name hash, making each settings class have a different default seed.

	/** If enabled, the execution dependency pin will require to be connected for execution (as any other required pin) and will be used for culling. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta=(EditCondition="CanUseExecutionDependencyPinForCulling()", EditConditionHides))
	bool bExecutionDependencyRequired = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TSet<FString> TagsAppliedOnOutput_DEPRECATED;

	UPROPERTY()
	EPCGSettingsExecutionMode ExecutionMode_DEPRECATED = EPCGSettingsExecutionMode::Enabled;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug, AdvancedDisplay, meta = (DisplayName="Determinism", NoResetToDefault, PCGNoHash))
	FPCGDeterminismSettings DeterminismSettings;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (PCGNoHash))
	bool bExposeToLibrary = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (PCGNoHash))
	FText Category;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (PCGNoHash))
	FText Description;
#endif

	// Returns Original UID when this is a duplicated settings so we can compare successive executions for reuse cases
	uint64 GetStableUID() const { return OriginalSettings ? OriginalSettings->UID : UID; }

	// Holds the original settings used to duplicate this object if it was overridden
	const UPCGSettings* OriginalSettings = nullptr;

	// Returns an array of all the input pin properties. You should not add manually a "params" pin, it is handled automatically by FillOverridableParamsPins
	UE_API virtual TArray<FPCGPinProperties> InputPinProperties() const;
	UE_API virtual TArray<FPCGPinProperties> OutputPinProperties() const;

protected:
	// BP version since EPCGDataType is uint32 and BP only supports uint8 enums.
	/** Bitwise union of the allowed types of each incident edge on pin. Returns None type if no common bits, or no edges. Use the BP function helpers to extract the types from the result. */
	UE_DEPRECATED(5.7, "Use GetTypeUnionIDOfIncidentEdges")
	UFUNCTION(BlueprintCallable, Category = "Settings|DynamicPins", DisplayName = "Get Type Union Of Incident Edges")
	int32 BP_GetTypeUnionOfIncidentEdges(const FName& PinLabel) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return static_cast<int32>(GetTypeUnionOfIncidentEdges(PinLabel));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	virtual FPCGElementPtr CreateElement() const PURE_VIRTUAL(UPCGSettings::CreateElement, return nullptr;);

	/** An additional custom version number that external system users can use to track versions. This version will be serialized into the asset and will be provided by UserDataVersion after load. */
	virtual FGuid GetUserCustomVersionGuid() { return FGuid(); }

	/** Can be overriden by child class if they ever got renamed to avoid changing the default seed for this one. Otherwise default is hash of the class name. */
	UE_API virtual uint32 GetTypeNameHash() const;

	/** Can be overriden by child class if some fixup code needs to run after duplication in the context of FPCGContext::InitializeSettings */
	virtual void OnOverrideSettingsDuplicatedInternal(bool bSkippedPostLoad) {};

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UE_DEPRECATED(5.4, "IsStructuralProperty is deprecated, return EPCGChangeType::Structural from GetChangeTypeForProperty instead.")
	virtual bool IsStructuralProperty(const FName& InPropertyName) const { return false; }

	/** Gets the change impact for a given property. Can be used to signal structural or cosmetic node changes for example - calls the named version by default. */
	UE_API virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const;

	/** Gets the change impact for a given property. Can be used to signal structural or cosmetic node changes for example. */
	UE_API virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const;

	/** Method that can be called to dirty the cache data from this settings objects if the operator== does not allow to detect changes */
	UE_API void DirtyCache();

	// We need the more complex function (with PropertyChain) to detect child properties in structs, if they are overridable
	UE_API virtual bool CanEditChange(const FEditPropertyChain& InPropertyChain) const override;

	// Passthrough for the simpler method, to avoid modifying the child settings already overriding this method.
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

	UE_DEPRECATED(5.5, "Implement the PCGSettings virtual UseSeed() override.")
	UPROPERTY(Transient, meta = (DeprecatedProperty, DeprecationMessage = "Implement the PCGSettings virtual UseSeed() override."))
	bool bUseSeed = false;

	/** Methods to remove boilerplate code across settings */
	UE_API TArray<FPCGPinProperties> DefaultPointInputPinProperties() const;
	UE_API TArray<FPCGPinProperties> DefaultPointOutputPinProperties() const;

public:
#if WITH_EDITORONLY_DATA
	/** The version number of the data after load and after any data migration. */
	int32 DataVersion = -1;

	/** If a custom version guid was provided through GetUserCustomVersionGuid(), this field will hold the version number after load and after any data migration. */
	int32 UserDataVersion = -1;
#endif // WITH_EDITORONLY_DATA

private:
	mutable FPCGElementPtr CachedElement;
	mutable PCG::FLock CacheLock;

	// Overridable param section
public:
	/** List of all the overridable params available for these settings. */
	virtual const TArray<FPCGSettingsOverridableParam>& OverridableParams() const { return CachedOverridableParams; }

	/** Check if we have some override. Can be overriden to force params pin for example */
	virtual bool HasOverridableParams() const { return !CachedOverridableParams.IsEmpty(); }

	/** Checks if a label matches an overridable param */
	UE_API virtual bool HasOverridableParam(FName InParamName) const;

	/** Check if we need to hook the output of the pre-task to this. One use is to compute overrides in the subgraph element and pass the overrides as data, to all nodes that needs it. */
	virtual bool RequiresDataFromPreTask() const { return false; }

	/** Creates an empty property bag instance from this settings. Values will be default initialized, not set from the settings instance. */
	FInstancedPropertyBag CreateEmptyPropertyBagInstance() const;

protected:
	/** Iterate over OverridableParams to automatically add param pins to the list. */
	UE_API void FillOverridableParamsPins(TArray<FPCGPinProperties>& OutPins) const;

#if WITH_EDITOR
	/** Can be overridden to add more custom params (like in BP). */
	UE_API virtual TArray<FPCGSettingsOverridableParam> GatherOverridableParams() const;
#endif // WITH_EDITOR

	/** Called after intialization to construct the list of all overridable params. 
	* In Editor, it will first gather all the overridable properties names and labels, based on their metadata.
	* Can request a "reset" if something changed in the settings.
	* And then, in Editor and Runtime, will gather the FProperty*.
	*/
	UE_API void InitializeCachedOverridableParams(bool bReset = false);

	// @todo_pcg: This could potentially be done once per class rather than once per class instance.
	/** Called on load to cache the overridden settings property bag. */
	void InitializeOverriddenSettingsPropertyBag();

	/**
	* There is a weird issue where the BP class is not set correctly in some Server cases.
	* We can try to recover if the PropertyClass is null.
	*/
	UE_API virtual void FixingOverridableParamPropertyClass(FPCGSettingsOverridableParam& Param) const;

	/** Needs to be serialized since property metadata (used to populate this array) is not available at runtime. */
	UPROPERTY(meta = (PCGNoHash))
	TArray<FPCGSettingsOverridableParam> CachedOverridableParams;

	/** Property bag used to create instances of overridden parameter structs for execution. */
	UPROPERTY(Transient)
	TObjectPtr<const UPropertyBag> OverriddenSettingsPropertyBag = nullptr;

	// We need to make sure that if we have hard references that are overridable, and they are overriden by paths
	// on objects that are not yet loaded, that we are loading it on the main thread.
	bool bHasAnyOverridableHardReferences = false;
	
public:
	bool HasAnyOverridableHardReferences() const { return bHasAnyOverridableHardReferences; }

	// GPU section
public:

	/** Whether this node should be executed on the GPU. */
	UE_API virtual bool ShouldExecuteOnGPU() const;

#if WITH_EDITOR
	/** Set whether this node should be executed on the GPU. */
	void SetExecuteOnGPU(bool bValue) { bExecuteOnGPU = bValue; }

	/** Create the GPU kernels and kernel edges that should execute for this node. */
	UE_DEPRECATED(5.8, "Override the overload that takes a UPCGNode.")
	UE_API virtual void CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const;

	/** Create the GPU kernels and kernel edges that should execute for this node. */
	virtual void CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const {}

	/** Whether to display GPU execution option in node settings UI. */
	virtual bool DisplayExecuteOnGPUSetting() const { return false; }

	/** Whether this node is GPU-friendly: either it has a GPU execution backend, or it runs on CPU but does not force GPU data readbacks or uploads. */
	UE_API virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const;
#endif

// Execution Dependency section
protected:
	/** This node should have an advanced Execution Dependency pin. */
	virtual bool HasExecutionDependencyPin() const { return true; }

private:
#if WITH_EDITOR
	UFUNCTION()
	UE_API bool DisplayExecuteOnGPUSettingCondition() const;
#endif

	/** Adds the universal Execution Dependency pin to the node's pin properties. */
	UE_API void AddExecutionDependencyPin(TArray<FPCGPinProperties>& OutPins) const;

protected:
	/** Whether this node should be executed on the GPU. */
	UPROPERTY(EditAnywhere, Category = "GPU", meta = (EditCondition = "DisplayExecuteOnGPUSettingCondition()", EditConditionHides, HideEditConditionToggle))
	bool bExecuteOnGPU = false;

public:
	/** Dump the cooked HLSL into the log after it is generated. */
	UPROPERTY(EditAnywhere, Category = "GPU", AdvancedDisplay, meta = (EditCondition = "bExecuteOnGPU", EditConditionHides))
	bool bDumpCookedHLSL = false;

	/** Dump the data descriptions of input/output pins to the log. */
	UPROPERTY(EditAnywhere, Category = "GPU", AdvancedDisplay, meta = (EditCondition = "bExecuteOnGPU", EditConditionHides))
	bool bDumpDataDescriptions = false;

	/** Enable use of 'WriteDebugValue(uint Index, float Value)' function in your kernel. Allows you to write float values to a buffer for logging on the CPU. */
	UPROPERTY(EditAnywhere, Category = "GPU", AdvancedDisplay, meta = (EditCondition = "bExecuteOnGPU", EditConditionHides))
	bool bPrintShaderDebugValues = false;

	/** Size (in number of floats) of the shader debug print buffer. */
	UPROPERTY(EditAnywhere, Category = "GPU", AdvancedDisplay, meta = (EditCondition="bExecuteOnGPU && bPrintShaderDebugValues", EditConditionHides))
	int DebugBufferSize = 16;

#if WITH_EDITORONLY_DATA
	/** Will trigger a render capture when this node executes and a debug object is selected in the graph editor. Transient and Editor-only. Render captures must be enabled (e.g. -AttachRenderDoc or -AttachPIX). */
	UPROPERTY(EditAnywhere, Transient, Category = "GPU", AdvancedDisplay, meta = (EditCondition = "bExecuteOnGPU", EditConditionHides))
	bool bTriggerRenderCapture = false;

	/** Index of kernel emitted by this node to repeatedly dispatch every frame to enable profiling. Set to -1 to disable profiling. PCG_GPU_KERNEL_PROFILING must be defined in PCG.Build.cs. */
	UPROPERTY(EditAnywhere, Transient, Category = "GPU", AdvancedDisplay, meta = (EditCondition = "bExecuteOnGPU", EditConditionHides))
	int ProfileKernelIndex = INDEX_NONE;
#endif

private:
	/** Calculate Crc for these settings and save it. */
	UE_API void CacheCrc();

#if WITH_EDITOR
	/** Self-subscribing handler that invalidates CachedCrc whenever OnSettingsChangedDelegate fires. */
	void OnSettingsChanged(UPCGSettings* InSettings, EPCGChangeType ChangeType);
#endif

#if WITH_EDITORONLY_DATA
	/** The cached Crc for these settings. */
	FPCGCrc CachedCrc;

	/** Cached pins when settings are used as assets. Will never be used otherwise. */
	UPROPERTY(AssetRegistrySearchable)
	FPCGAssetCachedPins CachedPins;

	/** Builds and returns list of pins as visible for graph creation purposes. */
	FPCGAssetCachedPins BuildCachedPins() const;
#endif

public:
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	virtual bool IsKernelValid(FPCGContext* InContext = nullptr, bool bQuiet = true) const { return false; }
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	virtual FString GetCookedKernelSource(const class UPCGComputeGraph* InComputeGraph) const { return FString(); }
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	virtual void GetKernelAttributeKeys(TArray<struct FPCGKernelAttributeKey>& OutKeys) const {}
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	virtual void AddStaticCreatedStrings(TArray<FString>& InOutStringTable) const {};
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	virtual int ComputeKernelThreadCount(const class UPCGDataBinding* Binding) const { return 0; };
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	bool ComputeOutputPinDataDesc(const FName& OutputPinLabel, const class UPCGDataBinding* InBinding, struct FPCGDataCollectionDesc& OutDesc) const { return false; }
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	virtual bool ComputeOutputPinDataDesc(const UPCGPin* OutputPin, const class UPCGDataBinding* InBinding, struct FPCGDataCollectionDesc& OutDesc) const { return false; }
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	virtual bool DoesOutputPinRequireElementCounters(const UPCGPin* OutputPin) const { return false; }
#if WITH_EDITOR
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	virtual void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<class UComputeDataInterface>>& OutDataInterfaces) const {}
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	virtual void CreateAdditionalOutputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<class UComputeDataInterface>>& OutDataInterfaces) const {}
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	virtual void CreateAdditionalOutputDataInterfaces(TArray<TObjectPtr<class UComputeDataInterface>>& OutDataInterfaces) const {}
	UE_DEPRECATED(5.6, "All GPU logic now resides in separate kernel objects, see implementations of UPCGSettings::CreateKernel().")
	virtual void GetDataLabels(FName InPinLabel, TArray<FString>& OutDataLabels) const {}
#endif
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSettingsInstance : public UPCGSettingsInterface
{
	GENERATED_BODY()

public:
	// ~Begin UPCGSettingsInterface
	virtual UPCGSettings* GetSettings() { return Settings.Get(); }
	virtual const UPCGSettings* GetSettings() const { return Settings.Get(); }
	// ~End UPCGSettingsInterface

	// ~Begin UObject interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PostEditImport() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// ~End UObject interface

	UE_API void SetSettings(UPCGSettings* InSettings);

protected:
#if WITH_EDITOR
	UE_API void OnSettingsChanged(UPCGSettings* InSettings, EPCGChangeType ChangeType);

	UE_API void PostSettingsChanged();
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, VisibleAnywhere, Category = Instance)
	TObjectPtr<UPCGSettings> OriginalSettings = nullptr; // Transient just for display
#endif

public:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Instance, AssetRegistrySearchable, meta = (EditInline))
	TObjectPtr<UPCGSettings> Settings = nullptr;
};

/** Trivial / Pass-through settings used for input/output nodes */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGTrivialSettings : public UPCGSettings
{
	GENERATED_BODY()
	
public:
	UPCGTrivialSettings();

protected:
	//~UPCGSettings implementation
	virtual FPCGElementPtr CreateElement() const override;
};

class FPCGTrivialElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool IsPassthrough(const UPCGSettings* InSettings) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};

#undef UE_API
