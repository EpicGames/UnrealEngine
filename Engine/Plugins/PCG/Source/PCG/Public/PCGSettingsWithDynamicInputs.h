// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGSettingsWithDynamicInputs.generated.h"

#define UE_API PCG_API

/**
* UPCGSettings subclass with functionality to dynamically add/remove input pins
*/
UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = (Procedural))
class UPCGSettingsWithDynamicInputs : public UPCGSettings
{
	GENERATED_BODY()

public:
	virtual FName GetDynamicInputPinsBaseLabel() const { return NAME_None; }

#if WITH_EDITOR
	/** Validate custom pin properties */
	virtual bool CustomPropertiesAreValid(const FPCGPinProperties& CustomProperties) { return true; }
	
	/** User driven event to add a dynamic source pin */
	UE_API virtual void OnUserAddDynamicInputPin();
	/** Overridden logic to add a default source pin */
	virtual void AddDefaultDynamicInputPin() PURE_VIRTUAL(PCGDynamicSettings::AddDefaultSourcePin, );
	/** Check if the pin to remove is dynamic */
	UE_API virtual bool CanUserRemoveDynamicInputPin(int32 PinIndex);
	/** User driven event to remove a dynamic source pin */
	UE_API virtual void OnUserRemoveDynamicInputPin(UPCGNode* InOutNode, int32 PinIndex);

	/** Get the number of static input pins. */
	UE_API int32 GetStaticInputPinNum() const;
	/** Get the number of dynamic input pins. */
	int32 GetDynamicInputPinNum() const { return DynamicInputPinProperties.Num(); }
#endif // WITH_EDITOR

	/** Get an array of pin labels to the pins defined by the settings. I.e. Not overrides, advanced pins, etc. */
	UE_API TArray<FName> GetNodeDefinedPinLabels() const;

protected:
#if WITH_EDITOR
	/** Add a new dynamic source pin with the specified properties */
	UE_API void AddDynamicInputPin(FPCGPinProperties&& CustomProperties);
#endif // WITH_EDITOR

	/** The input pin properties that are statically defined by the client class */
	UE_API virtual TArray<FPCGPinProperties> StaticInputPinProperties() const;

	/** Dynamic pin properties that the user can add or remove from */
	UPROPERTY()
	TArray<FPCGPinProperties> DynamicInputPinProperties;

	//~Begin UPCGSettings interface
	/** A concatenation of the static and dynamic input pin properties */
	UE_API virtual TArray<FPCGPinProperties> InputPinProperties() const override final;
	//~End UPCGSettings interface
};

#undef UE_API
