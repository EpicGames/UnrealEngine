// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGContext.h"
#include "PCGGraph.h"
#include "PCGSettings.h"

#include "UObject/ObjectPtr.h"

#include "PCGSubgraph.generated.h"

#define UE_API PCG_API

namespace PCGBaseSubgraphConstants
{
	UE_DEPRECATED(5.8, "Use PCG::Private::UserParameterTagData")
	inline const FString& UserParameterTagData = PCG::Private::UserParameterTagData;
}

UCLASS(MinimalAPI, Abstract)
class UPCGBaseSubgraphSettings : public UPCGSettings
{
	GENERATED_BODY()
public:
	UE_API UPCGGraph* GetSubgraph() const;
	virtual UPCGGraphInterface* GetSubgraphInterface() const { return nullptr; }
	
	// When we have nodes that supports subgraph overrides, we need to check if some parameters are overriden there.
	// By default, it should be the same as SubgraphInterface
	virtual UPCGGraphInterface* GetOriginalSubgraphInterface() const { return GetSubgraphInterface(); }

	/** Returns true if the subgraphs nodes were not inlined into the parent graphs tasks during compilation. */
	virtual bool IsDynamicGraph() const { return false; }

	// Use this method from the outside to set the subgraph, as it will connect editor callbacks
	UFUNCTION(BlueprintCallable, Category = Graph)
	UE_API virtual void SetSubgraph(UPCGGraphInterface* InGraph);

	virtual UPCGGraphInterface* GetSubgraphOverride() const { return nullptr; }

protected:
	//~Begin UObject interface implementation
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End UObject interface implementation

public:
	//~Begin UPCGSettings interface
	virtual bool RequiresDataFromPreTask() const override { return true; }
	virtual bool HasFlippedTitleLines() const override { return true; }
	// The graph may contain nodes that have side effects, don't assume we can cull even when unwired.
	// TODO: For static SGs we could probably compute this value based on the subgraph nodes.
	virtual bool CanCullTaskIfUnwired() const { return false; }

protected:
#if WITH_EDITOR
	UE_API virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
	UE_API virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif

	UE_API TArray<FPCGPinProperties> InputPinProperties() const override;
	UE_API TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface

	virtual void SetSubgraphInternal(UPCGGraphInterface* InGraph) {}

#if WITH_EDITOR
	UE_API void SetupCallbacks();
	UE_API void TeardownCallbacks();

	UE_API void OnSubgraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType);
#endif

protected:
	// Overrides
	UE_API virtual void FixingOverridableParamPropertyClass(FPCGSettingsOverridableParam& Param) const;
#if WITH_EDITOR
	UE_API virtual TArray<FPCGSettingsOverridableParam> GatherOverridableParams() const override;
#endif // WITH_EDITOR
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGSubgraphSettings : public UPCGBaseSubgraphSettings
{
	GENERATED_BODY()

public:
	UE_API UPCGSubgraphSettings(const FObjectInitializer& InObjectInitializer);

protected:
	//~Begin UObject interface implementation
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End UObject interface implementation

public:
	//~UPCGSettings interface implementation
	UE_API virtual UPCGNode* CreateNode() const override;

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Subgraph")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSubgraphSettings", "NodeTitle", "Subgraph"); }
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Subgraph; }
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual bool GetPinExtraIcon(const UPCGPin* InPin, FName& OutExtraIcon, FText& OutTooltip) const override { return false; }
#endif

	UE_API virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	UE_API virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	UE_API virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface implementation

	//~Begin UPCGBaseSubgraphSettings interface
public:
	UE_API virtual UPCGGraphInterface* GetSubgraphInterface() const override;
	UE_API virtual bool IsDynamicGraph() const override;
	virtual UPCGGraphInterface* GetOriginalSubgraphInterface() const { return SubgraphInstance; }
	virtual UPCGGraphInterface* GetSubgraphOverride() const override { return SubgraphOverride; }

	/** Used to filter the subgraph list based on the graph configuration. */
	UFUNCTION()
	UE_API bool SubgraphAssetFilter(const FAssetData& AssetData) const;

protected:
	UE_API virtual void SetSubgraphInternal(UPCGGraphInterface* InGraph) override;
	//~End UPCGBaseSubgraphSettings interface

	bool IsSubgraphOverridden() const;

public:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Properties, Instanced, meta = (NoResetToDefault))
	TObjectPtr<UPCGGraphInstance> SubgraphInstance;

	UPROPERTY(BlueprintReadOnly, Category = Properties, meta = (PCG_Overridable))
	TObjectPtr<UPCGGraphInterface> SubgraphOverride;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UPCGGraph> Subgraph_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI, Abstract)
class UPCGBaseSubgraphNode : public UPCGNode
{
	GENERATED_BODY()

public:
	UE_API TObjectPtr<UPCGGraph> GetSubgraph() const;
	virtual TObjectPtr<UPCGGraphInterface> GetSubgraphInterface() const { return nullptr; }
};

UCLASS(MinimalAPI, ClassGroup = (Procedural))
class UPCGSubgraphNode : public UPCGBaseSubgraphNode
{
	GENERATED_BODY()

public:
	/** ~Begin UPCGBaseSubgraphNode interface */
	virtual TObjectPtr<UPCGGraphInterface> GetSubgraphInterface() const override;
	/** ~End UPCGBaseSubgraphNode interface */
};

struct FPCGSubgraphContext : public FPCGContext
{
	TArray<FPCGTaskId> SubgraphTaskIds;
	bool bScheduledSubgraph = false;
	FInstancedStruct GraphInstanceParametersOverride;
	TSet<TObjectPtr<const UPCGData>> ReferencedObjects;

	// Analyze input data to detect if there is any override for the user parameters. If so will duplicate it to gather overrides.
	void InitializeUserParametersStruct();

	// If we have a subgraph override, update the underlying duplicated parameters with the overrides from the subgraph.
	void UpdateOverridesWithOverriddenGraph();

	void AddToReferencedObjects(const FPCGDataCollection& InDataCollection);

protected:
	virtual FStructView GetExternalStructContainerForOverridableParam(const FPCGSettingsOverridableParam& InParam) override;
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector);
};

class FPCGSubgraphElement : public IPCGElement
{
public:
	UE_API virtual FPCGContext* Initialize(const FPCGInitializeElementParams& InParams) override;
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	UE_API virtual bool ExecuteInternal(FPCGContext* Context) const override;
	UE_API virtual bool IsPassthrough(const UPCGSettings* InSettings) const override;
	void PrepareSubgraphData(const UPCGBaseSubgraphSettings* Settings, FPCGSubgraphContext* Context, const FPCGDataCollection& InputData, FPCGDataCollection& OutputData) const;
	void PrepareSubgraphUserParameters(const UPCGBaseSubgraphSettings* Settings, FPCGSubgraphContext* Context, FPCGDataCollection& OutputData) const;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
};

// Implementation note: this node forwards data, but does not keep that data alive. This is the responsibility of the corresponding FPCGSubgraphContext
class FPCGInputForwardingElement : public IPCGElement
{
public:
	PCG_API explicit FPCGInputForwardingElement(const FPCGDataCollection& InputToForward);

	// Since this class is stateful because it owns a FPCGDataCollection it can't be cached unless it implements a proper GetDependenciesCrc.
	// For now since it only forwards its input, we just disable the caching.
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

	// Should not verify if outputs are used multiples times, as we are just outputting the input collection we have.
	virtual bool ShouldVerifyIfOutputsAreUsedMultipleTimes(const UPCGSettings* InSettings) const override { return false; }
protected:
	PCG_API virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool IsPassthrough(const UPCGSettings* InSettings) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }

	FPCGDataCollection Input;
};

#undef UE_API
