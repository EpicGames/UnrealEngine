// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Volume.h"

#include "PCGVolume.generated.h"

#define UE_API PCG_API

class UPCGComponent;

UCLASS(MinimalAPI, BlueprintType, DisplayName = "PCG Volume")
class APCGVolume : public AVolume
{
	GENERATED_BODY()

public:
	UE_API APCGVolume(const FObjectInitializer& ObjectInitializer);

	//~ Begin AActor Interface
#if WITH_EDITOR
	UE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
	virtual FName GetCustomIconName() const { return NAME_None; }
#endif // WITH_EDITOR
	//~ End AActor Interface

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = PCG)
	TObjectPtr<UPCGComponent> PCGComponent;
};

#undef UE_API
