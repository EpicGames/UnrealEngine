// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Grid/PCGGridDescriptor.h"

#include "GameFramework/Actor.h"
#include "Misc/Guid.h"

#include "PCGWorldActor.generated.h"

class UPCGLandscapeCache;
namespace EEndPlayReason { enum Type : int; }

UCLASS(MinimalAPI, NotBlueprintable, NotPlaceable)
class APCGWorldActor : public AActor
{
	GENERATED_BODY()

public:
	APCGWorldActor(const FObjectInitializer& ObjectInitializer);

	//~Begin AActor Interface
	virtual void PostInitProperties() override;
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const override { return false; }
	virtual bool ActorTypeSupportsExternalDataLayer() const override { return false; }
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
	virtual bool IsUserManaged() const override { return false; }
	virtual bool ShouldExport() override { return false; }
	virtual bool ShouldImport(FStringView ActorPropString, bool IsMovingLevel) override { return false; }
	virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	virtual bool IsEditorOnly() const override;
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostLoad() override;
	//~End AActor Interface
#endif

	UFUNCTION(BlueprintCallable, Category = "PCG", meta=(DisplayName="Get PCG Grid Size"))
	int64 BP_GetPCGGridSize() const { return (int64)PartitionGridSize; }

	//~ Begin UObject Interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	virtual void BeginDestroy() override;
	virtual void Serialize(FArchive& Archive) override;
	//~ End UObject Interface.

	void MergeFrom(APCGWorldActor* OtherWorldActor);

#if WITH_EDITOR
	static APCGWorldActor* CreatePCGWorldActor(UWorld* InWorld);
#endif

	static inline constexpr uint32 DefaultPartitionGridSize = 25600; // 256m

	/** Size of the PCG partition actor grid for non-hierarchical-generation graphs. Min value: 400, MaxValue: 419430400 */
	UPROPERTY(config, EditAnywhere, Category = GenerationSettings)
	uint32 PartitionGridSize;

	/** Contains all the PCG data required to query the landscape complete. Serialized in cooked builds only */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Instanced, Category = CachedData, meta = (NoResetToDefault))
	TObjectPtr<UPCGLandscapeCache> LandscapeCacheObject = nullptr;

#if WITH_EDITORONLY_DATA
	/** This property was moved to the UPCGGraph but we keep it around to fixup existing data */
	UE_DEPRECATED(5.5, "bUse2DGrid is deprecated")
	UPROPERTY()
	bool bUse2DGrid = true;

	/** Allows any currently active editor viewport to act as a Runtime Generation Source. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RuntimeGeneration)
	bool bTreatEditorViewportAsGenerationSource = false;
#endif
	
	/** Allows any player controller to be considered as a Runtime Generation Source. Disabling this will require registering generation sources using UPCGGenSourceComponent or IPCGGenSourceBase sub-classing. */
	UPROPERTY(EditAnywhere, Category = RuntimeGeneration)
	bool bTreatPlayerControllersAsGenerationSources = true;

	/** Allows World Partition streaming sources to act as Runtime Generation Sources. */
	UE_DEPRECATED(5.7, "No longer required, by default, player controllers are considered generation sources and additional UPCGGenSourceComponent can be used for more complex use cases")
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RuntimeGeneration, meta = (EditCondition=bEnableWorldPartitionGenerationSources, EditConditionHides))
	bool bEnableWorldPartitionGenerationSources = false;

private:
	friend class FPCGTrackingManager;

	void RegisterToSubsystem();
	void UnregisterFromSubsystem();

#if WITH_EDITOR
	void OnPartitionGridSizeChanged();
#endif
};
