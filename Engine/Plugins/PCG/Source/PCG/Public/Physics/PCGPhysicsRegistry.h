// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#include "Delegates/Delegate.h"
#include "Misc/Guid.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakInterfacePtr.h"

class IPCGGraphExecutionSource;
class FPCGMetadataDomain;
struct FPCGWorldRaycastQueryParams;
struct FPCGWorldVolumetricQueryParams;
class UPCGMetadata;
class UWorld;

struct FHitResult;
struct FOverlapResult;

struct FPCGFilterHitResultParams
{
	explicit FPCGFilterHitResultParams(const FHitResult& InHitResult, const FPCGWorldRaycastQueryParams& InQueryParams, const TWeakInterfacePtr<IPCGGraphExecutionSource> InOriginatingSource, const TSet<FObjectKey>& InFilteredObjectReferences)
		: HitResult(InHitResult), QueryParams(InQueryParams), OriginatingSource(InOriginatingSource), FilteredObjectReferences(InFilteredObjectReferences)
	{
	}

	const FHitResult& HitResult;

	const FPCGWorldRaycastQueryParams& QueryParams;

	const TWeakInterfacePtr<IPCGGraphExecutionSource> OriginatingSource;

	const TSet<FObjectKey> FilteredObjectReferences;
};

struct FPCGFilterOverlapResultParams
{
	explicit FPCGFilterOverlapResultParams(const FOverlapResult& InOverlapResult, const FPCGWorldVolumetricQueryParams& InQueryParams, const TWeakInterfacePtr<IPCGGraphExecutionSource> InOriginatingSource, const TSet<FObjectKey>& InFilteredObjectReferences)
		: OverlapResult(InOverlapResult), QueryParams(InQueryParams), OriginatingSource(InOriginatingSource), FilteredObjectReferences(InFilteredObjectReferences)
	{
	}

	const FOverlapResult& OverlapResult;

	const FPCGWorldVolumetricQueryParams& QueryParams;

	const TWeakInterfacePtr<IPCGGraphExecutionSource> OriginatingSource;

	const TSet<FObjectKey> FilteredObjectReferences;
};

struct FPCGApplyHitResultAttributesParams
{
	explicit FPCGApplyHitResultAttributesParams(const FHitResult& InHitResult, const FPCGWorldRaycastQueryParams& InQueryParams, const FVector& InRayDirection, const FTransform& InTransform, const UWorld* InWorld, int64& InOutMetaDataEntry, UPCGMetadata& InOutMetadata)
		: HitResult(InHitResult), QueryParams(InQueryParams), RayDirection(InRayDirection), Transform(InTransform), World(InWorld), MetadataEntry(InOutMetaDataEntry), Metadata(InOutMetadata)
	{

	}

	const FHitResult& HitResult;
	
	const FPCGWorldRaycastQueryParams& QueryParams;
	
	const FVector& RayDirection;
	
	const FTransform& Transform;
	
	const UWorld* World = nullptr;

	int64& MetadataEntry;

	UPCGMetadata& Metadata;
};


// Returns true if filter is handling this call, the bool param is only valid if delegate returns true and indicates if Hit passes the filter.
DECLARE_DELEGATE_RetVal_TwoParams(bool, FPCGFilterHitResult, const FPCGFilterHitResultParams&, bool&);

// Returns true if filter is handling this call, the bool param is only valid if delegate returns true and indicates if Overlap passes the filter.
DECLARE_DELEGATE_RetVal_TwoParams(bool, FPCGFilterOverlapResult, const FPCGFilterOverlapResultParams&, bool&);

// Returns true if filter is handling this call, the bool param is only valid if delegate returns true and indicates success (true) or failure (false).
DECLARE_DELEGATE_RetVal_TwoParams(bool, FPCGApplyHitResultAttributes, const FPCGApplyHitResultAttributesParams&, bool&);

class FPCGPhysicsRegistry
{
public:
	PCG_API void RegisterHitResultFilter(const FGuid& InID, FPCGFilterHitResult InFilter);
	PCG_API void UnregisterHitResultFilter(const FGuid& InID);

	PCG_API void RegisterOverlapResultFilter(const FGuid& InID, FPCGFilterOverlapResult InFilter);
	PCG_API void UnregisterOverlapResultFilter(const FGuid& InID);

	PCG_API void RegisterApplyHitResultAttributes(const FGuid& InID, FPCGApplyHitResultAttributes InApplyHitResultAttributes);
	PCG_API void UnregisterApplyHitResultAttributes(const FGuid& InID);

	PCG_API bool FilterOverlapResult(const FPCGFilterOverlapResultParams& InParams) const;
	PCG_API bool FilterHitResult(const FPCGFilterHitResultParams& InParams) const;
	PCG_API bool ApplyHitResultAttributes(const FPCGApplyHitResultAttributesParams& InParams) const;

private:
	TMap<FGuid, FPCGFilterHitResult> HitResultFilters;
	TMap<FGuid, FPCGFilterOverlapResult> OverlapResultFilters;
	TMap<FGuid, FPCGApplyHitResultAttributes> ApplyHitResultAttributesFuncs;
};