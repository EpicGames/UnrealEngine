// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConvexVolume.h"
#include "UObject/Interface.h"

#include "PCGGenSourceBase.generated.h"

struct FPCGRuntimeGenContext;

UINTERFACE(MinimalAPI, BlueprintType)
class UPCGGenSourceBase : public UInterface
{
	GENERATED_BODY()
};

/**
 * A PCG Generation Source represents an object in the world that provokes nearby 
 * PCG Components to generate through the Runtime Generation Scheduler.
 */
class IPCGGenSourceBase
{
	GENERATED_BODY()

public:
	/** Update the generation source so that it can cache data that is queried often (e.g. view frustum). Should be called every tick on any active generation sources. */
	PCG_API virtual void Tick(const FPCGRuntimeGenContext& InContext);

	/** Returns the world space position of this gen source. */
	virtual TOptional<FVector> GetPosition() const PURE_VIRTUAL(UPCGGenSourceBase::GetPosition, return TOptional<FVector>(););

	/** Returns the normalized forward vector of this gen source. */
	virtual TOptional<FVector> GetDirection() const PURE_VIRTUAL(UPCGGenSourceBase::GetDirection, return TOptional<FVector>(););

	/** Returns the view frustum of this gen source. */
	virtual TOptional<FConvexVolume> GetViewFrustum(bool bIs2DGrid) const { return TOptional<FConvexVolume>(); }

	/** Returns true if this is a local gen source. (client-server specific) */
	virtual bool IsLocal() const { return true; }

	/** Whether a single runtime gen grid scan can be performed for both sources. */
	PCG_API virtual bool IsEquivalent(IPCGGenSourceBase* InOther, const FPCGRuntimeGenContext& InContext) const;

	virtual FString GetDebugName() const { return TEXT("AnonymousGenSource"); }

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "Implement version that takes runtime generation context instead.")
	virtual void Tick() {}
};
