// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RuntimeGen/GenSources/PCGGenSourceBase.h"

#include "Components/ActorComponent.h"

#include "PCGGenSourceComponent.generated.h"

namespace EEndPlayReason { enum Type : int; }

class FPCGGenSourceManager;

/**
 * UPCGGenSourceComponent makes the actor this is attached to act as a PCG runtime generation source.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural), DisplayName = "PCG Generation Source", meta = (BlueprintSpawnableComponent, PrioritizeCategories = "PCG"))
class UPCGGenSourceComponent : public UActorComponent, public IPCGGenSourceBase
{
	GENERATED_BODY()

	UPCGGenSourceComponent(const FObjectInitializer& InObjectInitializer) : Super(InObjectInitializer) {}
	~UPCGGenSourceComponent();

public:
	//~Begin UActorComponent Interface
	PCG_API virtual void BeginPlay() override;
	PCG_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	PCG_API virtual void OnRegister() override;
	PCG_API virtual void OnUnregister() override;
	PCG_API virtual void OnComponentCreated() override;
	PCG_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	//~End UActorComponent Interface

	//~Begin IPCGGenSourceInterface
	PCG_API virtual TOptional<FVector> GetPosition() const override;
	PCG_API virtual TOptional<FVector> GetDirection() const override;
	virtual FString GetDebugName() const override { return GetName(); }
	//~End IPCGGenSourceInterface

protected:
	PCG_API FPCGGenSourceManager* GetGenSourceManager() const;
};
