// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGenSourceBase.h"

#include "PCGGenSourceEditorCamera.generated.h"

#if WITH_EDITOR
class FEditorViewportClient;
#endif

/**
 * This GenerationSource captures active Editor Viewports per tick to provoke RuntimeGeneration. Editor Viewports
 * are not captured by default, but can be enabled on the PCGWorldActor via bTreatEditorViewportAsGenerationSource.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGenSourceEditorCamera : public UObject, public IPCGGenSourceBase
{
	GENERATED_BODY()

public:
	//~ Begin IPCGGenSourceInterface
	PCG_API virtual void Tick(const FPCGRuntimeGenContext& InContext) override;
	PCG_API virtual TOptional<FVector> GetPosition() const override;
	PCG_API virtual TOptional<FVector> GetDirection() const override;
	PCG_API virtual TOptional<FConvexVolume> GetViewFrustum(bool bIs2DGrid) const override;
	virtual FString GetDebugName() const override { return GetName(); }
	//~ End IPCGGenSourceInterface

public:
#if WITH_EDITORONLY_DATA
	FEditorViewportClient* EditorViewportClient = nullptr;
	TOptional<FConvexVolume> ViewFrustum;
#endif
};
