// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGenSourceBase.h"

#include "PCGGenSourcePlayer.generated.h"

class APlayerController;

/**
 * A UPCGGenSourcePlayer is automatically captured for all PlayerControllers in the level on Login/Logout.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGenSourcePlayer : public UObject, public IPCGGenSourceBase
{
	GENERATED_BODY()

public:
	//~ Begin IPCGGenSourceInterface
	PCG_API virtual void Tick(const FPCGRuntimeGenContext& InContext) override;
	PCG_API virtual TOptional<FVector> GetPosition() const override;
	PCG_API virtual TOptional<FVector> GetDirection() const override;
	virtual TOptional<FConvexVolume> GetViewFrustum(bool bIs2DGrid) const override { return ViewFrustum; }
	virtual bool IsLocal() const override;
	virtual FString GetDebugName() const override { return GetName(); }
	//~ End IPCGGenSourceInterface

	TWeakObjectPtr<const APlayerController> GetPlayerController() const { return PlayerController; }
	PCG_API void SetPlayerController(const APlayerController* InPlayerController);

	bool IsValid() const { return PlayerController.IsValid(); }

protected:
	TWeakObjectPtr<const APlayerController> PlayerController;
	TOptional<FConvexVolume> ViewFrustum;

	TOptional<FVector> CachedPosition;
	TOptional<FVector> CachedDirection;
};
