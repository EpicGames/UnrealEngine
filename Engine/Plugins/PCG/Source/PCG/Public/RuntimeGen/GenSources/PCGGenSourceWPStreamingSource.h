// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGenSourceBase.h"

#include "PCGGenSourceWPStreamingSource.generated.h"

struct FWorldPartitionStreamingSource;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UE_DEPRECATED(5.7, "No longer supported.") UPCGGenSourceWPStreamingSource : public UObject, public IPCGGenSourceBase
{
	GENERATED_BODY()

public:
	//~ Begin IPCGGenSourceInterface
	PCG_API virtual TOptional<FVector> GetPosition() const override;
	PCG_API virtual TOptional<FVector> GetDirection() const override;
	virtual FString GetDebugName() const override { return GetName(); }
	//~ End IPCGGenSourceInterface

public:
	const FWorldPartitionStreamingSource* StreamingSource;
};
