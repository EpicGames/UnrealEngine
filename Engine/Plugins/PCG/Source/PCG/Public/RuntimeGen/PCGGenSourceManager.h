// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RuntimeGen/PCGRuntimeGenContext.h"
#include "RuntimeGen/GenSources/PCGGenSourceBase.h"

class AController;
class AGameModeBase;
class APCGWorldActor;
class APlayerController;
class UPCGGenSourceComponent;
class UPCGGenSourceEditorCamera;
class UPCGGenSourceWPStreamingSource;
class UWorld;

/**
 * The runtime Generation Source Manager tracks generation sources in the world for use by the Runtime Generation Scheduler.
 */
class FPCGGenSourceManager
{
public:
	FPCGGenSourceManager(const FPCGRuntimeGenContext& InContext);
	~FPCGGenSourceManager();

	/** Marks the GenSourceManager as dirty so that the next call to 'GetAllGenSources()' will update tracked generation sources. */
	void Tick() { bDirty = true; }

	/** Creates the set of all generation sources tracked by the manager. Call 'Tick()' to keep tracked generation sources up to date. */
	TSet<IPCGGenSourceBase*> PCG_API GetAllGenSources(const APCGWorldActor* InPCGWorldActor);

	/** Creates a set of generation sources that are unique for a given runtime gen context (de-duplicating sources which will produce equivalent grid cell scans). */
	void GetUniqueGenSources(const FPCGRuntimeGenContext& InContext, TSet<IPCGGenSourceBase*>& OutUniqueGenSources);

	/** Adds a UPCGGenSource to be tracked by the GenSourceManager. */
	bool PCG_API RegisterGenSource(IPCGGenSourceBase* InGenSource, FName InGenSourceName = NAME_None);

	/** Removes a UPCGGenSource from being tracked by the GenSourceManager. */
	bool PCG_API UnregisterGenSource(const IPCGGenSourceBase* InGenSource);

	/** Removes a UPCGGenSource from being tracked by the GenSourceManaged on a name-basis. */
	bool PCG_API UnregisterGenSource(FName InGenSourceName);

	/** Add UObject references for GC */
	void AddReferencedObjects(FReferenceCollector& Collector);

protected:
	/** Updates tracked generation sources that should be refreshed per tick. */
	void UpdatePerTickGenSources(const APCGWorldActor* InPCGWorldActor);

	void UpdateWorldPartitionGenSources(const APCGWorldActor* InPCGWorldActor);
	void GatherWorldPartitionGenSources(TSet<IPCGGenSourceBase*>& OutGenSources) const;

protected:
	/** Tracks named registered generation sources, such as UPCGGenSourcePlayer. */
	TMap<FName, TScriptInterface<IPCGGenSourceBase>> RegisteredNamedGenSources;

	/** Tracks unnamed registered generation sources, such as UPCGGenSourceComponent. */
	TSet<TScriptInterface<IPCGGenSourceBase>> RegisteredGenSources;

#if WITH_EDITORONLY_DATA
	/** Tracks the active/main editor viewport client. This is refreshed every tick to keep a handle to whichever viewport is active. */
	TObjectPtr<UPCGGenSourceEditorCamera> EditorCameraGenSource = nullptr;
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Pool of GenSources dedicated to WorldPartition StreamingSources. This list grows as necessary and refreshes every tick. */
	TArray<TObjectPtr<UPCGGenSourceWPStreamingSource>> WorldPartitionGenSources;
	bool bLoggedWorldPartitionGenSourceWarning = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool bDirty = false;
	const FPCGRuntimeGenContext& Context;

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "Use constructor that takes runtime gen context.")
	FPCGGenSourceManager(const UWorld* InWorld);

private:
	FPCGRuntimeGenContext FallBackContext;
};
