// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class APCGWorldActor;
class UWorld;

/** Runtime generation scheduler context information. */
struct FPCGRuntimeGenContext
{
	UWorld* World = nullptr;
	const APCGWorldActor* PCGWorldActor = nullptr;

	bool bAnyRuntimeGenSourcesExist = false;
	bool bAnySourcesUseFrustumCulling = false;
	bool bAnySourcesUse2DGrids = false;
	bool bAnySourcesUse3DGrids = false;
};
