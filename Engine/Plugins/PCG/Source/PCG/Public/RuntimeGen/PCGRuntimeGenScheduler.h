// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGGraphExecutionStateInterface.h"
#include "Grid/PCGGridDescriptor.h"
#include "RuntimeGen/PCGRuntimeGenChangeDetection.h"
#include "RuntimeGen/PCGRuntimeGenContext.h"

#include "UObject/WeakObjectPtr.h"

#include "PCGRuntimeGenScheduler.generated.h"

class APCGPartitionActor;
class APCGWorldActor;
class FPCGTrackingManager;
#if WITH_EDITOR || !UE_BUILD_SHIPPING
class FAutoConsoleVariableSink;
#endif
class FPCGGenSourceManager;
class IPCGGenSourceBase;
class ULevel;
class ULevelStreaming;
class UPCGComponent;
class UPCGGraphInterface;
class UPCGRuntimeGenExecutionSource;
class UPCGSubsystem;
class UWorld;
class UWorldPartitionSubsystem;
enum class ELevelStreamingState : uint8;
enum class ELevelStreamingTargetState : uint8;
enum class EPCGGraphParameterEvent;
namespace PCGRuntimeGenSchedulerHelpers { struct FStatsOverlay; }

namespace PCGRuntimeGenSchedulerHelpers
{
	extern PCG_API TAutoConsoleVariable<float> CVarRuntimeGenerationRadiusMultiplier;
}

/** Used to inform what virtual textures to prime and on what grids they need to be present. */
USTRUCT(BlueprintType)
struct FPCGVirtualTexturePrimingInfo
{
	GENERATED_BODY()

	/** Virtual texture asset to be primed. */
	UPROPERTY(EditAnywhere, Category = "")
	TSoftObjectPtr<class URuntimeVirtualTexture> VirtualTexture;

	/** Largest grid on which this virtual texture is sampled. */
	UPROPERTY(EditAnywhere, Category = "")
	EPCGHiGenGrid Grid = EPCGHiGenGrid::Grid32;

	/** Desired world size (cm) of a texel in the primed virtual texture. Determines what mip level will be primed. */
	UPROPERTY(EditAnywhere, Category = "", meta = (ClampMin=0.1))
	float WorldTexelSize = 100.0f;
};

/**
 * The Runtime Generation Scheduler system handles the scheduling of PCG Components marked as GenerateAtRuntime.
 * It searches the level for Partitioned and Non-Partitioned execution sources in range of the currently active
 * UPCGGenSources in the level, and schedules them efficiently based on their UPCGSchedulingPolicy, creating 
 * APCGPartitionActors as necessary to support hierarchical generation.
 *
 * APCGPartitionActors can be created/destroyed on demand or provided by a dynamically growing pool of actors. If
 * enabled, the pool will double in capacity anytime the number of available PAs reaches zero.
 * 
 * Execution sources and partition actors created by the Runtime Generation Scheduler should be managed exclusively by the
 * runtime gen scheduling system.
 */
class FPCGRuntimeGenScheduler
{
	friend class UPCGSubsystem;

public:
	FPCGRuntimeGenScheduler(UWorld* InWorld, FPCGTrackingManager* InTrackingManager);
	~FPCGRuntimeGenScheduler();

	FPCGRuntimeGenScheduler(const FPCGRuntimeGenScheduler&) = delete;
	FPCGRuntimeGenScheduler(FPCGRuntimeGenScheduler&& other) = delete;
	FPCGRuntimeGenScheduler& operator=(const FPCGRuntimeGenScheduler& other) = delete;
	FPCGRuntimeGenScheduler& operator=(FPCGRuntimeGenScheduler&& other) = delete; 

	void Tick(APCGWorldActor* InPCGWorldActor, double InDeltaTime, double InEndTime);

	void OnOriginalComponentRegistered(UPCGComponent* InOriginalComponent);
	void OnOriginalComponentUnregistered(UPCGComponent* InOriginalComponent);
	void OnOriginalComponentReplaced(const UPCGComponent* InOldComponent, UPCGComponent* InNewComponent);

	/** Destroy all runtime gen partition actors (both generated and pooled). Executed in next tick. */
	void FlushAllGeneratedActors() { bActorFlushRequested = true; }

#if WITH_EDITOR
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances);
#endif

protected:
	// Grid size, grid coords, original source, local source
	struct FGridGenerationKey
	{
		FGridGenerationKey(uint32 InGridSize, const FIntVector& InGridCoords, IPCGGraphExecutionSource* InOriginalSource)
			: FGridGenerationKey(InGridSize, InGridCoords, InOriginalSource, /*InLocalSource=*/nullptr)
		{
		}

		FGridGenerationKey(uint32 InGridSize, const FIntVector& InGridCoords, IPCGGraphExecutionSource* InOriginalSource, IPCGGraphExecutionSource* InLocalSource);

		bool operator==(const FGridGenerationKey& Other) const
		{
			return GridSize == Other.GridSize
				&& GridCoords == Other.GridCoords
				&& bUse2DGrid == Other.bUse2DGrid
				&& OriginalSource == Other.OriginalSource;
		}

		bool IsValid() const { return !!OriginalSource.ResolveObjectPtr(); }

		bool Use2DGrid() const { return bUse2DGrid; }
		uint32 GetGridSize() const { return GridSize; }
		FIntVector GetGridCoords() const { return GridCoords; }

		IPCGGraphExecutionSource* GetOriginalSource() const { return Cast<IPCGGraphExecutionSource>(OriginalSource.ResolveObjectPtr()); }
		IPCGGraphExecutionSource* GetOriginalSourceEvenIfGarbage() const { return Cast<IPCGGraphExecutionSource>(OriginalSource.ResolveObjectPtrEvenIfGarbage()); }
		void SetOriginalSource(UObject* InOriginalSource) { OriginalSource = InOriginalSource; }

		IPCGGraphExecutionSource* GetCachedLocalSource() const { return CachedLocalSource.Get(); }
		void SetCachedLocalSource(IPCGGraphExecutionSource* InLocalSource) const
		{
			CachedLocalSource = InLocalSource;

			if (InLocalSource)
			{
				CachedBounds = InLocalSource->GetExecutionState().GetBounds();
			}
		}

		const TOptional<FBox>& GetCachedBounds() const { return CachedBounds; }

		FPCGGridDescriptor GetGridDescriptor() const;

	private:
		bool bUse2DGrid = false;
		uint32 GridSize = 0;
		FIntVector GridCoords = FIntVector(0);
		TObjectKey<UObject> OriginalSource;

		// Optional/opportunistic cached local execution source if one has been created.
		mutable TWeakInterfacePtr<IPCGGraphExecutionSource> CachedLocalSource;

		mutable TOptional<FBox> CachedBounds;

		friend uint32 GetTypeHash(const FGridGenerationKey& In)
		{
			return HashCombine(GetTypeHash(In.GridCoords), GetTypeHash(In.GridSize), GetTypeHash(In.bUse2DGrid), GetTypeHash(In.OriginalSource));
		}

		// Deprecated section
	protected:
		UE_DEPRECATED(5.8, "Use GetOriginalSource() instead.")
		PCG_API UPCGComponent* GetOriginalComponent() const;
		UE_DEPRECATED(5.8, "Use GetOriginalSourceEvenIfGarbage() instead.")
		PCG_API UPCGComponent* GetOriginalComponentEvenIfGarbage() const;
		UE_DEPRECATED(5.8, "Use SetOriginalSource() instead.")
		PCG_API void SetOriginalComponent(UPCGComponent* InComponent);
		UE_DEPRECATED(5.8, "Use GetCachedLocalSource() instead.")
		PCG_API IPCGGraphExecutionSource* GetCachedLocalComponent() const;
		UE_DEPRECATED(5.8, "Use SetCachedLocalSource() instead.")
		PCG_API void SetCachedLocalComponent(IPCGGraphExecutionSource* InLocalComponent) const;
	};

	/** Returns true if the scheduler should tick this frame. */
	bool ShouldTick(double InDeltaTime);

	// Add constructor, get moves for the sets
	struct FTickQueueSourcesForGenerationInputs
	{
		const TSet<IPCGGenSourceBase*>* GenSources = nullptr;
		const APCGWorldActor* PCGWorldActor = nullptr;
		TSet<IPCGGraphExecutionSource*> AllPartitionedExecutionSources;
		TSet<IPCGGraphExecutionSource*> AllNonPartitionedExecutionSources;
		TSet<FGridGenerationKey>* GeneratedSources = nullptr;
	};

	/** Queue nearby execution sources for generation. */
	void TickQueueSourcesForGeneration(const FTickQueueSourcesForGenerationInputs& Inputs, TMap<FGridGenerationKey, double>& OutSourcesToGenerate);

	/** Perform immediate cleanup on execution sources that become out of range. */
	void TickCleanup(const TSet<IPCGGenSourceBase*>& GenSources, const APCGWorldActor* InPCGWorldActor, double InEndTime);

	/** Schedule generation on execution sources in priority order. */
	void TickScheduleGeneration(TMap<FGridGenerationKey, double>& InOutSourcesToGenerate);

	/** Request any required virtual textures to be primed within the necessary generation radius. */
	void TickRequestVirtualTexturePriming(const TSet<IPCGGenSourceBase*>& InGenSources);

	/** Detects changes in RuntimeGen CVars to keep the PA pool in a valid state. */
	void TickCVars(const APCGWorldActor* InPCGWorldActor);

	/** Cleanup all local execution sources in the GeneratedSources set. */
	void CleanupLocalSources(const APCGWorldActor* InPCGWorldActor);

	/** Cleanup a execution source and remove it from the GeneratedSources set. */
	void CleanupSource(const FGridGenerationKey& GenerationKey, IPCGGraphExecutionSource* InGeneratedSource);

	/** Remove execution sources from the GeneratedSources set that have been marked for delayed refresh. Fully cleanup any that would be leaked otherwise. */
	void CleanupDelayedRefreshSources();

	/** Refresh a generated execution source. bRemovePartitionActors will also perform a full cleanup of PAs and local sources. */
	void RefreshExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bRemovePartitionActors = false);
	
	/** Grabs an empty RuntimeGen PA from the PartitionActorPool and initializes it at the given GridSize and GridCoords. If no PAs are available in the pool,
	* the pool capacity will double and new PAs will be created.
	*/
	APCGPartitionActor* GetPartitionActorFromPool(const FPCGGridDescriptor& GridDescriptor, const FIntVector& GridCoords);

	/** Adds Count new RuntimeGen PAs to the Runtime PA pool. */
	void AddPartitionActorPoolCount(int32 Count);

	/** Destroy all pooled partition actors and rebuild with the NewPoolSize. */
	void ResetPartitionActorPoolToSize(uint32 NewPoolSize);

	/** Add UObject references for GC */
	void AddReferencedObjects(FReferenceCollector& Collector);

private:
	/** Called on world streaming events. */
	void OnLevelStreamingStateChanged(UWorld* InWorld, const ULevelStreaming* InLevelStreaming, ULevel* InLevelIfLoaded, ELevelStreamingState InPreviousState, ELevelStreamingState InNewState);

	/** Called before network replay scrub. Unregisters all runtime-gen execution sources and suspends ticking. */
	void OnNetworkReplayScrub(UWorld* InWorld);

	/** Called after network replay scrub completes. Resumes ticking so sources can re-register naturally. */
	void OnNetworkReplayScrubComplete(UWorld* InWorld);

	/** Cleanup a local execution source along with its partition actor if it exists. */
	void CleanupLocalSource(APCGPartitionActor* InPartitionActor, IPCGGraphExecutionSource* InLocalSource);

	/** Cleanup the remaining execution sources that are not part of the GeneratedSources */
	void CleanupRemainingSources(IPCGGraphExecutionSource* InOriginalSource);

	/** Cache VT priming infos found in graph params of the given component. */
	void CacheVirtualTexturePrimingInfos(UPCGComponent* InOriginalComponent);

	/** Update RGS context information. */
	void RefreshContext();

	/** Whether the change detection is enabled which avoids scanning grids when not necessary. */
	bool IsChangeDetectionEnabled() const;

	void RegisterOriginalRuntimeGenExecutionSource(UPCGComponent* InOriginalComponent);
	void UnregisterOriginalRuntimeGenExecutionSource(UPCGComponent* InOriginalComponent);
	UPCGRuntimeGenExecutionSource* GetOriginalRuntimeGenExecutionSource(UPCGComponent* InOriginalComponent);

#if WITH_EDITOR
	void RegisterGraphParameterChangedEvent(const UPCGComponent* InOriginalComponent);
	void UnregisterGraphParameterChangedEvent(const UPCGComponent* InOriginalComponent);
	void OnGraphParametersChanged(UPCGGraphInterface* InGraph, EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName);
#endif

#if WITH_EDITOR || !UE_BUILD_SHIPPING
	/** Called once per frame whenever any console variable changes value. Checks registered sources for matching CVars. */
	void OnCVarSinkFired();
#endif // WITH_EDITOR || !UE_BUILD_SHIPPING

private:
	/** Tracks the generated execution sources managed by the RuntimeGenScheduler. For local execution sources, this generation key will hold the original execution source.
	* For non-partitioned execution sources, the generation key should have unbounded grid size and (0, 0, 0) grid coordinates.
	*/
	TSet<FGridGenerationKey> GeneratedSources;

	/** Tracks the execution sources which should be removed from the GeneratedSources set on the next tick. This helps us defer removal in case we get multiple
	* refreshes in a single tick. For example, a shallow refresh followed by a deep refresh would require the generated execution sources to persist, otherwise we
	* will leak Partition Actors.
	*/
	TSet<FGridGenerationKey> GeneratedSourcesToRemove;

	/** Mapping of execution source + coordinates to priorities - needed to compute max priority over all gen sources. */
	TMap<FGridGenerationKey, double> SourcesToGenerate;

	// Local to member functions but hoisted for efficiency.
	TSet<IPCGGenSourceBase*> GenSources;

	/** Pool of RuntimeGen PartitionActors used for hierarchical generation. */
	TArray<TObjectPtr<APCGPartitionActor>> PartitionActorPool;

	/** PartitionActorPoolSize represents the current maximum capacity of the PartitionActorPool. */
	int32 PartitionActorPoolSize = 0;

	FPCGTrackingManager* TrackingManager = nullptr;
	FPCGGenSourceManager* GenSourceManager = nullptr;
	UPCGSubsystem* Subsystem = nullptr;
	const UWorldPartitionSubsystem* WorldPartitionSubsystem = nullptr;

	bool bPoolingWasEnabledLastFrame = true;
	uint32 BasePoolSizeLastFrame = 0;

#if WITH_EDITOR || !UE_BUILD_SHIPPING
	/** Fires OnCVarSinkFired once per frame when any CVar changes, grouping all changes in that frame. */
	TUniquePtr<FAutoConsoleVariableSink> CVarRefreshSink;

	/** Last observed string value per CVar name watched by any registered source. Used to detect value changes. */
	TMap<FString, FString> WatchedCVarLastValues;
#endif // WITH_EDITOR || !UE_BUILD_SHIPPING

#if WITH_EDITOR
	bool bTreatEditorViewportAsGenSourcePreviousFrame = false;
#endif

	/** Requests to flush all actors are deferred so they can be handled at a known time during tick. */
	bool bActorFlushRequested = false;

	/** True while a network replay scrub is in progress. Prevents ticking until scrub completes. */
	bool bSuspendedForReplayScrub = false;

	/** Setting up a PA calls APCGPartitionActor::AddGraphInstance which later calls RefreshComponent, which can create
	* an infinite refresh loop. To break this loop we write the OC pointer to this variable, and if Refresh gets called for
	* this OC we early out. Basically we don't respond to refresh calls for a execution source we are midway through setting up.
	*/
	const IPCGGraphExecutionSource* OriginalSourceBeingRegistered = nullptr;

	int32 FramesUntilGeneration = 0;

	// Local to TickQueueSourcesForGeneration, cached here for efficiency.
	struct FStreamingCompleteQueryKey
	{
		FVector Location = FVector::ZeroVector;
		float GridSize = 0.0f;

		// Collision safety: TargetGridsHash is a 64-bit hash of the policy's WorldPartitionTargetGrids. A collision would return a false cache hit for a
		// different grid set, but with the expected cardinality (a handful of policies x a handful of grid names) 64-bit collisions are astronomically unlikely.
		uint64 TargetGridsHash = 0;

		bool operator==(const FStreamingCompleteQueryKey& Other) const = default;

		friend uint32 GetTypeHash(const FStreamingCompleteQueryKey& Key)
		{
			// Bucket-selection hash only needs to spread well; collapse the 64-bit grid hash down.
			// Correctness comes from operator==, which retains the full 64 bits.
			const uint32 GridsHash32 = uint32(Key.TargetGridsHash) ^ uint32(Key.TargetGridsHash >> 32);
			return HashCombine(HashCombine(GetTypeHash(Key.Location), GetTypeHash(Key.GridSize)), GridsHash32);
		}
	};
	TMap<FStreamingCompleteQueryKey, bool> CachedStreamingQueryResults;

	/** We should hold back generation until the non-spatial cell is done streaming, which happens once when the level loads. */
	bool bNonSpatialCellFinishedStreaming = false;

	/** Used to detect when world state (generation sources, cvars etc) have changed enough to warrant rescanning generation cells. */
	PCGRuntimeGenChangeDetection::FDetector ChangeDetector;

	/** Triggers scanning components for cleanup. If change detection is enabled, cleanup scans are only performed when needed rather than every frame. */
	bool bCleanupScanRequired = false;

	/** Meta-information about current context. */
	FPCGRuntimeGenContext Context;

	/** Whether Context needs to be refreshed. */
	bool bContextDirty = true;

#if WITH_EDITOR
	TMap<TObjectKey<UPCGComponent>, FDelegateHandle> GraphParamChangedEventHandles;
#endif

	/** Maps original component to a cache of VT priming infos. */
	TMap<TObjectKey<UPCGComponent>, TArray<FPCGVirtualTexturePrimingInfo>> CachedPrimingInfos;

	/** Maps an original PCG component to its runtime gen execution source. This relationship only exists for components whose graphs opt-in to actor-component-less generation. */
	TMap<TObjectKey<UPCGComponent>, TObjectPtr<UPCGRuntimeGenExecutionSource>> OriginalExecutionSources;

	double TimeToNextTick = 0.0;

	friend struct PCGRuntimeGenSchedulerHelpers::FStatsOverlay;

	// Deprecated section
protected:
	struct UE_DEPRECATED(5.8, "Use FTickQueueSourcesForGenerationInputs instead") FTickQueueComponentsForGenerationInputs
	{
		const TSet<IPCGGenSourceBase*>* GenSources = nullptr;
		const APCGWorldActor* PCGWorldActor = nullptr;
		TSet<IPCGGraphExecutionSource*> AllPartitionedExecutionSources;
		TSet<IPCGGraphExecutionSource*> AllNonPartitionedExecutionSources;
		TSet<FGridGenerationKey>* GeneratedComponents = nullptr;
	};
};
