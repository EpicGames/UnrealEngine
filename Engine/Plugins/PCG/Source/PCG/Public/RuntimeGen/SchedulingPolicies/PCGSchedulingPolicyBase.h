// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#include "PCGSchedulingPolicyBase.generated.h"

class IPCGGenSourceBase;

/** Determines grid dependencies for world streaming. */
UENUM()
enum class EPCGGridStreamingDependencyMode
{
	AllGridsExceptUnbounded = 0,
	AllGrids,
	SpecificGrids,
	NoGrids,
};

/** 
 * Scheduling Policies provide custom logic to efficiently schedule work for the Runtime Generation Scheduling system.
 * A higher priority value means the work will be scheduled sooner, and larger grid sizes will always have a higher
 * priority than lower grid sizes.
 *
 * If multiple Generation Sources overlap a component, the highest priority value will be used for scheduling.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = (Procedural))
class UPCGSchedulingPolicyBase : public UObject
{
	GENERATED_BODY()

public:
	//~ Begin UObject interface
	PCG_API virtual void PostLoad() override;
#if WITH_EDITOR
	PCG_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

	/** Calculate the runtime scheduling priority with respect to a Generation Source. Should return a value in the range [0, 1], where higher values will be scheduled sooner. */
	virtual double CalculatePriority(const IPCGGenSourceBase* InGenSource, const FBox& GenerationBounds, bool bUse2DGrid) const 
		PURE_VIRTUAL(UPCGSchedulingPolicyBase::CalculatePriority, return 0.0;);

	/** True if the generation source would consider the given bounds for generation. */
	virtual bool ShouldGenerate(const IPCGGenSourceBase* InGenSource, const FBox& GenerationBounds, bool bUse2DGrid) const { return true; }

	/** True if the generation source would cull the given bounds. Only applies to bounds within the cleanup generation radius. */
	virtual bool ShouldCull(const IPCGGenSourceBase* InGenSource, const FBox& GenerationBounds, bool bUse2DGrid) const { return false; }

	/** Should return true if this policy select cells for generation based on generation source direction (for e.g. frustum culling), and false otherwise. */
	virtual bool CullsBasedOnDirection() const { return true; }

	/** A SchedulingPolicy is equivalent to another SchedulingPolicy if they are the same (same ptr), or if they have the same type and parameter values. */
	PCG_API virtual bool IsEquivalent(const UPCGSchedulingPolicyBase* OtherSchedulingPolicy) const;

#if WITH_EDITOR
	/** Sets whether or not properties should be displayed in the editor.Used to hide instanced SchedulingPolicy properties when runtime generation is not enabled. */
	void SetShouldDisplayProperties(bool bInShouldDisplayProperties) { bShouldDisplayProperties = bInShouldDisplayProperties; }
#endif // WITH_EDITOR

	bool DoesGridDependOnWorldStreaming(uint32 InGridSize) const
	{
		if (GridStreamingDependencyMode == EPCGGridStreamingDependencyMode::NoGrids)
		{
			return false;
		}

		const EPCGHiGenGrid Grid = PCGHiGenGrid::GridSizeToGrid(InGridSize);

		if (Grid == EPCGHiGenGrid::Uninitialized)
		{
			return false;
		}

		return GridStreamingDependencyMode == EPCGGridStreamingDependencyMode::AllGrids
			|| (GridStreamingDependencyMode == EPCGGridStreamingDependencyMode::AllGridsExceptUnbounded && Grid != EPCGHiGenGrid::Unbounded)
			|| (GridStreamingDependencyMode == EPCGGridStreamingDependencyMode::SpecificGrids && GridsDependentOnWorldStreaming.Contains(Grid));
	}

	const TSet<FName>& GetWorldPartitionTargetGrids() const { return WorldPartitionTargetGrids; }

	/** 64-bit XXHash of WorldPartitionTargetGrids, cached on load/edit and used as a streaming-query cache key. */
	uint64 GetWorldPartitionTargetGridsHash() const { return WorldPartitionTargetGridsHash; }

public:
	/**
	 * Hi-Gen grids with a streaming dependency will only generate if the world within the generation volume is fully streamed. Enable this
	 * if the graph depends on actors in the world such as a landscape or a set of streamable actors.
	 */
	UPROPERTY(EditAnywhere, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (EditCondition = "bShouldDisplayProperties", EditConditionHides, HideEditConditionToggle))
	EPCGGridStreamingDependencyMode GridStreamingDependencyMode = EPCGGridStreamingDependencyMode::AllGridsExceptUnbounded;

	/** Hi-Gen grids that depend on world streaming. */
	UPROPERTY(EditAnywhere, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (EditCondition = "bShouldDisplayProperties && GridStreamingDependencyMode == EPCGGridStreamingDependencyMode::SpecificGrids", EditConditionHides, HideEditConditionToggle))
	TArray<EPCGHiGenGrid> GridsDependentOnWorldStreaming;

	/**
	 * Optional World Partition grid names to scope streaming queries. If empty, all WP grids are considered.
	 * When set, only the listed grids participate in the streaming-complete check that gates runtime generation.
	 */
	UPROPERTY(EditAnywhere, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (EditCondition = "bShouldDisplayProperties && GridStreamingDependencyMode != EPCGGridStreamingDependencyMode::NoGrids", EditConditionHides, HideEditConditionToggle))
	TSet<FName> WorldPartitionTargetGrids;

private:
	void UpdateWorldPartitionTargetGridsHash();

	/** Cached 64-bit hash of WorldPartitionTargetGrids. */
	uint64 WorldPartitionTargetGridsHash = 0;

#if WITH_EDITORONLY_DATA
	/** Hidden property to control display of SchedulingPolicy properties. */
	UPROPERTY(Transient, meta = (EditCondition = false, EditConditionHides))
	bool bShouldDisplayProperties = true;
#endif // WITH_EDITORONLY_DATA
};
