// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSchedulingPolicyBase.h"

#include "PCGSchedulingPolicyDistanceAndDirection.generated.h"

class IPCGGenSourceBase;

UENUM()
enum class EPCGSchedulingPolicyNetworkMode : uint8
{
	Client UMETA(ToolTip = "Considers generation sources that return true to IsLocal(). Can be used for standalone or listen server clients."),
	Server UMETA(ToolTip = "Considers genration sources that return false to IsLocal(). Can be used for dedicated server only generation."),
	All UMETA(ToolTip = "Considers all generation sources.")
};

/**
 * SchedulingPolicyDistanceAndDirection uses distance from the generating volume 
 * and alignment with view direction to choose the most important volumes to generate.
 *
 * Distance and Direction are calculated with respect to the Generation Source.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSchedulingPolicyDistanceAndDirection : public UPCGSchedulingPolicyBase
{
	GENERATED_BODY()

public:
	//~ Begin UPCGSchedulingPolicyBase interface
	virtual double CalculatePriority(const IPCGGenSourceBase* InGenSource, const FBox& GenerationBounds, bool bUse2DGrid) const override;
	virtual bool ShouldGenerate(const IPCGGenSourceBase* InGenSource, const FBox& GenerationBounds, bool bUse2DGrid) const override;
	virtual bool ShouldCull(const IPCGGenSourceBase* InGenSource, const FBox& GenerationBounds, bool bUse2DGrid) const override;
	virtual bool CullsBasedOnDirection() const override { return bUseFrustumCulling; }
	virtual bool IsEquivalent(const UPCGSchedulingPolicyBase* OtherSchedulingPolicy) const override;
	//~ End UPCGSchedulingPolicyBase interface

public:
	/** Toggle whether or not distance is used to calculate the scheduling priority. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (EditCondition = "bShouldDisplayProperties", EditConditionHides, HideEditConditionToggle))
	bool bUseDistance = true;

	/** Toggle whether or not direction is used to calculate the scheduling priority. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (EditCondition = "bShouldDisplayProperties", EditConditionHides, HideEditConditionToggle))
	bool bUseDirection = true;

	/** Scalar value used to increase/decrease the impact of direction in the scheduling priority. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (ClampMin = 0.0f, UIMax = 1.0f, EditCondition = "bShouldDisplayProperties && bUseDirection", EditConditionHides))
	float DirectionWeight = 1.0f;

	/** With frustum culling enabled, only components whose bounds overlap the view frustum will be generated. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (EditCondition = "bShouldDisplayProperties", EditConditionHides, HideEditConditionToggle))
	bool bUseFrustumCulling = false;

	/** Multiplier to scale bounds by when comparing against the view frustum for generation. Can help if components on the edge of the frustum are not generating as soon as you would like. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (ClampMin = 1.0f, EditCondition = "bShouldDisplayProperties && bUseFrustumCulling", EditConditionHides))
	float GenerateBoundsModifier = 1.0f;

	/** Multiplier to scale bounds by when comparing against the view frustum for clean up. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (ClampMin = 1.0f, EditCondition = "bShouldDisplayProperties && bUseFrustumCulling", EditConditionHides))
	float CleanupBoundsModifier = 1.2f;

	/** Client/Server policy mode will determine which generation sources to consider. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RuntimeGeneration|Scheduling Policy Parameters")
	EPCGSchedulingPolicyNetworkMode NetworkMode = EPCGSchedulingPolicyNetworkMode::All;

public:
	UE_DEPRECATED(5.7, "Distance weight no longer in use, use DirectionWeight instead.")
	UPROPERTY(BlueprintReadWrite, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (DeprecatedProperty), BlueprintGetter = "GetDistanceWeight_DEPRECATED", BlueprintSetter = "SetDistanceWeight_DEPRECATED")
	float DistanceWeight = 1.0f;

private:
	UFUNCTION(BlueprintPure, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (DeprecatedFunction, DeprecationMessage = "Use Direction Weight instead."))
	float GetDistanceWeight_DEPRECATED() const { return 0.0f; }

	UFUNCTION(BlueprintCallable, Category = "RuntimeGeneration|Scheduling Policy Parameters", meta = (DeprecatedFunction, DeprecationMessage = "Use Direction Weight instead."))
	void SetDistanceWeight_DEPRECATED(float Value) {}
};
