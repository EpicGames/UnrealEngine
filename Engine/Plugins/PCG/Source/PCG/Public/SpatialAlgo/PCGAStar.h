// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGBasePointData.h"
#include "Math/Vector.h"

namespace PCGSpatialAlgo::AStar
{
struct FPointDescription
{
	FPCGPointInputRange SrcPointRange;
	int32 PointIndexInRange = 0;
	FVector CachedLocation;
	
	FPointDescription() = default;
	FPointDescription(const FPCGPointInputRange& InSrcPointRange, int32 InPointIndexInRange):
		SrcPointRange(InSrcPointRange),
		PointIndexInRange(InPointIndexInRange),
		CachedLocation(SrcPointRange.PointData->GetTransform(SrcPointRange.RangeStartIndex + PointIndexInRange).GetLocation()) {}
	

	const FVector& GetLocation() const
	{
		return CachedLocation;
	}
};
	
namespace Cost
{
	double CalculateCost_EuclideanDistance(const FPointDescription& PreviousNodePoint, const double DistanceToPreviousNodeSquared, const FPointDescription& CurrentPoint);
}

namespace Heuristic
{
	double CalculateHeuristic_EuclideanDistance(const FVector& CurrentLocation, const FVector& GoalLocation);
}

struct FSearchSettings
{
	/** The max distance from each point to search for the next viable point in the path. */
	double SearchDistance = 1000;
	/** The heuristic estimates a faster path to speed up processing. A lower heuristic weight can be faster, but it may cease being the optimal path. A weight of 0 is essentially flood fill. */
	double HeuristicWeight = 1.0;
	/** Even if the path is not complete, return the most optimal and viable partial path to the goal. */
	bool bAcceptPartialPath = true;
	/** Copy the point data from the originating points. */
	bool bCopyOriginatingPoints = false;
};

struct FSearchState
{
	FSearchState();
	~FSearchState();
	/** Keep track of the point data for the search. */
	const UPCGBasePointData* OriginatingPointData = nullptr;
	
	/** Keep the goal point range to index them with FPointDescription */
	FPCGPointInputRange GoalPoints{};
	/** Cost function */
	TFunction<double(const FPointDescription& /*PreviousNodePoint*/, const double /*DistanceToPreviousNodeSquared*/, const FPointDescription& /*CurrentPoint*/)> CostFunction = Cost::CalculateCost_EuclideanDistance;
	/** Heuristic function */
	TFunction<double(const FVector& /*CurrentLocation*/, const FVector& /*GoalLocation*/)> HeuristicFunction = Heuristic::CalculateHeuristic_EuclideanDistance;

	struct FInternalSearchData;
	TUniquePtr<FInternalSearchData> InternalSearchData;
};

	
enum class ESearchResult
{
	Invalid = 0,
	Processing,
	Partial,
	Complete
};

/** Initialize the search state for AStar. Must be called before ExecuteSearchIteration. */
void Initialize(const FPCGPointInputRange& StartPoint, const FPCGPointInputRange& GoalPoints, FSearchState& OutSearchState);

/** Runs a single iteration of the A* algorithm. Intended to be called multiple times, whereupon it will return true when the algorithm is finished. */
ESearchResult ExecuteSearchIteration(const FSearchSettings& SearchSettings, FSearchState& SearchState, TArray<FPointDescription>& OutPath);
}
