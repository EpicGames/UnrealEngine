// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "Templates/Function.h"

namespace PCGSpatialAlgo
{
	struct FPCGMarchingSquareResult
	{
		TArray<FVector2D> LinkedGridCoordinates;
		bool bClosed = false;
	};

	/**
	* Marching Squares Types (16 of them). 0 in a corner means value is below Isovalue, and 1 means value is above (or equal) Isovalue.
	*  0---0   0---0   0---0   0---0   0---1   0---1   0---1   0---1
	*  |   |   |   |   |   |   |   |   |  \|   |/  |   | | |   |/  |
	*  |   |   |   |   |   |   |---|   |   |   |   |   | | |   |   |
	*  |   |   |\  |   |  /|   |   |   |   |   |  /|   | | |   |   |
	*  0---0   1---0   0---1   1---1   0---0   1---0   0---1   1---1
	* Type 0  Type 1  Type 2  Type 3  Type 4  Type 5  Type 6  Type 7
	*
	*  1---0   1---0   1---0   1---0   1---1   1---1   1---1   1---1
	*  |/  |   | | |   |  \|   |  \|   |   |   |   |   |   |   |   |
	*  |   |   | | |   |   |   |   |   |---|   |   |   |   |   |   |
	*  |   |   | | |   |\  |   |   |   |   |   |  /|   |\  |   |   |
	*  0---0   1---0   0---1   1---1   0---0   1---0   0---1   1---1
	* Type 8  Type 9  Type 10 Type 11 Type 12 Type 13 Type 14  Type 15
	*/

	/** Marching Squares Algorithm (cf. https://en.wikipedia.org/wiki/Marching_squares) 
	* It takes some kind of map of values (like a heightmap, texture, etc...) and will return a list of coordinates on a grid that will form a line where the value in the map on the line is
	* the isovalue passed as argument.
	* It can be used for example to find isolines on the landscape, where the line will be where the landscape is a the "Isovalue" height.
	* Marching Squares is discretizing the map of values into a 2D grid, of size CellCountX * CellCountY.
	* 
	* @param CellCountX               Size of the grid on the X axis
	* @param CellCountY               Size of the grid on the Y axis
	* @param Isovalue                 Defines the value where the lines computed will have in the map of values.
	* @param ValueQuery               Sampling function to get the value for the cell (X,Y).
	* @param bUseLinearInterpolation  If disabled, a point will be at the middle of cell edges. If we use linear interpolation the point will be linearly interpolated to be placed where the isovalue would be
	* (using the values on each extremity of the cell edge, so requires 2 more sampling).
	*/
	PCG_API TArray<FPCGMarchingSquareResult> MarchingSquares(const int32 CellCountX, const int32 CellCountY, const double Isovalue, TFunctionRef<double(int32 X, int32 Y)> ValueQuery, bool bUseLinearInterpolation);
}
