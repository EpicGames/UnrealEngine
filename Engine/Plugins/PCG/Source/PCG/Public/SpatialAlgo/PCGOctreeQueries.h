// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPoint.h"
#include "Data/PCGPointData.h"

#include "Math/Vector.h"
#include "Templates/Function.h"

#include "PCGOctreeQueries.generated.h"

UCLASS(MinimalAPI)
class UPCGOctreeQueries : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

private:
	// Can only be used with not UFUNCTION, as UFUNCTION requires the value to be explicitly defined.
	static inline constexpr double DefaultSearchDistance = 20000.0;

public:
	/** Query the internal octree to call a func on all points within some sphere. Callback takes the point (as const ref) and its squared distance to the center. */
	template <typename Func = TFunctionRef<void(const UPCGBasePointData* PointData, int32 Index, double SquaredDistance)>>
	static PCG_API void ForEachPointInsideSphere(const UPCGBasePointData* InPointData, const FVector& InCenter, const double InRadius, Func InCallback);

	/** Get the closest point to a given position within the search distance, If bInDiscardCenter is true, will reject any points that are exactly at the center. Returns null if not found. */
	static PCG_API const FPCGPoint* GetClosestPoint(const UPCGPointData* InPointData, const FVector& InCenter, const bool bInDiscardCenter, const double InSearchDistance = DefaultSearchDistance);
	static PCG_API int32 GetClosestPointIndex(const UPCGBasePointData* InPointData, const FVector& InCenter, const bool bInDiscardCenter, const double InSearchDistance = DefaultSearchDistance);

	/** Get the nearest point to a given point (that is not itself) within the search distance. Returns null if not found. */
	static PCG_API const FPCGPoint* GetClosestPointFromOtherPoint(const UPCGPointData* InPointData, const FPCGPoint& InPoint, const double InSearchDistance = DefaultSearchDistance);
	static PCG_API int32 GetClosestPointIndexFromOtherPointIndex(const UPCGBasePointData* InPointData, int32 OtherPointIndex, const double InSearchDistance = DefaultSearchDistance);

	/** Get the farthest point from a given position within the search distance. Returns null if not found. */
	static PCG_API const FPCGPoint* GetFarthestPoint(const UPCGPointData* InPointData, const FVector& InCenter, const double InSearchDistance = DefaultSearchDistance);
	static PCG_API int32 GetFarthestPointIndex(const UPCGBasePointData* InPointData, const FVector& InCenter, const double InSearchDistance = DefaultSearchDistance);

	/** Get the farthest point from a given point (that is not itself) within the search distance. Returns null if not found. */
	static PCG_API const FPCGPoint* GetFarthestPointFromOtherPoint(const UPCGPointData* InPointData, const FPCGPoint& InPoint, const double InSearchDistance = DefaultSearchDistance);
	static PCG_API int32 GetFarthestPointIndexFromOtherPointIndex(const UPCGBasePointData* InPointData, int32 OtherPointIndex, const double InSearchDistance);

private:
	// Blueprint functions are not exposed, use the functions above or the Octree directly

	/** Query the internal octree to return all the points within some bounds. */
	UFUNCTION(BlueprintCallable, Category = PointData)
	static TArray<FPCGPoint> GetPointsInsideBounds(const UPCGPointData* InPointData, const FBox& InBounds);

	/** Query the internal octree to return all the points within some sphere. */
	UFUNCTION(BlueprintCallable, Category = PointData)
	static TArray<FPCGPoint> GetPointsInsideSphere(const UPCGPointData* InPointData, const FVector& InCenter, const double InRadius);

	/** Get the closest point to a given position within the search distance. If bInDiscardCenter is true, will reject any points that is at the center exactly.*/
	UFUNCTION(BlueprintCallable, Category = PointData)
	static void GetClosestPoint(const UPCGPointData* InPointData, const FVector& InCenter, const bool bInDiscardCenter, bool& bOutFound, FPCGPoint& OutPoint, const double InSearchDistance = 20000.0);

	/** Get the nearest point to a given point within the search distance. The point is referenced by its point index in this point data. */
	UFUNCTION(BlueprintCallable, Category = PointData)
	static void GetClosestPointFromOtherPoint(const UPCGPointData* InPointData, const int32 InPointIndex, bool& bOutFound, FPCGPoint& OutPoint, const double InSearchDistance = 20000.0);

	/** Get the farthest point from a given position, within the search distance. */
	UFUNCTION(BlueprintCallable, Category = PointData)
	static void GetFarthestPoint(const UPCGPointData* InPointData, const FVector& InCenter, bool& bOutFound, FPCGPoint& OutPoint, const double InSearchDistance = 20000.0);

	/** Get the farthest point from a given point within the search distance. The point is referenced by its point index in this point data. */
	UFUNCTION(BlueprintCallable, Category = PointData)
	static void GetFarthestPointFromOtherPoint(const UPCGPointData* InPointData, const int32 InPointIndex, bool& bOutFound, FPCGPoint& OutPoint, const double InSearchDistance = 20000.0);
};

template <typename Func>
void UPCGOctreeQueries::ForEachPointInsideSphere(const UPCGBasePointData* InPointData, const FVector& InCenter, const double InRadius, Func Callback)
{
	if (!InPointData)
	{
		return;
	}

	const double Extents = UE_DOUBLE_SQRT_2 * InRadius;
	const double SquaredRadius = InRadius * InRadius;
	FBoxCenterAndExtent SearchBounds(InCenter, FVector(Extents, Extents, Extents));
	
	// To support previous callback
	const UPCGPointData* PointData = Cast<UPCGPointData>(InPointData);
	const TArray<FPCGPoint>* Points = PointData ? &PointData->GetPoints() : nullptr;
	const FPCGPointTransform::ConstValueRange TransformValueRange = InPointData->GetConstTransformValueRange();
		
	InPointData->GetPointOctree().FindElementsWithBoundsTest(SearchBounds, [InPointData, Points, &TransformValueRange, &InCenter, SquaredRadius, Callback = std::move(Callback)](const PCGPointOctree::FPointRef& PointRef)
	{
		if (!InPointData->IsValidRef(PointRef))
		{
			return;
		}

		const double SquaredDistance = FVector::DistSquared(InCenter, TransformValueRange[PointRef.Index].GetLocation());
		if (SquaredDistance <= SquaredRadius)
		{
			if constexpr (std::is_invocable_v<Func, const FPCGPoint&, double>)
			{
				check(Points);
				Callback((*Points)[PointRef.Index], SquaredDistance);
			}
			else
			{
				Callback(InPointData, PointRef.Index, SquaredDistance);
			}
		}
	});
}
