// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"

#include "PCGCommon.h"
#include "Elements/PCGActorSelector.h"
#include "Graph/PCGGraphPerExecutionCache.h"
#include "Utils/PCGNodeVisualLogs.h"
#include "UObject/Interface.h"

#include "UObject/ObjectKey.h"

#include "IPCGBaseSubsystem.generated.h"

class IPCGBaseSubsystem;
class IPCGElement;
class IPCGGraphCache;
class FPCGGraphCompiler;
class FPCGGraphExecutor;
class UPCGComputeGraph;
class UPCGData;
class UPCGDefaultExecutionSource;
class UPCGGraph;
class UPCGSettings;
class UWorld;
struct FPCGDataCollection;
struct FPCGScheduleGenericParams;
struct FPCGStack;

#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE_ThreeParams(FPCGOnPCGSourceGenerationDone, IPCGBaseSubsystem*, IPCGGraphExecutionSource*, EPCGGenerationStatus);
DECLARE_MULTICAST_DELEGATE_OneParam(FPCGOnPCGSourceUnregistered, IPCGGraphExecutionSource*);
#endif // WITH_EDITOR

UINTERFACE()
class UPCGBaseSubsystem : public UInterface
{
	GENERATED_BODY()
};

class IPCGBaseSubsystem
{
	GENERATED_BODY()

public:
	virtual UWorld* GetSubsystemWorld() const { return nullptr; }

	PCG_API void InitializeBaseSubsystem();
	PCG_API void DeinitializeBaseSubsystem();
	
	/** Subsystem must not be used without this condition being true. */
	// @todo_pcg: Hides function in WorldSubsystem
	//bool IsInitialized() const { return GraphExecutor != nullptr; }

	/** Called by graph executor when a graph is scheduled. */
	PCG_API void OnScheduleGraph(const FPCGStackContext& StackContext);

	// Schedule graph (used internally for dynamic subgraph execution)
	PCG_API FPCGTaskId ScheduleGraph(const FPCGScheduleGraphParams& InParams);

	// General job scheduling
	PCG_API FPCGTaskId ScheduleGeneric(const FPCGScheduleGenericParams& InParams);
	
	/** Cancels currently running generation */
	PCG_API void CancelGeneration(IPCGGraphExecutionSource* Source);

	/** Cancels currently running generation on given graph */
	PCG_API void CancelGeneration(UPCGGraph* Graph);

	/** Returns true if there are any tasks for this graph currently scheduled or executing. */
	PCG_API bool IsGraphCurrentlyExecuting(UPCGGraph* Graph);

	/** Returns true if any task is scheduled or executing for any graph */
	PCG_API bool IsAnyGraphCurrentlyExecuting() const;

	/** Cancels everything running */
	PCG_API void CancelAllGeneration();

	/** Gets the output data for a given task */
	PCG_API bool GetOutputData(FPCGTaskId InTaskId, FPCGDataCollection& OutData);

	/** Clears the output data for a given task. Should only be called on tasks with bNeedsManualClear set to true */
	PCG_API void ClearOutputData(FPCGTaskId InTaskId);

	/** Returns the interface to the cache, required for element per-data caching */
	PCG_API IPCGGraphCache* GetCache();

	/** Flushes the graph cache and graph compiler cache, and in editor trigger a GC. */
	PCG_API void FlushCache();

	/** True if graph cache debugging is enabled. */
	PCG_API bool IsGraphCacheDebuggingEnabled() const;
	
	PCG_API FPCGGraphCompiler* GetGraphCompiler();

	UE_DEPRECATED(5.8, "Use FPCGContext::GetComputeGraph instead, which resolves from the executing graph's compiler cache.")
	PCG_API UPCGComputeGraph* GetComputeGraph(const UPCGGraph* InGraph, uint32 GridSize, uint32 ComputeGraphIndex);

	/** Returns the cached value if the entry exists for the given source's task, unset otherwise. */
	template<typename T>
	static TOptional<typename T::ValueType> GetExecutionCacheEntry(const IPCGGraphExecutionSource* InSource, TPCGPerExecutionCacheId<T> InCacheId)
	{
		check(InSource);
		if (IPCGBaseSubsystem* PCGSubsystem = InSource->GetExecutionState().GetSubsystem())
		{
			if (FPCGPerExecutionCache* Cache = PCGSubsystem->GetCacheInternal())
			{
				return Cache->GetExecutionCacheEntry(InSource, InCacheId);
			}
		}

		return TOptional<typename T::ValueType>{};
	}

	/** Stores a new value for the given source's task and typed ID. Asserts if the slot is already occupied. */
	template<typename T>
	static void SetExecutionCacheEntry(const IPCGGraphExecutionSource* InSource, TPCGPerExecutionCacheId<T> InCacheId, typename T::ValueType InValue, bool bValidateWritable = true)
	{
		check(InSource);
		if (IPCGBaseSubsystem* PCGSubsystem = InSource->GetExecutionState().GetSubsystem())
		{
			if (FPCGPerExecutionCache* Cache = PCGSubsystem->GetCacheInternal())
			{
				Cache->SetExecutionCacheEntry(InSource, InCacheId, InValue, bValidateWritable);
			}
		}
	}

	/** Returns the cached value if present, otherwise calls InMakeEntry() to produce one.
	 *  If the source's task is valid the produced value is stored before returning.
	 *  If no subsystem is reachable from InSource InMakeEntry() is called and its result returned uncached. */
	template<typename T, typename FuncType>
	static typename T::ValueType GetOrCreateExecutionCacheValue(const IPCGGraphExecutionSource* InSource, TPCGPerExecutionCacheId<T> InCacheId, const FuncType& InMakeEntry, bool bValidateWritable = true)
	{
		check(InSource);
		if (IPCGBaseSubsystem* PCGSubsystem = InSource->GetExecutionState().GetSubsystem(); PCGSubsystem)
		{
			if (FPCGPerExecutionCache* Cache = PCGSubsystem->GetCacheInternal())
			{
				return Cache->GetOrCreateExecutionCacheValue(InSource, InCacheId, InMakeEntry, bValidateWritable, PCGSubsystem->IsGraphCacheDebuggingEnabled());
			}
		}
		return InMakeEntry();
	}

#if PCG_PROFILING_ENABLED
public:
	PCG_API virtual void OnPCGSourceGenerationDone(IPCGGraphExecutionSource* InExecutionSource, EPCGGenerationStatus InStatus);
#endif // PCG_PROFILING_ENABLED

#if WITH_EDITOR
public:
	/** Propagate to the graph compiler graph changes */
	PCG_API virtual void NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType);

	/** Cleans up the graph cache on an element basis. InSettings is used for debugging and is optional. */
	PCG_API void CleanFromCache(const IPCGElement* InElement, const UPCGSettings* InSettings = nullptr);

	/** Gets the execution stack information for the given component (depending on partitioning, grid size, etc.) but with no component frames. */
	PCG_API bool GetStackContext(const IPCGGraphExecutionSource* InSource, FPCGStackContext& OutStackContext);

	/** Gets the base execution stack information for a specific graph & grid size. */
	PCG_API bool GetStackContext(UPCGGraph* InGraph, uint32 GridSize, bool bIsPartitioned, FPCGStackContext& OutStackContext);

	/** Returns how many times InElement is present in the cache. */
	PCG_API uint32 GetGraphCacheEntryCount(IPCGElement* InElement) const;

	FPCGOnPCGSourceGenerationDone& GetOnPCGSourceGenerationDone() { return OnPCGSourceGenerationDoneDelegate; }
	FPCGOnPCGSourceUnregistered& GetOnPCGSourceUnregistered() { return OnPCGSourceUnregisteredDelegate; }

	/** Creates an execution source from the params and optionally tracks its lifetime and generation. Note this call is NOT THREADSAFE and as such this should be called only from the game thread. */
	template<typename SourceType>
		requires std::is_base_of_v<IPCGGraphExecutionSource, SourceType>
	static SourceType* CreateExecutionSource(const typename SourceType::ParamsType& InParams, UObject* Outer = nullptr)
	{
		check(IsInGameThread());
		if (!InParams.GraphInterface)
		{
			return nullptr;
		}

		SourceType* NewSource = NewObject<SourceType>(Outer ? Outer : GetTransientPackageAsObject());
		NewSource->Initialize(InParams);

		if (InParams.bFireAndForgetExecution)
		{
			IPCGBaseSubsystem* Subsystem = NewSource->GetExecutionState().GetSubsystem();
			check(Subsystem);
			
			FPCGOnPCGSourceGenerationDone GenerationCallbackWrapper;
			if (InParams.GenerationCallback.IsBound())
			{
				GenerationCallbackWrapper.AddLambda([GenerationCallback = InParams.GenerationCallback](IPCGBaseSubsystem*, IPCGGraphExecutionSource* Source, EPCGGenerationStatus Status)
				{
					GenerationCallback.ExecuteIfBound(Source, Status);
				});
			}

			Subsystem->GraphExecutions.Emplace(NewSource, GenerationCallbackWrapper);
			NewSource->Generate();
		}

		return NewSource;
	}
#endif // WITH_EDITOR
	
protected:
	/** Returns the expected end time of the tick. By default, it's the current time + the budget of the graph executor. */
	PCG_API virtual double GetTickEndTime() const;

	/** Tick the graph executor. Return expected end time. */
	PCG_API double Tick();
	
	PCG_API void CancelGeneration(IPCGGraphExecutionSource* Source, bool bCleanupUnusedResources);
	
	virtual void CancelGenerationInternal(IPCGGraphExecutionSource* Source, bool bCleanupUnusedResources) {};

	PCG_API FPCGPerExecutionCache* GetCacheInternal() const;

	TSharedPtr<FPCGGraphExecutor> GraphExecutor;

#if WITH_EDITOR
	PCG_API void SetDisableClearResults(bool bInDisableClearResults);

	PCG_API void AddReferencedObjects(FReferenceCollector& Collector);

	struct FGraphExecution
	{
		TObjectPtr<UPCGDefaultExecutionSource> ExecutionSource;
		FPCGOnPCGSourceGenerationDone GenerationCallback;
	};

	TArray<FGraphExecution> GraphExecutions;

	FPCGOnPCGSourceGenerationDone OnPCGSourceGenerationDoneDelegate;
	FPCGOnPCGSourceUnregistered OnPCGSourceUnregisteredDelegate;
#endif // WITH_EDITOR
};
