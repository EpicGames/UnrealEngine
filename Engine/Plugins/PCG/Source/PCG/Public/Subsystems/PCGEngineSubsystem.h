// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tickable.h"

#include "Subsystems/EngineSubsystem.h"
#include "Subsystems/IPCGBaseSubsystem.h"

#include "PCGEngineSubsystem.generated.h"

struct FPCGDefaultExecutionSourceParams;
class UPCGDefaultExecutionSource;
class UPCGGraphInterface;

DECLARE_DELEGATE_TwoParams(FPCGOnEditorGenerationDone, IPCGGraphExecutionSource*, EPCGGenerationStatus);

struct FPCGGenerateGraphParams
{
	/** Graph to generate. */
	TObjectPtr<UPCGGraphInterface> Graph = nullptr;

	/** Generation seed. */
	int32 Seed = 42;

	/** Optional callback when generation completes. */
	FPCGOnEditorGenerationDone GenerationCallback;
};

/**
 * Engine subsystem that allows running graph generations in Editor and headless configurations. 
 */
UCLASS(MinimalAPI)
class UPCGEngineSubsystem : public UEngineSubsystem, public IPCGBaseSubsystem, public FTickableGameObject
{
	GENERATED_BODY()
	
public:

	PCG_API static UPCGEngineSubsystem* Get();

#if WITH_EDITOR
	PCG_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	PCG_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	PCG_API virtual void Deinitialize() override;

	PCG_API void GenerateGraph(const FPCGGenerateGraphParams& InParams);

	PCG_API virtual void NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType) override;

	UE_DEPRECATED(5.8, "Use static IPCGBaseSubsystem version instead")
	PCG_API UPCGDefaultExecutionSource* CreateExecutionSource(const FPCGDefaultExecutionSourceParams& InParams);
#endif // WITH_EDITOR

protected:
	//** FTickableGameObject Interface
	PCG_API virtual void Tick(float DeltaTime) override;
	PCG_API virtual ETickableTickType GetTickableTickType() const override;
	PCG_API virtual TStatId GetStatId() const override;
	virtual bool IsTickableInEditor() const { return true; }
	//~ END FTickableGameObject Interface
};