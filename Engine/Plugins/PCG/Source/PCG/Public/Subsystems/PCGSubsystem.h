// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/IPCGBaseSubsystem.h"
#include "RHIShaderPlatform.h"
#include "Subsystems/WorldSubsystem.h"

#include "PCGCommon.h"
#include "Elements/PCGActorSelector.h"
#include "Grid/PCGGridDescriptor.h"
#include "Utils/PCGNodeVisualLogs.h"

#include "Scalability.h"
#include "UObject/ObjectKey.h"

#include "PCGSubsystem.generated.h"

class APCGPartitionActor;
class APCGWorldActor;
class FPCGTrackingManager;
class FPCGGenSourceManager;
class FPCGRuntimeGenScheduler;
class FViewport;
class UPCGComponent;
class UPCGComputeGraph;
class UPCGData;
class UPCGGraph;
class UPCGLandscapeCache;

enum class EPCGComponentDirtyFlag : uint8;
enum class ETickableTickType : uint8;
namespace ERHIFeatureLevel { enum Type : int; }

class IPCGGraphCache;
class IPCGGraphExecutionSource;
class FPCGGraphCompiler;
class FPCGGraphExecutor;
struct FPCGMoveResourceParams;
struct FPCGContext;
struct FPCGDataCollection;
struct FPCGScheduleGenericParams;
struct FPCGStack;
class UPCGSettings;

class IPCGElement;
typedef TSharedPtr<IPCGElement, ESPMode::ThreadSafe> FPCGElementPtr;

class FBoxCenterAndExtent;

class UWorld;

#if WITH_EDITOR

DECLARE_MULTICAST_DELEGATE_OneParam(FPCGOnComponentGenerationCompleteOrCancelled, UPCGSubsystem*);

/** Deprecated 5.7 - use FPCGOnPCGSourceGenerationDone */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FPCGOnPCGComponentGenerationDone, UPCGSubsystem*, UPCGComponent*, EPCGGenerationStatus);

/** Deprecated 5.8 - use FPCGOnPCGSourceUnregistered */
DECLARE_MULTICAST_DELEGATE_OneParam(FPCGOnPCGComponentUnregistered, UPCGComponent*);

// Event triggered when GenerateAllPCGComponents is done
DECLARE_MULTICAST_DELEGATE(FPCGOnAllComponentsGenerated);

// Event triggered when CleanupAllPCGComponents is done
DECLARE_MULTICAST_DELEGATE(FPCGOnAllComponentsCleanedup);

// Event triggered when ClearLinkForAllPCGComponents is done
DECLARE_MULTICAST_DELEGATE(FPCGOnAllComponentsClearedLink);
#endif // WITH_EDITOR

/**
* UPCGSubsystem
*/
UCLASS(MinimalAPI)
class UPCGSubsystem : public UTickableWorldSubsystem, public IPCGBaseSubsystem
{
	GENERATED_BODY()

public:
	friend class UPCGComponent;
	friend FPCGTrackingManager;
	friend struct FPCGWorldPartitionBuilder;

	PCG_API UPCGSubsystem();

	/** Add UObject references for GC */
	static PCG_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/** To be used when a PCG component can not have a world anymore, to unregister itself. */
	static PCG_API UPCGSubsystem* GetSubsystemForCurrentWorld();

	virtual UWorld* GetSubsystemWorld() const override { return GetWorld(); }

	//~ Begin USubsystem Interface.
	PCG_API virtual void Deinitialize() override;
	//~ End USubsystem Interface

	//~ Begin UWorldSubsystem Interface.
	PCG_API virtual void PostInitialize() override;
	// need UpdateStreamingState? 
	//~ End UWorldSubsystem Interface.

	//~ Begin FTickableGameObject
	PCG_API virtual void Tick(float DeltaSeconds) override;
	virtual bool IsTickableInEditor() const override { return true; }
	PCG_API virtual ETickableTickType GetTickableTickType() const override;
	PCG_API virtual TStatId GetStatId() const override;
	//~ End FTickableGameObject

	/** Will return the subsystem from the World if it exists and if it is initialized */
	static PCG_API UPCGSubsystem* GetInstance(UWorld* World);

	/** Adds an action that will be executed once at the beginning of this subsystem's next Tick(). */
	using FTickAction = TFunction<void()>;

	UE_DEPRECATED(5.6, "Use FPCGModule::ExecuteNextTick instead")
	PCG_API void RegisterBeginTickAction(FTickAction&& Action);

#if WITH_EDITOR
	/** Returns PIE world if it is active, otherwise returns editor world. */
	static PCG_API UPCGSubsystem* GetActiveEditorInstance();

	PCG_API void SetConstructionScriptSourceComponent(UPCGComponent* InComponent);
	PCG_API bool RemoveAndCopyConstructionScriptSourceComponent(AActor* InComponentOwner, FName InComponentName, UPCGComponent*& OutSourceComponent);

	/** Returns true if the Component is being unloaded through World Partition editor unloading or if the level the component is part of is being removed from world */
	PCG_API bool IsComponentBeingUnloaded(UActorComponent* InActorComponent) const;
#endif

	PCG_API APCGWorldActor* GetPCGWorldActor();
	PCG_API APCGWorldActor* FindPCGWorldActor();

	/** Returns current quality level between Low (0) and Cinematic (4). */
	static PCG_API int32 GetPCGQualityLevel();
	PCG_API void OnPCGQualityLevelChanged();

	void OnScalabilitySettingsChanged(const Scalability::FQualityLevels& QualityLevels);

#if WITH_EDITOR
	PCG_API void DestroyAllPCGWorldActors();
	PCG_API void DestroyCurrentPCGWorldActor();
	PCG_API void LogAbnormalComponentStates(bool bGroupByState) const;

	PCG_API AActor* GetOrCreateDebugActor();
#endif
	PCG_API void RegisterPCGWorldActor(APCGWorldActor* InActor);
	PCG_API void UnregisterPCGWorldActor(APCGWorldActor* InActor);

	PCG_API void OnOriginalExecutionSourceRegistered(IPCGGraphExecutionSource* InExecutionSource);
	PCG_API void OnOriginalExecutionSourceUnregistered(IPCGGraphExecutionSource* InExecutionSource);

	/** In case of blueprints, we need to remap the old execution source destroyed by the construction script to the new one. */
	PCG_API void OnOriginalExecutionSourceReplaced(const IPCGGraphExecutionSource* InOldExecutionSource, IPCGGraphExecutionSource* InNewExecutionSource);

	PCG_API UPCGLandscapeCache* GetLandscapeCache();

	/** Schedule graph(owner->graph) */
	PCG_API FPCGTaskId ScheduleComponent(UPCGComponent* PCGComponent, uint32 Grid, bool bForce, const TArray<FPCGTaskId>& InDependencies);

	UE_DEPRECATED(5.8, "Use the version that takes a uint32 Grid")
	PCG_API FPCGTaskId ScheduleComponent(UPCGComponent* PCGComponent, EPCGHiGenGrid Grid, bool bForce, const TArray<FPCGTaskId>& InDependencies);

	/** Schedule cleanup(owner->graph). */
	PCG_API FPCGTaskId ScheduleCleanup(UPCGComponent* PCGComponent, bool bRemoveComponents, const TArray<FPCGTaskId>& Dependencies);

	using IPCGBaseSubsystem::ScheduleGraph;
	
	PCG_API FPCGTaskId ScheduleGraph(
		UPCGGraph* Graph,
		IPCGGraphExecutionSource* ExecutionSource,
		FPCGElementPtr PreGraphElement,
		FPCGElementPtr InputElement,
		const TArray<FPCGTaskId>& Dependencies,
		const FPCGStack* InFromStack,
		bool bAllowHierarchicalGeneration);

	// Schedule graph (used internally for dynamic subgraph execution)
	PCG_API FPCGTaskId ScheduleGraph(IPCGGraphExecutionSource* ExecutionSource, const TArray<FPCGTaskId>& Dependencies);

	using IPCGBaseSubsystem::ScheduleGeneric;

	/** General job scheduling
	*  @param InOperation:               Callback that returns true if the task is done, false otherwise.
	*  @param ExecutionSource:           Execution source associated with this task. Can be null.
	*  @param TaskExecutionDependencies: Task will wait on these tasks to execute and won't take their output data as input.
	*/
	PCG_API FPCGTaskId ScheduleGeneric(TFunction<bool()> InOperation, IPCGGraphExecutionSource* ExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies) const;

	/** General job scheduling
	*  @param InOperation:               Callback that returns true if the task is done, false otherwise.
	*  @param InAbortOperation:          Callback that will be called if the generic task is cancelled for any reason.
	*  @param ExecutionSource:           Execution source associated with this task. Can be null.
	*  @param TaskExecutionDependencies: Task will wait on these tasks to execute and won't take their output data as input.
	*/
	PCG_API FPCGTaskId ScheduleGeneric(TFunction<bool()> InOperation, TFunction<void()> InAbortOperation, IPCGGraphExecutionSource* ExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies) const;

	/** General job scheduling with context
	*  @param InOperation:               Callback that takes a Context as argument and returns true if the task is done, false otherwise.
	*  @param ExecutionSource:           Execution source associated with this task. Can be null.
	*  @param TaskExecutionDependencies: Task will wait on these tasks to execute and won't take their output data as input.
	*  @param TaskDataDependencies:      Task will wait on these tasks to execute and will take their output data as input.
	*/
	PCG_API FPCGTaskId ScheduleGenericWithContext(TFunction<bool(FPCGContext*)> InOperation, IPCGGraphExecutionSource* ExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies, const TArray<FPCGTaskId>& TaskDataDependencies) const;

	/** General job scheduling with context
	*  @param InOperation:               Callback that takes a Context as argument and returns true if the task is done, false otherwise.
	*  @param InAbortOperation:          Callback that will be called if the generic task is cancelled for any reason.
	*  @param ExecutionSource:           Execution source associated with this task. Can be null.
	*  @param TaskExecutionDependencies: Task will wait on these tasks to execute and won't take their output data as input.
	*  @param TaskDataDependencies:      Task will wait on these tasks to execute and will take their output data as input.
	*/
	PCG_API FPCGTaskId ScheduleGenericWithContext(TFunction<bool(FPCGContext*)> InOperation, TFunction<void(FPCGContext*)> InAbortOperation, IPCGGraphExecutionSource* ExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies, const TArray<FPCGTaskId>& TaskDataDependencies) const;

	/** Asks the runtime generation scheduler to refresh a given GenerateAtRuntime execution source. ChangeType should be 'GenerationGrid' to perform a full cleanup of PAs and local sources. */
	PCG_API void RefreshRuntimeGenExecutionSource(IPCGGraphExecutionSource* RuntimeExecutionSource, EPCGChangeType ChangeType = EPCGChangeType::None);

	/** Refresh all GenerateAtRuntime execution sources. ChangeType should be 'GenerationGrid' to perform a full cleanup of PAs and local sources.
	 *  If reason is UserRequested then all sources will refresh; otherwise it depends on the refresh flags set on the graph for each source. */
	PCG_API void RefreshAllRuntimeGenExecutionSources(EPCGChangeType ChangeType = EPCGChangeType::None, ERuntimeGenRefreshReason Reason = ERuntimeGenRefreshReason::UserRequested);

	/** Marks execution sources dirty; on next tick they are refreshed via RefreshAllRuntimeGenExecutionSources with the union of all reasons queued since last tick.
	 *  Defer variant for events that may fire many times in quick succession (e.g. viewport resize). */
	PCG_API void DirtyRuntimeGenExecutionSources(EPCGChangeType ChangeType = EPCGChangeType::None, ERuntimeGenRefreshReason Reason = ERuntimeGenRefreshReason::UserRequested);

#if WITH_EDITOR
	/** Refresh all components selected by the filter (runtime generated or otherwise). */
	PCG_API void RefreshAllComponentsFiltered(const TFunction<bool(UPCGComponent*)>& ComponentFilter, EPCGChangeType ChangeType = EPCGChangeType::None);
#endif

	FPCGRuntimeGenScheduler* GetRuntimeGenScheduler() const { return RuntimeGenScheduler; }

	/** Register a new execution source or update it, will be added to the octree if it doesn't exists yet. Returns true if it was added/updated. Thread safe */
	PCG_API bool RegisterOrUpdateExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bDoPartitionMapping = true);

	/** In case of blueprints, we need to remap the old execution source destroyed by the construction script to the new one. Returns true if re-mapping succeeded. */
	PCG_API bool RemapExecutionSource(const IPCGGraphExecutionSource* InOldExecutionSource, IPCGGraphExecutionSource* InNewExecutionSource, bool bDoPartitionMapping);

	/** Unregister a execution source, will be removed from the octree. Can force it, if we have a delayed unregister. Thread safe */
	PCG_API void UnregisterExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bForce = false);

	/** Register a new Partition actor, will be added to a map and will query all intersecting volume to bind to them if asked. Thread safe */
	PCG_API void RegisterPartitionActor(APCGPartitionActor* InActor);

	/** Unregister a Partition actor, will be removed from the map and remove itself to all intersecting volumes. Thread safe */
	PCG_API void UnregisterPartitionActor(APCGPartitionActor* InActor);

	PCG_API TSet<IPCGGraphExecutionSource*> GetAllRegisteredPartitionedExecutionSources() const;
	PCG_API TSet<IPCGGraphExecutionSource*> GetAllRegisteredExecutionSources() const;

	/** Call the InFunc function to all local components registered to the original component. Thread safe*/
	PCG_API void ForAllRegisteredLocalComponents(UPCGComponent* InOriginalComponent, const TFunctionRef<void(UPCGComponent*)>& InFunc) const;

	/** Call the InFunc function to all local component registered to the original component within some bounds. Thread safe*/
	PCG_API void ForAllRegisteredIntersectingLocalComponents(UPCGComponent* InOriginalComponent, const FBoxCenterAndExtent& InBounds, const TFunctionRef<void(UPCGComponent*)>& InFunc) const;

	/** Gather all the Execution sources within some bounds. */
	PCG_API TArray<IPCGGraphExecutionSource*> GetAllIntersectingExecutionSources(const FBoxCenterAndExtent& InBounds) const;

	/** Traverses the hierarchy associated with the given component and calls InFunc for each overlapping component. */
	PCG_API void ForAllOverlappingComponentsInHierarchy(UPCGComponent* InComponent, const TFunctionRef<void(UPCGComponent*)>& InFunc) const;

	/** Call the InFunc on all partitioned execution source in specified bounds. */
	PCG_API void ForAllIntersectingPartitionedExecutionSources(const FBoxCenterAndExtent& InBounds, TFunctionRef<void(IPCGGraphExecutionSource*)> InFunc) const;

	UE_DEPRECATED(5.8, "Use OnOriginalExecutionSourceRegistered instead")
	PCG_API void OnOriginalComponentRegistered(UPCGComponent* InComponent);
	
	UE_DEPRECATED(5.8, "Use OnOriginalExecutionSourceUnregistered instead")
	PCG_API void OnOriginalComponentUnregistered(UPCGComponent* InComponent);

	UE_DEPRECATED(5.8, "Use RegisterOrUpdateExecutionSource instead")
	PCG_API bool RegisterOrUpdatePCGComponent(UPCGComponent* InComponent, bool bDoActorMapping = true);

	UE_DEPRECATED(5.8, "Use RemapExecutionSource instead")
	PCG_API bool RemapPCGComponent(const UPCGComponent* OldComponent, UPCGComponent* NewComponent, bool bDoActorMapping);

	UE_DEPRECATED(5.8, "Use UnregisterExecutionSource instead")
	PCG_API void UnregisterPCGComponent(UPCGComponent* InComponent, bool bForce = false);

	UE_DEPRECATED(5.8, "Use GetAllRegisteredPartitionedExecutionSources instead")
	PCG_API TSet<UPCGComponent*> GetAllRegisteredPartitionedComponents() const;

	UE_DEPRECATED(5.8, "Use GetAllRegisteredExecutionSources instead")
	PCG_API TSet<UPCGComponent*> GetAllRegisteredComponents() const;

	UE_DEPRECATED(5.8, "Use GetAllIntersectingExecutionSources instead")
	PCG_API TArray<UPCGComponent*> GetAllIntersectingComponents(const FBoxCenterAndExtent& InBounds) const;

	UE_DEPRECATED(5.8, "Use ForAllIntersectingPartitionedExecutionSources instead")
	PCG_API void ForAllIntersectingPartitionedComponents(const FBoxCenterAndExtent& InBounds, TFunctionRef<void(UPCGComponent*)> InFunc) const;

	UE_DEPRECATED(5.8, "Use RefreshRuntimeGenExecutionSource instead")
	PCG_API void RefreshRuntimeGenComponent(UPCGComponent* RuntimeComponent, EPCGChangeType ChangeType = EPCGChangeType::None);

	UE_DEPRECATED(5.8, "Use RefreshAllRuntimeGenExecutionSources instead")
	PCG_API void RefreshAllRuntimeGenComponents(EPCGChangeType ChangeType = EPCGChangeType::None);

	/**
	 * Call InFunc to all partition grid cells matching 'InGridSizes' and overlapping with 'InBounds'. 'InFunc' can schedule work or execute immediately.
	 * 'InGridSizes' should be sorted in descending order. If 'bCanCreateActor' is true, it will create the partition actor at that cell if necessary.
	 */
	PCG_API FPCGTaskId ForAllOverlappingCells(
		UPCGComponent* InPCGComponent,
		const FBox& InBounds,
		const PCGHiGenGrid::FSizeArray& InGridSizes,
		bool bCanCreateActor,
		const TArray<FPCGTaskId>& Dependencies,
		TFunctionRef<FPCGTaskId(APCGPartitionActor*, const FBox&)> InFunc,
		TFunctionRef<FPCGTaskId(const FPCGGridDescriptor&, const FIntVector&, const FBox&)> InUnloadedFunc = [](const FPCGGridDescriptor&, const FIntVector&, const FBox&) { return InvalidPCGTaskId; }) const;
		
	/** Immediately cleanup the local components associated with an original component. */
	PCG_API void CleanupLocalComponentsImmediate(UPCGComponent* InOriginalComponent, bool bRemoveComponents);

	/** Retrieves a local component using grid descriptor and grid coordinates, returns nullptr if no such component is found. */
	PCG_API UPCGComponent* GetLocalComponent(const FPCGGridDescriptor& GridDescriptor, const FIntVector& CellCoords, const UPCGComponent* InOriginalComponent) const;

	/** Retrieves a registered partition actor using grid size and grid coordinates, returns nullptr if no such partition actor is found. */
	PCG_API APCGPartitionActor* GetRegisteredPCGPartitionActor(const FPCGGridDescriptor& GridDescriptor, const FIntVector& GridCoords) const;

	/** Creates a new partition actor if one does not already exist with the same grid size, coords, and generation mode. */
	PCG_API APCGPartitionActor* FindOrCreatePCGPartitionActor(const FPCGGridDescriptor& GridDescriptor, const FIntVector& GridCoords, bool bCanCreateActor = true, bool bHideFromOutliner = false) const;
	
	/** Retrieves partition actors for this original component */
	PCG_API TSet<TObjectPtr<APCGPartitionActor>> GetPCGComponentPartitionActorMappings(UPCGComponent* InComponent) const;

	PCG_API FPCGGenSourceManager* GetGenSourceManager() const;

	void OnViewportResized(FViewport* Viewport, uint32 val);

	void ResetCacheStats();
	void LogCacheStats();

#if WITH_EDITOR
public:
	/** Schedule refresh on the current or next frame */
	PCG_API FPCGTaskId ScheduleRefresh(UPCGComponent* SourceComponent, bool bForceRefresh);

	/** Immediately dirties the partition actors in the given bounds */
	PCG_API void DirtyGraph(UPCGComponent* Component, const FBox& InBounds, EPCGComponentDirtyFlag DirtyFlag);

	/** Delete serialized partition actors in the level. If 'bOnlyDeleteUnused' is true, only PAs with no graph instances will be deleted. */
	PCG_API void DeleteSerializedPartitionActors(bool bOnlyDeleteUnused, bool bOnlyChildren = false);

	/** Update the tracking on a given component. */
	PCG_API void UpdateTracking(IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGSelectionKey>* OptionalChangedKeys = nullptr);

	UE_DEPRECATED(5.8, "Use UpdateTracking instead")
	void UpdateComponentTracking(IPCGGraphExecutionSource* InExecutionSource, bool bShouldDirtyActors, const TArray<FPCGSelectionKey>* OptionalChangedKeys = nullptr) { UpdateTracking(InExecutionSource, OptionalChangedKeys); }

	/** Propagates transient state change from an original component to the relevant partition actors */
	PCG_API void PropagateEditingModeToLocalComponents(UPCGComponent* InOriginalComponent, EPCGEditorDirtyMode EditingMode);

	/** Move all resources from sub actors to a new actor */
	PCG_API void ClearPCGLink(IPCGGraphExecutionSource* InExecutionSource, const FBox& InBounds, const FPCGMoveResourceParams& InParams);

	UE_DEPRECATED(5.8, "Use the version with the move resource params.")
	/** Move all resources from sub actors to a new actor */
	PCG_API void ClearPCGLink(UPCGComponent* InComponent, const FBox& InBounds, AActor* InNewActor);

	/** If the partition grid size change, call this to empty the Partition actors map */
	PCG_API void ResetPartitionActorsMap();

	/** Builds the landscape data cache. If force build is true, then it will build the cache even if it is never serialized. */
	PCG_API void BuildLandscapeCache(bool bQuiet = false, bool bForceBuild = true);

	/** Clears the landscape data cache */
	PCG_API void ClearLandscapeCache();

	/** Will gather all the components registered, and ask for generate. */
	PCG_API void GenerateAllPCGComponents(bool bForce) const;

	/** Will gather all the components registered, and ask for cleanup. */
	PCG_API void CleanupAllPCGComponents(bool bPurge) const;

	/** Will gather all the components registered, and ask for Clear PCG Link */
	PCG_API void ClearLinkForAllPCGComponents(const FPCGMoveResourceParams& InParams) const;

	/** Notification that a Selection key needs refreshing */
	PCG_API void NotifySelectionKeyChanged(const FPCGSelectionKey& InSelectionKey, const UObject* InOriginatingChangeObject, TArrayView<const FBox> InBounds);

	/** Notify that we exited the Landscape edit mode. */
	UE_DEPRECATED(5.8, "No longer used")
	void NotifyLandscapeEditModeExited() {}

	FPCGOnAllComponentsGenerated OnAllComponentsGenerated;
	FPCGOnAllComponentsCleanedup OnAllComponentsCleanedup;
	FPCGOnAllComponentsClearedLink OnAllComponentsClearedLink;
	
	UE_DEPRECATED(5.7, "Deprecated in favor of OnPCGSourceGenerationDone, will not be notified anymore")
	FPCGOnPCGComponentGenerationDone OnPCGComponentGenerationDone;

	UE_DEPRECATED(5.8, "Deprecated in favor of OnPCGSourceUnregistered, will not be notified anymore")
	FPCGOnPCGComponentUnregistered OnPCGComponentUnregistered;

	PCG_API void CreateMissingPartitionActors();

	void OnPreviewPlatformChanged(EShaderPlatform NewShaderPlatform);

private:
	void OnPCGGraphCancelled(IPCGGraphExecutionSource* InExecutionSource);
	void OnPCGGraphStartGenerating(IPCGGraphExecutionSource* InExecutionSource);
	void OnPCGGraphGenerated(IPCGGraphExecutionSource* InExecutionSource, TOptional<FBox> OptionalExecutionBounds = {});
	void OnPCGGraphCleaned(IPCGGraphExecutionSource* InExecutionSource);

	PCG_API void CreatePartitionActorsWithinBounds(UPCGComponent* InComponent, const FBox& InBounds, const PCGHiGenGrid::FSizeArray& InGridSizes);
	PCG_API void UpdateMappingPCGComponentPartitionActor(UPCGComponent* InComponent);

	PCG_API void SetChainedDispatchToLocalComponents(bool bInChainedDispatch);

	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances);
	void OnLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld);
	void OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld);
	void OnLoadedActorRemovedFromLevelPreEvent(const TArray<AActor*>& InActors);
	void OnLoadedActorRemovedFromLevelPostEvent(const TArray<AActor*>& InActors);
#endif // WITH_EDITOR

private:
	/** When registering PAs, we might not have a PCG World Actor already registered (at runtime). In that case we'll look for a World Actor in the same level.*/
	APCGWorldActor* GetPCGWorldActorForPartitionActor(APCGPartitionActor* InActor);

	void ExecuteBeginTickActions();

	void OnGlobalComponentReregisterContextDestroyed();

	APCGWorldActor* PCGWorldActor = nullptr;
	FPCGRuntimeGenScheduler* RuntimeGenScheduler = nullptr;
	bool bHasTickedOnce = false;
	TUniquePtr<FPCGTrackingManager> TrackingManager;

	/** Functions will be executed at the beginning of the tick and then removed from this array. */
	TArray<FTickAction> BeginTickActions;

	PCG::FLock PCGWorldActorLock;

	bool bDirtyRuntimeGenExecutionSources = false;
	EPCGChangeType DirtyRuntimeGenChangeType = EPCGChangeType::None;
	ERuntimeGenRefreshReason DirtyRuntimeGenReason = ERuntimeGenRefreshReason::None;

#if WITH_EDITOR
	/** Debug transient actor spawned to display in world PCG debug visualization data */
	TWeakObjectPtr<AActor> DebugActor;

	int32 NumLevelUnloadingActors = 0;

	using FConstructionScriptSourceComponents = TMap<FName, TObjectKey<UPCGComponent>>;
	TMap<TObjectKey<AActor>, FConstructionScriptSourceComponents> PerActorConstructionScriptSourceComponents;

	PCG_API static TSet<UWorld*> DisablePartitionActorCreationForWorld;

	// Used by UPCGWorldPartitonBuilder to disable PA creation while outside of a certain scope
	static void SetDisablePartitionActorCreationForWorld(UWorld* InWorld, bool bDisable) 
	{ 
		if (bDisable)
		{
			DisablePartitionActorCreationForWorld.Add(InWorld);
		}
		else
		{
			DisablePartitionActorCreationForWorld.Remove(InWorld);
		}
	}

	static bool IsPartitionActorCreationDisabledForWorld(UWorld* InWorld)
	{
		return DisablePartitionActorCreationForWorld.Contains(InWorld);
	}
#endif

protected:
	PCG_API virtual void CancelGenerationInternal(IPCGGraphExecutionSource* Source, bool bCleanupUnusedResources) override;
};
