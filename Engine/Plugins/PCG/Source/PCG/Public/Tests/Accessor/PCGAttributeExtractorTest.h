// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/UnrealType.h"

#include "PCGAttributeExtractorTest.generated.h"

UCLASS(meta = (Hidden))
class UPCGAttributeExtractorTestObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	double DoubleValue = 0;
};

USTRUCT(meta = (Hidden))
struct FPCGAttributeExtractorTestStructDepth2
{
	GENERATED_BODY()

	UPROPERTY()
	int32 IntValue = 0;
};

USTRUCT(meta = (Hidden))
struct FPCGAttributeExtractorTestStructDepth1
{
	GENERATED_BODY()

	UPROPERTY()
	FPCGAttributeExtractorTestStructDepth2 Depth2Struct;

	UPROPERTY()
	float FloatValue = 0.0f;
};

USTRUCT(meta = (Hidden))
struct FPCGAttributeExtractorTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	FPCGAttributeExtractorTestStructDepth1 DepthStruct;

	UPROPERTY()
	TObjectPtr<UPCGAttributeExtractorTestObject> Object;
};