// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#if WITH_EDITOR
#include "Containers/Map.h"

class UPCGSettings;

namespace PCGDeterminismTests
{
	class FNativeTestRegistry
	{
	public:
		/** Allocates the static class */
		static void Create();
		/** Deallocates the static class */
		static void Destroy();

		/** Registers a test function given the StaticClass of an element's settings */
		static void RegisterTestFunction(UClass* SettingsStaticClass, TFunction<bool()> TestFunction);
		/** Deregisters a test function given the StaticClass of an element's settings */
		static void DeregisterTestFunction(const UClass* SettingsStaticClass);
		/** Gets a native test function, given an element's UPCGSettings */
		static PCG_API TFunction<bool()> GetNativeTestFunction(const UPCGSettings* PCGSettings);

	private:
		FNativeTestRegistry() = default;
		FNativeTestRegistry(const FNativeTestRegistry&) = delete;
		FNativeTestRegistry& operator=(const FNativeTestRegistry&) = delete;
		FNativeTestRegistry(FNativeTestRegistry&&) = delete;
		FNativeTestRegistry& operator=(FNativeTestRegistry&&) = delete;

		static FNativeTestRegistry* RegistryPtr;
		TMap<UClass*, TFunction<bool()>> NativeTestMapping;
	};
}
#endif