// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"

#include "PCGDeterminismSettings.generated.h"

class UPCGDeterminismTestBlueprintBase;

USTRUCT(BlueprintType)
struct FPCGDeterminismSettings
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Determinism)
	bool bNativeTests = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Determinism)
	bool bUseBlueprintDeterminismTest = false;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Determinism, meta = (EditCondition = "bUseBlueprintDeterminismTest"))
	TSubclassOf<UPCGDeterminismTestBlueprintBase> DeterminismTestBlueprint;
};
