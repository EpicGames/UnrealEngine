// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "PCGDeterminismTestBlueprintBase.generated.h"

#define UE_API PCG_API

struct FDeterminismTestResult;

class UPCGNode;

UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, hidecategories = (Object))
class UPCGDeterminismTestBlueprintBase : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintNativeEvent, BlueprintPure, CallInEditor, Category = Determinism)
	UE_API void ExecuteTest(const UPCGNode* InPCGNode, UPARAM(ref)FDeterminismTestResult& InOutTestResult);
};

#undef UE_API
