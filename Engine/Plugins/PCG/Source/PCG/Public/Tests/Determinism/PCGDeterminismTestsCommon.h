// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Tests/PCGTestsCommon.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include "PCGDeterminismTestsCommon.generated.h"

class FPCGMetadataAttributeBase;

class UPCGComponent;
class UPCGData;
class UPCGMetadata;
class UPCGSpatialData;
class UPrimitiveComponent;
class USplineComponent;
struct FPCGDataCollection;

// This will include multiple values of different meanings, but we use an enum to facilitate data passing
UENUM()
enum class EDeterminismLevel : uint8
{
	None = 0u,
	NoDeterminism = None,
	Basic,
	OrderOrthogonal,
	OrderConsistent,
	OrderIndependent,
	Deterministic = OrderIndependent
};

USTRUCT(BlueprintType)
struct FDeterminismTestResult
{
	GENERATED_BODY()

	/** The test result index. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Determinism)
	int64 Index = INDEX_NONE;

	/** The node's title */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Determinism)
	FName TestResultTitle = TEXT("Untitled");

	/** The node's name */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Determinism)
	FString TestResultName = TEXT("Unnamed");

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Determinism)
	int32 Seed = -1;

	/** BitFlags for which data types were tested */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Determinism)
	FPCGDataTypeIdentifier DataTypesTested = EPCGDataType::None;

	/** A mapping of [test names : test results] */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Determinism)
	TMap<FName, EDeterminismLevel> TestResults;

	/** A mapping of [test name : additional details] */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Determinism)
	TArray<FString> AdditionalDetails;

	/** T/F whether a flag has been raised on this node's tests (for UI) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Determinism)
	bool bFlagRaised = false;

	// TODO: Chrono Duration of how long the tests took
};

namespace PCGDeterminismTests
{
	namespace Defaults
	{
		constexpr static int32 NumPointsToGenerate = 1;
		constexpr static int32 NumTestPointsToGenerate = 100;
		constexpr static int32 NumPolyLinePointsToGenerate = 6;
		constexpr static int32 NumTestPolyLinePointsToGenerate = 6;
		constexpr static int32 NumSamplingStepsPerDimension = 15;
		constexpr static int32 NumTestInputsPerPin = 2;

		constexpr static FVector::FReal SmallDistance = 50.0;
		constexpr static FVector::FReal MediumDistance = 200.0;
		constexpr static FVector::FReal LargeDistance = 500.0;
		/** This will expand the testing volume by an increase of this ratio - ex. 0.25 == 125% */
		constexpr static FVector::FReal TestingVolumeExpandByFactor = 0.25;

		const FVector SmallVector = FVector::OneVector * SmallDistance;
		const FVector MediumVector = FVector::OneVector * MediumDistance;
		const FVector LargeVector = FVector::OneVector * LargeDistance;

		const FName TestPinName = TEXT("Test");
		const FName GraphResultName = TEXT("Graph Result");
	}

	constexpr static EPCGDataType TestableDataTypes[6] =
	{
		EPCGDataType::None,
		EPCGDataType::Point,
		EPCGDataType::Volume,
		EPCGDataType::PolyLine,
		EPCGDataType::Primitive,
		EPCGDataType::Landscape
	};

	/** A default delegate to report an unset test */
	bool LogInvalidTest(const UPCGNode* InPCGNode, const FName& TestName, FDeterminismTestResult& OutResult);

	typedef TFunction<bool(const UPCGNode* InPCGNode, const FName& TestName, FDeterminismTestResult& OutResult)> TestFunction;

	struct FNodeTestInfo
	{
		FNodeTestInfo(FText Label, TestFunction Delegate, float LabelWidth = 140.f) :
			TestLabel(Label),
			TestName(Label.ToString()),
			TestDelegate(Delegate),
			TestLabelWidth(LabelWidth) {}

		FText TestLabel = NSLOCTEXT("PCGDeterminism", "UnnamedTest", "Unnamed Test");
		const FName TestName;
		TestFunction TestDelegate = LogInvalidTest;
		float TestLabelWidth = 140.f;
	};

	struct FNodeAndOptions
	{
		explicit FNodeAndOptions(const UPCGNode* PCGNode, const int32 Seed, bool bMultipleOptionsPerPin) :
			PCGNode(PCGNode),
			Seed(Seed),
			bMultipleOptionsPerPin(bMultipleOptionsPerPin) {}

		const UPCGNode* PCGNode;
		const int32 Seed;
		const bool bMultipleOptionsPerPin;
		TArray<TArray<FPCGDataTypeIdentifier>> BaseOptionsByPin;
	};

	/** Validates if a PCGNode is deterministic */
	PCG_API void RunDeterminismTest(const UPCGNode* InPCGNode, FDeterminismTestResult& OutResult, const FNodeTestInfo& TestToRun);

#if WITH_EDITOR
	/** Validates if a PCGGraph is deterministic */
	PCG_API void RunDeterminismTest(const UPCGGraph* InPCGGraph, const IPCGGraphExecutionSource* InSource, FDeterminismTestResult& OutResult);
#endif

	/** Validates all the generic determinism tests for any given node */
	PCG_API bool RunBasicTestSuite(const UPCGNode* InPCGNode, const FName& TestName, FDeterminismTestResult& OutResult);

	/** Validates the various levels of order independence for any given node */
	PCG_API bool RunOrderIndependenceSuite(const UPCGNode* InPCGNode, const FName& TestName, FDeterminismTestResult& OutResult);

	/** Validates minimal node determinism against the same single test data */
	bool RunBasicSelfTest(const FNodeAndOptions& NodeAndOptions);
	/** Validates minimal node determinism against two identical single test data */
	bool RunBasicCopiedSelfTest(const FNodeAndOptions& NodeAndOptions);

	/** Conducts tests on all permutations and determines the highest level of determinism */
	EDeterminismLevel GetHighestDeterminismLevel(const FNodeAndOptions& NodeAndOptions,
		int32 NumInputsPerPin = Defaults::NumTestInputsPerPin,
		EDeterminismLevel MaxLevel = EDeterminismLevel::OrderIndependent);
	/** Conducts simplified shuffled input tests and determines the highest level of determinism */
	EDeterminismLevel GetHighestDeterminismLevelSimpleShuffleInput(const FNodeAndOptions& NodeAndOptions,
		int32 NumInputsPerPin = Defaults::NumTestInputsPerPin,
		EDeterminismLevel MaxLevel = EDeterminismLevel::OrderIndependent);

	/** Adds input data to test data, based on input pins' allowed types */
	void AddRandomizedInputData(PCGTestsCommon::FTestData& TestData, FPCGDataTypeIdentifier DataType, const FName& PinName = Defaults::TestPinName);

	/** Helper for adding PointData with a single Point to an InputData */
	void AddSinglePointInputData(FPCGDataCollection& InputData, const FVector& Location, const FName& PinName = Defaults::TestPinName);
	/** Helper for adding PointData with multiple Points to an InputData */
	void AddMultiplePointsInputData(FPCGDataCollection& InputData, const TArray<FPCGPoint>& Points, const FName& PinName = Defaults::TestPinName);
	/** Helper for adding VolumeData to an InputData */
	void AddVolumeInputData(FPCGDataCollection& InputData, const FVector& Location, const FVector& HalfSize, const FVector& VoxelSize, const FName& PinName = Defaults::TestPinName);
	/** Helper for adding PolyLineData to an InputData */
	void AddPolyLineInputData(FPCGDataCollection& InputData, USplineComponent* SplineComponent, const FName& PinName = Defaults::TestPinName);
	/** Helper for adding PrimitiveData to an InputData */
	void AddPrimitiveInputData(FPCGDataCollection& InputData, UPrimitiveComponent* SplineComponent, const FVector& VoxelSize, const FName& PinName = Defaults::TestPinName);
	/** Helper for adding LandscapeData to an InputData */
	void AddLandscapeInputData(FPCGDataCollection& InputData);

	/** Adds randomized PointData with a single Point */
	void AddRandomizedSinglePointInputData(PCGTestsCommon::FTestData& TestData, int32 PointNum = Defaults::NumPointsToGenerate, const FName& PinName = Defaults::TestPinName);
	/** Adds randomized PointData with multiple Points */
	void AddRandomizedMultiplePointInputData(PCGTestsCommon::FTestData& TestData, int32 PointNum = Defaults::NumPointsToGenerate, const FName& PinName = Defaults::TestPinName);
	/** Adds randomized Volume Spatial Data */
	void AddRandomizedVolumeInputData(PCGTestsCommon::FTestData& TestData, const FName& PinName = Defaults::TestPinName);
	/** Adds randomized Surface Spatial Data */
	void AddRandomizedSurfaceInputData(PCGTestsCommon::FTestData& TestData, const FName& PinName = Defaults::TestPinName);
	/** Adds randomized PolyLine Spatial Data */
	void AddRandomizedPolyLineInputData(PCGTestsCommon::FTestData& TestData, int32 PointNum = Defaults::NumPolyLinePointsToGenerate, const FName& PinName = Defaults::TestPinName);
	/** Adds randomized Primitive Spatial Data */
	void AddRandomizedPrimitiveInputData(PCGTestsCommon::FTestData& TestData, const FName& PinName = Defaults::TestPinName);
	/** Adds randomized Landscape Spatial Data */
	void AddRandomizedLandscapeInputData(PCGTestsCommon::FTestData& TestData, const FName& PinName = Defaults::TestPinName);

	/** Validates whether two DataCollection objects are exactly identical */
	bool DataCollectionsAreIdentical(const FPCGDataCollection& FirstCollection, const FPCGDataCollection& SecondCollection);
	/** Validates whether two DataCollection objects have data in order relative to their inputs */
	bool DataCollectionsAreConsistent(const FPCGDataCollection& FirstCollection, const FPCGDataCollection& SecondCollection, int32 NumInputs);
	/** Validates whether two DataCollection objects contain all the same data */
	bool DataCollectionsContainSameData(const FPCGDataCollection& FirstCollection, const FPCGDataCollection& SecondCollection);
	/** Validates the data collections match and delivers matched indices and offsets */
	bool DataCollectionsMatch(const FPCGDataCollection& FirstCollection,
		const FPCGDataCollection& SecondCollection,
		TArray<int32>& OutIndexOffsets);
	/** Validates whether the internal data within the two DataCollection objects are consistent */
	bool InternalDataMatches(const UPCGData* FirstData, const UPCGData* SecondData, TArray<int32>& OutIndexOffsets);

	/** Validates whether two SpatialData objects are identical */
	bool SpatialDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData);
	/** Validates whether two PointData objects are identical */
	bool PointDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData);
	/** Validates whether two VolumeData objects are identical */
	bool VolumeDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData);
	/** Validates whether two SurfaceData objects are identical */
	bool SurfaceDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData);
	/** Validates whether two PolyLineData objects are identical */
	bool PolyLineDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData);
	/** Validates whether two PrimitiveData objects are identical */
	bool PrimitiveDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData);
	/** Validates whether two SpatialData objects are identical via Point Sampling */
	bool SampledSpatialDataIsIdentical(const UPCGSpatialData* FirstSpatialData, const UPCGSpatialData* SecondSpatialData);
	/** Validates the basics of SpatialData are identical */
	bool SpatialBasicsAreIdentical(const UPCGData* FirstData, const UPCGData* SecondData);

	/** Validates whether two SpatialData objects are consistent */
	bool SpatialDataIsConsistent(const UPCGData* FirstData, const UPCGData* SecondData);
	/** Validates whether two PointData objects are consistent */
	bool PointDataIsConsistent(const UPCGData* FirstData, const UPCGData* SecondData);

	/** Validates whether two SpatialData objects contain the same spatial data */
	bool SpatialDataIsOrthogonal(const UPCGData* FirstData, const UPCGData* SecondData);
	/** Validates whether two PointData objects contain the same points */
	bool PointDataIsOrthogonal(const UPCGData* FirstData, const UPCGData* SecondData);

	/** Validates whether two ParamData objects are identical */
	bool ParamDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData);

	/** Validates whether two Metadata objects are identical */
	bool MetadataIsIdentical(const UPCGMetadata* FirstMetadata, const UPCGMetadata* SecondMetadata);

	/** A catch function for unimplemented comparisons */
	bool ComparisonIsUnimplemented(const UPCGData* FirstData, const UPCGData* SecondData);
	/** Updates the test result output to reflect that the permutation limit has been exceeded */
	void UpdateTestResultForOverPermutationLimitError(FDeterminismTestResult& OutResult);

	/** Determines if a PCG DataType is a type that should be compared */
	bool DataTypeIsComparable(FPCGDataTypeIdentifier DataType);
	/** Validates PCGData and if its DataType is comparable */
	bool DataIsComparable(const UPCGData* Data);
	/** Validates that the data contains a shuffle-able array of data */
	bool DataCanBeShuffled(const UPCGData* Data);

	/** Randomizes the order of InputData collections */
	void ShuffleInputOrder(PCGTestsCommon::FTestData& TestData);
	/** Randomizes the order of OutputData collections */
	void ShuffleOutputOrder(PCGTestsCommon::FTestData& TestData);
	/** Randomizes the order of all internal data */
	void ShuffleAllInternalData(PCGTestsCommon::FTestData& TestData);
	/** Shifts the order of InputData collections */
	void ShiftInputOrder(PCGTestsCommon::FTestData& TestData, int32 NumShifts = 1);

	/** Gets an array of the union between an InputPin's allowed types and testable types */
	TArray<FPCGDataTypeIdentifier> FilterTestableDataTypes(FPCGDataTypeIdentifier AllowedDataTypes, int32 NumMultipleInputs = 1);
	/** Given input pins, updates an array with a permutation base of options */
	void RetrieveBaseOptionsPerPin(TArray<TArray<FPCGDataTypeIdentifier>>& InBaseOptionsArray,
		const TArray<TObjectPtr<UPCGPin>>& InputPins,
		FPCGDataTypeIdentifier& OutDataTypesTested,
		int32 NumMultipleInputs = 1);
	/** Gets the number of permutations from a base set of options */
	int32 GetNumPermutations(const TArray<TArray<FPCGDataTypeIdentifier>>& BaseOptionsArray);
	/** Gets the index of a pin's permutation based on the permutation iteration */
	FPCGDataTypeIdentifier GetPermutation(int32 PermutationIteration, int32 PinIndex, const TArray<TArray<FPCGDataTypeIdentifier>>& BaseOptionsPerPin);

	/** Helper to update a tests' outgoing results */
	void UpdateTestResults(FName TestName, FDeterminismTestResult& OutResult, EDeterminismLevel DeterminismLevel);

	/** Gets a comparison function to compare two data objects */
	TFunction<bool(const UPCGData*, const UPCGData*)> GetDataCompareFunction(FPCGDataTypeIdentifier DataType, EDeterminismLevel DeterminismLevel);
	/** Gets a comparison function to compare two data collection entry basics */
	TFunction<bool(const UPCGData*, const UPCGData*)> GetDataCollectionCompareFunction(FPCGDataTypeIdentifier DataType);

	/** Executes the element with given test data */
	void ExecuteWithTestData(PCGTestsCommon::FTestData& TestData, const UPCGNode* PCGNode);
	/** Executes the element with given test data against itself */
	void ExecuteWithSameTestData(const PCGTestsCommon::FTestData& TestData, const UPCGNode* PCGNode, FPCGDataCollection& OutFirstOutputData, FPCGDataCollection& OutSecondOutputData);
	/** Executes the element with given test data against itself, with the same element */
	void ExecuteWithSameTestDataSameElement(const PCGTestsCommon::FTestData& TestData, const UPCGNode* PCGNode, FPCGDataCollection& OutFirstOutputData, FPCGDataCollection& OutSecondOutputData);
	/** Execute the elements for each valid input and compare if all the outputs are at least orthogonally deterministic */
	bool ExecutionIsDeterministic(PCGTestsCommon::FTestData& FirstTestData, PCGTestsCommon::FTestData& SecondTestData, const UPCGNode* PCGNode = nullptr);
	/** Execute the elements for each valid input, convert output to points, and compare if all the outputs are at least orthogonally deterministic */
	bool ExecutionIsConcretelyDeterministic(PCGTestsCommon::FTestData& FirstTestData, PCGTestsCommon::FTestData& SecondTestData, const UPCGNode* PCGNode = nullptr);
	/** Execute the same element twice compare if all the outputs are at least orthogonally deterministic */
	bool ExecutionIsDeterministicSameData(const PCGTestsCommon::FTestData& TestData, const UPCGNode* PCGNode = nullptr);

	/** Validates whether both UPCGData can be cast to a specified subclass */
	template<typename DataType>
	bool BothDataCastsToDataType(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		check(FirstData && SecondData);

		return Cast<const DataType>(FirstData) != nullptr && Cast<const DataType>(SecondData) != nullptr;
	}

	template<typename MetadataAttributeType>
	bool MetadataAttributesAreEqual(const FPCGMetadataAttributeBase* FirstAttributeBase, const FPCGMetadataAttributeBase* SecondAttributeBase, PCGMetadataValueKey ValueKey)
	{
		check(FirstAttributeBase && SecondAttributeBase);

		if constexpr (std::is_same_v<FTransform, MetadataAttributeType>)
		{
			const FTransform FirstTypedAttribute = static_cast<const FPCGMetadataAttribute<FTransform>*>(FirstAttributeBase)->GetValue(ValueKey);
			const FTransform SecondTypedAttribute = static_cast<const FPCGMetadataAttribute<FTransform>*>(SecondAttributeBase)->GetValue(ValueKey);

			return FirstTypedAttribute.Equals(SecondTypedAttribute);
		}
		else
		{
			const MetadataAttributeType FirstTypedAttribute = static_cast<const FPCGMetadataAttribute<MetadataAttributeType>*>(FirstAttributeBase)->GetValue(ValueKey);
			const MetadataAttributeType SecondTypedAttribute = static_cast<const FPCGMetadataAttribute<MetadataAttributeType>*>(SecondAttributeBase)->GetValue(ValueKey);

			return (FirstTypedAttribute == SecondTypedAttribute);
		}
	}

	namespace Defaults
	{
		static const FNodeTestInfo DeterminismBasicTestInfo = FNodeTestInfo(NSLOCTEXT("PCGDeterminism", "BasicTest", "Basic Test"), RunBasicTestSuite, 90.f);
		static const FNodeTestInfo DeterminismOrderIndependenceInfo = FNodeTestInfo(NSLOCTEXT("PCGDeterminism", "OrderIndependenceTest", "Order Independence"), RunOrderIndependenceSuite, 140.f);
	}
}
