// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#if WITH_EDITOR

#include "HAL/Platform.h"

namespace PCGDeterminismTests
{
	namespace DifferenceElement
	{
		bool RunTestSuite();
	}
}

#endif