// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "PCGGraph.h"
#include "PCGPoint.h"
#include "PCGSettings.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPointArrayData.h"
#include "Elements/PCGCreateAttribute.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/PCGMetadataAttributeTraits.h"

#include "GameFramework/Actor.h"
#include "Misc/AutomationTest.h"

#define UE_API PCG_API

class ALandscape;
class IPCGElement;
class UPCGComponent;
class UPCGMetadata;
class UPCGNode;
class UPCGParamData;
class UPCGPolyLineData;
class UPCGPrimitiveData;
class UPCGSurfaceData;
class UPCGVolumeData;
struct FPCGContext;
struct FPCGPinProperties;

namespace PCGTestsCommon
{
	constexpr EAutomationTestFlags TestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;

	/** If you are using a FTestData, you can use FTestData::InitializeTestContext. Initialize a context and set the number of tasks available to 1. */
	TUniquePtr<FPCGContext> InitializeTestContext(IPCGElement* InElement, const FPCGDataCollection& InputData, UPCGComponent* InSourceComponent, const UPCGNode* InNode);

	struct FTestData
	{
		UE_API explicit FTestData(int32 Seed = 42, UPCGSettings* DefaultSettings = nullptr, TSubclassOf<AActor> ActorClass = AActor::StaticClass());
		UE_API ~FTestData();

		UE_API void Reset(UPCGSettings* InSettings = nullptr);

		/** Initialize a context and set the number of tasks available to 1. */
		UE_API TUniquePtr<FPCGContext> InitializeTestContext(const UPCGNode* InNode = nullptr) const;

		UE_API void SetCurrentGenerationTask(FPCGTaskId InTaskId);

		AActor* TestActor;
		UPCGComponent* TestPCGComponent;
		FPCGDataCollection InputData;
		FPCGDataCollection OutputData;
		UPCGSettings* Settings;
		int32 Seed;
		FRandomStream RandomStream;
	};

#if WITH_EDITOR
	/** Spawn a small flat landscape (Z=0 everywhere) in the given world. */
	ALandscape* CreateFlatLandscape(UWorld* World, const FTransform& Transform = FTransform::Identity);

	/** Spawn a landscape with layered Perlin noise (hills, medium detail, small bumps). Deterministic for a given Seed. */
	ALandscape* CreatePerlinLandscape(UWorld* World, const FTransform& Transform = FTransform::Identity, int32 Seed = 0);

	/** Add a CreatePoints node to a graph with the given transforms (world space). */
	UPCGNode* AddCreatePointsNode(UPCGGraph* Graph, TArrayView<const FTransform> Transforms);

	/** Add a CreateAttributeSet node to a graph. Sets the type and value from the C++ type. */
	template <typename T>
	UPCGNode* AddCreateAttributeSetNode(UPCGGraph* Graph, FName AttributeName, const T& Value)
	{
		UPCGCreateAttributeSetSettings* Settings = nullptr;
		UPCGNode* Node = Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(Settings);
		Settings->OutputTarget.SetAttributeName(AttributeName);

		FPCGMetadataTypesConstantStruct& Attr = Settings->AttributeTypes;
		if constexpr (std::is_same_v<T, bool>)            { Attr.Type = EPCGMetadataTypes::Boolean;    Attr.BoolValue = Value; }
		else if constexpr (std::is_same_v<T, int32>)      { Attr.Type = EPCGMetadataTypes::Integer32;  Attr.Int32Value = Value; }
		else if constexpr (std::is_same_v<T, int64>)      { Attr.Type = EPCGMetadataTypes::Integer64;  Attr.IntValue = Value; }
		else if constexpr (std::is_same_v<T, float>)      { Attr.Type = EPCGMetadataTypes::Float;      Attr.FloatValue = Value; }
		else if constexpr (std::is_same_v<T, double>)     { Attr.Type = EPCGMetadataTypes::Double;     Attr.DoubleValue = Value; }
		else if constexpr (std::is_same_v<T, FVector2D>)  { Attr.Type = EPCGMetadataTypes::Vector2;    Attr.Vector2Value = Value; }
		else if constexpr (std::is_same_v<T, FVector>)    { Attr.Type = EPCGMetadataTypes::Vector;     Attr.VectorValue = Value; }
		else if constexpr (std::is_same_v<T, FVector4>)   { Attr.Type = EPCGMetadataTypes::Vector4;    Attr.Vector4Value = Value; }
		else if constexpr (std::is_same_v<T, FQuat>)      { Attr.Type = EPCGMetadataTypes::Quaternion; Attr.QuatValue = Value; }
		else if constexpr (std::is_same_v<T, FRotator>)   { Attr.Type = EPCGMetadataTypes::Rotator;    Attr.RotatorValue = Value; }
		else if constexpr (std::is_same_v<T, FTransform>) { Attr.Type = EPCGMetadataTypes::Transform;  Attr.TransformValue = Value; }
		else if constexpr (std::is_same_v<T, FString>)    { Attr.Type = EPCGMetadataTypes::String;     Attr.StringValue = Value; }
		else if constexpr (std::is_same_v<T, FName>)      { Attr.Type = EPCGMetadataTypes::Name;       Attr.NameValue = Value; }
		else { static_assert(sizeof(T) == 0, "Unsupported type for AddCreateAttributeSetNode"); }

		return Node;
	}
#endif // WITH_EDITOR

	AActor* CreateTemporaryActor();
	UPCGParamData* CreateEmptyParamData();

	UPCGBasePointData* CreateEmptyBasePointData();

	/** Creates a PointData with a single point at the origin */
	template <typename PointDataType = UPCGPointData, std::enable_if_t<std::is_base_of_v<UPCGBasePointData, PointDataType>, int> = 0>
	PointDataType* CreateEmptyPointData()
	{
		return NewObject<PointDataType>();
	}

	UPCGBasePointData* CreateBasePointData();

	template <typename PointDataType = UPCGPointData, std::enable_if_t<std::is_base_of_v<UPCGBasePointData, PointDataType>, int> = 0>
	PointDataType* CreatePointData()
	{
		PointDataType* SinglePointData = CreateEmptyPointData<PointDataType>();

		check(SinglePointData);

		SinglePointData->SetNumPoints(1);

		return SinglePointData;
	}

	UPCGBasePointData* CreateBasePointData(const FVector& InLocation);

	template <typename PointDataType = UPCGPointData, std::enable_if_t<std::is_base_of_v<UPCGBasePointData, PointDataType>, int> = 0>
	PointDataType* CreatePointData(const FVector& InLocation)
	{
		PointDataType* SinglePointData = CreatePointData<PointDataType>();

		check(SinglePointData);
		check(SinglePointData->GetNumPoints() == 1);

		TPCGValueRange<FTransform> TransformRange = SinglePointData->GetTransformValueRange();
		
		TransformRange[0].SetLocation(InLocation);

		return SinglePointData;
	}

	UPCGBasePointData* CreateRandomBasePointData(int32 PointCount, int32 Seed, bool bRandomDensity = false);

	/** Creates a point data with PointCount many points, and randomizes the Transform, Color, and Density */
	template <typename PointDataType = UPCGPointData, std::enable_if_t<std::is_base_of_v<UPCGBasePointData, PointDataType>, int> = 0>
	PointDataType* CreateRandomPointData(int32 PointCount, int32 Seed, bool bRandomDensity = false)
	{
		TObjectPtr<PointDataType> PointData = PCGTestsCommon::CreateEmptyPointData<PointDataType>();
		
		PointData->SetNumPoints(PointCount);
		
		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		TPCGValueRange<int32> SeedRange = PointData->GetSeedValueRange();
		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();
		TPCGValueRange<FVector4> ColorRange = PointData->GetColorValueRange();

		FRandomStream RandomSource(Seed);
		for (int I = 0; I < PointCount; ++I)
		{
			FQuat Rotation(FRotator(RandomSource.FRandRange(0.f, 360.f)).Quaternion());
			FVector Scale(RandomSource.VRand());
			FVector Location(RandomSource.VRand());

			TransformRange[I] = FTransform(Rotation, Location, Scale);
			SeedRange[I] = I;
			DensityRange[I] = bRandomDensity ? RandomSource.GetFraction() : 1.0f;
			ColorRange[I] = RandomSource.VRand();
		}

		return PointData;
	}
		
	UPCGPolyLineData* CreatePolyLineData();
	UPCGSurfaceData* CreateSurfaceData();
	UPCGVolumeData* CreateVolumeData(const FBox& InBounds = FBox::BuildAABB(FVector::ZeroVector, FVector::OneVector * 100));
	UPCGPrimitiveData* CreatePrimitiveData();

	TArray<FPCGDataCollection> GenerateAllowedData(const FPCGPinProperties& PinProperties);

	/** Validates that two Spatial Points are identical */
	bool PointsAreIdentical(const FPCGPoint& FirstPoint, const FPCGPoint& SecondPoint);

#if WITH_EDITOR
	/**
	 * Verifies DataCollection contains exactly one item, then calls Validator on it.
	 * Returns false (with a logged failure) if the count check fails, otherwise returns
	 * whatever Validator returns.
	 */
	bool ValidateSingleTaggedData(FAutomationTestBase* Test, const FPCGDataCollection& DataCollection, TFunctionRef<bool(const FPCGTaggedData&)> Validator);

	/** Per-entry comparator for CompareDataCollections. Index is the TaggedData index. */
	using FTaggedDataComparator = TFunctionRef<bool(FAutomationTestBase* Test, int32 Index, const FPCGTaggedData& Expected, const FPCGTaggedData& Actual)>;

	/**
	 * Compare two data collections structurally (same tagged data count, matching data types per entry),
	 * then delegate per-entry content comparison to InComparator.
	 */
	bool CompareDataCollections(FAutomationTestBase* Test, const FPCGDataCollection& Expected, const FPCGDataCollection& Actual, FTaggedDataComparator InComparator);

	/** Convenience overload: compares data collections assuming all entries are point data. */
	bool ComparePointDataCollections(FAutomationTestBase* Test, const FPCGDataCollection& Expected, const FPCGDataCollection& Actual);

	/** Compare two UPCGBasePointData objects: native point properties and metadata attributes. */
	bool CompareBasePointData(FAutomationTestBase* Test, const UPCGBasePointData* Expected, const UPCGBasePointData* Actual);

	/** Compare metadata attributes. Checks attribute names, types, and per-entry values. */
	bool CompareMetadata(FAutomationTestBase* Test, const UPCGMetadata* Expected, const UPCGMetadata* Actual, int32 NumEntries);
#endif // WITH_EDITOR

	/** Generates settings based upon a UPCGSettings subclass */
	template<typename SettingsType>
	SettingsType* GenerateSettings(FTestData& TestData, TFunction<void(FTestData&)> ExtraSettingsDelegate = nullptr)
	{
		SettingsType* TypedSettings = NewObject<SettingsType>();
		check(TypedSettings);

		TestData.Settings = TypedSettings;
		TestData.Settings->Seed = TestData.Seed;

		TestData.InputData.TaggedData.Emplace_GetRef().Data = TestData.Settings;
		TestData.InputData.TaggedData.Last().Pin = FName(TEXT("Settings"));

		if (ExtraSettingsDelegate)
		{
			ExtraSettingsDelegate(TestData);
		}

		return TypedSettings;
	}

	template <class T, class... Ts>
	struct TIsAny : std::disjunction<std::is_same<T, Ts>...> {};

	// Numerical
	template <typename T, typename std::enable_if_t<std::is_arithmetic_v<T>, bool> = true>
	T GenerateRandomValue(FRandomStream& RandomStream)
	{
		if constexpr (std::is_same_v<bool, T>)
		{
			return RandomStream.FRand() > 0.5;
		}
		else
		{
			return T(RandomStream.FRandRange(-50.0, 50.0));
		}
	}

	 // Vectors
	template <typename T, typename std::enable_if_t<TIsAny<T, FVector2D, FVector, FVector4>::value, bool> = true>
	T GenerateRandomValue(FRandomStream& RandomStream)
	{
		return T(RandomStream.VRand());
	}

	 // Rotators/Quat
	template <typename T, typename std::enable_if_t<TIsAny<T, FRotator, FQuat>::value, bool> = true>
	T GenerateRandomValue(FRandomStream& RandomStream)
	{
		return T::MakeFromEuler(RandomStream.VRand() * 360.0);
	}

	 // String/FName
	template <typename T, typename std::enable_if_t<TIsAny<T, FName, FString>::value, bool> = true>
	T GenerateRandomValue(FRandomStream& RandomStream)
	{
		static constexpr const TCHAR* Dictionary[] = { TEXT("Foo"), TEXT("Bar"), TEXT("PCG"), TEXT("YOLO"), TEXT("Bla") };
		constexpr int32 NumWords = static_cast<int32>(sizeof(Dictionary) / sizeof(const TCHAR*));
		constexpr int32 MaxNumWords = 5;
		TStringBuilder<32> Builder;

		for (uint32 i = 0; i < (RandomStream.GetUnsignedInt() % (MaxNumWords - 1) + 1); ++i)
		{
			if (i != 0)
			{
				Builder += TEXT("_");
			}

			Builder += Dictionary[RandomStream.GetUnsignedInt() % NumWords];
		}

		return Builder.ToString();
	}

	// SoftObjectPath
	template <typename T, typename std::enable_if_t<TIsAny<T, FSoftObjectPath>::value, bool> = true>
	T GenerateRandomValue(FRandomStream& RandomStream)
	{
		static const FSoftObjectPath Dictionary[] = 
		{ 
			FSoftObjectPath(TEXT("/PCG/DebugObjects/PCG_Cube.PCG_Cube")),
			FSoftObjectPath(TEXT("Material'/PCG/DebugObjects/PCG_DebugMaterial.PCG_DebugMaterial'"))
		};

		constexpr int32 NumWords = static_cast<int32>(sizeof(Dictionary) / sizeof(FSoftObjectPath));
		return Dictionary[RandomStream.GetUnsignedInt() % NumWords];
	}

	// FSoftClassPath
	template <typename T, typename std::enable_if_t<TIsAny<T, FSoftClassPath>::value, bool> = true>
	T GenerateRandomValue(FRandomStream& RandomStream)
	{
		static const FSoftClassPath Dictionary[] =
		{
			UPCGData::StaticClass(),
			UPCGPointData::StaticClass(),
			UPCGMetadata::StaticClass(),
		};

		constexpr int32 NumWords = static_cast<int32>(sizeof(Dictionary) / sizeof(FSoftClassPath));
		return Dictionary[RandomStream.GetUnsignedInt() % NumWords];
	}

	template <typename T>
	void CreateAndFillRandomAttribute(UPCGData* InData, const FName AttributeName, T DefaultValue, const int32 NumValues, int32 Seed = 42, const bool* bForceAllowInterpolation = nullptr) 
	{
		check(InData);

		UPCGMetadata* Metadata = InData->MutableMetadata();
		UPCGBasePointData* PointData = Cast<UPCGBasePointData>(InData);
		TPCGValueRange<PCGMetadataEntryKey> MetadataEntryRange = PointData ? PointData->GetMetadataEntryValueRange() : TPCGValueRange<PCGMetadataEntryKey>();

		check(Metadata && (!PointData || NumValues == 0 || NumValues == PointData->GetNumPoints()));

		const bool bAllowInterpolation = bForceAllowInterpolation ? *bForceAllowInterpolation : PCG::Private::MetadataTraits<T>::CanInterpolate;
		FPCGMetadataAttribute<T>* NewAttribute = Metadata->CreateAttribute<T>(AttributeName, /*DefaultValue=*/std::move(DefaultValue), /*bAllowInterpolation=*/bAllowInterpolation, /*bOverrideParent=*/false);
		check(NewAttribute);

		FRandomStream RandomStream(Seed);

		for (int i = 0; i < NumValues; ++i)
		{
			PCGMetadataEntryKey EntryKey;
			if (PointData)
			{
				Metadata->InitializeOnSet(MetadataEntryRange[i]);
				EntryKey = MetadataEntryRange[i];
			}
			else
			{
				EntryKey = Metadata->AddEntry();
			}

			NewAttribute->SetValue(EntryKey, GenerateRandomValue<T>(RandomStream));
		}
	}
}

class FPCGTestBaseClass : public FAutomationTestBase
{
public:
	using FAutomationTestBase::FAutomationTestBase;
	/** Hook to expose private test function publicly */
	bool RunPCGTest(const FString& Parameters) { return RunTest(Parameters); }

protected:
	/** Generates all valid input combinations */
	bool SmokeTestAnyValidInput(UPCGSettings* InSettings, TFunction<bool(const FPCGDataCollection&, const FPCGDataCollection&)> ValidationFn = TFunction<bool(const FPCGDataCollection&, const FPCGDataCollection&)>());
};

#undef UE_API
