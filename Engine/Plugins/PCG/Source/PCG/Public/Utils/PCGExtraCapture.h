// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "UObject/WeakObjectPtr.h" // IWYU pragma: keep

#define UE_API PCG_API

class UPCGComponent;
class UPCGNode;
enum class EPCGExecutionPhase : uint8;
struct FPCGContext;
struct FPCGStack;

class IPCGElement;

namespace PCGUtils
{
#if PCG_PROFILING_ENABLED

	/** Timing data for one PCG element execution. */
	struct FCallTime
	{
		double PrepareDataStartTime = MAX_dbl;
		double PrepareDataTime = 0.0;
		double PrepareDataEndTime = 0.0;
		double ExecutionStartTime = MAX_dbl;
		double ExecutionTime = 0.0;
		double ExecutionEndTime = 0.0;
		double MinExecutionFrameTime = MAX_dbl;
		double MaxExecutionFrameTime = 0.0;
		double PostExecuteTime = 0.0;

		int32 ExecutionFrameCount = 0;
		int32 PrepareDataFrameCount = 0;
		TOptional<uint64> OutputCPUMemorySize;
		TOptional<uint64> OutputGPUMemorySize;
		TOptional<double> GPUTime;

		double PrepareDataWallTime() const { return PrepareDataEndTime - PrepareDataStartTime; }
		double ExecutionWallTime() const { return ExecutionEndTime - ExecutionStartTime; }
		double TotalTime() const { return ExecutionTime + PrepareDataTime; }
		double TotalWallTime() const { return ExecutionEndTime - PrepareDataStartTime; }
	};

#if WITH_EDITOR

	struct FCapturedMessage
	{
		int32 Index = 0;
		FName Namespace;
		FString Message;
		ELogVerbosity::Type Verbosity;
	};

	struct FCallTreeInfo
	{
		const UPCGNode* Node = nullptr;
		int32 LoopIndex = INDEX_NONE;
		FString Name; // overriden name for the task, will take precedence over the node name if not empty
		FCallTime CallTime;

		// Store the number of tasks executed within this call.
		int32 NumberOfTasks = 1;

		TArray<FCallTreeInfo> Children;
	};

	struct FScopedCall;

	struct FScopedCallOutputDevice : public FOutputDevice
	{
		UE_API FScopedCallOutputDevice();
		UE_API virtual ~FScopedCallOutputDevice();

		// FOutputDevice
		virtual bool IsMemoryOnly() const override { return true; }
		virtual bool CanBeUsedOnMultipleThreads() const override { return true; }
		virtual bool CanBeUsedOnAnyThread() const override { return true; }
		UE_API virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override;

		FScopedCall* ScopedCall = nullptr;
		const uint32 ThreadID;
	};

#else // !WITH_EDITOR

	struct FScopedCallOutputDevice
	{
	};

#endif // WITH_EDITOR

	struct FScopedCall;

	class FExtraCapture
	{
	public:
		UE_API void Update(const FScopedCall& InScopedCall);

#if WITH_EDITOR
		UE_API void ResetCapturedMessages();

		using TCapturedMessageMap = TMap<TWeakObjectPtr<const UPCGNode>, TArray<FCapturedMessage>>;

		const TCapturedMessageMap& GetCapturedMessages() const { return CapturedMessages; }

		UE_API FCallTreeInfo CalculateCallTreeInfo(const IPCGGraphExecutionSource* ExecutionSource, const FPCGStack& RootStack) const;

	private:
		mutable PCG::FLock Lock;
		TCapturedMessageMap CapturedMessages;
#endif // WITH_EDITOR
	};

	struct FScopedCall
	{
		UE_API FScopedCall(const IPCGElement* InOwner, FPCGContext* InContext, FScopedCallOutputDevice& InOutputDevice);
		UE_API ~FScopedCall();

		const IPCGElement* Owner;
		FPCGContext* Context;
		double StartTime;
		EPCGExecutionPhase Phase;
#if WITH_EDITOR
		TArray<FCapturedMessage> CapturedMessages;
#endif
		FScopedCallOutputDevice& OutputDevice;
	};

#else // PCG_PROFILING_ENABLED

	struct FScopedCallOutputDevice
	{
	};

	struct FScopedCall
	{
		FScopedCall(const IPCGElement* InOwner, FPCGContext* InContext, FScopedCallOutputDevice& InOutputDevice) {}
	};

#endif // PCG_PROFILING_ENABLED
}

#undef UE_API
