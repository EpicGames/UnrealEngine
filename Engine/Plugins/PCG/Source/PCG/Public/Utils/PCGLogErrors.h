// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"

struct FPCGContext;

namespace PCGLog
{
	/**
	 * Convenience function that would either log error on the graph if there is a context, or in the console if not.
	 * Note that if the context is not set, it will look for the current context for this thread.
	 */
	PCG_API void LogErrorOnGraph(const FText& InMsg, const FPCGContext* InContext = nullptr);
	
	/**
	 * Convenience function that would either log warning on the graph if there is a context, or in the console if not.
	 * Note that if the context is not set, it will look for the current context for this thread.
	 */
	PCG_API void LogWarningOnGraph(const FText& InMsg, const FPCGContext* InContext = nullptr);

	namespace InputOutput
	{
		namespace Format
		{
			PCG_API extern const FText InvalidInputData;

			PCG_API extern const FTextFormat TypedInputNotFound;
			PCG_API extern const FTextFormat FirstInputOnly;
			PCG_API extern const FTextFormat FileNotFound;
			PCG_API extern const FTextFormat DirectoryNotFound;
		}

		// Warnings
		PCG_API void LogTypedDataNotFoundWarning(EPCGDataType DataType, const FName PinLabel, const FPCGContext* InContext = nullptr);
		PCG_API void LogFirstInputOnlyWarning(const FName PinLabel, const FPCGContext* InContext = nullptr);

		// Errors
		PCG_API void LogInvalidInputDataError(const FPCGContext* InContext = nullptr);
		PCG_API void LogInvalidCardinalityError(const FName SourcePinLabel, const FName TargetPinLabel, const FPCGContext* InContext = nullptr);
		PCG_API void LogFileNotFound(const FString& FilePath, const FPCGContext* InContext = nullptr);
		PCG_API void LogInvalidFileType(const FString& FileType, const FString& ExpectedType, const FPCGContext* InContext = nullptr);
		PCG_API void LogDirectoryNotFound(const FString& DirectoryPath, const FPCGContext* InContext = nullptr);
	}

	namespace Metadata
	{
		namespace Format
		{
			PCG_API extern const FTextFormat CreateAttributeFailure;
			PCG_API extern const FTextFormat GetTypedAttributeFailure;
			PCG_API extern const FTextFormat GetTypedAttributeFailureNoAccessor;
			PCG_API extern const FTextFormat SetTypedAttributeFailure;
			PCG_API extern const FTextFormat SetTypedAttributeFailureNoAccessor;
			PCG_API extern const FTextFormat InvalidMetadataDomain;
			PCG_API extern const FText InvalidMetadata;
		}

		// Errors
		PCG_API void LogFailToCreateAccessorError(const FPCGAttributePropertySelector& Selector, const FPCGContext* InContext = nullptr);
		PCG_API void LogInvalidMetadataDomain(const FPCGAttributePropertySelector& Selector, const FPCGContext* InContext = nullptr);
		PCG_API void LogInvalidMetadata(const FPCGContext* InContext = nullptr);
		PCG_API void LogFailToCreateAttributeError(const FPCGMetadataAttributeDesc& Desc, const FPCGContext* InContext = nullptr);

		template <typename T>
		void LogFailToCreateAttributeError(const FText& AttributeName, const FPCGContext* InContext = nullptr)
		{
			LogFailToCreateAttributeError(PCG::Private::MakeAttributeDesc<T>(FName(AttributeName.ToString())), InContext);
		}

		template <typename T>
		void LogFailToCreateAttributeError(FName AttributeName, const FPCGContext* InContext = nullptr)
		{
			LogFailToCreateAttributeError(PCG::Private::MakeAttributeDesc<T>(AttributeName), InContext);
		}

		PCG_API void LogFailToGetAttributeError(const FText& AttributeName, const FPCGContext* InContext = nullptr);
		PCG_API void LogFailToGetAttributeError(FName AttributeName, const FPCGContext* InContext = nullptr);
		PCG_API void LogFailToGetAttributeError(const FPCGAttributePropertySelector& Selector, const FPCGContext* InContext = nullptr);
		PCG_API void LogIncomparableAttributesError(const FPCGAttributePropertySelector& FirstSelector, const FPCGAttributePropertySelector& SecondSelector, const FPCGContext* InContext = nullptr);

		template <typename T>
		void LogFailToGetAttributeError(const FText& AttributeName, const IPCGAttributeAccessor* Accessor, const FPCGContext* InContext = nullptr)
		{
			if (Accessor)
			{
				PCGLog::LogErrorOnGraph(FText::Format(Format::GetTypedAttributeFailure, AttributeName, PCG::Private::GetTypeNameText<T>(), PCG::Private::GetTypeNameText(Accessor->GetUnderlyingType())), InContext);
			}
			else
			{
				PCGLog::LogErrorOnGraph(FText::Format(Format::GetTypedAttributeFailureNoAccessor, AttributeName, PCG::Private::GetTypeNameText<T>()), InContext);
			}
		}

		template <typename T>
		void LogFailToGetAttributeError(FName AttributeName, const IPCGAttributeAccessor* Accessor, const FPCGContext* InContext = nullptr)
		{
			LogFailToGetAttributeError<T>(FText::FromName(AttributeName), Accessor, InContext);
		}

		template <typename T>
		void LogFailToGetAttributeError(const FPCGAttributePropertySelector& Selector, const IPCGAttributeAccessor* Accessor, const FPCGContext* InContext = nullptr)
		{
			return LogFailToGetAttributeError<T>(Selector.GetDisplayText(), Accessor, InContext);
		}
		
		template <typename T>
		void LogFailToSetAttributeError(const FText& AttributeName, const IPCGAttributeAccessor* Accessor, const FPCGContext* InContext = nullptr)
		{
			if (Accessor)
			{
				PCGLog::LogErrorOnGraph(FText::Format(Format::SetTypedAttributeFailure, AttributeName, PCG::Private::GetTypeNameText(Accessor->GetUnderlyingType()), PCG::Private::GetTypeNameText<T>()), InContext);
			}
			else
			{
				PCGLog::LogErrorOnGraph(FText::Format(Format::SetTypedAttributeFailureNoAccessor, AttributeName, PCG::Private::GetTypeNameText<T>()), InContext);
			}
		}

		template <typename T>
		void LogFailToSetAttributeError(FName AttributeName, const IPCGAttributeAccessor* Accessor, const FPCGContext* InContext = nullptr)
		{
			LogFailToSetAttributeError<T>(FText::FromName(AttributeName), Accessor, InContext);
		}

		template <typename T>
		void LogFailToSetAttributeError(const FPCGAttributePropertySelector& Selector, const IPCGAttributeAccessor* Accessor, const FPCGContext* InContext = nullptr)
		{
			return LogFailToSetAttributeError<T>(Selector.GetDisplayText(), Accessor, InContext);
		}
	}

	namespace Parsing
	{
		// Warnings
		PCG_API void LogEmptyExpressionWarning(const FPCGContext* InContext = nullptr);

		// Errors
		PCG_API void LogInvalidCharacterInParsedStringError(const FStringView& ParsedString, const FPCGContext* InContext = nullptr);
		PCG_API void LogInvalidExpressionInParsedStringError(const FStringView& ParsedString, const FPCGContext* InContext = nullptr);
	}

	namespace Component
	{
		PCG_API void LogComponentAttachmentFailedWarning(const FPCGContext* InContext = nullptr);
	}

	namespace Settings
	{
		PCG_API void LogInvalidPreconfigurationWarning(int32 PreconfigurationIndex, const FText& NodeTitle);
		PCG_API void LogInvalidConversionError(int32 PreconfigurationIndex, const FText& NodeTitle, const FText& Reason);
	}

	namespace Landscape
	{
		PCG_API void LogLandscapeCacheNotAvailableError(const FPCGContext* InContext = nullptr);
	}

	namespace Validation
	{
		PCG_API void LogInvalidRotationsReset(const FPCGContext* InContext = nullptr);
		PCG_API void LogInvalidRotationsSkipped(const FPCGContext* InContext = nullptr);
	}
}
