// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/PCGStackContext.h"

#include "Containers/Map.h"
#include "Internationalization/Text.h"
#include "Logging/LogVerbosity.h"
#include "UObject/WeakObjectPtr.h"

#define UE_API PCG_API

#if WITH_EDITOR

class UPCGComponent;
class UPCGNode;

struct FPCGNodeLogEntry
{
	FPCGNodeLogEntry() = default;

	explicit FPCGNodeLogEntry(const FText& InMessage, ELogVerbosity::Type InVerbosity)
		: Message(InMessage)
		, Verbosity(InVerbosity)
	{
	}

	bool operator==(const FPCGNodeLogEntry& Other) const
	{
		return Verbosity == Other.Verbosity && Message.EqualTo(Other.Message);
	}

	FText Message;
	ELogVerbosity::Type Verbosity;
};

typedef TArray<FPCGNodeLogEntry, TInlineAllocator<16>> FPCGPerNodeVisualLogs;

/** Collections per-node graph execution warnings and errors. */
class FPCGNodeVisualLogs
{
public:
	/** Log warnings and errors to be displayed on node in graph editor. */
	UE_API void Log(const FPCGStack& InPCGStack, ELogVerbosity::Type InVerbosity, const FText& InMessage);

	/** Returns true if any issues were logged during last execution. */
	UE_API bool HasLogs(const FPCGStack& InPCGStack) const;

	/** Returns true if an issue with given severity was logged during last execution, and writes the minimum encountered verbosity to OutMinVerbosity. */
	UE_API bool HasLogs(const FPCGStack& InPCGStack, ELogVerbosity::Type& OutMinVerbosity) const;

	/** Returns true if an issue with given severity was logged during last execution. */
	UE_API bool HasLogsOfVerbosity(const FPCGStack& InPCGStack, ELogVerbosity::Type InVerbosity) const;

	/** Returns all logs from a given stack */
	UE_API FPCGPerNodeVisualLogs GetLogs(const FPCGStack& InPCGStack) const;

	UE_DEPRECATED(5.7, "Use GetLogsAndSources instead")
	void GetLogs(const UPCGNode* InNode, FPCGPerNodeVisualLogs& OutLogs, TArray<const UPCGComponent*>& OutComponents) const {}

	/** Returns all logs for a given node */
	UE_API void GetLogsAndSources(const UPCGNode* InNode, FPCGPerNodeVisualLogs& OutLogs, TArray<const IPCGGraphExecutionSource*>& OutComponents) const;

	/** Calls a function for all stacks that match in the logs or until the function returns false. */
	UE_API void ForAllMatchingLogs(const FPCGStack& InPCGStack, TFunctionRef<bool(const FPCGStack&, const FPCGPerNodeVisualLogs&)> InFunc) const;

	/** Summary text of all visual logs produced while executing the provided base stack, appropriate for display in graph editor tooltip. */
	UE_API FText GetLogsSummaryText(const FPCGStack& InBaseStack, ELogVerbosity::Type* OutMinimumVerbosity = nullptr) const;

	/**
	* Returns summary text of visual logs from recent execution, appropriate for display in graph editor tooltip. Writes the minimum encountered verbosity
	* to OutMinimumVerbosity.
	*/
	UE_API FText GetLogsSummaryText(const UPCGNode* InNode, ELogVerbosity::Type& OutMinimumVerbosity) const;

	/** Clear all errors and warnings that occurred while executing stacks beginning with the given stack. */
	UE_API void ClearLogs(const FPCGStack& InPCGStack);

	/** Clear all errors and warnings corresponding to the given execution source. */
	UE_API void ClearLogs(const IPCGGraphExecutionSource* InSource);

	UE_DEPRECATED(5.7, "Use GetSummaryTextWithSources instead")
	static FText GetSummaryText(const FPCGPerNodeVisualLogs& InLogs, const TArray<const IPCGGraphExecutionSource*>* InComponents, ELogVerbosity::Type* OutMinimumVerbosity = nullptr) { return FText(); }

	/** Generates summary text from log list, with optional source pointers to show actor label. */
	static UE_API FText GetSummaryTextWithSources(const FPCGPerNodeVisualLogs& InLogs, const TArray<const IPCGGraphExecutionSource*>* InSources, ELogVerbosity::Type* OutMinimumVerbosity = nullptr);
private:
	TMap<FPCGStack, FPCGPerNodeVisualLogs> StackToLogs;
	mutable PCG::FSharedLock LogsLock;
};

#endif // WITH_EDITOR

#undef UE_API
