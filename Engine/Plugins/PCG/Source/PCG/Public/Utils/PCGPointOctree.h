// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/GenericOctreePublic.h"
#include "Math/GenericOctree.h"

namespace PCGPointOctree
{
	struct FPointRef
	{
		FPointRef() = default;

		FPointRef(int32 InIndex, const FBoxSphereBounds& InBounds)
		{
			Index = InIndex;
			Bounds = InBounds;
		}

		int32 Index = INDEX_NONE;
		FBoxSphereBounds Bounds;
	};

	struct FPointRefSemantics
	{
		enum { MaxElementsPerLeaf = 16 };
		enum { MinInclusiveElementsPerNode = 7 };
		enum { MaxNodeDepth = 12 };

		typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

		inline static const FBoxSphereBounds& GetBoundingBox(const FPointRef& InPoint)
		{
			return InPoint.Bounds;
		}

		inline static const bool AreElementsEqual(const FPointRef& A, const FPointRef& B)
		{
			// TODO: verify if that's sufficient
			return A.Index == B.Index;
		}

		inline static void ApplyOffset(FPointRef& InPoint)
		{
			ensureMsgf(false, TEXT("Not implemented"));
		}

		inline static void SetElementId(const FPointRef& Element, FOctreeElementId2 OctreeElementID)
		{
		}
	};

	typedef TOctree2<FPointRef, FPointRefSemantics> FPointOctree;
}
