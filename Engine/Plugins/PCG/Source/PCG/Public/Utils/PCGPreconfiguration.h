// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Metadata/PCGMetadataAttributeTraits.h"

#include "PCGPreconfiguration.generated.h"

#if WITH_EDITOR
struct FSlateBrush;
#endif // WITH_EDITOR

/**
* Pre-configured info
* Utility struct for use with any type of PCG preconfiguration.
* Example: A set of user actions available on a specific node or predefined settings configuration for a node element.
*/
USTRUCT(BlueprintType)
struct FPCGPreconfiguredInfo
{
	GENERATED_BODY()

	FPCGPreconfiguredInfo() = default;

	explicit FPCGPreconfiguredInfo(int32 InIndex, FText InLabel = FText{})
		: PreconfiguredIndex(InIndex)
		, Label(MoveTemp(InLabel))
	{}

#if WITH_EDITOR
	explicit FPCGPreconfiguredInfo(int32 InIndex, FText InLabel, FText InTooltip)
		: PreconfiguredIndex(InIndex)
		, Label(MoveTemp(InLabel))
		, Tooltip(MoveTemp(InTooltip))
	{}
#endif // WITH_EDITOR

	/**
	* Automatically fill all preconfigured settings depending on the provided enum.
	* Can also specify explicitly values that should not be included, i.e. cases that may not be available in non-editor builds.
	* Can pass in an optional function or lambda for filtering or finer control over the results, for things like metadata.
	* @param InValuesToSkip A set of enum values that should be skipped during population, like counts or hidden values.
	* @param InOptionalFormat An optional string format to fit the name of the action into, as it appears in the contextual search.
	* @param ProcessFunc A post-process or filtering callback for each enum value. Returns false if the enum should be filtered.
	* @returns An array of preconfigured info for schema actions or conversion, etc.
	*/
	template <typename EnumType, typename SubclassType = FPCGPreconfiguredInfo>
	static TArray<SubclassType> PopulateFromEnum(const TSet<EnumType>& InValuesToSkip = {}, const FTextFormat& InOptionalFormat = FTextFormat(), const FTextFormat& InOptionalTooltipFormat = FTextFormat(), const TFunction<bool(SubclassType& InOutInfo, const UEnum* EnumPtr, int32 ValueIndex)>& ProcessFunc = nullptr)
	{
		static_assert(std::is_base_of_v<FPCGPreconfiguredInfo, SubclassType>, "SubclassType must be a subclass of FPCGPreconfiguredInfo");

		static const FTextFormat EmptyFormat = INVTEXT("{0}");
		const FTextFormat Format = InOptionalFormat.GetSourceText().IsEmpty() ? EmptyFormat : InOptionalFormat;
		const bool bHasTooltipFormat = !InOptionalTooltipFormat.GetSourceText().IsEmpty();
		TArray<SubclassType> PreconfiguredInfo;

		if (const UEnum* EnumPtr = StaticEnum<EnumType>())
		{
			PreconfiguredInfo.Reserve(EnumPtr->NumEnums());
			for (int32 i = 0; i < EnumPtr->NumEnums(); ++i)
			{
#if WITH_EDITOR
				if (EnumPtr->HasMetaData(TEXT("Hidden"), i))
				{
					continue;
				}
#endif // WITH_EDITOR

				int64 Value = EnumPtr->GetValueByIndex(i);

				if (Value != EnumPtr->GetMaxEnumValue() && !InValuesToSkip.Contains(EnumType(Value)))
				{
					FText DisplayName = EnumPtr->GetDisplayNameTextByValue(Value);
					if (!DisplayName.IsEmpty())
					{
						SubclassType Info(Value, FText::Format(Format, MoveTemp(DisplayName)));

#if WITH_EDITOR
						if (bHasTooltipFormat)
						{
							Info.Tooltip = FText::Format(InOptionalTooltipFormat, EnumPtr->GetToolTipTextByIndex(i));
						}
#endif // WITH_EDITOR

						if (!ProcessFunc || ProcessFunc(Info, EnumPtr, i))
						{
							PreconfiguredInfo.Emplace(MoveTemp(Info));
						}
					}
				}
			}
		}

		return PreconfiguredInfo;
	}

	template <typename EnumType, typename SubclassType = FPCGPreconfiguredInfo>
	static TArray<SubclassType> PopulateFromEnum(const TSet<EnumType>& InValuesToSkip = {}, const FTextFormat& InOptionalFormat = FTextFormat(), const TFunction<bool(SubclassType& InOutInfo, const UEnum* EnumPtr, int32 ValueIndex)>& ProcessFunc = nullptr)
	{
		return PopulateFromEnum(InValuesToSkip, InOptionalFormat, {}, ProcessFunc);
	}

	/* Index used by the settings to know which preconfigured settings it needs to set. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "")
	int32 PreconfiguredIndex = -1;

	/* Label for the exposed asset. Can also be used instead of the index, if it is easier to deal with strings. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "")
	FText Label;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "")
	FText Tooltip;
#endif // WITH_EDITORONLY_DATA
};

// The type of descriptor used to generate an icon and color for an action menu item.
enum class EPCGActionIconDescriptorType : uint8
{
	Label = 0,
	Metadata
};

// Simple interface for overriding an action icon
class IPCGActionIconDescriptorBase
{
public:
	virtual ~IPCGActionIconDescriptorBase() = default;
	virtual EPCGActionIconDescriptorType GetActionIconDescriptorType() const = 0;
};

// Descriptor for setting the icon and color directly.
struct FPCGActionIconByLabelDescriptor : IPCGActionIconDescriptorBase
{
	virtual EPCGActionIconDescriptorType GetActionIconDescriptorType() const override { return EPCGActionIconDescriptorType::Label; }

	FName BrushLabel = NAME_None;
	FLinearColor Tint = FLinearColor::White;
};

// Descriptor to convert an EPCGMetadataTypes value directly.
struct FPCGActionIconByMetadataDescriptor : IPCGActionIconDescriptorBase
{
	explicit FPCGActionIconByMetadataDescriptor(const EPCGMetadataTypes InType, const EPCGContainerType InContainerType = EPCGContainerType::Element)
		: MetadataType(InType)
		, ContainerType(InContainerType) {}

	virtual EPCGActionIconDescriptorType GetActionIconDescriptorType() const override { return EPCGActionIconDescriptorType::Metadata; }
	EPCGMetadataTypes GetMetadataType() const { return MetadataType; }
	EPCGContainerType GetContainerType() const { return ContainerType; }

private:
	const EPCGMetadataTypes MetadataType;
	const EPCGContainerType ContainerType;
};

/**
 * @todo_pcg:
 * Look into converting these descriptors as a type trait for extensibility. Right now, MetadataDescriptor requires
 * a hardcoded lookup in the Editor, which could also potentially be made better extensible with a Factory pattern.
 *
 * Ex.
 * template <typename T>
 * struct IPCGActorIconTrait
 * {
 *	using ActionIconType = FPCGActionIconByLabelDescriptor;
 * };
 * 
 * template <>
 * struct IPCGActorIconTrait<EPCGMetadataTypes>
 * {
 * 	using ActionIconType = FPCGActionIconByMetadataDescriptor;
 * };
 */

/**
* Pre-configured settings info
* Will be passed to the settings to pre-configure the settings on creation. Also used for pre-configured node elements.
* Example: Maths operations: Add, Mul, Div etc...
*/
USTRUCT(BlueprintType)
struct FPCGPreConfiguredSettingsInfo : public FPCGPreconfiguredInfo
{
	GENERATED_BODY()

	static constexpr TCHAR SearchHintMetadataKey[] = TEXT("SearchHints");
	static constexpr TCHAR ActionIconMetadataKey[] = TEXT("ActionIcon");
	static constexpr TCHAR ActionIconTintMetadataKey[] = TEXT("ActionIconTint");

	FPCGPreConfiguredSettingsInfo() = default;

	explicit FPCGPreConfiguredSettingsInfo(int32 InIndex, FText InLabel = FText{})
		: FPCGPreconfiguredInfo(InIndex, InLabel)
	{}

#if WITH_EDITOR
	FPCGPreConfiguredSettingsInfo(const int32 InIndex, const FText& InLabel, const FText& InTooltip, const FText& InSearchHints = {}, TSharedPtr<IPCGActionIconDescriptorBase> InActionIconDescriptor = nullptr)
		: FPCGPreconfiguredInfo(InIndex, InLabel, InTooltip)
		, SearchHints(InSearchHints)
		, ActionIconDescriptor(MoveTemp(InActionIconDescriptor)) {}
#endif // WITH_EDITOR

	template <typename EnumType>
	static TArray<FPCGPreConfiguredSettingsInfo> PopulateFromEnum(const TSet<EnumType>& InValuesToSkip = {}, const FTextFormat& InOptionalFormat = FTextFormat(), const FTextFormat& InOptionalTooltipFormat = FTextFormat())
	{
		static_assert(std::is_enum_v<EnumType>, "Must use an enum to populate.");

#if WITH_EDITOR
		return FPCGPreconfiguredInfo::PopulateFromEnum<EnumType, FPCGPreConfiguredSettingsInfo>(InValuesToSkip, InOptionalFormat, InOptionalTooltipFormat, [](FPCGPreConfiguredSettingsInfo& InOutInfo, const UEnum* EnumPtr, const int32 ValueIndex) -> bool
		{
			if (EnumPtr)
			{
				if (EnumPtr->HasMetaData(SearchHintMetadataKey, ValueIndex))
				{
					InOutInfo.SearchHints = FText::FromString(EnumPtr->GetMetaData(SearchHintMetadataKey, ValueIndex));
					InOutInfo.EnumPtr = EnumPtr;
				}

				// For anything we populate from metadata type enum, add the type icon as the palette icon
				if constexpr (std::is_same_v<EnumType, EPCGMetadataTypes>)
				{
					if (!InOutInfo.ActionIconDescriptor)
					{
						InOutInfo.ActionIconDescriptor = MakeShared<FPCGActionIconByMetadataDescriptor>(static_cast<EPCGMetadataTypes>(EnumPtr->GetValueByIndex(ValueIndex)));
					}
				}
				// Get the brush icon and color directly from the metadata
				else if (EnumPtr->HasMetaData(ActionIconMetadataKey, ValueIndex))
				{
					FPCGActionIconByLabelDescriptor IconDescriptor;
					IconDescriptor.BrushLabel = FName(EnumPtr->GetMetaData(ActionIconMetadataKey, ValueIndex));

					// @todo_pcg: Parse the metadata into RGBA channels. For now just make it white.
					// if (EnumPtr->HasMetaData(ActionIconTintMetadataKey, ValueIndex)) {}
					IconDescriptor.Tint = FLinearColor::White;

					InOutInfo.ActionIconDescriptor = MakeShared<FPCGActionIconByLabelDescriptor>(MoveTemp(IconDescriptor));
				}
			}

			return true;
		});
#else
		return FPCGPreconfiguredInfo::PopulateFromEnum<EnumType, FPCGPreConfiguredSettingsInfo>(InValuesToSkip, InOptionalFormat, InOptionalTooltipFormat);
#endif // WITH_EDITOR
	}

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "")
	FText SearchHints;

	TSharedPtr<IPCGActionIconDescriptorBase> ActionIconDescriptor = nullptr;

private:
	TObjectPtr<const UEnum> EnumPtr = nullptr;
#endif // WITH_EDITORONLY_DATA
};
