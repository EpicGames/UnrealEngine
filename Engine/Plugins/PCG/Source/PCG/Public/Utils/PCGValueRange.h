// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StridedView.h"

#include "HAL/PlatformString.h" // for INT64_FMT

struct PCGValueRangeHelpers;

/*
* TPCGValueRange is used to represent a list of InNumElements values but those
* values are provided by an underlying TStridedView which can have a different number of values.
*
* Most common use case is where we have N Points, a TPCGValueRange might be used to iterate through all the Point Colors.
* 
* For UPCGPointData, all Points have a memory allocation representing those Colors so the PCGValueRange number of elements will match the underlying TStridedView number of elements
* 
* but if we have another type of Point data which doesn't allocate Colors by default and just uses a single value, We can still use a TPCGValueRange with the same number of elements with
* an underlying TStridedView with a single element.
* 
* This allows us to write processing functions that operate on ranges without knowledge of the underlying memory layout
* 
* The range index will be modulated against the underlying TStridedView num elements.
* 
* ex: 
* 
* Range with 5 Elements & StridedView of 1 Element
* Range Indices [0, 1, 2, 3 ,4] StridedView Indices [0, 0, 0, 0, 0]
* 
* Range with 5 Elements & StridedView of 3 Element(s)
* Range Indices [0, 1, 2, 3, 4] StridedView Indices [0, 1, 2, 0, 1]
* 
* Range with 5 Elements & StridedView of 5 Elements(s) or more
* Range Indices [0, 1, 2, 3, 4] StridedView Indices [0, 1, 2, 3, 4]
* 
* Note: For multithreaded usage it is preferred to declare the ranges in each thread so that the 'StubValue' is thread local
* 
*/
template<typename ElementType, typename ViewType = TStridedView<ElementType>>
class TPCGValueRange
{
	friend PCGValueRangeHelpers;

public:
	using SizeType = typename ViewType::SizeType;
	using NonConstElementType = std::remove_const_t<ElementType>;

	TPCGValueRange()
		: TPCGValueRange(ViewType(), 0)
	{
	}

	explicit TPCGValueRange(ViewType InElementView)
		: TPCGValueRange(InElementView, InElementView.Num())
	{
	}

	explicit TPCGValueRange(ViewType InElementView, SizeType InNumElements)
		: ElementView(InElementView), NumElements(InNumElements)
	{
	}
		
	[[nodiscard]] inline TOptional<ElementType> GetSingleValue() const
	{
		return ViewNum() == 1 && Num() > 0 ? TOptional<ElementType>(GetElement(0)) : TOptional<ElementType>();
	}

	[[nodiscard]] inline bool IsValidIndex(SizeType Index) const
	{
		return (Index >= 0) && (Index < NumElements);
	}

	[[nodiscard]] inline bool IsEmpty() const
	{
		return NumElements == 0;
	}

	[[nodiscard]] inline SizeType Num() const
	{
		return NumElements;
	}

	[[nodiscard]] inline SizeType ViewNum() const
	{
		return FMath::Min(NumElements, ElementView.Num());
	}

	[[nodiscard]] inline ElementType& operator[](SizeType Index) const
	{
		return GetElement(Index);
	}

	inline void Set(const ElementType& Value)
	{
		// Set all values in underlying view to same value
		for (SizeType i = 0; i < ElementView.Num(); ++i)
		{
			ElementView[i] = Value;
		}
	}

	struct FIterator
	{
		const TPCGValueRange* Owner;
		SizeType Index;

		inline FIterator& operator++()
		{
			++Index;
			return *this;
		}

		inline ElementType& operator*()
		{
			return Owner->GetElementUnsafe(Index);
		}

		[[nodiscard]] inline bool operator==(const FIterator& Other) const
		{
			return Owner == Other.Owner
				&& Index == Other.Index;
		}

		[[nodiscard]] inline bool operator!=(const FIterator& Other) const
		{
			return !(*this == Other);
		}
	};

	[[nodiscard]] inline FIterator begin() const { return FIterator{ this, 0 }; }
	[[nodiscard]] inline FIterator end() const { return FIterator{ this, Num() }; }

private:
	inline void RangeCheck(SizeType Index) const
	{
		checkf((Index >= 0) & (Index < NumElements), TEXT("Array index out of bounds: %" INT64_FMT " from an array of size %" INT64_FMT), int64(Index), int64(NumElements))
	}
	
	[[nodiscard]] inline ElementType& GetElementUnsafe(SizeType Index) const
	{
		const SizeType ViewIndex = Index % ElementView.Num();
		ElementType& Value = ElementView.GetUnsafe(ViewIndex);

		// Accessing the value for read in a multithreaded context is safe so no need for the StubValue here.
		if constexpr (std::is_const_v<ElementType>)
		{
			return Value;
		}
		else
		{			
			// It is possible that our ElementView is a single value for non-allocated properties in which case 
			// we want to prevent multithreaded writes to this single value.
			// Only allowing to return the actual value reference if Index == ViewIndex.
			// If not we return a StubValue reference which is local to the ValueRange object.
			if (Index == ViewIndex)
			{
				return Value;
			}
			else
			{
				return StubValue;
			}
		}
	}

	[[nodiscard]] inline ElementType& GetElement(SizeType Index) const
	{
		RangeCheck(Index);
		return GetElementUnsafe(Index);
	}

private:
	ViewType ElementView;
	SizeType NumElements;
	mutable NonConstElementType StubValue;
};

template<typename ElementType> using TConstPCGValueRange = TPCGValueRange<const ElementType, TConstStridedView<ElementType>>;

struct PCGValueRangeHelpers
{
	// Not instantiatable
	~PCGValueRangeHelpers() = delete;
	
	/** For Array views */
	template<typename ElementType>
	static TPCGValueRange<ElementType> MakeValueRange(TArrayView<ElementType> InView)
	{
		return TPCGValueRange<ElementType>(MakeStridedView(InView));
	}

	template<typename ElementType>
	static TConstPCGValueRange<ElementType> MakeConstValueRange(TArrayView<const ElementType> InView)
	{
		return TConstPCGValueRange<ElementType>(MakeConstStridedView(InView));
	}

	/** For Array */
	template<typename ElementType, typename ...Args>
	static TPCGValueRange<ElementType> MakeValueRange(const TArray<ElementType, Args...>& InArray)
	{
		return TPCGValueRange<ElementType>(MakeStridedView(InArray));
	}

	template<typename ElementType, typename ...Args>
	static TConstPCGValueRange<ElementType> MakeConstValueRange(const TArray<const ElementType, Args...>& InArray)
	{
		return TConstPCGValueRange<ElementType>(MakeConstStridedView(InArray));
	}

	template<typename ElementType, typename ...Args>
	static TConstPCGValueRange<ElementType> MakeConstValueRange(const TArray<ElementType, Args...>& InArray)
	{
		return TConstPCGValueRange<ElementType>(MakeConstStridedView(InArray));
	}

	/** For TPCGValueRange */
	template<typename ElementType>
	static TConstPCGValueRange<ElementType> MakeConstValueRange(TPCGValueRange<ElementType> InView)
	{
		return TConstPCGValueRange<ElementType>(MakeConstStridedView(InView.ElementView.GetStride(), &InView.ElementView.GetUnsafe(0), InView.Num()));
	}

	// Const -> Non-const, use it at your own risk.
	template<typename ElementType>
	static TPCGValueRange<ElementType> MakeValueRange_Unsafe(TConstPCGValueRange<ElementType> InView)
	{
		return TPCGValueRange<ElementType>(MakeStridedView(InView.ElementView.GetStride(), const_cast<ElementType*>(&InView.ElementView.GetUnsafe(0)), InView.Num()));
	}
};