// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphFwd.h"

class FRDGBuilder;

namespace PCGDilate
{
	/** Grows valid data (alpha == 1) outward by one texel per iteration into uncovered neighbors. */
	PCGCOMPUTE_API bool AddDilatePass(FRDGBuilder& GraphBuilder, FRDGTextureRef OutputTexture, int32 Iterations);
}
