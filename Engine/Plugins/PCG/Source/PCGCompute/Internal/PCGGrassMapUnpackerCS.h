// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ShaderParameterStruct.h"

/** Compute shader for unpacking grass map textures rendered by FLandscapeGrassWeightExporter.
 * Note: This class is subject to change without deprecation.
 */
class FPCGGrassMapUnpackerCS : public FGlobalShader
{
public:
	static constexpr uint32 ThreadGroupDim = 8;
	static constexpr uint32 MaxNumLandscapeComponents = 64;

public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FPCGGrassMapUnpackerCS, PCGCOMPUTE_API);
	SHADER_USE_PARAMETER_STRUCT(FPCGGrassMapUnpackerCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, PCGCOMPUTE_API)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InPackedGrassMaps)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int32>, InComponentHeightOffsets)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FIntVector2>, InComponentGrassmapOffsetAndChannels)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int32>, InOutputIndexToExportIndex)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, OutUnpackedHeight)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float>, OutUnpackedGrassMaps)
		SHADER_PARAMETER_ARRAY(FIntVector4, InLinearTileIndexToComponentIndex, [MaxNumLandscapeComponents])
		SHADER_PARAMETER(FUintVector2, InNumTiles)
		SHADER_PARAMETER(FIntPoint, InTileGridOrigin)
		SHADER_PARAMETER(uint32, InLandscapeComponentResolution)
		SHADER_PARAMETER(int32, InNumExportedGrassMaps)
		SHADER_PARAMETER(float, InLandscapeScaleZ)
		SHADER_PARAMETER(float, InLandscapeRootZ)
		SHADER_PARAMETER(uint32, InOutputHeight)
		SHADER_PARAMETER(FUintVector2, InDispatchOffset)
		SHADER_PARAMETER(FUintVector2, InDispatchSize)
	END_SHADER_PARAMETER_STRUCT()

public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};
