// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"
#include "RenderGraphFwd.h"

#if RHI_RAYTRACING
class FSceneInterface;
class FSceneView;
#endif

namespace PCGRaytraceConstants
{
	constexpr int RAY_CULLED = -1;
	constexpr int RAY_TRACE_PACKED_BUFFER_STRIDE_UINTS = 12;
}

#if RHI_RAYTRACING
struct FPCGRayTraceParams
{
	const FSceneInterface* Scene = nullptr;
	const FSceneView* View = nullptr;
	int32 NumRays = 0;
	bool bNeedsUVData = false;
	int32 TexCoordsChannelIndex = 0;
	FRDGBufferUAVRef PackedDataUAV;
};

namespace PCGRayTrace
{
	void PCGCOMPUTE_API RenderPCGRayTraceInline(FRDGBuilder& GraphBuilder, const FPCGRayTraceParams& InParams);
};
#endif // RHI_RAYTRACING
