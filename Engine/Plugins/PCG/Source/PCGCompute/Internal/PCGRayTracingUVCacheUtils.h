// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FSceneInterface;

namespace PCGRayTracingUVCache
{
	/** Enable the PCG ray tracing UV cache for a given scene. Enqueues a render command that builds the cache from all existing scene primitives. No-op if already enabled or if ray tracing is disabled. */
	PCGCOMPUTE_API void RequestEnable_GameThread(FSceneInterface* SceneInterface);
}
