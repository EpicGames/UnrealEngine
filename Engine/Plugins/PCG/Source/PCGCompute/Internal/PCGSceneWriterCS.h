// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "GPUSceneWriter.h"
#include "ShaderParameterStruct.h"

/** [EXPERIMENTAL] Compute shader for writing instance data into GPU Scene.
 * Note: This class is subject to change without deprecation.
 */
class FPCGSceneWriterCS : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FPCGSceneWriterCS, PCGCOMPUTE_API);
	SHADER_USE_PARAMETER_STRUCT(FPCGSceneWriterCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, PCGCOMPUTE_API)
		SHADER_PARAMETER(uint32, InPrimitiveIndex)
		SHADER_PARAMETER(uint32, InNumInstancesAllocatedInGPUScene)
		SHADER_PARAMETER(uint32, InInstanceOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FUintVector4>, InInstanceData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint32>, InInstanceCustomFloatData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint32>, InWriteCounters)
		SHADER_PARAMETER(uint32, InPrimitiveId)
		SHADER_PARAMETER(uint32, InCustomDataCount)
		SHADER_PARAMETER(uint32, InPayloadDataFlags)
		SHADER_PARAMETER(uint32, InSeed)
		SHADER_PARAMETER(uint32, InNumCullingCellsTotal)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneWriterParameters, GPUSceneWriterParameters)
	END_SHADER_PARAMETER_STRUCT()

public:
	static constexpr uint32 NumThreadsPerGroup = 64;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};
