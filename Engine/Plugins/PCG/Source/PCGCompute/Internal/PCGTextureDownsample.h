// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"
#include "RendererInterface.h"

#include "PCGTextureDownsample.generated.h"

UENUM()
enum class EPCGTextureDownsampleMode : uint8
{
	Average = 0,
	Min = 1,
	Max = 2,
	Sum = 3,
};

namespace PCGTextureDownsample
{
	struct FParams
	{
		FRDGTextureRef Texture = nullptr;
		FRHISamplerState* Sampler = nullptr;
		int SliceIndex = 0;
		int NumSlices = 1;
		EPCGTextureDownsampleMode Mode = EPCGTextureDownsampleMode::Average;
	};

	PCGCOMPUTE_API void DownsampleTexture(FRDGBuilder& GraphBuilder, FParams& InParams);
};
