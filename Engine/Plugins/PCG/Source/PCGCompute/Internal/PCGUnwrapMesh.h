// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIFwd.h"
#include "RenderGraphFwd.h"
#include "Math/IntPoint.h"

class FRDGBuilder;
struct FStaticMeshLODResources;

namespace PCGUnwrapMesh
{
	/** Per-texel mesh attribute written by the unwrap pixel shader. */
	enum class EMeshAttribute : uint8
	{
		/** RGB = local-space vertex position interpolated to the texel; alpha = 1 marks "covered". */
		LocalPosition = 0,

		/** Coverage mask. Writes 1 on every channel so the mask is readable from any RT format (R8, R16F, RGBA, ...). */
		Mask = 1,

		Num,
	};

	/** Inputs for a single mesh-to-texture unwrap draw. */
	struct FUnwrapParams
	{
		FBufferRHIRef PositionBufferRHI;
		FBufferRHIRef TexCoordBufferRHI;
		FBufferRHIRef IndexBufferRHI;
		uint32 NumVerts = 0;
		uint32 NumTris = 0;
		uint32 NumTexCoords = 1;
		uint32 UVChannelIndex = 0;
		bool bFullPrecisionUVs = true;
		EMeshAttribute Attribute = EMeshAttribute::LocalPosition;
		FIntPoint Resolution = FIntPoint::ZeroValue;

		PCGCOMPUTE_API void InitFromLOD(const FStaticMeshLODResources& LOD);
	};

	/** Check params are valid for rendering and log errors if not. */
	PCGCOMPUTE_API bool ValidateParams(const FUnwrapParams& Params);

	/**
	 * Adds an RDG raster pass that draws the input mesh into OutputTexture, with triangles placed in UV space
	 * (UV mapped to clip space). The pixel shader writes the selected mesh attribute at every covered texel.
	 */
	PCGCOMPUTE_API bool AddUnwrapMeshPass(FRDGBuilder& GraphBuilder, FRDGTextureRef OutputTexture, const FUnwrapParams& Params);
}
