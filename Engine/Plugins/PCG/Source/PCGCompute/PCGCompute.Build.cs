// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class PCGCompute : ModuleRules
	{
		public PCGCompute(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Projects",
					"RHI",
					"RenderCore",
					"Renderer",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
				});

			PrivateIncludePaths.AddRange(
				new string[] {
					Path.Combine(GetModuleDirectory("Renderer"), "Private"),
				});
		}
	}
}
