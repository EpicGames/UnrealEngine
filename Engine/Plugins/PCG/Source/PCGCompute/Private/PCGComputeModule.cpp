// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGComputeModule.h"

#include "GlobalShader.h"
#include "Interfaces/IPluginManager.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FPCGComputeModule"

class FPCGComputeModule final : public IModuleInterface
{
public:
	//~ IModuleInterface implementation
#if WITH_EDITOR
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
#endif
	//~ End IModuleInterface implementation
};

#if WITH_EDITOR
void FPCGComputeModule::StartupModule()
{
	// Maps virtual shader source directory to the plugin's actual shaders directory.
	const FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("PCG"))->GetBaseDir(), TEXT("Shaders/Private"));
	AddShaderSourceDirectoryMapping(TEXT("/PCGComputeShaders"), PluginShaderDir);
}

void FPCGComputeModule::ShutdownModule()
{
}
#endif

IMPLEMENT_MODULE(FPCGComputeModule, PCGCompute);

DEFINE_LOG_CATEGORY(LogPCGCompute);

LLM_DEFINE_TAG(PCGCompute);

#undef LOCTEXT_NAMESPACE
