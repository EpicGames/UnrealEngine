// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGDilate.h"

#include "GlobalShader.h"
#include "PixelShaderUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterStruct.h"

DEFINE_LOG_CATEGORY_STATIC(LogPCGDilate, Log, All);

namespace PCGDilate
{
	class FPCGDilatePS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FPCGDilatePS);
		SHADER_USE_PARAMETER_STRUCT(FPCGDilatePS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
			SHADER_PARAMETER(FIntPoint, ResolutionMinusOne)
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()
	};
}

IMPLEMENT_GLOBAL_SHADER(PCGDilate::FPCGDilatePS, "/PCGComputeShaders/PCGDilate.usf", "MainPS", SF_Pixel);

namespace PCGDilate
{
	bool AddDilatePass(FRDGBuilder& GraphBuilder, FRDGTextureRef OutputTexture, int32 Iterations)
	{
		if (Iterations <= 0)
		{
			return true;
		}

		if (!OutputTexture)
		{
			UE_LOG(LogPCGDilate, Error, TEXT("PCGDilate::AddDilatePass: OutputTexture is null."));
			return false;
		}

		const FRDGTextureDesc& OutputDesc = OutputTexture->Desc;
		FRDGTextureDesc ScratchDesc = OutputDesc;
		ScratchDesc.Flags |= TexCreate_ShaderResource | TexCreate_RenderTargetable;

		FRDGTextureRef Scratch = GraphBuilder.CreateTexture(ScratchDesc, TEXT("PCGDilate.Scratch"));

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FPCGDilatePS> PixelShader(ShaderMap);

		const FIntPoint ResolutionMinusOne(FMath::Max(0, OutputDesc.Extent.X - 1), FMath::Max(0, OutputDesc.Extent.Y - 1));
		const FIntRect Viewport(0, 0, OutputDesc.Extent.X, OutputDesc.Extent.Y);

		// Ping-pong: each iteration reads Source, writes Dest, then swaps. After N iterations Source aliases the texture that received the most recent write.
		// If that's not OutputTexture, copy it back so callers always see results in the texture they passed in.
		FRDGTextureRef Source = OutputTexture;
		FRDGTextureRef Dest = Scratch;

		for (int32 Iteration = 0; Iteration < Iterations; ++Iteration)
		{
			FPCGDilatePS::FParameters* Parameters = GraphBuilder.AllocParameters<FPCGDilatePS::FParameters>();
			Parameters->InputTexture = Source;
			Parameters->ResolutionMinusOne = ResolutionMinusOne;
			Parameters->RenderTargets[0] = FRenderTargetBinding(Dest, ERenderTargetLoadAction::ENoAction);

			FPixelShaderUtils::AddFullscreenPass(GraphBuilder, ShaderMap, RDG_EVENT_NAME("PCGDilate(%d/%d)", Iteration + 1, Iterations), PixelShader, Parameters, Viewport);

			Swap(Source, Dest);
		}

		if (Source != OutputTexture)
		{
			AddCopyTexturePass(GraphBuilder, Source, OutputTexture);
		}

		return true;
	}
}
