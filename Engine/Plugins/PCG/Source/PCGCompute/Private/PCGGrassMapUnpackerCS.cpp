// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGGrassMapUnpackerCS.h"

void FPCGGrassMapUnpackerCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), ThreadGroupDim);
	OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), ThreadGroupDim);
	OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Z"), 1);
	OutEnvironment.SetDefine(TEXT("PCG_MAX_NUM_LANDSCAPE_COMPONENTS"), MaxNumLandscapeComponents);
}

IMPLEMENT_GLOBAL_SHADER(FPCGGrassMapUnpackerCS, "/PCGComputeShaders/PCGGrassMapUnpackerCS.usf", "PCGGrassMapUnpacker_CS", SF_Compute);
