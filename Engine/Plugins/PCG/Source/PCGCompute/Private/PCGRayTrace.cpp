// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGRayTrace.h"

#include "PCGComputeModule.h"
#include "PCGRayTracingUVCache.h"

#include "BuiltInRayTracingShaders.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "EngineModule.h"
#include "FXRenderingUtils.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "RHI.h"
#include "RHIGPUReadback.h"
#include "RayTracingPayloadType.h"
#include "RayTracingShaderBindingLayout.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RendererUtils.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneUniformBuffer.h"
#include "SceneView.h"
#include "ShaderParameterStruct.h"

#if RHI_RAYTRACING
class FPCGRayTraceCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPCGRayTraceCS)
	SHADER_USE_PARAMETER_STRUCT(FPCGRayTraceCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, InOutPackedData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, InTLAS)
		SHADER_PARAMETER_RESOURCE_COLLECTION(FResourceCollection, InPrimitiveUVCollection)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, InPrimitiveUVOffsets)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FRayTracingSceneMetadataRecord>, InRayTracingSceneMetadata)
		SHADER_PARAMETER(FVector3f, InPreViewTranslation)
		SHADER_PARAMETER(int32, InNumRays)
		SHADER_PARAMETER(int32, InTexCoordsChannelIndex)

		// These might just be here to prompt RG to create a resource. Its bound statically in addpass.
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracing)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParameters, GPUSceneParameters)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform)
			&& IsBindlessEnabledForRayTracing(FShaderPlatformConfig::GetBindlessConfiguration(Parameters.Platform));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("PCG_RAY_CULLED"), FString::FromInt(PCGRaytraceConstants::RAY_CULLED));
		OutEnvironment.SetDefine(TEXT("PCG_RAY_TRACE_PACKED_BUFFER_STRIDE_UINTS"), FString::FromInt(PCGRaytraceConstants::RAY_TRACE_PACKED_BUFFER_STRIDE_UINTS));
		
		OutEnvironment.CompilerFlags.Add(CFLAG_SupportsMinimalBindless);
		OutEnvironment.CompilerFlags.Add(CFLAG_InlineRayTracing);
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}
};
IMPLEMENT_GLOBAL_SHADER(FPCGRayTraceCS, "/PCGComputeShaders/PCGRayTrace.usf", "PCGRayTraceMainCS", SF_Compute);

void PCGRayTrace::RenderPCGRayTraceInline(FRDGBuilder& GraphBuilder, const FPCGRayTraceParams& InParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PCGRayTrace::RenderPCGRayTraceInline);
	check(IsInRenderingThread());

	if (!IsRayTracingEnabled())
	{
		UE_LOGF(LogPCGCompute, Error, "Failed to submit inline raytrace shader. Ray tracing is not enabled.");
		return;
	}

	if (!GRHISupportsInlineRayTracing)
	{
		UE_LOGF(LogPCGCompute, Error, "Failed to submit inline raytrace shader. Inline ray tracing is not supported.");
		return;
	}

	if (!IsBindlessEnabledForRayTracing(FShaderPlatformConfig::GetBindlessConfiguration(GMaxRHIShaderPlatform)))
	{
		UE_LOGF(LogPCGCompute, Error, "Failed to submit inline raytrace shader. Bindless rendering is not enabled for platform '%ls'. Check your project settings.", *LexToString(GMaxRHIShaderPlatform));
		return;
	}

	if (!InParams.Scene)
	{
		UE_LOGF(LogPCGCompute, Error, "Failed to submit inline raytrace shader. Invalid scene.");
		return;
	}

	if (!InParams.View)
	{
		UE_LOGF(LogPCGCompute, Error, "Failed to submit inline raytrace shader. Invalid view.");
		return;
	}

	if (!InParams.View->GetInlineRayTracingBindingDataBuffer())
	{
		UE_LOGF(LogPCGCompute, Error, "Failed to submit inline raytrace shader. Inline ray tracing binding data buffer is not available.");
		return;
	}

	const FScene* RenderScene = InParams.Scene->GetRenderScene();

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FPCGRayTraceCS> ComputeShader(ShaderMap);

	// Populate UV collection members and indirection data.
	TArray<FRHIResourceCollectionMember, SceneRenderingAllocator> PrimitiveUVCollectionMembers;
	FRDGUploadData<FUintPoint> PrimitiveIndirections(GraphBuilder, 1);
	PrimitiveIndirections[0] = { ~0u, 0u };

	if (InParams.bNeedsUVData && InParams.TexCoordsChannelIndex >= 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGRayTrace::BuildPrimitiveUVData);

		const FPCGRayTracingUVCacheExtension* UVCache = RenderScene->GetExtensionPtr<FPCGRayTracingUVCacheExtension>();

		if (UVCache && UVCache->Cache.Num() > 0)
		{
			// Build from the persistent cache — no per-primitive proxy calls needed.
			PrimitiveIndirections = FRDGUploadData<FUintPoint>(GraphBuilder, UVCache->MaxPersistentIndex + 1);
			FMemory::Memset(PrimitiveIndirections.GetData(), 0xFF, PrimitiveIndirections.GetTotalSize());

			for (const auto& [PrimitiveIdx, Entry] : UVCache->Cache)
			{
				PrimitiveIndirections[PrimitiveIdx] = { static_cast<uint32>(PrimitiveUVCollectionMembers.Num()), Entry.NumCoordinates };
				PrimitiveUVCollectionMembers.Add(FRHIResourceCollectionMember(Entry.BufferSRV));
			}
		}
		else
		{
			// Fallback: cache not available — build UV data on demand by scanning all primitives.
			// Note: this can be extremely expensive.
			TArray<TPair<FRaytracingVFUVData, uint32>, SceneRenderingAllocator> UVBuffers;
			uint32 MaxPrimitiveId = 0;

			for (int32 PrimitiveIndex = 0; PrimitiveIndex < RenderScene->Primitives.Num(); PrimitiveIndex++)
			{
				const FPrimitiveSceneInfo* Primitive = RenderScene->Primitives[PrimitiveIndex];

				if (!Primitive || !Primitive->Proxy || !Primitive->Proxy->HasRayTracingRepresentation())
				{
					continue;
				}

				const uint32 PrimitiveId = Primitive->GetPersistentIndex().Index;
				MaxPrimitiveId = FMath::Max(MaxPrimitiveId, PrimitiveId);

				const FRaytracingVFAttributeData VFAttributeData = Primitive->Proxy->GetRaytracingVFAttributeData(InParams.View);

				if (VFAttributeData.Sections.IsEmpty() || !VFAttributeData.Sections[0].UV.BufferSRV)
				{
					continue;
				}

				UVBuffers.Emplace(VFAttributeData.Sections[0].UV, PrimitiveId);
			}

			PrimitiveIndirections = FRDGUploadData<FUintPoint>(GraphBuilder, MaxPrimitiveId + 1);
			FMemory::Memset(PrimitiveIndirections.GetData(), 0xFF, PrimitiveIndirections.GetTotalSize());

			for (auto&& [UVBuffer, Key] : UVBuffers)
			{
				PrimitiveIndirections[Key] = { static_cast<uint32>(PrimitiveUVCollectionMembers.Num()), UVBuffer.NumCoordinates };
				PrimitiveUVCollectionMembers.Add(FRHIResourceCollectionMember(UVBuffer.BufferSRV));
			}
		}
	}

	FRHIResourceCollectionRef PrimitiveUVCollection = GraphBuilder.RHICmdList.CreateResourceCollection(PrimitiveUVCollectionMembers);
	FRDGBufferRef PrimitiveIndirectionsRDG = CreateUploadBuffer(
		GraphBuilder,
		TEXT("PCG::UVIndirectionBuffer"),
		/*BytesPerElement=*/sizeof(FUintPoint),
		/*NumElements=*/FMath::RoundUpToPowerOfTwo(PrimitiveIndirections.Num()),
		PrimitiveIndirections);

	FSceneUniformBuffer SceneUniformBuffer;
	TRDGUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRDG = UE::FXRenderingUtils::GetSceneUniformBuffer(GraphBuilder, SceneUniformBuffer);

	FPCGRayTraceCS::FParameters* ShaderParameters = GraphBuilder.AllocParameters<FPCGRayTraceCS::FParameters>();
	ShaderParameters->InOutPackedData = InParams.PackedDataUAV;
	ShaderParameters->InTLAS = UE::FXRenderingUtils::RayTracing::GetRayTracingSceneViewRDG(*InParams.Scene, *InParams.View);
	ShaderParameters->InPrimitiveUVCollection = PrimitiveUVCollection;
	ShaderParameters->InPrimitiveUVOffsets = GraphBuilder.CreateSRV(PrimitiveIndirectionsRDG, PF_R32_UINT);
	ShaderParameters->InRayTracingSceneMetadata = GraphBuilder.CreateSRV(InParams.View->GetInlineRayTracingBindingDataBuffer());
	ShaderParameters->InPreViewTranslation = FVector3f(InParams.View->ViewMatrices.GetPreViewTranslation());
	ShaderParameters->InNumRays = InParams.NumRays;
	ShaderParameters->InTexCoordsChannelIndex = InParams.TexCoordsChannelIndex;
	ShaderParameters->NaniteRayTracing = Nanite::GetPublicGlobalRayTracingUniformBuffer();
	ShaderParameters->Scene = SceneUniformBufferRDG;
	ShaderParameters->View = InParams.View->ViewUniformBuffer;
	ShaderParameters->GPUSceneParameters = RenderScene->GPUScene.GetShaderParameters(GraphBuilder);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("PCGRayTrace"),
		ShaderParameters,
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		[NumRays=InParams.NumRays, ShaderParameters, ComputeShader](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FComputeShaderUtils::Dispatch(
				RHICmdList,
				ComputeShader,
				*ShaderParameters,
				FIntVector(FMath::DivideAndRoundUp(NumRays, 32), 1, 1));
		});
}
#endif // RHI_RAYTRACING
