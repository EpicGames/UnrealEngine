// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGRayTracingUVCache.h"
#include "PCGRayTracingUVCacheUtils.h"

#include "RenderingThread.h"
#include "SceneInterface.h"

#if RHI_RAYTRACING

#include "PCGComputeModule.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "HAL/IConsoleManager.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "ScenePrivate.h"
#include "ScenePrimitiveUpdates.h"

static TAutoConsoleVariable<bool> CVarPCGRayTracingUVCache(
	TEXT("pcg.GPU.EnableRayTracingUVCache"),
	true,
	TEXT("When enabled, PCG caches per-primitive UV data in a scene extension to avoid rebuilding it every ray trace dispatch."),
	ECVF_Default);

IMPLEMENT_SCENE_EXTENSION(FPCGRayTracingUVCacheExtension);

namespace PCGRayTracingUVCache
{
	void AddPrimitiveToCache(FPCGRayTracingUVCacheExtension& Extension, const FPrimitiveSceneInfo* PrimitiveSceneInfo)
	{
		FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo ? PrimitiveSceneInfo->Proxy : nullptr;
		if (!Proxy || !Proxy->HasRayTracingRepresentation())
		{
			return;
		}

		// Pass nullptr for View to force LOD 0 — PCG always wants highest-detail geometry.
		const FRaytracingVFAttributeData VFData = Proxy->GetRaytracingVFAttributeData(nullptr);
		if (VFData.Sections.IsEmpty() || !VFData.Sections[0].UV.BufferSRV)
		{
			return;
		}

		const int32 Index = PrimitiveSceneInfo->GetPersistentIndex().Index;

		FPCGRayTracingUVCacheExtension::FCachedEntry Entry;
		Entry.BufferSRV = VFData.Sections[0].UV.BufferSRV;
		Entry.NumCoordinates = VFData.Sections[0].UV.NumCoordinates;

		Extension.Cache.Add(Index, MoveTemp(Entry));

		// MaxPersistentIndex grows monotonically.
		Extension.MaxPersistentIndex = FMath::Max(Extension.MaxPersistentIndex, static_cast<uint32>(Index));
	}
}

class FPCGRayTracingUVCacheUpdater : public ISceneExtensionUpdater
{
	DECLARE_SCENE_EXTENSION_UPDATER(FPCGRayTracingUVCacheUpdater, FPCGRayTracingUVCacheExtension);

public:
	FPCGRayTracingUVCacheUpdater(FPCGRayTracingUVCacheExtension& InExtension)
		: Extension(InExtension)
	{
	}

	virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override
	{
		if (!Extension.bEnabled)
		{
			return;
		}

		LLM_SCOPE_BYTAG(PCGCompute);
		for (const FPersistentPrimitiveIndex& Id : ChangeSet.RemovedPrimitiveIds)
		{
			Extension.Cache.Remove(Id.Index);
		}
	}

	virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override
	{
		if (!Extension.bEnabled)
		{
			return;
		}

		LLM_SCOPE_BYTAG(PCGCompute);

		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ChangeSet.AddedPrimitiveSceneInfos)
		{
			PCGRayTracingUVCache::AddPrimitiveToCache(Extension, PrimitiveSceneInfo);
		}
	}

private:
	FPCGRayTracingUVCacheExtension& Extension;
};

FPCGRayTracingUVCacheExtension::FPCGRayTracingUVCacheExtension(FScene& InScene)
	: ISceneExtension(InScene)
{
}

bool FPCGRayTracingUVCacheExtension::ShouldCreateExtension(FScene& Scene)
{
	return IsRayTracingEnabled() && CVarPCGRayTracingUVCache.GetValueOnAnyThread();
}

ISceneExtensionUpdater* FPCGRayTracingUVCacheExtension::CreateUpdater()
{
	return new FPCGRayTracingUVCacheUpdater(*this);
}

#endif // RHI_RAYTRACING

void PCGRayTracingUVCache::RequestEnable_GameThread(FSceneInterface* SceneInterface)
{
#if RHI_RAYTRACING
	if (!SceneInterface)
	{
		return;
	}

	FScene* RenderScene = SceneInterface->GetRenderScene();
	if (!RenderScene)
	{
		return;
	}

	ENQUEUE_RENDER_COMMAND(EnablePCGRTUVCache)([RenderScene](FRHICommandListImmediate&)
	{
		FPCGRayTracingUVCacheExtension* Ext = RenderScene->GetExtensionPtr<FPCGRayTracingUVCacheExtension>();
		if (!Ext || Ext->IsEnabled())
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(PCGRayTracingUVCache::Initialize);
		LLM_SCOPE_BYTAG(PCGCompute);

		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : RenderScene->Primitives)
		{
			PCGRayTracingUVCache::AddPrimitiveToCache(*Ext, PrimitiveSceneInfo);
		}

		Ext->SetEnabled(true);

		UE_LOGF(LogPCGCompute, Verbose, "PCGRayTracingUVCache: Initialized with %d entries from %d scene primitives",
			Ext->Cache.Num(), RenderScene->Primitives.Num());
	});
#endif
}
