// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIResources.h"
#include "SceneExtensions.h"

#if RHI_RAYTRACING

/** Scene extension that caches per-primitive UV SRV pointers for PCG GPU ray tracing.
 *  Populated on-demand when the first runtime gen original component registers, then maintained incrementally by the scene updater.
 */
class FPCGRayTracingUVCacheExtension : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(PCGCOMPUTE_API, FPCGRayTracingUVCacheExtension);

public:
	struct FCachedEntry
	{
		FShaderResourceViewRHIRef BufferSRV = nullptr;
		uint32 NumCoordinates = 0;
	};

	explicit FPCGRayTracingUVCacheExtension(FScene& InScene);

	static bool ShouldCreateExtension(FScene& Scene);
	virtual ISceneExtensionUpdater* CreateUpdater() override;

	bool IsEnabled() const { return bEnabled; }
	void SetEnabled(bool bInEnabled) { bEnabled = bInEnabled; }

	/** Keyed by FPersistentPrimitiveIndex::Index. */
	TMap<int32, FCachedEntry> Cache;
	uint32 MaxPersistentIndex = 0;

private:
	friend class FPCGRayTracingUVCacheUpdater;

	/** Set after initial build completes. */
	bool bEnabled = false;
};

#endif // RHI_RAYTRACING
