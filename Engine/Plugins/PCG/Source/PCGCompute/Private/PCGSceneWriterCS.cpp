// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGSceneWriterCS.h"

void FPCGSceneWriterCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1); // Fixes errors that Primitive uniform buffer unbound.
	OutEnvironment.SetDefine(TEXT("USE_GPU_SCENE_DATA_RW"), 1); // Directs shader to use GPUSceneFrameNumber from global scope.
	OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), NumThreadsPerGroup);
}

IMPLEMENT_GLOBAL_SHADER(FPCGSceneWriterCS, "/PCGComputeShaders/PCGSceneWriter.usf", "SceneWriter_CS", SF_Compute);
