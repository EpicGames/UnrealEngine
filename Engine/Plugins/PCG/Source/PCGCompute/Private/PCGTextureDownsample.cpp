// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTextureDownsample.h"

#include "PCGComputeModule.h"

#include "GlobalShader.h"
#include "RHIGPUReadback.h"
#include "RenderGraphUtils.h"
#include "Async/Async.h"

#define LOCTEXT_NAMESPACE "PCGCompute"

class FPCGTextureDownsampleCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FPCGTextureDownsampleCS)
	SHADER_USE_PARAMETER_STRUCT(FPCGTextureDownsampleCS, FGlobalShader)

	class FDownsampleTextureArray : SHADER_PERMUTATION_BOOL("PCG_DOWNSAMPLE_TEXTUREARRAY");
	class FDownsampleSRGB : SHADER_PERMUTATION_BOOL("PCG_DOWNSAMPLE_SRGB");
	class FDownsampleMode : SHADER_PERMUTATION_INT("PCG_DOWNSAMPLE_MODE", 4);
	using FPermutationDomain = TShaderPermutationDomain<FDownsampleTextureArray, FDownsampleSRGB, FDownsampleMode>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FUintVector2, MipOutSize)
		SHADER_PARAMETER(FVector2f, TexelSize)
		SHADER_PARAMETER_SAMPLER(SamplerState, MipSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, MipInSRV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, MipOutUAV)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray, MipArrayInSRV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, MipArrayOutUAV)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FPCGTextureDownsampleCS, "/PCGComputeShaders/PCGTexureDownsample.usf", "TextureDownsampleCS", SF_Compute);

namespace PCGTextureDownsample
{
	void DownsampleTexture(FRDGBuilder& GraphBuilder, FParams& InParams)
	{
		FRDGTextureRef Texture = InParams.Texture;
		FRHISamplerState* Sampler = InParams.Sampler;
		check(Texture);
		check(Sampler);

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		const FRDGTextureDesc& TextureDesc = Texture->Desc;

		// Select compute shader variant (normal vs. sRGB etc.)
		const bool bMipsSRGB = EnumHasAnyFlags(TextureDesc.Flags, TexCreate_SRGB);
		FPCGTextureDownsampleCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FPCGTextureDownsampleCS::FDownsampleTextureArray>(InParams.NumSlices > 1);
		PermutationVector.Set<FPCGTextureDownsampleCS::FDownsampleSRGB>(bMipsSRGB);
		PermutationVector.Set<FPCGTextureDownsampleCS::FDownsampleMode>(static_cast<int>(InParams.Mode));
		TShaderMapRef<FPCGTextureDownsampleCS> ComputeShader(ShaderMap, PermutationVector);

		// Loop through each level of the mips that require creation and add a dispatch pass per level.
		for (uint8 MipLevel = 1, MipCount = TextureDesc.NumMips; MipLevel < MipCount; ++MipLevel)
		{
			const FUintVector2 DestTextureSize(
				FMath::Max(TextureDesc.Extent.X >> MipLevel, 1),
				FMath::Max(TextureDesc.Extent.Y >> MipLevel, 1));

			FPCGTextureDownsampleCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPCGTextureDownsampleCS::FParameters>();
			PassParameters->MipOutSize = DestTextureSize;
			PassParameters->TexelSize = FVector2f(1.0f / DestTextureSize.X, 1.0f / DestTextureSize.Y);
			PassParameters->MipSampler = Sampler;

			if (InParams.NumSlices > 1)
			{
				PassParameters->MipArrayInSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(Texture, MipLevel - 1));
				PassParameters->MipArrayOutUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Texture, MipLevel));
				PassParameters->MipInSRV = nullptr;
				PassParameters->MipOutUAV = nullptr;
			}
			else
			{
				PassParameters->MipInSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(Texture, MipLevel - 1));
				PassParameters->MipOutUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Texture, MipLevel));
				PassParameters->MipArrayInSRV = nullptr;
				PassParameters->MipArrayOutUAV = nullptr;
			}

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("GenerateMips DestMipLevel=%d", MipLevel),
				ComputeShader,
				PassParameters,
				FIntVector(FMath::DivideAndRoundUp<int>(DestTextureSize.X, 8), FMath::DivideAndRoundUp<int>(DestTextureSize.Y, 8), InParams.NumSlices));
		}
	}
}

#undef LOCTEXT_NAMESPACE
