// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTextureReadback.h"

#include "PCGComputeModule.h"

#include "GlobalShader.h"
#include "RHIGPUReadback.h"
#include "RenderGraphUtils.h"
#include "Async/Async.h"

#define LOCTEXT_NAMESPACE "PCGCompute"
#define PCG_NUM_THREADS_PER_GROUP_DIMENSION 8

class PCGCOMPUTE_API FPCGTextureReadbackCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FPCGTextureReadbackCS);
	SHADER_USE_PARAMETER_STRUCT(FPCGTextureReadbackCS, FGlobalShader);
 
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D<float4>, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2DArray<float4>, SourceTextureArray)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceSampler)
		SHADER_PARAMETER(FVector2f, SourceDimensions)
		SHADER_PARAMETER(int32, SourceTextureIndex)
		SHADER_PARAMETER_UAV(RWTexture2D<float4>, OutputTexture)
	END_SHADER_PARAMETER_STRUCT()
 
public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), PCG_NUM_THREADS_PER_GROUP_DIMENSION);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), PCG_NUM_THREADS_PER_GROUP_DIMENSION);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Z"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FPCGTextureReadbackCS, "/PCGComputeShaders/PCGTextureReadback.usf", "PCGTextureReadback_CS", SF_Compute);

void FPCGTextureReadbackInterface::Dispatch_RenderThread(FRHICommandListImmediate& RHICmdList, const FPCGTextureReadbackDispatchParams& Params, const TFunction<void(void* OutBuffer, int32 ReadbackWidth, int32 ReadbackHeight)>& AsyncCallback)
{
	LLM_SCOPE_BYTAG(PCGCompute);
	check(Params.SourceTexture && Params.SourceSampler);

	const bool bIsTextureArray = Params.SourceTexture->GetDesc().Dimension == ETextureDimension::Texture2DArray;

	FPCGTextureReadbackCS::FParameters PassParameters;
	PassParameters.SourceSampler = Params.SourceSampler;
	PassParameters.SourceDimensions = { (float)Params.SourceDimensions.X, (float)Params.SourceDimensions.Y };

	if (bIsTextureArray)
	{
		const FRHITextureCreateDesc DummyTextureDesc =
			FRHITextureCreateDesc::Create2D(TEXT("PCGDummyTexture"), 1, 1, PF_G8)
			.SetFlags(ETextureCreateFlags::ShaderResource);

		FTextureRHIRef DummyTexture = RHICmdList.CreateTexture(DummyTextureDesc);

		{
			uint32 DestStride;
			uint8* DestBuffer = (uint8*)RHICmdList.LockTexture2D(DummyTexture, 0, RLM_WriteOnly, DestStride, false);
			*DestBuffer = 0;
			RHICmdList.UnlockTexture2D(DummyTexture, 0, false);
		}

		PassParameters.SourceTexture = MoveTemp(DummyTexture);
		PassParameters.SourceTextureArray = Params.SourceTexture;
		PassParameters.SourceTextureIndex = Params.SourceTextureIndex;
	}
	else
	{
		const FRHITextureCreateDesc DummyTextureDesc =
			FRHITextureCreateDesc::Create2DArray(TEXT("PCGDummyTextureArray"), 1, 1, 1, PF_G8)
			.SetFlags(ETextureCreateFlags::ShaderResource);

		FTextureRHIRef DummyTexture = RHICmdList.CreateTexture(DummyTextureDesc);

		{
			uint32 DestStride;
			uint8* DestBuffer = (uint8*)RHICmdList.LockTexture2DArray(DummyTexture, 0, 0, RLM_WriteOnly, DestStride, false);
			*DestBuffer = 0;
			RHICmdList.UnlockTexture2DArray(DummyTexture, 0, 0, false);
		}

		PassParameters.SourceTexture = Params.SourceTexture;
		PassParameters.SourceTextureArray = MoveTemp(DummyTexture);
		PassParameters.SourceTextureIndex = -1;
	}

	FRHITextureCreateDesc TargetTextureDesc =
		FRHITextureCreateDesc::Create2D(TEXT("PCGTexture Readback Compute Target"), Params.SourceDimensions.X, Params.SourceDimensions.Y, Params.OutputFormat)
			.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::UAV |
				ETextureCreateFlags::ShaderResource |
				ETextureCreateFlags::NoTiling
			)
			.SetInitialState(ERHIAccess::UAVCompute)
			.DetermineInititialState();
	check(TargetTextureDesc.IsValid());
	
	// Create temporary output texture
	FTextureRHIRef OutputTexture = RHICmdList.CreateTexture(TargetTextureDesc);
	PassParameters.OutputTexture = RHICmdList.CreateUnorderedAccessView(OutputTexture, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(OutputTexture));

	TShaderMapRef<FPCGTextureReadbackCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, PassParameters,
		FIntVector(FMath::DivideAndRoundUp(Params.SourceDimensions.X, PCG_NUM_THREADS_PER_GROUP_DIMENSION),
			FMath::DivideAndRoundUp(Params.SourceDimensions.Y, PCG_NUM_THREADS_PER_GROUP_DIMENSION), 1));

	// Prepare OutputTexture to be copied
	RHICmdList.Transition(FRHITransitionInfo(OutputTexture, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));

	TUniquePtr<FRHIGPUTextureReadback> GPUTextureReadback = MakeUnique<FRHIGPUTextureReadback>(TEXT("PCGTextureReadbackCopy"));
	GPUTextureReadback->EnqueueCopy(RHICmdList, PassParameters.OutputTexture->GetTexture());

	auto ExecuteAsync = [](auto&& RunnerFunc) -> void
	{
		if(IsInActualRenderingThread())
		{
			AsyncTask(ENamedThreads::ActualRenderingThread, [RunnerFunc]()
			{
				RunnerFunc(RunnerFunc);
			});
		}
		else
		{
			// In specific cases (Server, -onethread, etc) the RenderingThread is actually the same as the GameThread.
			// When this happens we want to avoid calling AsyncTask which could put us in a infinite task execution loop. 
			// The reason is that if we are running this callback through the task graph we might stay in an executing loop until it has no tasks to execute,
			// since we are pushing a new task as long as our data isn't ready and we are not advancing the GameThread as we are already on the GameThread this causes a infinite task execution.
			// Instead delay to GameThread with ExecuteOnGameThread
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [RunnerFunc]()
			{
				RunnerFunc(RunnerFunc);
			});
		}
	};

	auto RunnerFunc = [GPUTextureReadBackPtr = GPUTextureReadback.Release(), AsyncCallback, ExecuteAsync](auto&& RunnerFunc) -> void
	{
		LLM_SCOPE_BYTAG(PCGCompute);

		if (GPUTextureReadBackPtr->IsReady())
		{
			// NOTE: Lock() returns the row pitch in pixels as ReadbackWidth, which may be larger than the source texture width due to GPU alignment (e.g. D3D12 256-byte row pitch).
			// Call sites must capture SourceDimensions and use ReadbackWidth only as a row stride when unpacking the buffer.
			// See PCGTextureData and PCGSaveTextureToAsset for examples.
			// @todo_pcg: Maybe we should just unpack the buffer here, but it would cost some perf.
			int32 ReadbackWidth = 0, ReadbackHeight = 0;
			void* OutBuffer = GPUTextureReadBackPtr->Lock(ReadbackWidth, &ReadbackHeight);

			AsyncCallback(OutBuffer, ReadbackWidth, ReadbackHeight);

			GPUTextureReadBackPtr->Unlock();
			delete GPUTextureReadBackPtr;
		}
		else
		{
			ExecuteAsync(RunnerFunc);
		}
	};

	ExecuteAsync(RunnerFunc);
}

void FPCGTextureReadbackInterface::Dispatch_GameThread(const FPCGTextureReadbackDispatchParams& Params, const TFunction<void(void* OutBuffer, int32 ReadbackWidth, int32 ReadbackHeight)>& AsyncCallback)
{
	ENQUEUE_RENDER_COMMAND(SceneDrawCompletion)(
		[Params, AsyncCallback](FRHICommandListImmediate& RHICmdList)
		{
			Dispatch_RenderThread(RHICmdList, Params, AsyncCallback);
		});
}

void FPCGTextureReadbackInterface::Dispatch(const FPCGTextureReadbackDispatchParams& Params, const TFunction<void(void* OutBuffer, int32 ReadbackWidth, int32 ReadbackHeight)>& AsyncCallback)
{
	if (IsInRenderingThread())
	{
		Dispatch_RenderThread(FRHICommandListImmediate::Get(), Params, AsyncCallback);
	}
	else
	{
		Dispatch_GameThread(Params, AsyncCallback);
	}
}

#undef LOCTEXT_NAMESPACE
#undef PCG_NUM_THREADS_PER_GROUP_DIMENSION
