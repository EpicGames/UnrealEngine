// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGUnwrapMesh.h"

#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "RHIStaticStates.h"
#include "RenderGraphBuilder.h"
#include "ShaderParameterStruct.h"
#include "StaticMeshResources.h"

DEFINE_LOG_CATEGORY_STATIC(LogPCGUnwrapMesh, Log, All);

namespace PCGUnwrapMesh
{
	class FPCGUnwrapMeshVS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FPCGUnwrapMeshVS);
		SHADER_USE_PARAMETER_STRUCT(FPCGUnwrapMeshVS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		END_SHADER_PARAMETER_STRUCT()
	};

	class FPCGUnwrapMeshPS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FPCGUnwrapMeshPS);
		SHADER_USE_PARAMETER_STRUCT(FPCGUnwrapMeshPS, FGlobalShader);

		class FAttribute : SHADER_PERMUTATION_INT("PCG_MESH_ATTRIBUTE", static_cast<int32>(EMeshAttribute::Num));
		using FPermutationDomain = TShaderPermutationDomain<FAttribute>;

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("PCG_MESH_ATTRIBUTE_LOCALPOSITION"), static_cast<int32>(EMeshAttribute::LocalPosition));
			OutEnvironment.SetDefine(TEXT("PCG_MESH_ATTRIBUTE_MASK"), static_cast<int32>(EMeshAttribute::Mask));
		}
	};
}

IMPLEMENT_GLOBAL_SHADER(PCGUnwrapMesh::FPCGUnwrapMeshVS, "/PCGComputeShaders/PCGUnwrapMesh.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(PCGUnwrapMesh::FPCGUnwrapMeshPS, "/PCGComputeShaders/PCGUnwrapMesh.usf", "MainPS", SF_Pixel);

namespace PCGUnwrapMesh
{
	static FVertexDeclarationRHIRef BuildVertexDeclaration(bool bFullPrecisionUVs, uint32 NumTexCoords, uint32 UVChannelIndex)
	{
		// Layout is invariant:
		// * PositionVertexBuffer is packed FVector3f
		// * TexCoordVertexBuffer is packed Half2/Float2 UVs with stride = NumTexCoords * ElementSize.

		const uint32 UVElementSize = bFullPrecisionUVs ? sizeof(FVector2f) : sizeof(FVector2DHalf);
		const uint32 UVStride = NumTexCoords * UVElementSize;
		const uint32 UVOffset = UVChannelIndex * UVElementSize;

		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(/*StreamIndex=*/0, /*Offset=*/0, VET_Float3, /*AttributeIndex=*/0, /*Stride=*/sizeof(FVector3f)));
		Elements.Add(FVertexElement(/*StreamIndex=*/1, UVOffset, bFullPrecisionUVs ? VET_Float2 : VET_Half2, /*AttributeIndex=*/1, UVStride));
		return PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	void FUnwrapParams::InitFromLOD(const FStaticMeshLODResources& LOD)
	{
		PositionBufferRHI = LOD.VertexBuffers.PositionVertexBuffer.VertexBufferRHI;
		TexCoordBufferRHI = LOD.VertexBuffers.StaticMeshVertexBuffer.TexCoordVertexBuffer.VertexBufferRHI;
		IndexBufferRHI = LOD.IndexBuffer.IndexBufferRHI;
		NumVerts = LOD.GetNumVertices();
		NumTris = LOD.GetNumTriangles();
		NumTexCoords = LOD.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
		bFullPrecisionUVs = LOD.VertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs();
	}

	bool ValidateParams(const FUnwrapParams& Params)
	{
		if (!Params.PositionBufferRHI || !Params.TexCoordBufferRHI || !Params.IndexBufferRHI)
		{
			UE_LOG(LogPCGUnwrapMesh, Error, TEXT("ValidateParams: RHI buffer ref null."));
			return false;
		}

		// FRHIBuffer can exist with zero-size GPU storage (streamed-out LOD or CPU-data-discarded mesh).
		if (Params.PositionBufferRHI->GetSize() == 0 || Params.TexCoordBufferRHI->GetSize() == 0 || Params.IndexBufferRHI->GetSize() == 0)
		{
			UE_LOG(LogPCGUnwrapMesh, Error, TEXT("ValidateParams: GPU storage zero-sized."));
			return false;
		}

		return true;
	}

	bool AddUnwrapMeshPass(FRDGBuilder& GraphBuilder, FRDGTextureRef OutputTexture, const FUnwrapParams& Params)
	{
		if (!Params.PositionBufferRHI || !Params.TexCoordBufferRHI || !Params.IndexBufferRHI || Params.NumTris == 0)
		{
			UE_LOG(LogPCGUnwrapMesh, Error,
				TEXT("PCGUnwrapMesh::AddUnwrapMeshPass: invalid params (Position=%p, TexCoord=%p, Index=%p, NumTris=%u)"),
				Params.PositionBufferRHI.GetReference(), Params.TexCoordBufferRHI.GetReference(), Params.IndexBufferRHI.GetReference(), Params.NumTris);
			return false;
		}

		if (Params.Resolution.X <= 0 || Params.Resolution.Y <= 0)
		{
			UE_LOG(LogPCGUnwrapMesh, Error, TEXT("PCGUnwrapMesh::AddUnwrapMeshPass: invalid Resolution=%dx%d"), Params.Resolution.X, Params.Resolution.Y);
			return false;
		}

		if (static_cast<uint8>(Params.Attribute) >= static_cast<uint8>(EMeshAttribute::Num))
		{
			UE_LOG(LogPCGUnwrapMesh, Error, TEXT("PCGUnwrapMesh::AddUnwrapMeshPass: invalid Attribute=%u (Num=%u)"), static_cast<uint8>(Params.Attribute), static_cast<uint8>(EMeshAttribute::Num));
			return false;
		}

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		FPCGUnwrapMeshPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FPCGUnwrapMeshPS::FAttribute>(static_cast<int32>(Params.Attribute));

		TShaderMapRef<FPCGUnwrapMeshVS> VertexShader(ShaderMap);
		TShaderMapRef<FPCGUnwrapMeshPS> PixelShader(ShaderMap, PermutationVector);

		FPCGUnwrapMeshPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPCGUnwrapMeshPS::FParameters>();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::EClear);

		FVertexDeclarationRHIRef VertexDecl = BuildVertexDeclaration(Params.bFullPrecisionUVs, Params.NumTexCoords, Params.UVChannelIndex);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("PCGBakeMeshAttr(%dx%d, %u tris)", Params.Resolution.X, Params.Resolution.Y, Params.NumTris),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, VertexShader, PixelShader, Params, VertexDecl](FRHICommandList& RHICmdList)
			{
				FPCGUnwrapMeshVS::FParameters VSParams;
				FPCGUnwrapMeshPS::FParameters PSParams = *PassParameters;

				FGraphicsPipelineStateInitializer PSOInit;
				RHICmdList.ApplyCachedRenderTargets(PSOInit);
				PSOInit.BlendState = TStaticBlendState<>::GetRHI();
				PSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
				PSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				PSOInit.BoundShaderState.VertexDeclarationRHI = VertexDecl;
				PSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				PSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				PSOInit.PrimitiveType = PT_TriangleList;
				SetGraphicsPipelineState(RHICmdList, PSOInit, /*StencilRef=*/0);

				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSParams);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PSParams);

				RHICmdList.SetStreamSource(/*StreamIndex=*/0, Params.PositionBufferRHI, /*Offset=*/0);
				RHICmdList.SetStreamSource(/*StreamIndex=*/1, Params.TexCoordBufferRHI, /*Offset=*/0);
				RHICmdList.SetViewport(/*MinX=*/0, /*MinY=*/0, /*MinZ=*/0.0f, Params.Resolution.X, Params.Resolution.Y, /*MaxZ=*/1.0f);
				RHICmdList.DrawIndexedPrimitive(Params.IndexBufferRHI, /*BaseVertexIndex=*/0, /*FirstInstance=*/0, Params.NumVerts, /*StartIndex=*/0, Params.NumTris, /*NumInstances=*/1);
			});

		return true;
	}
}
