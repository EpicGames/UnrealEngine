// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"

// Logs
PCGCOMPUTE_API DECLARE_LOG_CATEGORY_EXTERN(LogPCGCompute, Log, All);

// Stats
DECLARE_STATS_GROUP(TEXT("PCGCompute"), STATGROUP_PCGCompute, STATCAT_Advanced)

LLM_DECLARE_TAG(PCGCompute);
