// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"

#define UE_API PCGCOMPUTE_API
 
/**
 * Parameters to drive execution of a GPU texture readback. These mirror the fields in the texture readback compute shader (PCGTextureReadback.usf).
 */
struct FPCGTextureReadbackDispatchParams
{
	/** Source texture to sample from. Can be a UTexture2D or UTexture2DArray. */
	FTextureRHIRef SourceTexture;

	/** Sampler used to sample the SourceTexture. Should use an SF_Point filter for precise per-pixel readback. */
	FSamplerStateRHIRef SourceSampler;

	/** Width and Height of the SourceTexture. Should match the underlying dimensions exactly for precise per-pixel readback. */
	FIntPoint SourceDimensions;

	/** Optional texture index. Should be 0 if the SourceTexture is not a texture array. */
	uint32 SourceTextureIndex = 0;

	/** Pixel format for the staging texture used during readback. Must be a 4-channel uncompressed format (e.g. PF_B8G8R8A8, PF_FloatRGBA, PF_A32B32G32R32F). */
	EPixelFormat OutputFormat = PF_B8G8R8A8;
};
 
/**
 * API for dispatching TextureReadback operations to the GPU.
 */
class FPCGTextureReadbackInterface
{
public:
	static UE_API void Dispatch_RenderThread(FRHICommandListImmediate& RHICmdList, const FPCGTextureReadbackDispatchParams& Params, const TFunction<void(void* OutBuffer, int32 ReadbackWidth, int32 ReadbackHeight)>& AsyncCallback);
	static UE_API void Dispatch_GameThread(const FPCGTextureReadbackDispatchParams& Params, const TFunction<void(void* OutBuffer, int32 ReadbackWidth, int32 ReadbackHeight)>& AsyncCallback);

	/** Dispatches the texture readback compute shader. Can be called from any thread. */
	static UE_API void Dispatch(const FPCGTextureReadbackDispatchParams& Params, const TFunction<void(void* OutBuffer, int32 ReadbackWidth, int32 ReadbackHeight)>& AsyncCallback);
};

#undef UE_API
