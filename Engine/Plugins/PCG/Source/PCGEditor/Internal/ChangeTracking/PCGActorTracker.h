// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ChangeTracking/PCGChangeTrackingRegistry.h"

#include "Misc/Guid.h"
#include "UObject/ObjectKey.h"

class FPCGTrackingManager;

class AActor;
class FTransactionObjectEvent;
class ILevelInstanceInterface;
class UObject;

namespace PCGActorTracker
{
	static const FPCGTrackingChangeID ActorDeleted(0x99FF9E41, 0x3940482C, 0xBD5F0BA0, 0x7DAE68EE);
	static const FPCGTrackingChangeID ActorAdded(0x10A0FD3D, 0x9A8546B8, 0xA7D5FB5F, 0xCD398DC1);
	static const FPCGTrackingChangeID ActorChanged(0xD822BD05, 0xE0B94C35, 0xA8A36F1D, 0x7A05F444);
}

class FPCGActorTracker : public IPCGChangeTracker
{
public:
	virtual ~FPCGActorTracker();

	static TUniquePtr<IPCGChangeTracker> MakeInstance(FPCGTrackingManager* InOwner);
	static FName GetName();

	virtual void Tick() override;

	virtual bool OnObjectPropertyChanged(const UObject* InObject, FPropertyChangedEvent& InEvent) override;
	virtual bool OnNotifyObjectChanged(const UObject* InObject, const FPCGTrackingChangeID& InChangeID, const UObject* InOriginatingChangeObject) override;

private:
	// Typedef to store the previous position and previous tags of changed actors.
	using FActorPreviousData = TTuple<FBox, TSet<FName>, double>;

	FPCGActorTracker(FPCGTrackingManager* InOwner);

	bool IsActorTracked(const AActor* InActor) const;
	bool ShouldDelayActor(const AActor* InActor) const;

	void OnLevelInstancesUpdated(const TArray<ILevelInstanceInterface*>& InLevelInstances);
	void OnLevelInstanceEditCancelled(ILevelInstanceInterface* InLevelInstance, bool bInHasDiscardedChanges);
	void OnActorLoaded(AActor& InActor);
	void OnActorAdded(AActor* InActor);
	void OnActorAdded_Internal(const AActor* InActor, bool bShouldDirty);
	void OnActorChanged_Internal(const AActor* InActor, const UObject* InOriginatingChangeObject, const FPCGTrackingChangeID& InChangeID);
	void OnActorChanged_Recursive(const AActor* InActor, const UObject* InOriginatingChangeObject, const FPCGTrackingChangeID& InChangeID);
	void OnActorChanged_Impl(const AActor* InActor, const UObject* InOriginatingChangeObject, const FPCGTrackingChangeID& InChangeID, FActorPreviousData* InPreviousData);
	void OnActorUnloaded(AActor& InActor);
	void OnActorDeleted(AActor* InActor);
	void OnActorDeleted_Internal(const AActor* InActor, bool bShouldDirty);
	void OnObjectModified(UObject* InObject);
	void OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& Event);
	
	static const FLazyName Name;

	// Keep track of actors that aren't yet ready (or if the subsystem is not yet ready), whether we should dirty them in next tick.
	TMap<TObjectKey<AActor>, TTuple<TObjectKey<UObject>, FPCGTrackingChangeID>> DelayedChangedActors;

	/** Transient map of owners and their previous data, it's set in the pre object change to be able to track changes (such as tags or positions) */
	TMap<TObjectKey<AActor>, FActorPreviousData> ActorToPreviousDataMap;

	// Time keeper for cleaning up cached previous actor data
	double LastPreviousActorDataCleanup = -1.0;
};

#endif // WITH_EDITOR