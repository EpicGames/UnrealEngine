// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "PCGComponent.h"
#include "PCGGraphExecutionStateInterface.h"
#include "ChangeTracking/PCGChangeTrackingRegistry.h"

class FPCGTrackingManager;

class FPCGComponentChangeHandler : public IPCGChangeHandler
{
public:
	virtual ~FPCGComponentChangeHandler();

	static TUniquePtr<IPCGChangeHandler> MakeInstance(FPCGTrackingManager* InOwner);
	static FName GetName();

	virtual void BeginChangeHandling(const TSharedRef<FPCGChangeHandlerChange>& InChange) override;
	virtual void HandleChange(IPCGGraphExecutionSource* InExecutionSource) override;
	virtual void HandleBoundedChange(IPCGGraphExecutionSource* InExecutionSource, const FBox& InExecutionSourceBounds, const FBox& InChangeBounds) override;
	virtual void EndChangeHandling(bool bSkipRefresh) override;

private:
	FPCGComponentChangeHandler(FPCGTrackingManager* InOwner);

	bool ShouldDiscardComponent(UPCGComponent* InComponent) const;

	struct FCurrentChange
	{
		TWeakPtr<FPCGChangeHandlerChange> Change;

		TArray<const UObject*> ChangedObjects;

		TOptional<FPCGDynamicTrackingPriority> ChangedObjectDynamicTrackingPriority;
		EPCGComponentDirtyFlag DirtyFlag = EPCGComponentDirtyFlag::None;
		TSet<UPCGComponent*> DirtyComponents;
	};

	TUniquePtr<FCurrentChange> CurrentChange;
	
	static const FLazyName Name;
};

#endif // WITH_EDITOR