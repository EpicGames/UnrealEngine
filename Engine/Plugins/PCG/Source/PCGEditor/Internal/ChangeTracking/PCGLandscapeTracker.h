// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ChangeTracking/PCGChangeTrackingRegistry.h"

#include "UObject/ObjectKey.h"

class FPCGTrackingManager;

class ALandscapeProxy;
class FLandscapeProxyComponentDataChangedParams;

typedef FName FEditorModeID;

class FPCGLandscapeTracker : public IPCGChangeTracker
{
public:
	virtual ~FPCGLandscapeTracker();

	static TUniquePtr<IPCGChangeTracker> MakeInstance(FPCGTrackingManager* InOwner);
	static FName GetName();

	virtual void Tick() override;
	virtual bool ShouldSkipRefresh(const UObject* InChangedObject) override;

private:
	FPCGLandscapeTracker(FPCGTrackingManager* InOwner);

	void OnLandscapeChanged(ALandscapeProxy* InLandscape, const FLandscapeProxyComponentDataChangedParams& InChangeParams);
	void ApplyLandscapeChanges(ALandscapeProxy* InLandscape);
	void OnEditorModeIDChanged(const FEditorModeID& EditorModeID, bool bIsEntering);
	void OnLandscapeEditModeExited();

	static const FLazyName Name;

	// Part for the delayed landscape change update
	double LastLandscapeDirtyTime = -1.0;
	TArray<TObjectKey<ALandscapeProxy>> DelayedModifiedLandscapes;

	// Keep track of all dirtied landscapes and force a refresh on them when exiting edit mode.
	TArray<TObjectKey<ALandscapeProxy>> DirtiedLandscapes;
	
	// Temp boolean to indicate we are currently exiting the landscape edit mode
	bool bIsCurrentlyExitingLandscapeEditMode = false;
};

#endif // WITH_EDITOR