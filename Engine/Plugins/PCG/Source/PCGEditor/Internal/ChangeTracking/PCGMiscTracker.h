// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ChangeTracking/PCGChangeTrackingRegistry.h"

#include "UObject/ObjectSaveContext.h"

class FPCGTrackingManager;
class UObject;

// For now this is a "everything" else tracker that tracks specific asset type saves
class FPCGMiscTracker : public IPCGChangeTracker
{
public:
	virtual ~FPCGMiscTracker();

	static TUniquePtr<IPCGChangeTracker> MakeInstance(FPCGTrackingManager* InOwner);
	static FName GetName();

private:
	FPCGMiscTracker(FPCGTrackingManager* InOwner);

	void OnObjectSaved(UObject* InObject, FObjectPreSaveContext InObjectSaveContext);

	static const FLazyName Name;
};

#endif // WITH_EDITOR