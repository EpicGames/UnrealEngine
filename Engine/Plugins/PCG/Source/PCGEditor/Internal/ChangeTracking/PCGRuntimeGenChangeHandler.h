// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ChangeTracking/PCGChangeTrackingRegistry.h"

class FPCGTrackingManager;
class IPCGGraphExecutionSource;
class UPCGRuntimeGenExecutionSource;

class FPCGRuntimeGenChangeHandler : public IPCGChangeHandler
{
public:
	static TUniquePtr<IPCGChangeHandler> MakeInstance(FPCGTrackingManager* InOwner);
	static FName GetName();

	//~ Begin IPCGChangeHandler Interface
	virtual void BeginChangeHandling(const TSharedRef<FPCGChangeHandlerChange>& InChange) override;
	virtual void HandleChange(IPCGGraphExecutionSource* InExecutionSource) override;
	virtual void HandleBoundedChange(IPCGGraphExecutionSource* InExecutionSource, const FBox& InExecutionSourceBounds, const FBox& InChangeBounds) override;
	virtual void EndChangeHandling(bool bSkipRefresh) override;
	//~ End IPCGChangeHandler Interface

private:
	FPCGRuntimeGenChangeHandler(FPCGTrackingManager* InOwner);

	void HandleChangeInternal(IPCGGraphExecutionSource* InExecutionSource, const FBox* InExecutionSourceBounds = nullptr, const FBox* InChangeBounds = nullptr);
	bool ShouldDiscardExecutionSource(UPCGRuntimeGenExecutionSource* InExecutionSource) const;

	struct FCurrentChange
	{
		TWeakPtr<FPCGChangeHandlerChange> Change;
		TArray<const UObject*> ChangedObjects;
		TSet<UPCGRuntimeGenExecutionSource*> DirtyExecutionSources;
	};

	TUniquePtr<FCurrentChange> CurrentChange;
};

#endif // WITH_EDITOR
