// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PCGEditor : ModuleRules
	{
		public PCGEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"Projects",
					"Engine",
					"CoreUObject",
					"PlacementMode", 
					"PCG",
					"StructUtilsEditor",
					"DataHierarchyEditor"
				});

			if (Target.WithAutomationTests)
			{
				PublicDependencyModuleNames.AddRange(
					new string[]
					{
						"LevelEditor"
					});
			}

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AdvancedPreviewScene",
					"AppFramework",
					"ApplicationCore",
					"AssetDefinition",
					"AssetRegistry",
					"AssetTools",
					"BlueprintGraph",
					"ClassViewer",
					"ComponentVisualizers",
					"ContentBrowser",
					"ContentBrowserData",
					"DesktopWidgets",
					"DetailCustomizations",
					"DeveloperSettings",
					"EditorFramework",
					"EditorScriptingUtilities",
					"EditorStyle",
					"EditorSubsystem",
					"EditorWidgets",
					"GameProjectGeneration",
					"GraphEditor",
					"InputCore",
					"InteractiveToolsFramework",
					"Kismet",
					"KismetWidgets",
					"LevelEditor",
					"PropertyEditor",
					"RHI",
					"RenderCore",
					"SceneOutliner",
					"ScriptableEditorWidgets",
					"Slate",
					"SlateCore",
					"SourceControl",
					"StatusBar", 
					"SubobjectEditor",
					"SubobjectDataInterface",
					"ToolMenus",
					"ToolWidgets",
					"TypedElementFramework",
					"TypedElementRuntime",
					"UnrealEd",
					"WidgetRegistration", 
					"ModelingComponents",
					"ModelingComponentsEditorOnly",
					"Landscape",
					"GeometryCore",
					"GeometryFramework"
				});

			// This module needs AutoRTFM so disable the auto disable since its in a plugin.
			// The call to the delegate `OnObjectModified` calls into here.
			bDisableAutoRTFMInstrumentation = false;
		}
	}
}
