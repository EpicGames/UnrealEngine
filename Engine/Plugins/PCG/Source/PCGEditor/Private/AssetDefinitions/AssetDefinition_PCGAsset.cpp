// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PCGAsset.h"

#include "PCGDataAsset.h"
#include "PCGEditorCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PCGAsset)

#define LOCTEXT_NAMESPACE "AssetDefinition_PCGAsset"

FText UAssetDefinition_PCGAsset::GetAssetDisplayName() const
{
	return LOCTEXT("DisplayName", "PCG Data Asset");
}

FLinearColor UAssetDefinition_PCGAsset::GetAssetColor() const
{
	return FColor::Emerald;
}

TSoftClassPtr<UObject> UAssetDefinition_PCGAsset::GetAssetClass() const
{
	return UPCGDataAsset::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_PCGAsset::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FPCGEditorCommon::PCGAssetCategoryPath };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
