// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PCGBuilderSettings.h"

#include "PCGEditorCommon.h"

#include "WorldPartitionBuilder/PCGBuilderSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PCGBuilderSettings)

#define LOCTEXT_NAMESPACE "AssetDefinition_PCGBuilderSettings"

FText UAssetDefinition_PCGBuilderSettings::GetAssetDisplayName() const
{
	return LOCTEXT("DisplayName", "PCG Builder Settings");
}

FLinearColor UAssetDefinition_PCGBuilderSettings::GetAssetColor() const
{
	return FColor::Purple;
}

TSoftClassPtr<UObject> UAssetDefinition_PCGBuilderSettings::GetAssetClass() const
{
	return UPCGBuilderSettings::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_PCGBuilderSettings::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FPCGEditorCommon::PCGAdvancedAssetCategoryPath };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
