// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PCGComputeSource.h"

#include "Compute/PCGComputeSource.h"

#include "PCGEditorCommon.h"

#include "AssetToolsModule.h"
#include "Misc/FileHelper.h"
#include "Settings/EditorLoadingSavingSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PCGComputeSource)

#define LOCTEXT_NAMESPACE "AssetDefinition_PCGComputeSource"

FText UAssetDefinition_PCGComputeSource::GetAssetDisplayName() const
{
	return LOCTEXT("DisplayName", "PCG Compute Source");
}

FLinearColor UAssetDefinition_PCGComputeSource::GetAssetColor() const
{
	return FColor::Orange;
}

TSoftClassPtr<UObject> UAssetDefinition_PCGComputeSource::GetAssetClass() const
{
	return UPCGComputeSource::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_PCGComputeSource::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FPCGEditorCommon::PCGAdvancedAssetCategoryPath };
	return Categories;
}

EAssetCommandResult UAssetDefinition_PCGComputeSource::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	const UPCGComputeSource* OldSource = Cast<UPCGComputeSource>(DiffArgs.OldAsset);
	const UPCGComputeSource* NewSource = Cast<UPCGComputeSource>(DiffArgs.NewAsset);
	if (OldSource == nullptr && NewSource == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}

	// Obtain temp text file names.
	FString OldTextFilename;
	{
		const FString AssetName = OldSource ? OldSource->GetName() : TEXT("empty");

		if (DiffArgs.OldRevision.Changelist != -1)
		{
			OldTextFilename = FPaths::ConvertRelativePathToFull(FString::Printf(TEXT("%s%s-%d.txt"), *FPaths::DiffDir(), *AssetName, DiffArgs.OldRevision.Changelist));
		}
		else
		{
			OldTextFilename = FPaths::ConvertRelativePathToFull(FString::Printf(TEXT("%s%s-A.txt"), *FPaths::DiffDir(), *AssetName));
		}
	}

	FString NewTextFilename;
	{
		const FString AssetName = NewSource ? NewSource->GetName() : TEXT("empty");

		if (DiffArgs.NewRevision.Changelist != -1)
		{
			NewTextFilename = FPaths::ConvertRelativePathToFull(FString::Printf(TEXT("%s%s-%d.txt"), *FPaths::DiffDir(), *AssetName, DiffArgs.NewRevision.Changelist));
		}
		else
		{
			NewTextFilename = FPaths::ConvertRelativePathToFull(FString::Printf(TEXT("%s%s-B.txt"), *FPaths::DiffDir(), *AssetName));
		}
	}

	// Save source to files for diff.
	FFileHelper::SaveStringToFile(OldSource ? OldSource->GetSource() : TEXT(""), *OldTextFilename);
	FFileHelper::SaveStringToFile(NewSource ? NewSource->GetSource() : TEXT(""), *NewTextFilename);

	// Run diff tool.
	const FString DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;
	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetTools.CreateDiffProcess(DiffCommand, OldTextFilename, NewTextFilename);

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
