// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PCGGraph.h"

#include "PCGEditor.h"
#include "PCGGraph.h"
#include "Widgets/SPCGEditorGraphDiff.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PCGGraph)

#define LOCTEXT_NAMESPACE "AssetDefinition_PCGGraph"

FText UAssetDefinition_PCGGraph::GetAssetDisplayName() const
{
	return LOCTEXT("DisplayName", "PCG Graph");
}

TSoftClassPtr<UObject> UAssetDefinition_PCGGraph::GetAssetClass() const
{
	return UPCGGraph::StaticClass();
}

EAssetCommandResult UAssetDefinition_PCGGraph::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FPCGEditor::OpenAssets(OpenArgs);
	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_PCGGraph::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	const UPCGGraph* OldGraph = Cast<UPCGGraph>(DiffArgs.OldAsset);
	const UPCGGraph* NewGraph = Cast<UPCGGraph>(DiffArgs.NewAsset);

	if (NewGraph == nullptr && OldGraph == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}

	SPCGEditorGraphDiff::CreateDiffWindow(OldGraph, NewGraph, DiffArgs.OldRevision, DiffArgs.NewRevision);
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
