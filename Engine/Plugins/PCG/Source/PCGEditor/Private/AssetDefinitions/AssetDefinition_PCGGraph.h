// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinition_PCGGraphInterface.h"

#include "AssetDefinition_PCGGraph.generated.h"

UCLASS()
class UAssetDefinition_PCGGraph : public UAssetDefinition_PCGGraphInterface
{
	GENERATED_BODY()

public:
	virtual FText GetAssetDisplayName() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
};
