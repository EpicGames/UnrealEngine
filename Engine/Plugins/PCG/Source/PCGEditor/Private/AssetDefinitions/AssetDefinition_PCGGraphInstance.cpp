// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PCGGraphInstance.h"

#include "PCGGraph.h"

#include "SDetailsDiff.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PCGGraphInstance)

#define LOCTEXT_NAMESPACE "AssetDefinition_PCGGraphInstance"

FText UAssetDefinition_PCGGraphInstance::GetAssetDisplayName() const
{
	return LOCTEXT("DisplayName", "PCG Graph Instance");
}

FLinearColor UAssetDefinition_PCGGraphInstance::GetAssetColor() const
{
	FLinearColor Color = UAssetDefinition_PCGGraphInterface::GetAssetColor();
	// Make it darker
	Color *= 0.25f;
	Color.A = 1.0f;
	return Color;
}

TSoftClassPtr<UObject> UAssetDefinition_PCGGraphInstance::GetAssetClass() const
{
	return UPCGGraphInstance::StaticClass();
}

EAssetCommandResult UAssetDefinition_PCGGraphInstance::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	const UPCGGraphInstance* OldGraph = Cast<UPCGGraphInstance>(DiffArgs.OldAsset);
	const UPCGGraphInstance* NewGraph = Cast<UPCGGraphInstance>(DiffArgs.NewAsset);

	if (NewGraph == nullptr && OldGraph == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}

	SDetailsDiff::CreateDiffWindow(OldGraph, NewGraph, DiffArgs.OldRevision, DiffArgs.NewRevision, UPCGGraphInstance::StaticClass());
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
