// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PCGGraphInterface.h"

#include "PCGGraph.h"

#include "PCGDefaultExecutionSource.h"
#include "PCGDefaultWorldObjectExecutionSource.h"
#include "PCGEditorCommon.h"
#include "PCGEditorStyle.h"
#include "PCGGraphFactory.h"
#include "PCGSubsystem.h"
#include "Subsystems/PCGEngineSubsystem.h"

#include "ContentBrowserMenuContexts.h"
#include "Editor.h"
#include "IAssetTools.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/DelayedAutoRegister.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PCGGraphInterface)

#define LOCTEXT_NAMESPACE "AssetDefinition_PCGGraphInterface"

FText UAssetDefinition_PCGGraphInterface::GetAssetDisplayName() const
{
	return LOCTEXT("DisplayName", "PCG Graph Interface");
}

FLinearColor UAssetDefinition_PCGGraphInterface::GetAssetColor() const
{
	return FColor::Turquoise;
}

TSoftClassPtr<UObject> UAssetDefinition_PCGGraphInterface::GetAssetClass() const
{
	return UPCGGraphInterface::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_PCGGraphInterface::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FPCGEditorCommon::PCGAssetSubCategoryPath };
	return Categories;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_PCGGraphInterface
{
	static void ExecuteNewPCGGraphInstance(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);

		IAssetTools::Get().CreateAssetsFrom<UPCGGraphInterface>(
			CBContext->LoadSelectedObjects<UPCGGraphInterface>(), UPCGGraphInstance::StaticClass(), TEXT("_Inst"), [](UPCGGraphInterface* SourceObject)
			{
				UPCGGraphInstanceFactory* Factory = NewObject<UPCGGraphInstanceFactory>();
				Factory->ParentGraph = SourceObject;
				return Factory;
			}
		);
	}

	/** Returns true if the loaded graph has at least one user-defined graph parameter. */
	static bool HasGraphParameters(const UPCGGraphInterface* GraphInterface)
	{
		const FInstancedPropertyBag* UserParams = GraphInterface ? GraphInterface->GetUserParametersStruct() : nullptr;
		return UserParams && UserParams->GetNumPropertiesInBag() > 0;
	}

	/** Opens a modal dialog that lets the user configure graph parameters on a temporary instance before executing. */
	static void ShowConfigureAndExecuteDialog(UPCGGraphInterface* GraphInterface, TFunction<void(UPCGGraphInstance*)> OnExecute)
	{
		check(GraphInterface);

		// Create a transient graph instance to hold the parameter overrides for this one-time execution.
		UPCGGraphInstance* TempInstance = NewObject<UPCGGraphInstance>(GetTransientPackage(), NAME_None, RF_Transient);
		// We'll root the instance to make sure it does not get garbage collected before we're done.
		TempInstance->AddToRoot();
		TempInstance->SetGraph(GraphInterface);

		// Build a details view for the temp instance, hiding the "Graph" property since it is already set.
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bShowScrollBar = true;

		TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda([](const FPropertyAndParent& PropertyAndParent) -> bool
		{
			return PropertyAndParent.Property.GetFName() != GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, Graph);
		}));
		DetailsView->RegisterInstancedCustomPropertyLayout(UPCGGraphInstance::StaticClass(), DetailsView->GetGenericLayoutDetailsDelegate());
		DetailsView->SetObject(TempInstance);
		DetailsView->SetRootExpansionStates(/*bExpand=*/true, /*bRecurse=*/false);

		bool bShouldExecute = false;

		TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(FText::Format(LOCTEXT("ConfigureAndExecuteTitle", "Configure: {0}"), GraphInterface->GetDisplayName()))
		.ClientSize(FVector2D(500, 400))
		.SizingRule(ESizingRule::UserSized)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.AutoCenter(EAutoCenter::PreferredWorkArea);

		Window->SetContent(
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				StaticCastSharedRef<SWidget>(DetailsView)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(FMargin(8.f, 4.f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(4.f, 0.f))
				[
					SNew(SButton)
					.Text(LOCTEXT("ExecuteButton", "Execute"))
					.OnClicked_Lambda([&bShouldExecute, WeakWindow = TWeakPtr<SWindow>(Window)]() -> FReply
					{
						bShouldExecute = true;
						if (TSharedPtr<SWindow> PinnedWindow = WeakWindow.Pin())
						{
							PinnedWindow->RequestDestroyWindow();
						}
						return FReply::Handled();
					})
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(4.f, 0.f))
				[
					SNew(SButton)
					.Text(LOCTEXT("CancelButton", "Cancel"))
					.OnClicked_Lambda([WeakWindow = TWeakPtr<SWindow>(Window)]() -> FReply
					{
						if (TSharedPtr<SWindow> PinnedWindow = WeakWindow.Pin())
						{
							PinnedWindow->RequestDestroyWindow();
						}
						return FReply::Handled();
					})
				]
			]
		);

		FSlateApplication::Get().AddModalWindow(Window, FSlateApplication::Get().GetActiveTopLevelWindow());

		if (bShouldExecute)
		{
			OnExecute(TempInstance);
		}

		TempInstance->RemoveFromRoot();
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UPCGGraphInterface::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const TAttribute<FText> Label = LOCTEXT("PCGGraph_NewInstance", "Create PCG Graph Instance");
				const TAttribute<FText> ToolTip = LOCTEXT("PCGGraph_NewInstanceToolTip", "Creates a parameterized PCG graph using this graph as a base.");
				const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewPCGGraphInstance);

				InSection.AddMenuEntry("PCGGraph_NewInstance", Label, ToolTip, FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "ClassIcon.PCGGraphInstance"), UIAction);
			}));

			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection);
				if (!CBContext)
				{
					return;
				}

				TArray<FAssetData> AssetGraphs;
				TArray<FAssetData> LevelGraphs;

				auto GetGraphsByType = [CBContext](EPCGGraphUsage Usage)
				{
					TArray<FAssetData> Graphs;
					Algo::CopyIf(CBContext->SelectedAssets, Graphs, [Usage](const FAssetData& InAsset)
					{
						bool bTagValue = false;
						return InAsset.IsInstanceOf<UPCGGraphInterface>() &&
							InAsset.GetTagValue(GET_MEMBER_NAME_CHECKED(UPCGGraphInterface, bExposeGenerationInAssetExplorer), bTagValue) &&
							bTagValue &&
							UPCGGraphInterface::GetGraphUsage(InAsset) == Usage;
					});

					return Graphs;
				};

				AssetGraphs = GetGraphsByType(EPCGGraphUsage::Asset);
				LevelGraphs = GetGraphsByType(EPCGGraphUsage::Level);

				// When exactly one asset is selected, check whether it has parameters to offer the configure option.
				// TODO: Loading the asset at menu-build time to check for parameters - revisit if this proves too slow for large unloaded assets.
				const bool bShowConfigureInstance = CBContext->SelectedAssets.Num() == 1 && (!AssetGraphs.IsEmpty() || !LevelGraphs.IsEmpty()) && HasGraphParameters(Cast<UPCGGraphInterface>(CBContext->SelectedAssets[0].GetAsset()));
				const FSlateIcon Icon = FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "ClassIcon.PCGGraph");

				auto AddMenuEntry = [bShowConfigureInstance, &InSection, &Icon](FName SubmenuName, const TAttribute<FText>& Label, const TAttribute<FText>& ToolTip, auto&& ExecuteGraphs, auto&& ConfigureAndExecuteGraph)
				{
					if (bShowConfigureInstance)
					{
						FToolMenuEntry& SubMenuEntry = InSection.AddSubMenu(
							SubmenuName,
							Label,
							ToolTip,
							FNewToolMenuDelegate::CreateLambda([ExecuteGraphs, ConfigureAndExecuteGraph](UToolMenu* SubMenu)
							{
								FToolMenuSection& SubSection = SubMenu->FindOrAddSection(NAME_None);

								FToolUIAction GenerateAction;
								GenerateAction.ExecuteAction.BindLambda(ExecuteGraphs);
								SubSection.AddMenuEntry(
									"GenerateStandaloneGraphsDirect",
									LOCTEXT("GenerateLabel", "Generate"),
									LOCTEXT("GenerateToolTip", "Generate the graph immediately."),
									FSlateIcon(),
									GenerateAction);

								FToolUIAction ConfigureAction;
								ConfigureAction.ExecuteAction.BindLambda(ConfigureAndExecuteGraph);

								SubSection.AddMenuEntry(
									"ConfigureAndGenerateStandaloneGraph",
									LOCTEXT("ConfigureAndGenerateLabel", "Configure and Generate..."),
									LOCTEXT("ConfigureAndGenerateToolTip", "Configure graph parameters before generating."),
									FSlateIcon(),
									ConfigureAction);
							}));

						SubMenuEntry.Icon = Icon;
					}
					else
					{
						FToolUIAction UIAction;
						UIAction.ExecuteAction.BindLambda(ExecuteGraphs);
						InSection.AddMenuEntry(SubmenuName, Label, ToolTip, Icon, UIAction);
					}
				};

				if(!AssetGraphs.IsEmpty())
				{
					auto ExecuteAssetGraphs = [AssetGraphs](const FToolMenuContext&)
					{
						if (UPCGEngineSubsystem* EngineSubsystem = UPCGEngineSubsystem::Get())
						{
							for (const FAssetData& AssetData : AssetGraphs)
							{
								if (UPCGGraphInterface* GraphInterface = Cast<UPCGGraphInterface>(AssetData.GetAsset()))
								{
									IPCGBaseSubsystem::CreateExecutionSource<UPCGDefaultExecutionSource>({ .GraphInterface = GraphInterface, .bFireAndForgetExecution = true });
								}
							}
						}
					};

					auto ConfigureAndExecuteAssetGraph = [AssetGraphs](const FToolMenuContext&)
					{
						if (UPCGEngineSubsystem* EngineSubsystem = UPCGEngineSubsystem::Get(); EngineSubsystem && AssetGraphs.Num() == 1)
						{
							if (UPCGGraphInterface* GraphInterface = Cast<UPCGGraphInterface>(AssetGraphs[0].GetAsset()))
							{
								ShowConfigureAndExecuteDialog(GraphInterface, [](UPCGGraphInstance* TemporaryInstance)
								{
									IPCGBaseSubsystem::CreateExecutionSource<UPCGDefaultExecutionSource>({ .GraphInterface = TemporaryInstance, .bFireAndForgetExecution = true });
								});
							}
						}
					};

					const TAttribute<FText> Label = LOCTEXT("GenerateAssetGraphsMenuLabel", "Generate Asset Graph(s)");
					const TAttribute<FText> ToolTip = LOCTEXT("GenerateAssetGraphsMenuToolTip", "Will generate every asset graph in the selection.");

					AddMenuEntry("GenerateAssetGraphsMenu", Label, ToolTip, ExecuteAssetGraphs, ConfigureAndExecuteAssetGraph);
				}

				if (UPCGSubsystem* Subsystem = UPCGSubsystem::GetSubsystemForCurrentWorld(); Subsystem && !LevelGraphs.IsEmpty())
				{
					auto ExecuteLevelGraphs = [LevelGraphs](const FToolMenuContext&)
					{
						UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld();
						if (!PCGSubsystem)
						{
							return;
						}

						for (const FAssetData& AssetData : LevelGraphs)
						{
							ensure(UPCGGraphInterface::GetGraphUsage(AssetData) == EPCGGraphUsage::Level);
							if (UPCGGraphInterface* GraphInterface = Cast<UPCGGraphInterface>(AssetData.GetAsset()))
							{
								FPCGDefaultWorldObjectExecutionSourceParams Params;
								Params.GraphInterface = GraphInterface;
								Params.WorldObject = PCGSubsystem;
								Params.bFireAndForgetExecution = true;

								IPCGBaseSubsystem::CreateExecutionSource<UPCGDefaultWorldObjectExecutionSource>(Params);
							}
						}
					};

					auto ConfigureAndExecuteLevelGraph = [LevelGraphs](const FToolMenuContext&)
					{
						UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld();
						if (!PCGSubsystem || LevelGraphs.Num() != 1)
						{
							return;
						}

						ensure(UPCGGraphInterface::GetGraphUsage(LevelGraphs[0]) == EPCGGraphUsage::Level);
						if (UPCGGraphInterface* GraphInterface = Cast<UPCGGraphInterface>(LevelGraphs[0].GetAsset()))
						{
							ShowConfigureAndExecuteDialog(GraphInterface, [PCGSubsystem](UPCGGraphInstance* TemporaryInstance)
							{
								FPCGDefaultWorldObjectExecutionSourceParams Params;
								Params.GraphInterface = TemporaryInstance;
								Params.WorldObject = PCGSubsystem;
								Params.bFireAndForgetExecution = true;

								IPCGBaseSubsystem::CreateExecutionSource<UPCGDefaultWorldObjectExecutionSource>(Params);
							});
						}
					};

					const TAttribute<FText> Label = LOCTEXT("GenerateGraphInWorldMenuLabel", "Runs Graph(s) in Current World");
					const TAttribute<FText> ToolTip = LOCTEXT("GenerateGraphInWorldMenuToolTip", "Will generate the graph(s) in the world context.");

					AddMenuEntry("GenerateGraphInWorld", Label, ToolTip, ExecuteLevelGraphs, ConfigureAndExecuteLevelGraph);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
