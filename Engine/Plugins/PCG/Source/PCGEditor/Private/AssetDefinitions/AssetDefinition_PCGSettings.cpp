// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PCGSettings.h"

#include "PCGEditorCommon.h"
#include "PCGSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PCGSettings)

#define LOCTEXT_NAMESPACE "AssetDefinition_PCGSettings"

FText UAssetDefinition_PCGSettings::GetAssetDisplayName() const
{
	return LOCTEXT("DisplayName", "PCG Settings");
}

FLinearColor UAssetDefinition_PCGSettings::GetAssetColor() const
{
	return FColor::Turquoise;
}

TSoftClassPtr<UObject> UAssetDefinition_PCGSettings::GetAssetClass() const
{
	return UPCGSettings::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_PCGSettings::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FPCGEditorCommon::PCGAssetCategoryPath };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
