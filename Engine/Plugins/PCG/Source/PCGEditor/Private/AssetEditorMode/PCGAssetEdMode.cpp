// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetEditorMode/PCGAssetEdMode.h"

#include "PCGEditor.h"
#include "AssetEditorMode/PCGAssetEditorToolRegistry.h"
#include "AssetEditorMode/Tools/PCGAssetEditorInteractiveTool.h"

#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Tools/EdModeInteractiveToolsContext.h"

#include "Widgets/SPCGEditorGraphAttributeListView.h"

#define LOCTEXT_NAMESPACE "UPCGAssetEditorMode"

const FEditorModeID UPCGAssetEditorMode::EM_PCGAssetEditorModeId = TEXT("EM_PCGAssetEditorMode");

UPCGAssetEditorMode::UPCGAssetEditorMode()
{
	Info = FEditorModeInfo(
		EM_PCGAssetEditorModeId,
		LOCTEXT("PCGAssetEditorModeDisplayName", "PCG"),
		FSlateIcon("PCGEditorModeStyle", "PCGEditorModeIcon"),
		/*InVisibility*/ false
	);
}

void UPCGAssetEditorMode::Enter()
{
	Super::Enter();

	UInteractiveToolManager* ToolManager = GetToolManager();
	if (!ToolManager)
	{
		return;
	}

	const TObjectPtr<UEditorInteractiveToolsContext> ToolsContext = GetInteractiveToolsContext();
	if (!ToolsContext)
	{
		return;
	}

	UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(ToolsContext);
	ToolsContext->SetForceCombinedGizmoMode(true);

	// ToolManager->OnToolPostBuild.AddUObject(this, &UPCGAssetEditorMode::OnToolPostBuild);
	ToolManager->OnToolShutdownRequest.BindLambda([this](UInteractiveToolManager*, UInteractiveTool*, const EToolShutdownType ShutdownType)
		{
			if (UEditorInteractiveToolsContext* Context = GetInteractiveToolsContext())
			{
				Context->EndTool(ShutdownType);
				return true;
			}
			return false;
		});

	// Register one generic builder per unique tool class (ToolId = ToolClass->GetName())
	for (const TSubclassOf<UPCGAssetEditorInteractiveTool>& ToolClass : FPCGAssetEditorToolRegistry::Get().GetAllToolClasses())
	{
		UPCGAssetEditorInteractiveToolBuilder* Builder = NewObject<UPCGAssetEditorInteractiveToolBuilder>();
		Builder->ToolClass = ToolClass;
		ToolManager->RegisterToolType(ToolClass->GetName(), Builder);
	}

	ConfigureRealTimeViewportsOverride(/*bEnable=*/true);
}

void UPCGAssetEditorMode::Exit()
{
	// Clear realtime viewport override
	ConfigureRealTimeViewportsOverride(/*bEnable=*/false);

	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		ToolManager->OnToolShutdownRequest.Unbind();
		ToolManager->OnToolPostBuild.RemoveAll(this);
	}
	
	if (const TObjectPtr<UEditorInteractiveToolsContext> ToolsContext = GetInteractiveToolsContext())
	{
		UE::TransformGizmoUtil::DeregisterTransformGizmoContextObject(ToolsContext);
	}

	// Call base Exit method to ensure proper cleanup
	Super::Exit();
}

bool UPCGAssetEditorMode::ShouldToolStartBeAllowed(const FString& ToolIdentifier) const
{
	// Don't allow starting a new tool if one is already active
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		if (IsAnyToolActive())
		{
			ToolManager->DisplayMessage(
				LOCTEXT("ToolAlreadyActive", "Another tool is already active. Complete or cancel the current tool first."),
				EToolMessageLevel::UserWarning);
			return false;
		}
	}

	return Super::ShouldToolStartBeAllowed(ToolIdentifier);
}

bool UPCGAssetEditorMode::IsAnyToolActive() const
{
	if (const UInteractiveToolManager* ToolManager = GetToolManager())
	{
		return ToolManager->HasAnyActiveTool();
	}
	else
	{
		return false;
	}
}

void UPCGAssetEditorMode::ConfigureRealTimeViewportsOverride(const bool bEnable)
{
	if (const FEditorModeTools* ModeManager = GetModeManager())
	{
		if (FEditorViewportClient* const FocusedViewport = ModeManager->GetFocusedViewportClient())
		{
			if (bEnable)
			{
				FocusedViewport->AddRealtimeOverride(bEnable, LOCTEXT("PCGAssetEditorModeDisplayName", "PCG"));
			}
			else
			{
				FocusedViewport->RemoveRealtimeOverride(LOCTEXT("PCGAssetEditorModeDisplayName", "PCG"), /*bCheckMissingOverride=*/ false);
			}
		}
	}
}

// @todo_pcg: Activate when the asset editor mode might need to track anything during a tool's lifecycle.

// void UPCGAssetEditorMode::OnToolStarted(
// 	UInteractiveToolManager* Manager,
// 	UInteractiveTool* Tool
// )
// {}

// void UPCGAssetEditorMode::OnToolEnded(
// 	UInteractiveToolManager* Manager,
// 	UInteractiveTool* Tool
// )
// {}

// void UPCGAssetEditorMode::OnToolPostBuild(
// 	UInteractiveToolManager* InToolManager,
// 	EToolSide InSide,
// 	UInteractiveTool* InBuiltTool,
// 	UInteractiveToolBuilder* InToolBuilder,
// 	const FToolBuilderState& ToolState
// )
// {}

#undef LOCTEXT_NAMESPACE
