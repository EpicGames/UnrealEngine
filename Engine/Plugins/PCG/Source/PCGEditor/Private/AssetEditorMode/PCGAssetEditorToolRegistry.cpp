// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGAssetEditorToolRegistry.h"

#include "PCGEditorModule.h"
#include "AssetEditorMode/Tools/PCGAssetEditorInteractiveTool.h"

#include "Misc/LazySingleton.h"

FPCGAssetEditorToolRegistry& FPCGAssetEditorToolRegistry::Get()
{
	return TLazySingleton<FPCGAssetEditorToolRegistry>::Get();
}

void FPCGAssetEditorToolRegistry::TearDown()
{
	TLazySingleton<FPCGAssetEditorToolRegistry>::TearDown();
}

void FPCGAssetEditorToolRegistry::RegisterTool(UClass* SettingsClass, TSubclassOf<UPCGAssetEditorInteractiveTool> ToolClass)
{
	if (SettingsClass == nullptr)
	{
		UE_LOGF(LogPCGEditor, Fatal, "Settings Class is null");
	}
	
	if (ToolClass == nullptr)
	{
		UE_LOGF(LogPCGEditor, Fatal, "Tool Class is null");
	}
	
	if (const auto FoundToolClass = SettingsToToolMap.Find(TObjectKey<UClass>(SettingsClass)))
	{
		UE_LOGF(LogPCGEditor, Fatal, "Settings Class %ls already registered with Tool %ls", *SettingsClass->GetName(), *((*FoundToolClass)->GetName()));
	}
	
	SettingsToToolMap.Add(TObjectKey<UClass>(SettingsClass), ToolClass);
}

const TSubclassOf<UPCGAssetEditorInteractiveTool>* FPCGAssetEditorToolRegistry::FindToolForSettings(const UClass* SettingsClass) const
{
	const UClass* Class = SettingsClass;
	while (Class && Class != UObject::StaticClass())
	{
		if (const TSubclassOf<UPCGAssetEditorInteractiveTool>* Found = SettingsToToolMap.Find(TObjectKey<UClass>(Class)))
		{
			return Found;
		}
		Class = Class->GetSuperClass();
	}
	return nullptr;
}

TArray<TSubclassOf<UPCGAssetEditorInteractiveTool>> FPCGAssetEditorToolRegistry::GetAllToolClasses() const
{
	TSet<TSubclassOf<UPCGAssetEditorInteractiveTool>> Unique;
	for (const TPair<TObjectKey<UClass>, TSubclassOf<UPCGAssetEditorInteractiveTool>>& Pair : SettingsToToolMap)
	{
		Unique.Add(Pair.Value);
	}
	return Unique.Array();
}
