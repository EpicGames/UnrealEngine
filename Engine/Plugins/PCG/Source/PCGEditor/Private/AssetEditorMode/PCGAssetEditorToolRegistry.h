// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "UObject/ObjectKey.h"

class UPCGAssetEditorInteractiveTool;

/**
 * Maps UPCGSettings subclasses to the interactive tool that handles them.
 * Populated at module startup via GetDerivedClasses + CDO::GetSupportedSettingsClasses().
 */
class FPCGAssetEditorToolRegistry
{
public:
	static FPCGAssetEditorToolRegistry& Get();
	static void TearDown();

	void RegisterTool(UClass* SettingsClass, TSubclassOf<UPCGAssetEditorInteractiveTool> ToolClass);

	/** Walks the settings class hierarchy; returns null if no tool is registered. */
	const TSubclassOf<UPCGAssetEditorInteractiveTool>* FindToolForSettings(const UClass* SettingsClass) const;

	/** Returns deduplicated list of all registered tool classes, for tool-manager registration. */
	TArray<TSubclassOf<UPCGAssetEditorInteractiveTool>> GetAllToolClasses() const;

private:
	TMap<TObjectKey<UClass>, TSubclassOf<UPCGAssetEditorInteractiveTool>> SettingsToToolMap;
};
