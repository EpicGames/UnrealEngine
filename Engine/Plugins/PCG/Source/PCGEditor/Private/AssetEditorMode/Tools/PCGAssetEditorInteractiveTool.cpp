// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetEditorMode/Tools/PCGAssetEditorInteractiveTool.h"

#include "PCGGraphExecutionStateInterface.h"

#include "ContextObjectStore.h"
#include "InteractiveToolManager.h"

#define LOCTEXT_NAMESPACE "PCGAssetEditorInteractiveTool"

//////////////////////////////////////////////////////////////////////////
// UPCGNodeToolContext

IPCGGraphExecutionSource* UPCGNodeToolContext::GetExecutionSource() const
{
	return Cast<IPCGGraphExecutionSource>(ExecutionSourceObject);
}

//////////////////////////////////////////////////////////////////////////
// UPCGAssetEditorInteractiveTool

void UPCGAssetEditorInteractiveTool::Setup()
{
	Super::Setup();
	
	// Create and add tool settings
	ToolSettings = NewObject<UPCGAssetEditorInteractiveToolSettings>(this);
	AddToolPropertySource(ToolSettings);
	
	// Retrieve the context object populated by FPCGEditor::OnNodeToolStarted
	NodeToolContext = GetToolManager()->GetContextObjectStore()->FindContext<UPCGNodeToolContext>();
	checkf(NodeToolContext, TEXT("Invalid NodeToolContext"));
	
	ExecutionSource = NodeToolContext->GetExecutionSource();
	checkf(ExecutionSource, TEXT("Invalid ExecutionSource"));
	
	// Don't allow Undo/Redo to start/shutdown previous/current tool
	GetToolManager()->ConfigureChangeTrackingMode(EToolChangeTrackingMode::NoChangeTracking);
	
	NodeToolContext->NodeSettings->OnSettingsChangedDelegate.AddUObject(this, &UPCGAssetEditorInteractiveTool::OnNodeSettingsChanged);
}

void UPCGAssetEditorInteractiveTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept || ShutdownType == EToolShutdownType::Completed)
	{
		OnAccept();
	}
	else if (ShutdownType == EToolShutdownType::Cancel)
	{
		OnCancel();
	}
	
	if (NodeToolContext)
	{
		NodeToolContext->NodeSettings->OnSettingsChangedDelegate.RemoveAll(this);
	}

	NodeToolContext = nullptr;
	ExecutionSource = nullptr;

	Super::Shutdown(ShutdownType);
}

//////////////////////////////////////////////////////////////////////////
// UPCGAssetEditorInteractiveToolBuilder

bool UPCGAssetEditorInteractiveToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return ToolClass && 
		SceneState.ToolManager &&
		SceneState.ToolManager->GetContextObjectStore()->FindContext<UPCGNodeToolContext>() != nullptr;
}

UInteractiveTool* UPCGAssetEditorInteractiveToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UPCGAssetEditorInteractiveTool* NewTool = NewObject<UPCGAssetEditorInteractiveTool>(SceneState.ToolManager, ToolClass);
	NewTool->PostBuild(SceneState);
	return NewTool;
}

#undef LOCTEXT_NAMESPACE
