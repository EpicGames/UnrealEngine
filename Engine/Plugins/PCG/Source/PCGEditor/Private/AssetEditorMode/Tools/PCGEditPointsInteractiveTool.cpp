// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditPointsInteractiveTool.h"

#include "PCGDebug.h"
#include "PCGGraph.h"
#include "PCGGraphExecutionInspection.h"
#include "PCGGraphExecutionStateInterface.h"
#include "Data/PCGBasePointData.h"
#include "Graph/PCGSourceDataContainer.h"
#include "Graph/DataOverride/PCGDataOverride.h"
#include "Graph/DataOverride/PCGDataOverrideHelpers.h"
#include "Graph/DataOverride/PCGDataOverridePoints.h"
#include "Subsystems/IPCGBaseSubsystem.h"

#include "ContextObjectStore.h"
#include "Editor.h"
#include "EditorModes.h"
#include "HitProxies.h"
#include "InteractiveToolManager.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "StaticMeshResources.h"
#include "ToolContextInterfaces.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "Components/Viewport.h"
#include "Engine/StaticMesh.h"
#include "Math/Bounds.h"

#include "Misc/TransactionObjectEvent.h"

#define LOCTEXT_NAMESPACE "PCGEditPointsInteractiveTool"

//////////////////////////////////////////////////////////////////////////
// HPCGCreatePointsHitProxy

/** Hit proxy identifying which point box was clicked in the viewport. */
struct HPCGCreatePointsHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY()

	int32 PointIndex = INDEX_NONE;
	FTransform PointTransform;
	FBox PointBounds;
	FPCGSourceDataStorageKey StorageKey;

	HPCGCreatePointsHitProxy(
		int32 InIndex,
		FTransform InPointTransform,
		FBox InBounds,
		const FPCGSourceDataStorageKey& InStorageKey
	)
		: HHitProxy(HPP_UI)
		, PointIndex(InIndex)
		, PointTransform(InPointTransform)
		, PointBounds(InBounds)
		, StorageKey(InStorageKey)
	{}
};

IMPLEMENT_HIT_PROXY(HPCGCreatePointsHitProxy, HHitProxy)

namespace PCGCreatePointsTool
{
	// @todo_pcg: Make these common/shared values with the Level Editor Viewport Interaction
	constexpr FLinearColor DefaultColor(0.0f, 0.55f, 0.45f, 0.3f);
	constexpr FLinearColor SelectedColor(1.0f, 0.8f, 0.3f, 0.5f);
}

namespace PCGCreatePointsToolHelpers
{
	void DrawMeshWireframe(FPrimitiveDrawInterface* PDI, const FTransform& Transform,
	                       const TArray<uint32>& Indices, const FPositionVertexBuffer* PositionBuffer, const FLinearColor& Color)
	{
		for (int32 i = 0; i + 2 < Indices.Num(); i += 3)
		{
			const FVector P0 = Transform.TransformPosition(FVector(PositionBuffer->VertexPosition(Indices[i])));
			const FVector P1 = Transform.TransformPosition(FVector(PositionBuffer->VertexPosition(Indices[i + 1])));
			const FVector P2 = Transform.TransformPosition(FVector(PositionBuffer->VertexPosition(Indices[i + 2])));
			PDI->DrawLine(P0, P1, Color, SDPG_World);
			PDI->DrawLine(P1, P2, Color, SDPG_World);
			PDI->DrawLine(P2, P0, Color, SDPG_World);
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// UPCGEditPointsInteractiveTool

void UPCGEditPointsInteractiveTool::Setup()
{
	Super::Setup();

	// Register single-click input behavior
	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>(this);
	ClickBehavior->Initialize(this);
	AddInputBehavior(ClickBehavior);

	// Create transform proxy and gizmo
	TransformProxy = NewObject<UTransformProxy>(this);
	TransformProxy->OnBeginTransformEdit.AddUObject(this, &UPCGEditPointsInteractiveTool::OnGizmoTransformBegin);
	TransformProxy->OnTransformChanged.AddUObject(this, &UPCGEditPointsInteractiveTool::OnGizmoTransformChanged);
	TransformProxy->OnEndTransformEdit.AddUObject(this, &UPCGEditPointsInteractiveTool::OnGizmoTransformEnd);

	CombinedGizmo = UE::TransformGizmoUtil::Create3AxisTransformGizmo(GetToolManager(), this);
	if (ensure(CombinedGizmo))
	{
		CombinedGizmo->SetActiveTarget(TransformProxy, GetToolManager());
		CombinedGizmo->SetVisibility(false);
		CombinedGizmo->ActiveGizmoMode = EToolContextTransformGizmoMode::Combined;
		CombinedGizmo->bUseContextGizmoMode = false;
		CombinedGizmo->bUseContextCoordinateSystem = false;
	}

	if (IPCGBaseSubsystem* Subsystem = ExecutionSource->GetExecutionState().GetSubsystem())
	{
		Subsystem->GetOnPCGSourceGenerationDone().AddUObject(this, &UPCGEditPointsInteractiveTool::OnGraphExecuted);
	}

	CachedNodeStacks = ExecutionSource->GetExecutionState().GetInspection().GetExecutedNodeStacks();

	BuildCachedMesh();
}

void UPCGEditPointsInteractiveTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (GEditor->IsTransactionActive())
	{
		GEditor->EndTransaction();
	}
	
	if (CombinedGizmo)
	{
		CombinedGizmo->ClearActiveTarget();
		CombinedGizmo->Shutdown();
		CombinedGizmo = nullptr;
	}

	TransformProxy = nullptr;

	if (IPCGBaseSubsystem* Subsystem = ExecutionSource->GetExecutionState().GetSubsystem())
	{
		Subsystem->GetOnPCGSourceGenerationDone().RemoveAll(this);
	}

	CachedMesh.Reset();
	CachedMeshIndices.Empty();
	CachedPositionBuffer = nullptr;
	CachedNodeStacks.Empty();

	Super::Shutdown(ShutdownType);
}

void UPCGEditPointsInteractiveTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	using namespace PCGCreatePointsTool;
	using namespace PCGCreatePointsToolHelpers;

	if (!NodeToolContext)
	{
		return;
	}

	// Resolve debug mesh (node's custom mesh, or fall back to default PCG point mesh)
	const UPCGSettings* NodeSettings = NodeToolContext->NodeSettings;
	check(NodeSettings);

	if (!CachedMesh.IsValid() || !CachedPositionBuffer || CachedMeshIndices.IsEmpty())
	{
		return;
	}

	const float PointScale = NodeSettings->DebugSettings.PointScale;
	const bool bIsAbsolute = NodeSettings->DebugSettings.ScaleMethod == EPCGDebugVisScaleMethod::Absolute;
	const bool bIsRelative = NodeSettings->DebugSettings.ScaleMethod == EPCGDebugVisScaleMethod::Relative;
	const bool bScaleWithExtents = NodeSettings->DebugSettings.ScaleMethod == EPCGDebugVisScaleMethod::Extents;
	const FVector MeshExtents = CachedMesh->GetBoundingBox().GetExtent();

	FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
	check(PDI);

	const bool bHitTesting = PDI->IsHitTesting();

	const FPCGPointTransformDelta* CurrentTransformDelta = nullptr;
	if (const FPCGDeltaCollection* Collection = GetCollection(CurrentStorageKey))
	{
		// While dragging, provide explicit wireframe feedback for the selected point
		const TInstancedStruct<FPCGDeltaBase>* EditDelta = Collection->Find(CurrentDeltaKey);
		CurrentTransformDelta = GetTransformDelta(EditDelta);
		if (CurrentTransformDelta && bIsTransforming)
		{
			DrawMeshWireframe(PDI, CurrentTransformDelta->TransformOverride, CachedMeshIndices, CachedPositionBuffer, SelectedColor);
			return;
		}
	}

	const FPCGGraphExecutionInspection& Inspection = ExecutionSource->GetExecutionState().GetInspection();

	// Find the node corresponding to our settings
	for (const auto& [ObjectKey, StackSet] : CachedNodeStacks)
	{
		const UPCGNode* Node = ObjectKey.ResolveObjectPtr();
		if (!Node || Node->GetSettings() != NodeSettings)
		{
			continue;
		}

		const TArray<TObjectPtr<UPCGPin>>& OutputPins = Node->GetOutputPins();
		if (OutputPins.IsEmpty())
		{
			continue;
		}

		// @todo_pcg: Check for multiple OutputPins (?)
		const UPCGPin* OutputPin = OutputPins[0];

		for (const FPCGGraphExecutionInspection::FNodeExecutedNotificationData& StackData : StackSet)
		{
			FPCGStack PinStack = StackData.Stack;
			PinStack.PushFrame(Node);
			PinStack.PushFrame(OutputPin);

			TSharedPtr<FPCGInspectionData> InspectionData = Inspection.GetInspectionDataPtr(PinStack);
			if (!InspectionData || !InspectionData->Data)
			{
				continue;
			}

			const FPCGSourceDataStorageKey StorageKey(
				PCG::DataOverride::Constants::DefaultOverrideLabel,
				PCG::DataOverride::Helpers::ComputePinOverrideKey(StackData.Stack, Node, OutputPin->Properties.Label).Hash
			);

			const TArray<FPCGTaggedData>& DataArray = InspectionData->Data->GetAllInputs();
			int32 GlobalPointIndex = 0;
			for (const FPCGTaggedData& Input : DataArray)
			{
				const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Input.Data);
				if (!PointData)
				{
					continue;
				}

				TConstPCGValueRange<FTransform> TransformRange = PointData->GetConstTransformValueRange();
				TConstPCGValueRange<FVector> BoundsMinRange = PointData->GetConstBoundsMinValueRange();
				TConstPCGValueRange<FVector> BoundsMaxRange = PointData->GetConstBoundsMaxValueRange();
				for (int32 i = 0; i < PointData->GetNumPoints(); ++i)
				{
					const FTransform& Transform = TransformRange[i];

					const bool bIsSelected = CurrentTransformDelta && CurrentTransformDelta->TransformOverride.Equals(Transform);

					const FLinearColor Color = bIsSelected ? SelectedColor : DefaultColor;

					if (bHitTesting)
					{
						PDI->SetHitProxy(new HPCGCreatePointsHitProxy(GlobalPointIndex, Transform, PointData->GetBounds(), StorageKey));
					}

					FTransform InstanceTransform = Transform;
					if (bIsRelative)
					{
						InstanceTransform.SetScale3D(InstanceTransform.GetScale3D() * PointScale);
					}
					else if (bScaleWithExtents)
					{
						const FVector Extents = PCGPointHelpers::GetExtents(BoundsMinRange[i], BoundsMaxRange[i]);
						const FVector LocalCenter = PCGPointHelpers::GetLocalCenter(BoundsMinRange[i], BoundsMaxRange[i]);

						const FVector ScaleWithExtents = Extents / MeshExtents;
						const FVector TransformedBoxCenterWithOffset = InstanceTransform.TransformPosition(LocalCenter) - InstanceTransform.
							GetLocation();
						InstanceTransform.SetTranslation(InstanceTransform.GetTranslation() + TransformedBoxCenterWithOffset);
						InstanceTransform.SetScale3D(InstanceTransform.GetScale3D() * ScaleWithExtents);
					}
					else if (bIsAbsolute)
					{
						InstanceTransform.SetScale3D(FVector(PointScale));
					}
					DrawMeshWireframe(PDI, InstanceTransform, CachedMeshIndices, CachedPositionBuffer, Color);

					if (bHitTesting)
					{
						PDI->SetHitProxy(nullptr);
					}

					++GlobalPointIndex;
				}
			}
		}
		break; // Only process the first matching node
	}
}

FInputRayHit UPCGEditPointsInteractiveTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	// Always consume clicks so OnClicked can determine if a box was hit
	return FInputRayHit(1.0f);
}

void UPCGEditPointsInteractiveTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	FViewport* FocusedViewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	const HHitProxy* HitProxy = FocusedViewport
		? FocusedViewport->GetHitProxy(ClickPos.ScreenPosition.X, ClickPos.ScreenPosition.Y)
		: nullptr;

	const HPCGCreatePointsHitProxy* PointProxy = HitProxy && HitProxy->IsA(HPCGCreatePointsHitProxy::StaticGetType())
		? static_cast<const HPCGCreatePointsHitProxy*>(HitProxy)
		: nullptr;
	const int32 HitPointIndex = PointProxy ? PointProxy->PointIndex : INDEX_NONE;

	if (FPCGDeltaCollection* Collection = GetCollection(CurrentStorageKey))
	{
		// If the selected point's delta was never moved (OriginalTransform == TransformOverride), remove it to avoid redundant entries.
		const TInstancedStruct<FPCGDeltaBase>* EditDelta = Collection->Find(CurrentDeltaKey);
		const FPCGPointTransformDelta* Delta = GetTransformDelta(EditDelta);
		if (Collection && Delta && Delta->TransformOverride.Equals(Delta->OriginalTransform))
		{
			const FPCGDeltaKey ToRemoveKey = PCG::DataOverride::Keys::FPCGCompositeTransformDeltaKey(
				Delta->OriginalTransform,
				Collection->Settings.SpatialTolerance,
				/*bPosition*/ true,
				/*bRotation*/ true,
				/*bScale*/ true,
				FPCGPointTransformDelta::GetDeltaNameStatic()
			);
			Collection->Remove(ToRemoveKey);
			MarkDataContainerDirty();
		}
	}

	CurrentEditDelta = nullptr;
	CurrentDeltaKey = FPCGDeltaKey();

	if (PointProxy == nullptr)
	{
		CurrentCollection = nullptr;
		CurrentStorageKey = FPCGSourceDataStorageKey();
		HideGizmo();
		return;
	}

	CurrentStorageKey = PointProxy->StorageKey;
	CurrentCollection = GetOrCreateCollection(CurrentStorageKey);
	check(CurrentCollection);

	FPCGPointTransformDelta PointTransformDelta;
	PointTransformDelta.OriginalTransform = PointProxy->PointTransform;
	PointTransformDelta.TransformOverride = PointProxy->PointTransform;
	PointTransformDelta.Bounds = PointProxy->PointBounds;
#if WITH_EDITORONLY_DATA
	PointTransformDelta.ElementIndex = HitPointIndex;
#endif

	const FPCGDeltaKey DeltaKey = PCG::DataOverride::Keys::FPCGCompositeTransformDeltaKey(
		PointTransformDelta.OriginalTransform,
		CurrentCollection->Settings.SpatialTolerance,
		/*bPosition*/ true,
		/*bRotation*/ true,
		/*bScale*/ true,
		FPCGPointTransformDelta::GetDeltaNameStatic()
	);
	CurrentDeltaKey = DeltaKey;
	CurrentEditDelta = CurrentCollection->Find(DeltaKey);
	if (!CurrentEditDelta)
	{
		CurrentEditDelta = &CurrentCollection->Add_GetRef(DeltaKey, TInstancedStruct<FPCGPointTransformDelta>::Make(PointTransformDelta));
	}

	ShowGizmoAt(PointProxy->PointTransform);
}

bool UPCGEditPointsInteractiveTool::MatchesContext(
	const FTransactionContext& InContext,
	const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts
) const
{
	if (!NodeToolContext)
	{
		return false;
	}

	for (const auto& [Object, Event] : TransactionObjectContexts)
	{
		if (Object == NodeToolContext->ExecutionSourceObject)
		{
			return true;
		}
	}

	return false;
}

void UPCGEditPointsInteractiveTool::PostUndo(bool bSuccess)
{
	if (!bSuccess)
	{
		return;
	}

	// Re-fetch the collection — Undo may have deserialized it, making CurrentEditDelta a dangling pointer.
	CurrentEditDelta = nullptr;
	CurrentCollection = GetCollection(CurrentStorageKey);
	if (CurrentCollection)
	{
		CurrentEditDelta = CurrentCollection->Find(CurrentDeltaKey);
	}

	if (!CurrentEditDelta)
	{
		CurrentCollection = nullptr;
		CurrentStorageKey = FPCGSourceDataStorageKey();
		CurrentDeltaKey = FPCGDeltaKey();
		HideGizmo();
	}

	MarkDataContainerDirty();
	NotifyGraphChanged();
}

void UPCGEditPointsInteractiveTool::OnAccept()
{
	// @todo_pcg: Allow Accepting tool's output. Currently all tools are set to auto accept, as there is no UI to accept/cancel
	// Deltas are already stored — just ensure the graph re-executes with them.
	NotifyGraphChanged();
	HideGizmo();
}

void UPCGEditPointsInteractiveTool::OnCancel()
{
	// @todo_pcg: Allow Cancelling tool's output. Currently all tools are set to auto accept, as there is no UI to accept/cancel
	// Remove all deltas created during this tool session
	FPCGSourceDataContainer* DataContainer = GetDataContainer();
	if (DataContainer)
	{
		DataContainer->Remove(CurrentStorageKey);
		DataContainer->MarkDirty();
	}

	HideGizmo();
	NotifyGraphChanged();
}

void UPCGEditPointsInteractiveTool::OnNodeSettingsChanged(UPCGSettings* InNodeSettings, EPCGChangeType InChangeType)
{
	Super::OnNodeSettingsChanged(InNodeSettings, InChangeType);

	BuildCachedMesh();
}

void UPCGEditPointsInteractiveTool::OnGraphExecuted(
	IPCGBaseSubsystem* InSubsystem,
	IPCGGraphExecutionSource* InExecutionSource,
	EPCGGenerationStatus InGenerationStatus
)
{
	if (InGenerationStatus != EPCGGenerationStatus::Completed)
	{
		return;
	}

	if (ExecutionSource == InExecutionSource)
	{
		CachedNodeStacks = ExecutionSource->GetExecutionState().GetInspection().GetExecutedNodeStacks();
	}
}

void UPCGEditPointsInteractiveTool::OnGizmoTransformBegin(UTransformProxy* Proxy)
{
	MarkDataContainerDirty();
	
	GEditor->BeginTransaction(LOCTEXT("MovePoint", "Move PCG Point"));

	bIsTransforming = true;

	NodeToolContext->ExecutionSourceObject->Modify();
	
	if (CurrentCollection)
	{
		CurrentEditDelta = CurrentCollection->Find(CurrentDeltaKey);
	}
}

void UPCGEditPointsInteractiveTool::OnGizmoTransformChanged(UTransformProxy* Proxy, FTransform NewTransform)
{
	if (FPCGPointTransformDelta* const TransformDelta = GetTransformDelta(CurrentEditDelta))
	{
		TransformDelta->TransformOverride = NewTransform;
	}
}

void UPCGEditPointsInteractiveTool::OnGizmoTransformEnd(UTransformProxy* Proxy)
{
	if (GEditor->IsTransactionActive())
	{
		GEditor->EndTransaction();
	}

	bIsTransforming = false;
	NotifyGraphChanged();
}

void UPCGEditPointsInteractiveTool::ShowGizmoAt(const FTransform& Transform)
{
	if (CombinedGizmo && TransformProxy)
	{
		TransformProxy->SetTransform(Transform);
		CombinedGizmo->ReinitializeGizmoTransform(Transform);
		CombinedGizmo->SetVisibility(true);
	}
}

void UPCGEditPointsInteractiveTool::HideGizmo()
{
	if (CombinedGizmo)
	{
		CombinedGizmo->SetVisibility(false);
	}
}

void UPCGEditPointsInteractiveTool::BuildCachedMesh()
{
	const UPCGSettings* NodeSettings = NodeToolContext->NodeSettings;
	check(NodeSettings);

	UStaticMesh* DebugMesh = NodeSettings->DebugSettings.PointMesh.Get();
	if (!DebugMesh)
	{
		DebugMesh = LoadObject<UStaticMesh>(nullptr, *PCGDebugVisConstants::DefaultPointMesh.ToString());
	}

	// Rebuild index/position cache if mesh changed
	if (DebugMesh != CachedMesh.Get())
	{
		CachedMesh.Reset();
		CachedMeshIndices.Empty();
		CachedPositionBuffer = nullptr;

		if (const FStaticMeshRenderData* RenderData = DebugMesh ? DebugMesh->GetRenderData() : nullptr)
		{
			if (!RenderData->LODResources.IsEmpty() && RenderData->LODResources[0].IndexBuffer.GetNumIndices() > 0)
			{
				RenderData->LODResources[0].IndexBuffer.GetCopy(CachedMeshIndices);
				CachedPositionBuffer = &RenderData->LODResources[0].VertexBuffers.PositionVertexBuffer;
				CachedMesh = DebugMesh;
			}
		}
	}
}

void UPCGEditPointsInteractiveTool::NotifyGraphChanged()
{
	// @todo_pcg: Use the UPCGComponent::Refresh Method here instead of re-executing the whole Graph
	// Cast<UPCGComponent>(NodeToolContext->ExecutionSourceObject)->Refresh(EPCGChangeType::ExternalModification, /*bCancelExistingRefresh=*/false);
	// Or extend IPCGGraphExecutionSource with Refresh method which all execution sources can inherit, to dynamically allow any execution source to refresh
	// ExecutionSource->Refresh(EPCGChangeType::ExternalModification, /*bCancelExistingRefresh=*/false);
	// This allows Asset Editor Viewport Tools to not depend on PCGComponent directly, which is useful in non-PCG/non-Level-Editor contexts.
	if (const TObjectPtr<UPCGGraphInstance> GraphInstance = NodeToolContext->GraphInstance)
	{
		GraphInstance->OnGraphChangedDelegate.Broadcast(GraphInstance, EPCGChangeType::ExternalModification);
	}
}

FPCGSourceDataContainer* UPCGEditPointsInteractiveTool::GetDataContainer() const
{
	return ExecutionSource->GetExecutionState().GetSourceDataContainer();
}

bool UPCGEditPointsInteractiveTool::MarkDataContainerDirty()
{
	if (FPCGSourceDataContainer* DataContainer = GetDataContainer())
	{
		DataContainer->MarkDirty();
		return true;
	}
	else
	{
		return false;
	}
}

FPCGDeltaCollection* UPCGEditPointsInteractiveTool::GetCollection(const FPCGSourceDataStorageKey& StorageKey) const
{
	if (StorageKey.Hash == 0)
	{
		return nullptr;
	}
	
	FPCGSourceDataContainer* DataContainer = GetDataContainer();
	const FSharedStruct SharedStruct = DataContainer ? DataContainer->GetMutable<FPCGDeltaCollection>(StorageKey) : FSharedStruct();
	return SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
}

FPCGDeltaCollection* UPCGEditPointsInteractiveTool::GetOrCreateCollection(const FPCGSourceDataStorageKey& StorageKey) const
{
	if (StorageKey.Hash == 0)
	{
		return nullptr;
	}
	
	FPCGSourceDataContainer* DataContainer = GetDataContainer();
	if (!DataContainer)
	{
		return nullptr;
	}

	const FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(StorageKey);
	FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
	if (!Collection)
	{
		// Create collection if it doesn't exist
		FPCGDeltaCollection NewCollection;
		NewCollection.Settings.KeyPolicy = EPCGDataOverrideKeyPolicy::Spatial;
		NewCollection.Settings.KeyTarget = EPCGSpatialKeyTarget::Position;
		NewCollection.Settings.SpatialTolerance = PCG::DataOverride::Constants::SpatialToleranceDefault;
		DataContainer->Store(StorageKey, NewCollection);
		DataContainer->MarkDirty();

		Collection = DataContainer->GetMutable<FPCGDeltaCollection>(StorageKey).GetPtr<FPCGDeltaCollection>();
		check(Collection);
	}
	return Collection;
}

FPCGPointTransformDelta* UPCGEditPointsInteractiveTool::GetTransformDelta(TInstancedStruct<FPCGDeltaBase>* InstancedStructDelta)
{
	return InstancedStructDelta ? InstancedStructDelta->GetMutablePtr<FPCGPointTransformDelta>() : nullptr;
}

const FPCGPointTransformDelta* UPCGEditPointsInteractiveTool::GetTransformDelta(const TInstancedStruct<FPCGDeltaBase>* InstancedStructDelta) const
{
	return InstancedStructDelta ? InstancedStructDelta->GetPtr<FPCGPointTransformDelta>() : nullptr;
}


#undef LOCTEXT_NAMESPACE
