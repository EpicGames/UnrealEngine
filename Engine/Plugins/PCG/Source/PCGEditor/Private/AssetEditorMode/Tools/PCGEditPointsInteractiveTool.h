// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGraphExecutionInspection.h"
#include "AssetEditorMode/Tools/PCGAssetEditorInteractiveTool.h"
#include "Elements/PCGCreatePoints.h"
#include "Elements/PCGCreatePointsGrid.h"
#include "Elements/PCGCreatePointsSphere.h"
#include "Elements/PCGEditPoints.h"
#include "Graph/PCGSourceDataContainer.h"
#include "Graph/DataOverride/PCGDataOverride.h"
#include "Graph/DataOverride/PCGDataOverridePoints.h"
#include "Subsystems/IPCGBaseSubsystem.h"

#include "EditorUndoClient.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "PCGEditPointsInteractiveTool.generated.h"

class FPositionVertexBuffer;
class UCombinedTransformGizmo;
class UStaticMesh;
class UTransformProxy;

/**
 * Interactive viewport tool for UPCGCreatePointsSettings.
 *
 * Renders a wire box for each point in the node's output.
 * Clicking a box shows a combined transform gizmo; dragging it writes a
 * FPCGPointTransformDelta into FPCGSourceDataContainer and triggers graph re-execution.
 */
UCLASS()
class UPCGEditPointsInteractiveTool : public UPCGAssetEditorInteractiveTool, public IClickBehaviorTarget, public FSelfRegisteringEditorUndoClient
{
	GENERATED_BODY()

public:
	PCG_DECLARE_SUPPORTED_NODES(
		UPCGCreatePointsSettings,
		UPCGCreatePointsGridSettings,
		UPCGCreatePointsSphereSettings,
		UPCGEditPointsSettings
	);

	//~ Begin UInteractiveTool Interface
	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	//~ End UInteractiveTool Interface

	//~ Begin IClickBehaviorTarget Interface
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPos) override;
	//~ End IClickBehaviorTarget Interface
	
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

protected:
	//~ Begin UPCGAssetEditorInteractiveTool Interface
	virtual void OnAccept() override;
	virtual void OnCancel() override;

	virtual void OnNodeSettingsChanged(UPCGSettings* NodeSettings, EPCGChangeType ChangeType) override;
	//~ End UPCGAssetEditorInteractiveTool Interface

private:
	void OnGraphExecuted(IPCGBaseSubsystem* InSubsystem, IPCGGraphExecutionSource* InExecutionSource, EPCGGenerationStatus InGenerationStatus);

	void OnGizmoTransformBegin(UTransformProxy* Proxy);
	void OnGizmoTransformChanged(UTransformProxy* Proxy, FTransform NewTransform);
	void OnGizmoTransformEnd(UTransformProxy* Proxy);

	void ShowGizmoAt(const FTransform& Transform);
	void HideGizmo();

	void BuildCachedMesh();

	/** Trigger graph re-execution after delta changes. */
	void NotifyGraphChanged();

	FPCGSourceDataContainer* GetDataContainer() const;
	bool MarkDataContainerDirty();

	FPCGDeltaCollection* GetCollection(const FPCGSourceDataStorageKey& StorageKey) const;
	FPCGDeltaCollection* GetOrCreateCollection(const FPCGSourceDataStorageKey& StorageKey) const;
	FPCGPointTransformDelta* GetTransformDelta(TInstancedStruct<FPCGDeltaBase>* InstancedStructDelta);
	const FPCGPointTransformDelta* GetTransformDelta(const TInstancedStruct<FPCGDeltaBase>* InstancedStructDelta) const;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> CombinedGizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	FPCGDeltaKey CurrentDeltaKey;
	FPCGSourceDataStorageKey CurrentStorageKey;
	FPCGDeltaCollection* CurrentCollection = nullptr;
	TInstancedStruct<FPCGDeltaBase>* CurrentEditDelta = nullptr;

	bool bIsTransforming = false;

	// Mesh wireframe cache — parallels FCachedMeshWireframe in PCGComponentVisualizer.cpp
	TWeakObjectPtr<UStaticMesh> CachedMesh;
	TArray<uint32> CachedMeshIndices;
	const FPositionVertexBuffer* CachedPositionBuffer = nullptr;

	// Cached copy of executed node stacks, updated via OnGraphExecuted callback to avoid a per-frame copy.
	TMap<TObjectKey<const UPCGNode>, TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData>> CachedNodeStacks;
};
