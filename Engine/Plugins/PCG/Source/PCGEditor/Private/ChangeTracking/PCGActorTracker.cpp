// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "ChangeTracking/PCGActorTracker.h"

#include "PCGTrackingManager.h"
#include "PCGWorldActor.h"
#include "Helpers/PCGHelpers.h"

#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "LevelInstance/LevelInstanceEditorPivotInterface.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Misc/TransactionObjectEvent.h"
#include "PackedLevelActor/PackedLevelActor.h"
#include "UObject/UObjectGlobals.h"
#include "WorldPartition/WorldPartition.h"

namespace PCGActorTracker
{
	static TAutoConsoleVariable<bool> CVarDisableDelayedActorRegistering(
		TEXT("pcg.DisableDelayedActorRegistering"),
		false,
		TEXT("If delayed actor registering when their components aren't registered yet is introducing bad behavior, disables it, allowing people to continue working while we investigate."));

	static TAutoConsoleVariable<int> CVarActorModifiedPreviousDataCleanupDelayMS(
		TEXT("pcg.ActorModifiedPreviousDataCleanupDelayMS"),
		10000,
		TEXT("Time in MS between a cached previous data from modified actor and the time we remove it from our cache."));

	static TAutoConsoleVariable<int> CVarActorModifiedPreviousDataCheckDelayMS(
		TEXT("pcg.ActorModifiedPreviousDataCheckDelayMS"),
		1000,
		TEXT("Delay in MS between cleanup checks on previous data from modified actors."));

	bool ShouldIgnoreActor(const AActor* InActor)
	{
		if (!InActor)
		{
			return true;
		}

		return InActor->bIsEditorPreviewActor || InActor->IsA<ALevelInstanceEditorInstanceActor>() || InActor->IsA<APCGWorldActor>() || InActor->Implements<ULevelInstanceEditorPivotInterface>();
	}

	void PropagateToLevelInstanceActors(const ILevelInstanceInterface* InLevelInstance, TFunctionRef<bool(const AActor* LevelActor)> InFunc)
	{
		if (InLevelInstance)
		{
			if (const ULevelInstanceSubsystem* LevelInstanceSubsystem = InLevelInstance->GetLevelInstanceSubsystem())
			{
				// Handle streamed out level for game worlds 
				if (LevelInstanceSubsystem->GetWorld()->IsGameWorld())
				{
					ULevel* LoadedLevel = LevelInstanceSubsystem->GetLevelInstanceLevel(InLevelInstance);
					if (!LoadedLevel || !LoadedLevel->OwningWorld)
					{
						return;
					} 
				}

				LevelInstanceSubsystem->ForEachActorInLevelInstance(InLevelInstance, [InFunc](const AActor* LevelActor)
				{
					if (!ShouldIgnoreActor(LevelActor))
					{
						return InFunc(LevelActor);
					}
					return true;
				});
			}
		}
	}

	void PropagateToLevelInstanceActors(const AActor* InActor, TFunctionRef<bool(const AActor* LevelActor)> InFunc)
	{
		PropagateToLevelInstanceActors(Cast<ILevelInstanceInterface>(InActor), InFunc);
	}
}

const FLazyName FPCGActorTracker::Name = TEXT("ActorTracker");

FPCGActorTracker::FPCGActorTracker(FPCGTrackingManager* InOwner)
	: IPCGChangeTracker(InOwner)
{
	GEngine->OnLevelActorAdded().AddRaw(this, &FPCGActorTracker::OnActorAdded);
	GEngine->OnLevelActorDeleted().AddRaw(this, &FPCGActorTracker::OnActorDeleted);
	FCoreUObjectDelegates::OnObjectModified.AddRaw(this, &FPCGActorTracker::OnObjectModified);
	FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &FPCGActorTracker::OnObjectTransacted);
	
	UWorld* World = Owner->GetWorld();
	check(World);

	if (World->PersistentLevel && !World->IsGameWorld())
	{
		World->PersistentLevel->OnLoadedActorAddedToLevelEvent.AddRaw(this, &FPCGActorTracker::OnActorLoaded);
		World->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.AddRaw(this, &FPCGActorTracker::OnActorUnloaded);

		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(World))
		{
			LevelInstanceSubsystem->OnLevelInstancesUpdated().AddRaw(this, &FPCGActorTracker::OnLevelInstancesUpdated);
			LevelInstanceSubsystem->OnLevelInstanceEditCancelled().AddRaw(this, &FPCGActorTracker::OnLevelInstanceEditCancelled);
		}
	}
}

FPCGActorTracker::~FPCGActorTracker()
{
	GEngine->OnLevelActorAdded().RemoveAll(this);
	GEngine->OnLevelActorDeleted().RemoveAll(this);
	FCoreUObjectDelegates::OnObjectModified.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);

	UWorld* World = Owner->GetWorld();
	check(World);

	if (World->PersistentLevel && !World->IsGameWorld())
	{
		World->PersistentLevel->OnLoadedActorAddedToLevelEvent.RemoveAll(this);
		World->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.RemoveAll(this);

		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(World))
		{
			LevelInstanceSubsystem->OnLevelInstancesUpdated().RemoveAll(this);
			LevelInstanceSubsystem->OnLevelInstanceEditCancelled().RemoveAll(this);
		}
	}
}

TUniquePtr<IPCGChangeTracker> FPCGActorTracker::MakeInstance(FPCGTrackingManager* InOwner)
{
	return TUniquePtr<IPCGChangeTracker>(new FPCGActorTracker(InOwner));
}

FName FPCGActorTracker::GetName()
{
	return Name.Resolve();
}

void FPCGActorTracker::Tick()
{
	if (DelayedChangedActors.IsEmpty())
	{
		return;
	}

	auto LocalDelayedChangedActors = MoveTemp(DelayedChangedActors);

	for (const auto& ActorToObject : LocalDelayedChangedActors)
	{
		AActor* Actor = ActorToObject.Key.ResolveObjectPtr();
		UObject* Object = ActorToObject.Value.Get<0>().ResolveObjectPtr();
		const FPCGTrackingChangeID& Reason = ActorToObject.Value.Get<1>();

		if (!Actor)
		{
			continue;
		}

		OnActorChanged_Internal(Actor, Object, Reason);
	}

	const double CurrentTime = FApp::GetCurrentTime();
	// Cleaning up previous data gathered by the OnObjectModified function but not consumed.
	if (LastPreviousActorDataCleanup < 0.0 || ((CurrentTime - LastPreviousActorDataCleanup) * 1000) > PCGActorTracker::CVarActorModifiedPreviousDataCheckDelayMS.GetValueOnAnyThread())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTrackingManager::Tick::PreviousActorDataCleanupCheck);

		LastPreviousActorDataCleanup = CurrentTime;
		TArray<TObjectKey<AActor>> Keys;
		ActorToPreviousDataMap.GetKeys(Keys);
		for (const TObjectKey<AActor>& Key : Keys)
		{
			if (((CurrentTime - ActorToPreviousDataMap[Key].Get<2>()) * 1000) > PCGActorTracker::CVarActorModifiedPreviousDataCleanupDelayMS.GetValueOnAnyThread())
			{
				ActorToPreviousDataMap.Remove(Key);
			}
		}
	}
}

bool FPCGActorTracker::IsActorTracked(const AActor* InActor) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGActorTracker::IsActorTracked);

	const FPCGSelectionKey ActorKey = FPCGSelectionKey::CreateFromObjectPtr(InActor);
	bool bFoundMatch = false;
	TSet<FName> EmptySet;
	
	Owner->ForEachTrackedKey([&bFoundMatch, &ActorKey, &EmptySet](const FPCGSelectionKey& Key, const TSet<IPCGGraphExecutionSource*>& ExecutionSources) -> bool
	{
		if(Key.IsMatching(ActorKey, EmptySet, ExecutionSources, nullptr))
		{
			bFoundMatch = true;
			return false;
		}

		return true;
	});

	return bFoundMatch;
}

bool FPCGActorTracker::ShouldDelayActor(const AActor* InActor) const
{
	const bool bDisableDelayedActorRegistering = PCGActorTracker::CVarDisableDelayedActorRegistering.GetValueOnAnyThread();
	if (!InActor->HasActorRegisteredAllComponents() && !bDisableDelayedActorRegistering)
	{
		return true;
	}

	bool bShouldDelay = false;

	// Check that whole Level Instance hierarchy is loaded
	if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(InActor))
	{
		// If Level Instance isn't loaded yet it should be delayed
		if (!LevelInstance->IsLoaded() && !InActor->IsA<APackedLevelActor>())
		{
			return true;
		}

		PCGActorTracker::PropagateToLevelInstanceActors(LevelInstance, [this, &bShouldDelay](const AActor* InChildActor)
		{
			if (ShouldDelayActor(InChildActor))
			{
				bShouldDelay = true;
				return false;
			}
			return true;
		});
	}

	return bShouldDelay;
}

void FPCGActorTracker::OnLevelInstancesUpdated(const TArray<ILevelInstanceInterface*>& InLevelInstances)
{
	for (ILevelInstanceInterface* LevelInstance : InLevelInstances)
	{
		OnActorAdded(Cast<AActor>(LevelInstance));
	}
}

void FPCGActorTracker::OnLevelInstanceEditCancelled(ILevelInstanceInterface* InLevelInstance, bool bInHasDiscardedChanges)
{
	if (bInHasDiscardedChanges)
	{
		OnActorAdded(Cast<AActor>(InLevelInstance));
	}
}

void FPCGActorTracker::OnActorLoaded(AActor& InActor)
{
	// We have to make sure to not create a infinite loop
	if (PCGActorTracker::ShouldIgnoreActor(&InActor) || InActor.GetWorld() != Owner->GetWorld())
	{
		return;
	}

	// Loaded actors should not dirty.
	OnActorAdded_Internal(&InActor, /*bShouldDirty=*/ false);
}

void FPCGActorTracker::OnActorAdded(AActor* InActor)
{
	// We have to make sure to not create a infinite loop
	if (PCGActorTracker::ShouldIgnoreActor(InActor) || InActor->GetWorld() != Owner->GetWorld())
	{
		return;
	}

	// Implementation note: We delay adding because OnActorAdded fires before an actor's properties are set,
	// so the actor is not ready for processing until the next tick.
	// Only dirty if we aren't currently inside a loading path
	OnActorAdded_Internal(InActor, /*bShouldDirty=*/ !UE::GetIsEditorLoadingPackage());
}

void FPCGActorTracker::OnActorAdded_Internal(const AActor* InActor, bool bShouldDirty)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGActorTracker::OnActorAdded);
	check(!PCGActorTracker::ShouldIgnoreActor(InActor) && InActor->GetWorld() == Owner->GetWorld());


	if (!bShouldDirty || DelayedChangedActors.Contains(InActor))
	{
		// Nothing to do
		return;
	}

	// We delay adding because OnActorAdded fires before an actor's properties are set,
	// so the actor is not ready for processing until the next tick.
	DelayedChangedActors.Emplace(InActor, { nullptr, PCGActorTracker::ActorAdded });
}

void FPCGActorTracker::OnActorChanged_Internal(const AActor* InActor, const UObject* InOriginatingChangeObject, const FPCGTrackingChangeID& InChangeID)
{
	// Currently delayed, return
	if (DelayedChangedActors.Contains(InActor))
	{
		return;
	}

	// Should still be delayed, add to delayed actors, return
	if (ShouldDelayActor(InActor))
	{
		DelayedChangedActors.Emplace(InActor, { InOriginatingChangeObject, InChangeID });
		return;
	}

	// Call Recursive method
	OnActorChanged_Recursive(InActor, InOriginatingChangeObject, InChangeID);
}

void FPCGActorTracker::OnActorChanged_Recursive(const AActor* InActor, const UObject* InOriginatingChangeObject, const FPCGTrackingChangeID& InChangeID)
{
	check(!DelayedChangedActors.Contains(InActor));

	// Process actors recursively
	auto OnActorChanged = [this, InOriginatingChangeObject, InChangeID](const AActor* InActor, auto&& OnActorChangedRecursive) -> void
	{
		if (InActor)
		{
			FActorPreviousData* PreviousData = ActorToPreviousDataMap.Find(InActor);
			
			OnActorChanged_Impl(InActor, InOriginatingChangeObject, InChangeID, PreviousData);

			if (PreviousData)
			{
				ActorToPreviousDataMap.Remove(InActor);
			}
		}

		PCGActorTracker::PropagateToLevelInstanceActors(InActor, [this, OnActorChangedRecursive](const AActor* LevelActor)
		{
			OnActorChangedRecursive(LevelActor, OnActorChangedRecursive);
			return true;
		});
	};

	OnActorChanged(InActor, OnActorChanged);
}

void FPCGActorTracker::OnActorChanged_Impl(const AActor* InActor, const UObject* InOriginatingChangeObject, const FPCGTrackingChangeID& InChangeID, FActorPreviousData* InPreviousData)
{
	check(InActor);
	
	if (!Owner->IsTracking())
	{
		return;
	}

	// Discard everything related to self for actor added/deleted
	const bool bNoRefreshOwner = InChangeID == PCGActorTracker::ActorAdded || InChangeID == PCGActorTracker::ActorDeleted;

	ensure(InActor->GetWorld() == Owner->GetWorld());

	TSet<FName> RemovedTags;
	if (InPreviousData && !InPreviousData->Get<1>().IsEmpty())
	{
		RemovedTags = InPreviousData->Get<1>().Difference(TSet<FName>(InActor->Tags));
	}

	FPCGSelectionKey SelectionKey = FPCGSelectionKey::CreateFromObjectPtr(InActor);

	TArray<FBox, TInlineAllocator<2>> Bounds;
	const FBox ActorBounds = PCGHelpers::GetActorBounds(InActor, /*bIgnorePCGCreatedComponents=*/false);
	Bounds.Add(ActorBounds);
	
	// Also provide previous bounds if we do have previous data
	if (InPreviousData)
	{
		const FBox& PreviousBounds = InPreviousData->Get<0>();
		if (PreviousBounds.IsValid && !PreviousBounds.Equals(ActorBounds))
		{
			Bounds.Add(PreviousBounds);
		}
	}

	SelectionKey.OptionalBounds = Bounds;

	Owner->OnSelectionKeyChanged(SelectionKey, InOriginatingChangeObject, RemovedTags, bNoRefreshOwner, Bounds);
}

void FPCGActorTracker::OnActorUnloaded(AActor& InActor)
{
	// Don't dirty on unload (to mirror the behavior in load)
	OnActorDeleted_Internal(&InActor, /*bShouldDirty=*/false);
}

void FPCGActorTracker::OnActorDeleted(AActor* InActor)
{
	if (PCGActorTracker::ShouldIgnoreActor(InActor) || InActor->GetWorld() != Owner->GetWorld())
	{
		return;
	}

	// Only dirty if we aren't currently inside a loading path
	OnActorDeleted_Internal(InActor, /*bShouldDirty=*/ !UE::GetIsEditorLoadingPackage());
}

void FPCGActorTracker::OnActorDeleted_Internal(const AActor* InActor, bool bShouldDirty)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGActorTracker::OnActorDeleted_Internal);
	check(InActor && InActor->GetWorld() == Owner->GetWorld());

	if (!bShouldDirty)
	{
		// Nothing to do
		return;
	}

	PCGActorTracker::PropagateToLevelInstanceActors(InActor, [this, bShouldDirty](const AActor* LevelActor)
	{
		OnActorDeleted_Internal(LevelActor, bShouldDirty);
		return true;
	});

	// Notify all components that the actor has changed (was removed), but the Refresh will only happen AFTER the actor was actually removed from the world (because of delayed refresh).
	OnActorChanged_Impl(InActor, nullptr, PCGActorTracker::ActorDeleted, /*InPreviousData=*/nullptr);
}

void FPCGActorTracker::OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& Event)
{
	// We only care about deleted actors through undo redo
	const bool bProcessChange = Event.GetEventType() == ETransactionObjectEventType::UndoRedo && Event.HasPendingKillChange() && !IsValid(Object);
	if (!bProcessChange)
	{
		return;
	}

	const AActor* Actor = Cast<AActor>(Object);

	// If this isn't an actor early out / or not owned by the world we are tracking
	if (!Actor || (Actor->GetWorld() != Owner->GetWorld()))
	{
		return;
	}

	OnActorDeleted_Internal(Actor, /*bShouldDirty=*/ true);
}

void FPCGActorTracker::OnObjectModified(UObject* InObject)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGActorTracker::OnObjectModified);

	if (!Owner->IsTracking())
	{
		return;
	}

	AActor* Actor = Cast<AActor>(InObject);

	// Otherwise, if it's an actor component, track it as well
	if (!Actor)
	{
		if (UActorComponent* ActorComponent = Cast<UActorComponent>(InObject))
		{
			Actor = ActorComponent->GetOwner();
		}
	}

	if (!Actor || (Actor->GetWorld() != Owner->GetWorld()))
	{
		return;
	}

	if (PCGActorTracker::ShouldIgnoreActor(Actor))
	{
		return;
	}

	auto StorePreviousData = [this](const AActor* InActor, auto&& RecursiveCall) -> void
	{
		if (!ActorToPreviousDataMap.Contains(InActor))
		{
			if (IsActorTracked(InActor))
			{
				FActorPreviousData& PreviousData = ActorToPreviousDataMap.Add(InActor);
				PreviousData.Get<0>() = PCGHelpers::GetActorBounds(InActor, /*bIgnorePCGCreatedComponents=*/false);
				PreviousData.Get<1>() = TSet<FName>(InActor->Tags);
				PreviousData.Get<2>() = FApp::GetCurrentTime();
			}

			// Also propagate the pre-change to all child actors if it is within a level instance.
			PCGActorTracker::PropagateToLevelInstanceActors(InActor, [this, RecursiveCall](const AActor* LevelActor)
			{
				RecursiveCall(LevelActor, RecursiveCall);
				return true;
			});
		}
	};

	StorePreviousData(Actor, StorePreviousData);
}

bool FPCGActorTracker::OnNotifyObjectChanged(const UObject* InObject, const FPCGTrackingChangeID& InChangeID, const UObject* InOriginatingChangeObject)
{
	if (InChangeID != PCGActorTracker::ActorChanged)
	{
		return false;
	}

	const AActor* Actor = Cast<AActor>(InObject);
	if (PCGActorTracker::ShouldIgnoreActor(Actor) || Actor->GetWorld() != Owner->GetWorld())
	{
		return true;
	}

	OnActorChanged_Internal(Actor, InOriginatingChangeObject, InChangeID);
	return true;
}

bool FPCGActorTracker::OnObjectPropertyChanged(const UObject* InObject, FPropertyChangedEvent& InEvent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGActorTracker::OnObjectPropertyChanged);
	check(Owner->IsTracking());
	
	// First check if it is an actor
	const AActor* Actor = Cast<AActor>(InObject);

	// Otherwise, if it's an actor component, track it as well
	if (!Actor)
	{
		if (const UActorComponent* ActorComponent = Cast<UActorComponent>(InObject))
		{
			Actor = ActorComponent->GetOwner();
		}
	}

	// Not an actor, early out
	if (!Actor)
	{
		return false;
	}

	if ((Actor->GetWorld() != Owner->GetWorld()) || PCGActorTracker::ShouldIgnoreActor(Actor))
	{
		return true;
	}

	const bool bInteractiveChange = (InEvent.ChangeType == EPropertyChangeType::Interactive);
	// Special exception for actor tags, as we can't track otherwise an actor "losing" a tag
	const bool bActorTagChange = (InEvent.Property && InEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AActor, Tags));

	const bool bNoOperation = (bInteractiveChange && !bActorTagChange) || (Actor && (DelayedChangedActors.Contains(Actor)));

	if (bNoOperation)
	{
		// Don't remove it as it will be needed when the delay will be called.
		if (Actor && !(DelayedChangedActors.Contains(Actor)))
		{
			auto RemoveAll = [this](const AActor* InActor, auto&& RecursiveCall) -> void
			{
				ActorToPreviousDataMap.Remove(InActor);
				PCGActorTracker::PropagateToLevelInstanceActors(InActor, [this, RecursiveCall](const AActor* LevelActor)
				{
					RecursiveCall(LevelActor, RecursiveCall);
					return true;
				});
			};

			RemoveAll(Actor, RemoveAll);
		}

		return true;
	}

	OnActorChanged_Internal(Actor, /*InOriginatingChangeObject=*/ InObject, PCGActorTracker::ActorChanged);
	return true;
}

#endif // WITH_EDITOR