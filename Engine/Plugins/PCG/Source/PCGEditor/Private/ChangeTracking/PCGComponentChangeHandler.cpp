// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "ChangeTracking/PCGComponentChangeHandler.h"

#include "PCGTrackingManager.h"
#include "Grid/PCGPartitionActor.h"
#include "Utils/PCGGraphExecutionLogging.h"

#include "LandscapeProxy.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"

const FLazyName FPCGComponentChangeHandler::Name = TEXT("PCGComponentChangeHandler");

namespace PCGComponentChangeHandler
{
	bool ShouldDirtyGenerated(UPCGComponent* InPCGComponent, const UObject* InChangedObject)
	{
		// @todo_pcg: to remove usage
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		check(InPCGComponent);
		return InPCGComponent->ShouldTrackLandscape() && InChangedObject && InChangedObject->IsA<ALandscapeProxy>();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

FPCGComponentChangeHandler::FPCGComponentChangeHandler(FPCGTrackingManager* InOwner)
	: IPCGChangeHandler(InOwner)
{
}

FPCGComponentChangeHandler::~FPCGComponentChangeHandler()
{
}

TUniquePtr<IPCGChangeHandler> FPCGComponentChangeHandler::MakeInstance(FPCGTrackingManager* InOwner)
{
	return TUniquePtr<IPCGChangeHandler>(new FPCGComponentChangeHandler(InOwner));
}

FName FPCGComponentChangeHandler::GetName()
{
	return Name.Resolve();
}

void FPCGComponentChangeHandler::BeginChangeHandling(const TSharedRef<FPCGChangeHandlerChange>& InChange)
{
	check(InChange->ChangedObject);

	CurrentChange = MakeUnique<FCurrentChange>();
	CurrentChange->Change = InChange.ToWeakPtr();

	CurrentChange->ChangedObjects.Add(InChange->ChangedObject);
	if (InChange->OriginatingChangeObject != nullptr && InChange->ChangedObject != InChange->OriginatingChangeObject)
	{
		CurrentChange->ChangedObjects.Add(InChange->OriginatingChangeObject);
	}

	// @todo_pcg: Find a better way to get this info as this doesn't work for managed objects that were spawned under a different outer
	if (AActor* OuterActor = InChange->OriginatingChangeObject ? InChange->OriginatingChangeObject->GetTypedOuter<AActor>() : nullptr)
	{
		TArray<UPCGComponent*> OriginatingComponents;
		OuterActor->GetComponents<UPCGComponent>(OriginatingComponents);
		for (UPCGComponent* OriginatingComponent : OriginatingComponents)
		{
			if (!OriginatingComponent->IsCleaningUp() && OriginatingComponent->IsAnyObjectManagedByResource(CurrentChange->ChangedObjects))
			{
				CurrentChange->ChangedObjectDynamicTrackingPriority = OriginatingComponent->GetExecutionState().GetDynamicTrackingPriority();
				break;
			}
		}
	}

	CurrentChange->DirtyFlag = EPCGComponentDirtyFlag::Actor;
	if (InChange->ChangedObject->IsA<ALandscapeProxy>())
	{
		CurrentChange->DirtyFlag = CurrentChange->DirtyFlag | EPCGComponentDirtyFlag::Landscape;
	}
}

void FPCGComponentChangeHandler::HandleChange(IPCGGraphExecutionSource* InExecutionSource)
{
	check(CurrentChange.IsValid());

	TSharedPtr<FPCGChangeHandlerChange> Change = CurrentChange->Change.Pin();
	if (!Change)
	{
		return;
	}

	UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource);
	if (!PCGComponent)
	{
		return;
	}
	
	if (CurrentChange->DirtyComponents.Contains(PCGComponent) || ShouldDiscardComponent(PCGComponent))
	{
		return;
	}

	// Don't mark "Owner changed" if the change originate from a PCG Component. It will be delegated to the ClearCacheForActor.
	// It is necessary to avoid infine loops when there are multiple PCG components on one actor, and one component was generated.
	const bool bOwnerChanged = (PCGComponent->GetOwner() == Change->ChangedObject) && (!Change->OriginatingChangeObject || !Change->OriginatingChangeObject->IsA<UPCGComponent>());
	bool bShouldDirty = bOwnerChanged || PCGComponentChangeHandler::ShouldDirtyGenerated(PCGComponent, Change->ChangedObject);

	// Since when we clear the cache for a settings, we clear it all, it's only necessary to do it once on the original component. And we will dirty all the local components (with dirty dispatch)
	// Technically, a key that is always tracked can also be culled tracked, so maybe having an intersection test here to not
	// dirty components that culled tracked and don't intersect with the component would be more efficient.
	// In practice, it's mostly relevent for partitioned graphs and we will dirty all the locals anyway.
	bShouldDirty |= Owner->ClearCacheForKeys(Change->MatchedKeys, PCGComponent, /*bIntersect=*/true, Change->OriginatingChangeObject);
	if (bShouldDirty)
	{
		PCGComponent->DirtyGenerated(CurrentChange->DirtyFlag, /*bDispatchToLocalComponents=*/true);
		CurrentChange->DirtyComponents.Add(PCGComponent);
	}
}

void FPCGComponentChangeHandler::HandleBoundedChange(IPCGGraphExecutionSource* InExecutionSource, const FBox& InExecutionSourceBounds, const FBox& InChangeBounds)
{
	check(CurrentChange.IsValid());

	TSharedPtr<FPCGChangeHandlerChange> Change = CurrentChange->Change.Pin();
	if (!Change)
	{
		return;
	}

	UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource);
	if (!PCGComponent)
	{
		return;
	}

	if (ShouldDiscardComponent(PCGComponent))
	{
		return;
	}

	if (PCGComponent->IsPartitioned())
	{
		// Don't dirty if the component is not tracked, the origin of the change or its owner is the changed actor and we should not refresh.
				// We can "re-dirty" it because changes can impact different local components, from the same
				// original component.
		const bool bIntersect = InChangeBounds.Intersect(InExecutionSourceBounds);
		if (!bIntersect)
		{
			return;
		}

		const FBox Overlap = InChangeBounds.Overlap(InExecutionSourceBounds);
		bool bShouldDirty = PCGComponentChangeHandler::ShouldDirtyGenerated(PCGComponent, Change->ChangedObject);
		bShouldDirty |= Owner->ClearCacheForKeys(Change->MatchedKeys, PCGComponent, /*bIntersect=*/true, Change->OriginatingChangeObject);
		bool bWasDirtied = false;

		// Since when we clear the cache for a settings, we clear it all, it's only necessary to do it once on the original component, then only dirty the local that intersects.
		if (bShouldDirty)
		{
			Owner->ForAllIntersectingPartitionActors(Overlap, [PCGComponent, &bWasDirtied, this](APCGPartitionActor* InPartitionActor) -> void
			{
				if (UPCGComponent* LocalComponent = InPartitionActor->GetLocalComponent(PCGComponent))
				{
					bWasDirtied = true;
					LocalComponent->DirtyGenerated(CurrentChange->DirtyFlag);
				}
			});
		}

		if (bWasDirtied)
		{
			// Don't dispatch
			PCGComponent->DirtyGenerated(CurrentChange->DirtyFlag, /*bDispatchToLocalComponents=*/false);
			CurrentChange->DirtyComponents.Add(PCGComponent);
		}
	}
	else
	{
		// Don't dirty if the component was already dirtied, not tracked, the origin of the change or its owner is the changed actor and we should not refresh.
		if (CurrentChange->DirtyComponents.Contains(PCGComponent))
		{
			return;
		}

		bool bShouldDirty = PCGComponentChangeHandler::ShouldDirtyGenerated(PCGComponent, Change->ChangedObject);
		bShouldDirty |= Owner->ClearCacheForKeys(Change->MatchedKeys, PCGComponent, /*bIntersect=*/true, Change->OriginatingChangeObject);
		if (bShouldDirty)
		{
			PCGComponent->DirtyGenerated(CurrentChange->DirtyFlag);
			CurrentChange->DirtyComponents.Add(PCGComponent);
		}
	}
}

// We discard the component if:
	//   * It is null.
	//   * It is one of the objects that changed.
	//   * We should not refresh the owner and its owner is one of the objects that changed.
	//   * The objects that changed are managed by it (safe-guard, would probably induce an infinite loop).
	//   * The objects that changed are managed by a lower tracking priority component. (lower tracking priority value means higher priority)
	//   * One of the objects that changed is ignored by its original component.
bool FPCGComponentChangeHandler::ShouldDiscardComponent(UPCGComponent* InComponent) const
{
	check(CurrentChange.IsValid());

	TSharedPtr<FPCGChangeHandlerChange> Change = CurrentChange->Change.Pin();
	if (!Change)
	{
		return true;
	}

	if (!InComponent
		|| CurrentChange->ChangedObjects.Contains(InComponent)
		|| (Change->bNoRefreshOwner && CurrentChange->ChangedObjects.Contains(InComponent->GetOwner()))
		|| InComponent->IsCleaningUp()
		|| InComponent->IsAnyObjectManagedByResource(CurrentChange->ChangedObjects)
		|| (CurrentChange->ChangedObjectDynamicTrackingPriority.IsSet() && InComponent->GetExecutionState().GetDynamicTrackingPriority() < CurrentChange->ChangedObjectDynamicTrackingPriority.GetValue()))
	{
		return true;
	}

	const UObject* ObjectIgnored = nullptr;
	const UPCGComponent* OriginalComponent = InComponent->GetConstOriginalComponent();
	EPCGIgnoreChangeOriginReason Reason;

	if (ensure(OriginalComponent) && OriginalComponent->IsIgnoringAnyChangeOrigins(CurrentChange->ChangedObjects, ObjectIgnored, Reason))
	{
		PCGGraphExecutionLogging::LogChangeOriginIgnoredForComponent(ObjectIgnored, InComponent, Reason);
		return true;
	}

	return false;
}

void FPCGComponentChangeHandler::EndChangeHandling(bool bSkipRefresh)
{
	check(CurrentChange.IsValid());
	
	// Release current change
	ON_SCOPE_EXIT
	{
		CurrentChange.Reset();
	};

	if (bSkipRefresh)
	{
		return;
	}

	TSharedPtr<FPCGChangeHandlerChange> Change = CurrentChange->Change.Pin();
	if (!Change)
	{
		return;
	}

	ULevelInstanceSubsystem* LevelInstanceSubsystem = Owner->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>();
	const IPCGGraphExecutionSource* OriginatingExecutionSource = Cast<const IPCGGraphExecutionSource>(Change->OriginatingChangeObject);

	// And refresh all dirtied components
	for (UPCGComponent* Component : CurrentChange->DirtyComponents)
	{
		if (!ensure(Component))
		{
			continue;
		}

		// This part checks if the change originates from a PCG Component. If so, we check the dirty component has no more other PCG dependencies.
		// In that case we can proceed for the refresh, otherwise early out, this will be woken up by another dependency change.
		if (OriginatingExecutionSource && !Owner->RemoveExecutionSourceDependency(OriginatingExecutionSource, Component))
		{
			continue;
		}

		const AActor* ComponentOwner = Component->GetOwner();
		const bool bOwnerHasChanged = ComponentOwner == Change->ChangedObject;

		if ((!Change->bNoRefreshOwner || !bOwnerHasChanged) && (!Component->bOnlyTrackItself || bOwnerHasChanged))
		{
			const AActor* Actor = Cast<AActor>(Change->ChangedObject);

			// When an object changes, we need to make sure that we don't trigger a refresh on PCG components that are "higher" in the
			// level hierarchy, otherwise we will end up generating in the Level Instance level, which is wrong.
			// Note that in some instances (e.g. when something happens at the Level Instance level) we need to make sure that the
			// PCG components higher-up are properly updated, hence the level instance depth
			if (LevelInstanceSubsystem && Actor)
			{
				// Immediate Level Instance Parent (Actor)
				const ILevelInstanceInterface* ActorLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor);

				// Immediate Level Instance Parent (PCG Component)
				const ILevelInstanceInterface* ComponentLevelInstance = ComponentOwner ? LevelInstanceSubsystem->GetParentLevelInstance(ComponentOwner) : nullptr;

				// Is the Actor under a Level Instance that is the same or under the Components owning Level Instance
				bool bActorLevelInstanceInComponentLevelInstanceHierarchy = false;

				// Go up Level Instance hierarchy for Actor and see if we are part of the Component's level instance hierarchy, stop if we find an Editing Level instance as we don't want to impact top level 
				// Components from an editing level instance
				if (ComponentLevelInstance)
				{
					LevelInstanceSubsystem->ForEachLevelInstanceAncestors(Actor, [ComponentLevelInstance, &bActorLevelInstanceInComponentLevelInstanceHierarchy](const ILevelInstanceInterface* Ancestor)
					{
						if (Ancestor == ComponentLevelInstance)
						{
							bActorLevelInstanceInComponentLevelInstanceHierarchy = true;
							return false; // stop iterating
						}

						return !Ancestor->IsEditing();
					});
				}

				// Actor in same level instance as Component or under the Components Level instance
				if (bActorLevelInstanceInComponentLevelInstanceHierarchy)
				{
					// Allow update if Component is inside Editing level instance or if it is only in preview mode (generates transient data because it is in a non editing Level instance)
					if (!ComponentLevelInstance->IsEditing() && Component->GetEditingMode() != EPCGEditorDirtyMode::Preview)
					{
						continue;
					}
				} // Allow update of Component from a Editing Level instance contained actor if the Component being refreshed is in preview mode and not itself in a Level Instance
				else if (Actor->IsInEditLevelInstanceHierarchy() && (Component->GetEditingMode() != EPCGEditorDirtyMode::Preview || ComponentLevelInstance != nullptr))
				{
					continue;
				}
				// If ComponentLevelInstance is non-null and Actor Level instance isn't in its hierarchy then skip it
				else if (ActorLevelInstance && ComponentLevelInstance != nullptr)
				{
					continue;
				}
			}

			Component->Refresh();
		}
	}
}

#endif // WITH_EDITOR