// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "ChangeTracking/PCGLandscapeTracker.h"

#include "PCGTrackingManager.h"
#include "ChangeTracking/PCGActorTracker.h"
#include "Landscape/PCGLandscapeCVars.h"
#include "Subsystems/PCGSubsystem.h"

#include "Editor.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "ILevelEditor.h"
#include "Landscape.h"
#include "LandscapeSubsystem.h"
#include "LevelEditor.h"
#include "Engine/World.h"
#include "Modules/ModuleManager.h"

const FLazyName FPCGLandscapeTracker::Name = TEXT("LandscapeTracker");

FPCGLandscapeTracker::FPCGLandscapeTracker(FPCGTrackingManager* InOwner)
	: IPCGChangeTracker(InOwner)
{
	if (ULandscapeSubsystem* LandscapeSubsystem = Owner->GetWorld()->GetSubsystem<ULandscapeSubsystem>())
	{
		LandscapeSubsystem->OnLandscapeProxyComponentDataChanged().AddRaw(this, &FPCGLandscapeTracker::OnLandscapeChanged);
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		FirstLevelEditor->GetEditorModeManager().OnEditorModeIDChanged().AddRaw(this, &FPCGLandscapeTracker::OnEditorModeIDChanged);
	}
}

FPCGLandscapeTracker::~FPCGLandscapeTracker()
{
	if (ULandscapeSubsystem* LandscapeSubsystem = Owner->GetWorld()->GetSubsystem<ULandscapeSubsystem>())
	{
		LandscapeSubsystem->OnLandscapeProxyComponentDataChanged().RemoveAll(this);
	}

	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		if (TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor())
		{
			FirstLevelEditor->GetEditorModeManager().OnEditorModeIDChanged().RemoveAll(this);
		}
	}
}

TUniquePtr<IPCGChangeTracker> FPCGLandscapeTracker::MakeInstance(FPCGTrackingManager* InOwner)
{
	return TUniquePtr<IPCGChangeTracker>(new FPCGLandscapeTracker(InOwner));
}

FName FPCGLandscapeTracker::GetName()
{
	return Name.Resolve();
}

void FPCGLandscapeTracker::Tick()
{
	const double CurrentTime = FApp::GetCurrentTime();

	if (!DelayedModifiedLandscapes.IsEmpty() && LastLandscapeDirtyTime > 0.0 && ((CurrentTime - LastLandscapeDirtyTime) * 1000.0) > PCGLandscapeCVars::CVarLandscapeRefreshTimeDelay.GetValueOnAnyThread())
	{
		LastLandscapeDirtyTime = -1.0;
		for (TObjectKey<ALandscapeProxy> Landscape : DelayedModifiedLandscapes)
		{
			ApplyLandscapeChanges(Landscape.ResolveObjectPtr());
		}

		DelayedModifiedLandscapes.Empty();
	}
}

bool FPCGLandscapeTracker::ShouldSkipRefresh(const UObject* InChangedObject)
{
	const ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(InChangedObject);

	if (!LandscapeProxy)
	{
		return false;
	}

	// If refresh is globably disabled, never refresh
	if (PCGLandscapeCVars::CVarLandscapeDisableRefreshTracking.GetValueOnAnyThread())
	{
		return true;
	}

	// If refresh is not disabled in editing, always refresh
	if (!PCGLandscapeCVars::CVarLandscapeDisableRefreshTrackingInLandscapeEditingMode.GetValueOnAnyThread())
	{
		return false;
	}

	// Skip refresh if we are currently in landscape edit mode
	const ALandscape* Landscape = LandscapeProxy->GetLandscapeActor();
	if (Landscape && Landscape->HasLandscapeEdMode() && !bIsCurrentlyExitingLandscapeEditMode)
	{
		DirtiedLandscapes.AddUnique(LandscapeProxy);
		return true;
	}

	return false;
}

void FPCGLandscapeTracker::OnLandscapeChanged(ALandscapeProxy* InLandscape, const FLandscapeProxyComponentDataChangedParams& InChangeParams)
{
	if (!InLandscape)
	{
		return;
	}

	if (PCGLandscapeCVars::CVarLandscapeRefreshTimeDelay.GetValueOnAnyThread() > 0)
	{
		LastLandscapeDirtyTime = FApp::GetCurrentTime();
		DelayedModifiedLandscapes.AddUnique(InLandscape);
	}
	else
	{
		ApplyLandscapeChanges(InLandscape);
	}
}

void FPCGLandscapeTracker::ApplyLandscapeChanges(ALandscapeProxy* InLandscape)
{
	if (!InLandscape || !Owner->IsTracking())
	{
		return;
	}

#if WITH_EDITOR
	if (PCGLandscapeCVars::CVarLandscapeForceRefreshRuntimeGen.GetValueOnAnyThread())
	{
		if (UPCGSubsystem* Subsystem = UPCGSubsystem::GetInstance(Owner->GetWorld()))
		{
			// @todo_pcg: Temporarily inject a runtime-only refresh path when applying changes from landscape actors.
			// This is necessary for clearing and regenerating grass/detail on landscapes while painting.
			// Should be removed when change tracking can propagate changes for runtime gen specifically.
			Subsystem->RefreshAllRuntimeGenExecutionSources(EPCGChangeType::GenerationGrid);
		}
	}
#endif

	// Forward as an object change event that can be processed by another tracker
	Owner->NotifyObjectChanged(InLandscape, PCGActorTracker::ActorChanged);
}

void FPCGLandscapeTracker::OnEditorModeIDChanged(const FEditorModeID& EditorModeID, bool bIsEntering)
{
	if (EditorModeID == FBuiltinEditorModes::EM_Landscape && !bIsEntering)
	{
		OnLandscapeEditModeExited();
	}
}

void FPCGLandscapeTracker::OnLandscapeEditModeExited()
{
	TGuardValue<bool> Guard(bIsCurrentlyExitingLandscapeEditMode, true);

	// When the landscape edit mode is exited, force the refresh on all modified/dirtied landscapes.
	for (TObjectKey<ALandscapeProxy> Landscape : DelayedModifiedLandscapes)
	{
		DirtiedLandscapes.AddUnique(Landscape);
	}

	DelayedModifiedLandscapes.Empty();

	for (TObjectKey<ALandscapeProxy> Landscape : DirtiedLandscapes)
	{
		ApplyLandscapeChanges(Landscape.ResolveObjectPtr());
	}

	DirtiedLandscapes.Empty();
}

#endif // WITH_EDITOR