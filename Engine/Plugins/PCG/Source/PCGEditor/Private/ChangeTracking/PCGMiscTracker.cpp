// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "ChangeTracking/PCGMiscTracker.h"

#include "PCGDataAsset.h"
#include "PCGTrackingManager.h"

#include "Engine/DataTable.h"
#include "Engine/StaticMesh.h"
#include "UObject/UObjectGlobals.h"

const FLazyName FPCGMiscTracker::Name = TEXT("MiscTracker");

FPCGMiscTracker::FPCGMiscTracker(FPCGTrackingManager* InOwner)
	: IPCGChangeTracker(InOwner)
{
	FCoreUObjectDelegates::OnObjectPreSave.AddRaw(this, &FPCGMiscTracker::OnObjectSaved);
}

FPCGMiscTracker::~FPCGMiscTracker()
{
	FCoreUObjectDelegates::OnObjectPreSave.RemoveAll(this);
}

TUniquePtr<IPCGChangeTracker> FPCGMiscTracker::MakeInstance(FPCGTrackingManager* InOwner)
{
	return TUniquePtr<IPCGChangeTracker>(new FPCGMiscTracker(InOwner));
}

FName FPCGMiscTracker::GetName()
{
	return Name.Resolve();
}

void FPCGMiscTracker::OnObjectSaved(UObject* InObject, FObjectPreSaveContext InObjectSaveContext)
{
	check(Owner);

	// Nothing to do if we track nothing
	if (!InObject || !Owner->IsTracking())
	{
		return;
	}

	// Only trigger a refresh on save for limited data classes.
	// At this point in time, We only track data tables and PCG data assets because in most cases we probably will catch other changes with OnObjectPropertyChanged.
	// This is especially important to make sure we don't trigger refresh multiple times (less a problem for PCG assets, but a big problem for data tables).
	if (!InObjectSaveContext.IsProceduralSave() && (InObject->IsA<UDataTable>() || InObject->IsA<UPCGDataAsset>() || InObject->IsA<UStaticMesh>()))
	{
		FPCGSelectionKey SelectionKey = FPCGSelectionKey::CreateFromObjectPtr(InObject);
		Owner->OnSelectionKeyChanged(SelectionKey);
	}
}

#endif // WITH_EDITOR