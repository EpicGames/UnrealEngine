// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "ChangeTracking/PCGRuntimeGenChangeHandler.h"

#include "PCGSubsystem.h"
#include "PCGContext.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGTrackingManager.h"
#include "RuntimeGen/PCGRuntimeGenExecutionSource.h"

namespace PCGRuntimeGenChangeHandler
{
	constexpr FLazyName Name("PCGRuntimeGenChangeHandler");
}

FPCGRuntimeGenChangeHandler::FPCGRuntimeGenChangeHandler(FPCGTrackingManager* InOwner)
	: IPCGChangeHandler(InOwner)
{
}

TUniquePtr<IPCGChangeHandler> FPCGRuntimeGenChangeHandler::MakeInstance(FPCGTrackingManager* InOwner)
{
	return MakeUnique<IPCGChangeHandler>(InOwner);
}

FName FPCGRuntimeGenChangeHandler::GetName()
{
	return PCGRuntimeGenChangeHandler::Name;
}

void FPCGRuntimeGenChangeHandler::BeginChangeHandling(const TSharedRef<FPCGChangeHandlerChange>& InChange)
{
	check(InChange->ChangedObject);

	CurrentChange = MakeUnique<FCurrentChange>();
	CurrentChange->Change = InChange.ToWeakPtr();
	CurrentChange->ChangedObjects.Add(InChange->ChangedObject);

	if (InChange->OriginatingChangeObject != nullptr && InChange->ChangedObject != InChange->OriginatingChangeObject)
	{
		CurrentChange->ChangedObjects.Add(InChange->OriginatingChangeObject);
	}
}

void FPCGRuntimeGenChangeHandler::HandleChange(IPCGGraphExecutionSource* InExecutionSource)
{
	HandleChangeInternal(InExecutionSource);
}

void FPCGRuntimeGenChangeHandler::HandleBoundedChange(IPCGGraphExecutionSource* InExecutionSource, const FBox& InExecutionSourceBounds, const FBox& InChangeBounds)
{
	HandleChangeInternal(InExecutionSource, &InExecutionSourceBounds, &InChangeBounds);
}

void FPCGRuntimeGenChangeHandler::EndChangeHandling(bool bSkipRefresh)
{
	check(CurrentChange.IsValid());
	
	// Release current change
	ON_SCOPE_EXIT
	{
		CurrentChange.Reset();
	};

	if (bSkipRefresh)
	{
		return;
	}

	TSharedPtr<FPCGChangeHandlerChange> Change = CurrentChange->Change.Pin();

	if (!Change)
	{
		return;
	}

	// And refresh all dirtied execution sources.
	for (UPCGRuntimeGenExecutionSource* DirtyExecutionSource : CurrentChange->DirtyExecutionSources)
	{
		if (!ensure(DirtyExecutionSource))
		{
			continue;
		}

		const bool bOwnerHasChanged = DirtyExecutionSource->GetExecutionState().GetTarget() == Change->ChangedObject;

		if (!Change->bNoRefreshOwner || !bOwnerHasChanged)
		{
			if (UPCGSubsystem* Subsystem = Cast<UPCGSubsystem>(DirtyExecutionSource->GetExecutionState().GetSubsystem()))
			{
				Subsystem->RefreshRuntimeGenExecutionSource(DirtyExecutionSource);
			}
		}
	}
}

void FPCGRuntimeGenChangeHandler::HandleChangeInternal(IPCGGraphExecutionSource* InExecutionSource, const FBox* InExecutionSourceBounds, const FBox* InChangeBounds)
{
	check(CurrentChange.IsValid());
	
	TSharedPtr<FPCGChangeHandlerChange> Change = CurrentChange->Change.Pin();

	if (!Change)
	{
		return;
	}

	UPCGRuntimeGenExecutionSource* ExecutionSource = Cast<UPCGRuntimeGenExecutionSource>(InExecutionSource);

	if (!ExecutionSource || CurrentChange->DirtyExecutionSources.Contains(ExecutionSource))
	{
		return;
	}

	if (ShouldDiscardExecutionSource(ExecutionSource))
	{
		return;
	}

	if (InExecutionSourceBounds && InChangeBounds && !InChangeBounds->Intersect(*InExecutionSourceBounds))
	{
		return;
	}

	if (Owner->ClearCacheForKeys(Change->MatchedKeys, ExecutionSource, /*bIntersect=*/true, Change->OriginatingChangeObject))
	{
		CurrentChange->DirtyExecutionSources.Add(ExecutionSource);
	}
}

bool FPCGRuntimeGenChangeHandler::ShouldDiscardExecutionSource(UPCGRuntimeGenExecutionSource* InExecutionSource) const
{
	check(CurrentChange.IsValid());

	TSharedPtr<FPCGChangeHandlerChange> Change = CurrentChange->Change.Pin();

	if (!Change)
	{
		return true;
	}

	// Discard null execution sources.
	if (!InExecutionSource)
	{
		return true;
	}

	// Avoid loops if the change originates from our own execution.
	if (FPCGContext* CurrentContext = FPCGContext::GetContextForThread(); CurrentContext && CurrentContext->ExecutionSource.Get() == InExecutionSource)
	{
		return true;
	}

	if (Change->bNoRefreshOwner && CurrentChange->ChangedObjects.Contains(InExecutionSource->GetExecutionState().GetTarget()))
	{
		return true;
	}

	return false;
}

#endif // WITH_EDITOR
