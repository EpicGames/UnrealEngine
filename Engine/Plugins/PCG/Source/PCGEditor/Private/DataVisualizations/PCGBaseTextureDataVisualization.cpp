// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGBaseTextureDataVisualization.h"

#include "DataVisualizations/PCGVisualizationTexture2D.h"

#include "Data/PCGTextureData.h"

#include "AdvancedPreviewScene.h"
#include "EditorViewportClient.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/AssetManager.h"
#include "Engine/Texture.h"
#include "Materials/MaterialInstanceDynamic.h"

TArray<TSharedPtr<FStreamableHandle>> FPCGBaseTextureDataVisualization::LoadRequiredResources(const UPCGData* Data) const
{
	TArray<TSharedPtr<FStreamableHandle>> LoadHandles;
	LoadHandles.Add(UAssetManager::GetStreamableManager().RequestAsyncLoad(PCGTextureVisualizationConstants::PlaneMeshPath, nullptr));
	LoadHandles.Add(UAssetManager::GetStreamableManager().RequestAsyncLoad(PCGTextureVisualizationConstants::DebugMaterialPath, nullptr));

	return LoadHandles;
}

FPCGSetupSceneFunc FPCGBaseTextureDataVisualization::GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const
{
	return [this, WeakData=TWeakObjectPtr<const UPCGTexture2DSingleBaseData>(Cast<UPCGTexture2DSingleBaseData>(Data))](FPCGSceneSetupParams& InOutParams)
	{
		check(InOutParams.Scene);
		check(InOutParams.EditorViewportClient);
		check(InOutParams.EditorViewportClient->Viewport);

		if (InOutParams.Resources.Num() != 2)
		{
			return;
		}

		if (!WeakData.IsValid())
		{
			UE_LOGF(LogPCG, Error, "Failed to setup data viewport, the data was lost or invalid.");
			return;
		}

		TObjectPtr<UTexture> Texture = nullptr;
		bool bIsTextureArray = false;

		if (WeakData->GetTextureResourceType() == EPCGTextureResourceType::TextureObject)
		{
			Texture = WeakData->GetTexture();
			bIsTextureArray = WeakData->GetTextureRHI() && WeakData->GetTextureRHI()->GetDesc().Dimension == ETextureDimension::Texture2DArray;
		}
		else if (WeakData->GetTextureResourceType() == EPCGTextureResourceType::ExportedTexture)
		{
			FPCGVisualizationTexture2DParams VisParams;
			VisParams.TextureRHI = WeakData->GetTextureRHI();
			VisParams.Resolution = WeakData->GetResolution();
			VisParams.SliceIndex = WeakData->GetTextureSlice();

			Texture = UPCGVisualizationTexture2D::Create(VisParams);
			InOutParams.ManagedResources.Add(Texture);
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Texture data uses an unsupported resource type for data viewport visualization.");
			return;
		}

		UMaterialInterface* DebugMaterial = Cast<UMaterialInterface>(InOutParams.Resources[1]);
		UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(DebugMaterial, GetTransientPackage());

		if (bIsTextureArray)
		{
			MaterialInstance->SetTextureParameterValue(FName(TEXT("DebugTextureArray")), Texture);
			MaterialInstance->SetScalarParameterValue(FName(TEXT("SliceIndex")), WeakData->GetTextureSlice());
			MaterialInstance->SetScalarParameterValue(FName(TEXT("UseTextureArray")), 1.0f);
		}
		else
		{
			MaterialInstance->SetTextureParameterValue(FName(TEXT("DebugTexture")), Texture);
		}

		InOutParams.ManagedResources.Add(MaterialInstance);

		TObjectPtr<UStaticMeshComponent> MeshComponent = NewObject<UStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		MeshComponent->SetStaticMesh(Cast<UStaticMesh>(InOutParams.Resources[0]));
		MeshComponent->OverrideMaterials.Add(MaterialInstance);
		InOutParams.ManagedResources.Add(MeshComponent);

		if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() <= ERHIFeatureLevel::ES3_1)
		{
			MeshComponent->SetMobility(EComponentMobility::Static);
		}

		static const FTransform MeshTransform(FRotator(0.0, -90.0, 0.0), FVector::ZeroVector, FVector::OneVector);
		InOutParams.Scene->AddComponent(MeshComponent, MeshTransform);

		// @todo_pcg: These settings should all be exposed through InOutParams. Textures should probably have their own preview scene profile that gets selected automatically.
		InOutParams.Scene->SetFloorVisibility(false);
		InOutParams.Scene->SetEnvironmentVisibility(false);
		InOutParams.PreferredViewportType = ELevelViewportType::LVT_OrthoTop;
		InOutParams.EditorViewportClient->SetViewMode(EViewModeIndex::VMI_Unlit);

		const FIntPoint ViewportSize = InOutParams.EditorViewportClient->Viewport->GetSizeXY();

		if (ViewportSize.X > 0 && ViewportSize.Y > 0)
		{
			// Bounds will be updated already by SetStaticMesh() call.
			InOutParams.FocusBounds = MeshComponent->Bounds;

			// Fit the orthographic zoom to the mesh. 0.8f is chosen arbitrarily to add some padding around the mesh.
			const float MeshUnitsPerPixel = FMath::Max(InOutParams.FocusBounds.BoxExtent.X / ViewportSize.X, InOutParams.FocusBounds.BoxExtent.Y / ViewportSize.Y) * 2.0f;
			InOutParams.FocusOrthoZoom = FMath::Clamp(MeshUnitsPerPixel * DEFAULT_ORTHOZOOM * 0.8f, MIN_ORTHOZOOM, MAX_ORTHOZOOM);
		}
	};
}
