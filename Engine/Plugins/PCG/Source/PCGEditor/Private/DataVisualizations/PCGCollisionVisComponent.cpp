// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGCollisionVisComponent.h"

#include "MeshElementCollector.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/Engine.h"
#include "MaterialEditor/MaterialEditorMeshComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "Physics/PhysicsInterfaceDeclares.h"
#include "PhysicsEngine/BodySetup.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCollisionVisComponent)

class FPCGCollisionVisProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FPCGCollisionVisProxy(UPCGCollisionVisComponent* Component)
		: FPrimitiveSceneProxy(Component)
	{
		bWillEverBeLit = false;
		BodySetups = Component->BodySetups;
		BodyTransforms = Component->BodyTransforms;
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		if (AllowDebugViewmodes())
		{
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FLinearColor DrawColor(GEngine->C_BrushWire);
					const FColor DrawColorInFColor = DrawColor.ToFColor(true);
					auto SolidMaterialInstance = new FColoredMaterialRenderProxy(
						GEngine->ShadedLevelColorationUnlitMaterial->GetRenderProxy(),
						DrawColor
					);

					Collector.RegisterOneFrameMaterialProxy(SolidMaterialInstance);

					const FSceneView* View = Views[ViewIndex];

					for (int Instance = 0; Instance < BodySetups.Num(); ++Instance)
					{
						UBodySetup* BodySetup = BodySetups[Instance];

						if (BodySetup)
						{
							FTransform GeomTransform = BodyTransforms[Instance] * FTransform(GetLocalToWorld());
							BodySetup->AggGeom.GetAggGeom(GeomTransform, DrawColorInFColor, /*Material=*/SolidMaterialInstance, false, /*bSolid=*/ true, AlwaysHasVelocity(), ViewIndex, Collector);
						}
					}
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		bool bVisible = false;

		if (IsShown(View))
		{
			bVisible = true;
		}

		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = bVisible;
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = IsShadowCast(View);

		return Result;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }
	uint32 GetAllocatedSize(void) const { return(FPrimitiveSceneProxy::GetAllocatedSize()); }

	TArray<UBodySetup*> BodySetups;
	TArray<FTransform> BodyTransforms;
};

FPrimitiveSceneProxy* UPCGCollisionVisComponent::CreateSceneProxy()
{
	return BodySetups.IsEmpty() ? nullptr : new FPCGCollisionVisProxy(this);
}

FBoxSphereBounds UPCGCollisionVisComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (BodySetups.IsEmpty())
	{
		return FBoxSphereBounds(LocalToWorld.GetLocation(), FVector::ZeroVector, 0.f);
	}
	else
	{
		FBoxSphereBounds NewBounds(EForceInit::ForceInit);
		bool bHasBounds = false;

		for (int Instance = 0; Instance < BodySetups.Num(); ++Instance)
		{
			FBoxSphereBounds InstanceBounds;
			BodySetups[Instance]->AggGeom.CalcBoxSphereBounds(InstanceBounds, BodyTransforms[Instance] * LocalToWorld);

			if (bHasBounds)
			{
				NewBounds = NewBounds + InstanceBounds;
			}
			else
			{
				bHasBounds = true;
				NewBounds = InstanceBounds;
			}
		}

		return NewBounds;
	}
}

void UPCGCollisionVisComponent::SetBodySetup(UBodySetup* InBodySetup)
{
	BodySetups.Reset();
	BodyTransforms.Reset();

	if (InBodySetup)
	{
		BodySetups.Add(InBodySetup);
		BodyTransforms.Add(FTransform::Identity);
	}
}

void UPCGCollisionVisComponent::AddBodySetup(UBodySetup* InBodySetup, const FTransform& InBodyTransform)
{
	if (InBodySetup)
	{
		BodySetups.Add(InBodySetup);
		BodyTransforms.Add(InBodyTransform);
	}
}
