// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGDataViewVisualization.h"

#include "Data/DataView/PCGDataView.h"
#include "Data/DataView/PCGDataViewData.h"
#include "DataVisualizations/PCGDataVisualizationHelpers.h"

void FPCGDataViewVisualization::ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const
{
	/* Do nothing. */
}

// @todo_pcg: It would be nice to preset the filter to only show selected attributes, even with custom visualizer.
FPCGTableVisualizerInfo FPCGDataViewVisualization::GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
{
	const UPCGDataViewData* DataViewData = Cast<UPCGDataViewData>(Data);
	if (!DataViewData)
	{
		return FPCGTableVisualizerInfo{};
	}

	// Pull the visualization of the underlying type
	const FPCGDataView& DataView = DataViewData->GetDataView();
	if (const UPCGData* ViewedData = DataView.ViewedData)
	{
		const FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetConstPCGDataVisualizationRegistry();

		if (const IPCGDataVisualization* DataVisualization = DataVisRegistry.GetDataVisualization(ViewedData->GetClass()))
		{
			return DataVisualization->GetTableVisualizerInfoWithDomain(ViewedData, DomainID);
		}

		FPCGTableVisualizerInfo Info;
		Info.Data = ViewedData;

		if (DataView.Selection.bAllAttributes)
		{
			PCGDataVisualizationHelpers::CreateDefaultMetadataColumnInfos(ViewedData, DomainID);
			return Info;
		}

		// Any data beyond here has no custom visualizer and isn't selecting all attributes. Add attributes one at a time.
		const UPCGMetadata* Metadata = ViewedData->ConstMetadata();
		check(Metadata);

		const FPCGMetadataDomain* MetadataDomain = Metadata ? Metadata->GetConstMetadataDomain(DomainID) : nullptr;
		if (!MetadataDomain)
		{
			return FPCGTableVisualizerInfo{};
		}

		// Sort the index to the front
		TArray<FPCGAttributePropertySelector> Attributes = DataView.Selection.Attributes;
		FPCGAttributePropertySelector IndexSelector = FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index);
		const int32 IndexIndex = Attributes.IndexOfByPredicate([&IndexSelector](const FPCGAttributePropertySelector& Selector)
		{
			return Selector == IndexSelector;
		});

		if (IndexIndex > 0)
		{
			Attributes.Swap(0, IndexIndex);
		}

		for (const FPCGAttributePropertySelector& Selector : Attributes)
		{
			const FName AttributeName = Selector.IsBasicAttribute() ? Selector.GetAttributeName() : Selector.GetPropertyName();
			FPCGMetadataDomainID ID = ViewedData->GetMetadataDomainIDFromSelector(Selector);
			if (MetadataDomain->HasAttribute(AttributeName) && ID == DomainID)
			{
				auto AddInfo = [ViewedData, &Info]<typename T>(T, const FPCGAttributePropertySelector& InSelector)
				{
					PCGDataVisualizationHelpers::AddTypedColumnInfo<T>(Info, ViewedData, InSelector);
				};

				const FPCGMetadataAttributeBase* AttributeBase = MetadataDomain->GetConstAttribute(AttributeName);
				PCGMetadataAttribute::CallbackWithRightType(AttributeBase->GetTypeId(), AddInfo, Selector);
			}
		}

		if (!Info.ColumnInfos.IsEmpty())
		{
			// Sort by $Index, if it exists. Otherwise, sort by the first column. 
			const bool bHasIndex = Info.ColumnInfos.ContainsByPredicate([&IndexSelector](const FPCGTableVisualizerColumnInfo& ColumnInfo)
			{
				return ColumnInfo.Id == IndexSelector.GetPropertyName();
			});

			if (bHasIndex)
			{
				// Attributes were sorted, so this should be guaranteed
				ensure(Info.ColumnInfos[0].Id == IndexSelector.GetExtraNames()[0]);
				Info.SortingColumn = Info.ColumnInfos[0].Id;
			}
			else // Create an index column
			{
				ViewedData->SetDomainFromDomainID(DomainID, IndexSelector);
				PCGDataVisualizationHelpers::AddColumnInfo(Info, ViewedData, IndexSelector);
				Info.SortingColumn = Info.ColumnInfos.Last().Id;
			}
		}

		return Info;
	}

	return FPCGTableVisualizerInfo{};
}

FString FPCGDataViewVisualization::GetDomainDisplayNameForInspection(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
{
	if (const UPCGDataViewData* DataViewData = Cast<UPCGDataViewData>(Data))
	{
		const FPCGDataView& DataView = DataViewData->GetDataView();
		if (const UPCGData* ViewedData = Cast<UPCGData>(DataView.ViewedData))
		{
			FPCGAttributePropertySelector Selector;
			ViewedData->SetDomainFromDomainID(DomainID, Selector);
			return Selector.GetDomainName().ToString();
		}
	}

	return {};
}
