// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGDataVisualizationHelpers.h"

#include "Widgets/SPCGEditorGraphAttributeListView.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#define LOCTEXT_NAMESPACE "PCGDataVisualizationHelpers"

namespace PCGDataVisualizationConstants
{
	const FString NoneAttributeId = TEXT("@None");

	/** Labels of the columns. */
	const FText TEXT_Index = LOCTEXT("Index", "$Index");

	/** Text to flag an attribute as being full-width allocated, instead of representing the range of elements with a single value. */
	const FText TEXT_ConstantValueCompressedAttribute = LOCTEXT("ConstantValueCompressedAttribute", "Utilizes constant value compression - a single value is shared by all entries.");
}

namespace PCGDataVisualizationHelpers
{
	void AddColumnInfo(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides)
	{
		FPCGTableVisualizerColumnInfo& ColumnInfo = OutInfo.ColumnInfos.Emplace_GetRef();
		
		ColumnInfo.Accessor = Overrides.CreateAccessorFuncOverride ? Overrides.CreateAccessorFuncOverride() : TSharedPtr<const IPCGAttributeAccessor>(PCGAttributeAccessorHelpers::CreateConstAccessor(Data, InSelector).Release());
		ColumnInfo.AccessorKeys = Overrides.CreateAccessorKeysFuncOverride ? Overrides.CreateAccessorKeysFuncOverride() : TSharedPtr<const IPCGAttributeAccessorKeys>(PCGAttributeAccessorHelpers::CreateConstKeys(Data, InSelector).Release());

		if (ensure(ColumnInfo.Accessor))
		{
			const uint16 Type = Overrides.TypeOverride == EPCGMetadataTypes::Unknown ? ColumnInfo.Accessor->GetUnderlyingType() : static_cast<uint16>(Overrides.TypeOverride);

			if (!Overrides.TooltipOverride.IsEmpty())
			{
				ColumnInfo.Tooltip = Overrides.TooltipOverride;
			}

			if (Overrides.bAddTypeToTooltip)
			{
				FText TypeNameText;
				if (Overrides.TypeOverride != EPCGMetadataTypes::Unknown)
				{
					FPCGMetadataAttributeDesc Desc{};
					Desc.ValueType = Overrides.TypeOverride;
					TypeNameText = Desc.GetTypeText();
				}
				else
				{
					TypeNameText = ColumnInfo.Accessor->GetUnderlyingDesc().GetTypeText();
				}

				if (!ColumnInfo.Tooltip.IsEmpty())
				{
					ColumnInfo.Tooltip = FText::Format(INVTEXT("{0} ({1})"), ColumnInfo.Tooltip, TypeNameText);
				}
				else
				{
					ColumnInfo.Tooltip = TypeNameText;
				}
			}

			if (Overrides.bIsConstantValueCompressed)
			{
				ColumnInfo.Tooltip = FText::Format(INVTEXT("{0}\n\n{1}"), ColumnInfo.Tooltip, PCGDataVisualizationConstants::TEXT_ConstantValueCompressedAttribute);
			}

			if (PCG::Private::IsOfTypes<FString>(Type))
			{
				ColumnInfo.CellAlignment = EPCGTableVisualizerCellAlignment::Left;
				ColumnInfo.Width = PCGEditorGraphAttributeListView::MaxColumnWidth;
			}
			
			if (!Overrides.LabelOverride.IsEmpty())
			{
				ColumnInfo.Label = Overrides.LabelOverride;
			}
			else if (InSelector.GetName() == NAME_None)
			{
				ColumnInfo.Label = FText::Format(LOCTEXT("NoneLabelFormat", "{1} ({0})"), UEnum::GetDisplayValueAsText(static_cast<EPCGMetadataTypes>(ColumnInfo.Accessor->GetUnderlyingType())), InSelector.GetDisplayText(/*bSkipDomain=*/true));
			}
			else
			{
				ColumnInfo.Label = InSelector.GetDisplayText(/*bSkipDomain=*/true);
			}

			ColumnInfo.Id = *ColumnInfo.Label.ToString();
		}
	}

	void AddColumnInfoExtraNames(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& BaseSelector, const TArray<FString>& AllExtraNames, const FColumnInfoOverrides& Overrides)
	{
		FPCGAttributePropertySelector SelectorCopy = BaseSelector;
		FString& SelectorExtraName = SelectorCopy.GetExtraNamesMutable().Emplace_GetRef();
		for (const FString& ExtraName : AllExtraNames)
		{
			SelectorExtraName = ExtraName;
			AddColumnInfo(OutInfo, Data, SelectorCopy, Overrides);
		}
	}

	template <> void AddTypedColumnInfo_Impl<FVector2D>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides)
	{
		AddColumnInfoExtraNames(OutInfo, Data, InSelector, {TEXT("X"), TEXT("Y")}, Overrides);
	}
	
	template <> void AddTypedColumnInfo_Impl<FVector>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides)
	{
		AddColumnInfoExtraNames(OutInfo, Data, InSelector, {TEXT("X"), TEXT("Y"), TEXT("Z")}, Overrides);
	}
	
	template <> void AddTypedColumnInfo_Impl<FVector4>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides)
	{
		AddColumnInfoExtraNames(OutInfo, Data, InSelector, {TEXT("X"), TEXT("Y"), TEXT("Z"), TEXT("W")}, Overrides);
	}

	template <> void AddTypedColumnInfo_Impl<FLinearColor>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides)
	{
		AddColumnInfoExtraNames(OutInfo, Data, InSelector, {TEXT("R"), TEXT("G"), TEXT("B"), TEXT("A")}, Overrides);
	}
	
	template <> void AddTypedColumnInfo_Impl<FQuat>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides)
	{
		return AddTypedColumnInfo<FVector4>(OutInfo, Data, InSelector, Overrides);
	}
	
	template <> void AddTypedColumnInfo_Impl<FRotator>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides)
	{
		AddColumnInfoExtraNames(OutInfo, Data, InSelector, {TEXT("Roll"), TEXT("Pitch"), TEXT("Yaw")}, Overrides);
	}
	
	template <> void AddTypedColumnInfo_Impl<FTransform>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides)
	{
		FPCGAttributePropertySelector SelectorCopy = InSelector;
		FString& SelectorExtraName = SelectorCopy.GetExtraNamesMutable().Emplace_GetRef();

		SelectorExtraName = TEXT("Position");
		AddTypedColumnInfo<FVector>(OutInfo, Data, SelectorCopy, Overrides);

		SelectorExtraName = TEXT("Rotation");
		AddTypedColumnInfo<FRotator>(OutInfo, Data, SelectorCopy, Overrides);
		
		SelectorExtraName = TEXT("Scale");
		AddTypedColumnInfo<FVector>(OutInfo, Data, SelectorCopy, Overrides);
	}

	void CreateMetadataColumnInfos(const UPCGData* Data, FPCGTableVisualizerInfo& OutInfo, const FPCGMetadataDomainID& DomainID)
	{
		const UPCGMetadata* Metadata = Data->ConstMetadata();
		if (!Metadata)
		{
			return;
		}

		const FPCGMetadataDomain* MetadataDomain = Metadata->GetConstMetadataDomain(DomainID);
		if (!MetadataDomain)
		{
			return;
		}
		
		TArray<FName> AttributeNames;
		TArray<EPCGMetadataTypes> AttributeTypes;
		MetadataDomain->GetAttributes(AttributeNames, AttributeTypes);

		// Domain is constant for all the attributes
		FPCGAttributePropertySelector Selector{};
		Data->SetDomainFromDomainID(DomainID, Selector);
		
		auto AddInfo = [Data, &OutInfo, &Selector]<typename T>(T, const FName AttributeName) -> bool
		{
			Selector.SetAttributeName(AttributeName);
			AddTypedColumnInfo<T>(OutInfo, Data, Selector);
			return true;
		};

		for (int32 I = 0; I < AttributeNames.Num(); I++)
		{
			const FPCGMetadataAttributeBase* Attribute = MetadataDomain->GetConstAttribute(AttributeNames[I]);

			const bool bColumnAdded = PCGMetadataAttribute::CallbackWithRightType(static_cast<uint16>(AttributeTypes[I]), AddInfo, AttributeNames[I]);

			// If it failed (there was no column added), and we have an attribute, then configure the column to get the attribute with the generic viewer.
			if (!bColumnAdded && Attribute)
			{
				FPCGTableVisualizerColumnInfo& ColumnInfo = OutInfo.ColumnInfos.Emplace_GetRef();
				ColumnInfo.Label = FText::FromName(AttributeNames[I]);
				ColumnInfo.Id = AttributeNames[I] == NAME_None ? TEXT("AttrNone") : AttributeNames[I];
				ColumnInfo.UnderlyingAttribute = Attribute;
				Selector.SetAttributeName(AttributeNames[I]);
				ColumnInfo.Accessor = TSharedPtr<const IPCGAttributeAccessor>(PCGAttributeAccessorHelpers::CreateConstAccessor(Data, Selector).Release());
				ColumnInfo.AccessorKeys = TSharedPtr<const IPCGAttributeAccessorKeys>(PCGAttributeAccessorHelpers::CreateConstKeys(Data, Selector).Release());
				ColumnInfo.Tooltip = ColumnInfo.Accessor ? ColumnInfo.Accessor->GetUnderlyingDesc().GetTypeText() : FText{};
			}
		}
	}

	FPCGTableVisualizerInfo CreateDefaultMetadataColumnInfos(const UPCGData* Data, const FPCGMetadataDomainID& DomainID)
	{
		check(Data);
		
		const UPCGMetadata* Metadata = Data->ConstMetadata();
		const FPCGMetadataDomain* MetadataDomain = Metadata ? Metadata->GetConstMetadataDomain(DomainID) : nullptr;
		if (!MetadataDomain)
		{
			return {};
		}
		
		FPCGTableVisualizerInfo Info;
		Info.Data = Data;

		FPCGAttributePropertySelector IndexSelector = FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index);
		Data->SetDomainFromDomainID(DomainID, IndexSelector);
		
		AddColumnInfo(Info, Data, IndexSelector);
		Info.SortingColumn = Info.ColumnInfos.Last().Id;

		// Add Metadata Columns
		CreateMetadataColumnInfos(Data, Info, DomainID);

		return Info;
	}
}

#undef LOCTEXT_NAMESPACE
