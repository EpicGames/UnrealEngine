// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGLandscapeDataVisualization.h"

#include "PCGEditorSettings.h"
#include "Data/PCGLandscapeData.h"
#include "Data/PCGBasePointData.h"
#include "Elements/PCGSurfaceSampler.h"

const UPCGBasePointData* IPCGLandscapeDataVisualization::CollapseToDebugBasePointData(FPCGContext* Context, const UPCGData* Data) const
{
	const UPCGLandscapeData* LandscapeData = Cast<UPCGLandscapeData>(Data);
	
	if (!LandscapeData)
	{
		return nullptr;
	}
	
	const int32 NumPointsUpperBound = FMath::Max(1, GetDefault<UPCGEditorSettings>()->TargetNumPointsForDebug);
	
	const FBox Bounds = LandscapeData->GetBounds();
	const double BoundsSurface = (Bounds.Max.X - Bounds.Min.X) * (Bounds.Max.Y - Bounds.Min.Y);
	const double Extents = FMath::Max(1.0, FMath::Sqrt(BoundsSurface / NumPointsUpperBound) * 0.5);
	
	const PCGSurfaceSampler::FSurfaceSamplerParams SamplerParams
	{
		.PointsPerSquaredMeter = 99999999.0f, // Big number to make sure we saturate the random process to always select a cell to spawn in.
		.PointExtents = FVector(Extents, Extents, Extents),
		.Looseness = 0,
		.bApplyDensityToPoints = false
	};
	
	return PCGSurfaceSampler::SampleSurface(Context, SamplerParams, LandscapeData, nullptr, Bounds);
}
