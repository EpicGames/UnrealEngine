// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGParamDataVisualization.h"

#include "PCGData.h"
#include "DataVisualizations/PCGDataVisualizationHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#define LOCTEXT_NAMESPACE "PCGParamDataVisualization"

FPCGTableVisualizerInfo IPCGParamDataVisualization::GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
{
	return PCGDataVisualizationHelpers::CreateDefaultMetadataColumnInfos(Data, DomainID);
}

#undef LOCTEXT_NAMESPACE
