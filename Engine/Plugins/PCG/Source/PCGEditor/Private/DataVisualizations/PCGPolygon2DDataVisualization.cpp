// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGPolygon2DDataVisualization.h"

#include "Data/PCGPolygon2DData.h"
#include "Data/PCGPolygon2DInteriorData.h"
#include "DataVisualizations/PCGDataVisualizationHelpers.h"

#include "AdvancedPreviewScene.h"
#include "EditorViewportClient.h"
#include "Components/SplineComponent.h"

FPCGTableVisualizerInfo FPCGPolygon2DDataVisualization::GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
{
	using namespace PCGDataVisualizationHelpers;

	const UPCGPolygon2DData* PolygonData = Cast<UPCGPolygon2DData>(Data);
	if (!PolygonData)
	{
		if (const UPCGPolygon2DInteriorSurfaceData* PolygonSurfaceData = Cast<UPCGPolygon2DInteriorSurfaceData>(Data))
		{
			PolygonData = PolygonSurfaceData->GetPolygonData();
		}
	}

	FPCGTableVisualizerInfo Info;
	Info.Data = PolygonData;

	if (!PolygonData)
	{
		return Info;
	}

	if (DomainID == PCGMetadataDomainID::Data)
	{
		// Column Sorting
		FPCGAttributePropertySelector IndexSelector = FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index, PCGDataConstants::DataDomainName);
		AddColumnInfo(Info, PolygonData, IndexSelector);
		Info.SortingColumn = Info.ColumnInfos.Last().Id;

		// Columns
		AddPropertyEnumColumnInfo<FTransform>(Info, PolygonData, EPCGPolygon2DDataProperties::Transform);
		AddPropertyEnumColumnInfo<double>(Info, PolygonData, EPCGPolygon2DDataProperties::Area);
		AddPropertyEnumColumnInfo<double>(Info, PolygonData, EPCGPolygon2DDataProperties::Perimeter);
		AddPropertyEnumColumnInfo<FVector2d>(Info, PolygonData, EPCGPolygon2DDataProperties::BoundsMin);
		AddPropertyEnumColumnInfo<FVector2d>(Info, PolygonData, EPCGPolygon2DDataProperties::BoundsMax);
		AddPropertyEnumColumnInfo<int32>(Info, PolygonData, EPCGPolygon2DDataProperties::SegmentCount);
		AddPropertyEnumColumnInfo<int32>(Info, PolygonData, EPCGPolygon2DDataProperties::OuterSegmentCount);
		AddPropertyEnumColumnInfo<int32>(Info, PolygonData, EPCGPolygon2DDataProperties::HoleCount);
		AddPropertyEnumColumnInfo<int32>(Info, PolygonData, EPCGPolygon2DDataProperties::LongestOuterSegmentIndex);
		AddPropertyEnumColumnInfo<bool>(Info, PolygonData, EPCGPolygon2DDataProperties::IsClockwise);

		// Add Metadata Columns
		CreateMetadataColumnInfos(PolygonData, Info, PCGMetadataDomainID::Data);
	}
	else if (DomainID == PCGMetadataDomainID::Elements)
	{
		// Column Sorting
		FPCGAttributePropertySelector IndexSelector = FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index, PCGPolygon2DData::VertexDomainName);
		AddColumnInfo(Info, PolygonData, IndexSelector);
		Info.SortingColumn = Info.ColumnInfos.Last().Id;

		// Columns
		AddPropertyEnumColumnInfo<FVector>(Info, PolygonData, EPCGPolygon2DProperties::Position);
		AddPropertyEnumColumnInfo<FRotator>(Info, PolygonData, EPCGPolygon2DProperties::Rotation);
		AddPropertyEnumColumnInfo<int32>(Info, PolygonData, EPCGPolygon2DProperties::SegmentIndex);
		AddPropertyEnumColumnInfo<int32>(Info, PolygonData, EPCGPolygon2DProperties::HoleIndex);
		AddPropertyEnumColumnInfo<double>(Info, PolygonData, EPCGPolygon2DProperties::SegmentLength);
		AddPropertyEnumColumnInfo<FVector2d>(Info, PolygonData, EPCGPolygon2DProperties::LocalPosition);
		AddPropertyEnumColumnInfo<FRotator>(Info, PolygonData, EPCGPolygon2DProperties::LocalRotation);

		// Add Metadata Columns
		CreateMetadataColumnInfos(PolygonData, Info, PCGMetadataDomainID::Elements);

		// Focus on data behavior
		Info.FocusOnDataCallback = [](const UPCGData* Data, TArrayView<const int> Indices)
		{
			const UPCGPolygon2DData* PolygonData = Cast<UPCGPolygon2DData>(Data);
			if (!PolygonData)
			{
				if (const UPCGPolygon2DInteriorSurfaceData* PolygonSurfaceData = Cast<UPCGPolygon2DInteriorSurfaceData>(Data))
				{
					PolygonData = PolygonSurfaceData->GetPolygonData();
				}
			}

			if (!PolygonData)
			{
				return;
			}

			FBox BoundingBox(EForceInit::ForceInit);

			if (Indices.IsEmpty())
			{
				BoundingBox = PolygonData->GetBounds();
			}
			else
			{
				const FTransform& Transform = PolygonData->GetTransform();
				const TMap<int, TPair<int, int>>& VertexIndexToSegmentAndHoleIndices = PolygonData->GetSegmentIndexToSegmentAndHoleIndices();
				const UE::Geometry::FGeneralPolygon2d& Polygon = PolygonData->GetPolygon();
				const FVector HalfExtents(50.0);

				for (const int& Index : Indices)
				{
					if (const TPair<int, int>* SegmentAndHoleIndex = VertexIndexToSegmentAndHoleIndices.Find(Index))
					{
						FVector2D VertexPosition = Polygon.Segment(SegmentAndHoleIndex->Key, SegmentAndHoleIndex->Value).StartPoint();
						FVector VertexWorldPosition = Transform.TransformPosition(FVector(VertexPosition, 0.0));

						FBox PointBoundingBox(VertexWorldPosition + HalfExtents, VertexWorldPosition - HalfExtents);
						BoundingBox += PointBoundingBox;
					}
				}
			}

			if (GEditor && BoundingBox.IsValid)
			{
				GEditor->MoveViewportCamerasToBox(BoundingBox, /*bActiveViewportOnly=*/true, /*DrawDebugBoxTimeInSeconds=*/2.5f);
			}
		};
	}

	return Info;
}

FPCGSetupSceneFunc FPCGPolygon2DDataVisualization::GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const
{
	return[this, Data](FPCGSceneSetupParams& InOutParams)
	{
		check(InOutParams.Scene);
		check(InOutParams.EditorViewportClient);

		const UPCGPolygon2DData* PolygonData = Cast<UPCGPolygon2DData>(Data);

		if (!PolygonData)
		{
			if (const UPCGPolygon2DInteriorSurfaceData* PolygonSurfaceData = Cast<UPCGPolygon2DInteriorSurfaceData>(Data))
			{
				PolygonData = PolygonSurfaceData->GetPolygonData();
			}
		}

		if (!PolygonData)
		{
			return;
		}

		const FTransform& Transform = PolygonData->GetTransform();

		auto CreateSplineComponent = [PolygonData, &InOutParams, &Transform](const UE::Geometry::FPolygon2d& Polygon)
		{
			TObjectPtr<USplineComponent> SplineComponent = NewObject<USplineComponent>(GetTransientPackage(), NAME_None, RF_Transient);
			
			// Setup spline - done similarly to the Create Spline node.
			{
				// Create points
				TArray<FSplinePoint> SplinePoints;
				const TArray<FVector2d>& PolygonVertices = Polygon.GetVertices();

				SplinePoints.Reserve(PolygonVertices.Num());

				for(int VertexIndex = 0; VertexIndex < PolygonVertices.Num(); ++VertexIndex)
				{
					SplinePoints.Emplace(static_cast<float>(VertexIndex),
						FVector(PolygonVertices[VertexIndex], 0.0),
						ESplinePointType::Linear);
				}

				// Set points to component
				SplineComponent->SetWorldTransform(Transform);
				SplineComponent->DefaultUpVector = Transform.GetRotation().GetUpVector();
				SplineComponent->ClearSplinePoints(/*bUpdateSpline=*/false);
				SplineComponent->AddPoints(SplinePoints, /*bUpdateSpline=*/false);
				SplineComponent->SetClosedLoop(true);
				SplineComponent->UpdateSpline();
				SplineComponent->UpdateBounds();
			}

			if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() <= ERHIFeatureLevel::ES3_1)
			{
				SplineComponent->SetMobility(EComponentMobility::Static);
			}

			InOutParams.ManagedResources.Add(SplineComponent);
			InOutParams.Scene->AddComponent(SplineComponent, Transform);

			return SplineComponent;
		};

		TObjectPtr<USplineComponent> OuterPolygon = CreateSplineComponent(PolygonData->GetPolygon().GetOuter());

		for (const UE::Geometry::FPolygon2d& Hole : PolygonData->GetPolygon().GetHoles())
		{
			CreateSplineComponent(Hole);
		}

		InOutParams.FocusBounds = OuterPolygon->CalcBounds(Transform);
	};
}