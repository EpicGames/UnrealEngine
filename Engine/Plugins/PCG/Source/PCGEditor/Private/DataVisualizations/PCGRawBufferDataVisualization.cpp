// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGRawBufferDataVisualization.h"

#include "Compute/Data/PCGRawBufferData.h"
#include "DataVisualizations/PCGDataVisualizationHelpers.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

namespace PCGRawBufferVisualization
{
	/** Simple keys implementation that reports the number of uint32 values in the buffer. */
	class FKeys : public IPCGAttributeAccessorKeys
	{
	public:
		explicit FKeys(int32 InCount)
			: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ true)
			, Count(InCount)
		{}

		virtual int32 GetNum() const override { return Count; }

	private:
		int32 Count;
	};

	/** Read-only accessor that reads uint32 values from the raw buffer, exposing them as int32. */
	class FValueAccessor : public IPCGAttributeAccessorT<FValueAccessor>
	{
	public:
		using Type = int32;
		using Super = IPCGAttributeAccessorT<FValueAccessor>;

		explicit FValueAccessor(const TArray<uint32>& InData)
			: Super(/*bInReadOnly=*/ true)
			, Data(InData)
		{}

		bool GetRangeImpl(TArrayView<int32> OutValues, int32 Index, const IPCGAttributeAccessorKeys& InKeys) const
		{
			const int32 NumKeys = InKeys.GetNum();
			if (NumKeys == 0)
			{
				return false;
			}

			int32 Current = Index;
			if (Current >= NumKeys)
			{
				Current %= NumKeys;
			}

			for (int32 i = 0; i < OutValues.Num(); ++i)
			{
				OutValues[i] = Data.IsValidIndex(Current) ? static_cast<int32>(Data[Current]) : 0;
				if (++Current >= NumKeys)
				{
					Current = 0;
				}
			}

			return true;
		}

		bool SetRangeImpl(TArrayView<const int32>, int32, IPCGAttributeAccessorKeys&, EPCGAttributeAccessorFlags)
		{
			return false;
		}

	private:
		const TArray<uint32>& Data;
	};
}

void FPCGRawBufferDataVisualization::ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const
{
	// No spatial debug display for raw buffer data.
}

FPCGTableVisualizerInfo FPCGRawBufferDataVisualization::GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
{
	FPCGTableVisualizerInfo Info;

	const UPCGRawBufferData* RawBufferData = Cast<UPCGRawBufferData>(Data);
	if (!RawBufferData)
	{
		return Info;
	}

	Info.Data = RawBufferData;

	const int32 NumValues = RawBufferData->GetNumUint32s();
	TSharedRef<PCGRawBufferVisualization::FKeys> Keys = MakeShared<PCGRawBufferVisualization::FKeys>(NumValues);

	// Index column
	{
		FPCGTableVisualizerColumnInfo& Col = Info.ColumnInfos.Emplace_GetRef();
		Col.Id = PCGDataVisualizationConstants::NAME_Index;
		Col.Label = FText::FromString(TEXT("$Index"));
		Col.Accessor = MakeShared<FPCGIndexAccessor>();
		Col.AccessorKeys = Keys;
	}

	// Value column
	{
		FPCGTableVisualizerColumnInfo& Col = Info.ColumnInfos.Emplace_GetRef();
		Col.Id = FName(TEXT("Value"));
		Col.Label = FText::FromString(TEXT("Value"));
		Col.Tooltip = FText::FromString(TEXT("Raw uint32 value"));
		Col.Accessor = MakeShared<PCGRawBufferVisualization::FValueAccessor>(RawBufferData->GetConstData());
		Col.AccessorKeys = Keys;
	}

	Info.SortingColumn = PCGDataVisualizationConstants::NAME_Index;

	return Info;
}
