// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGSpatialDataVisualization.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGData.h"
#include "PCGDebug.h"
#include "PCGEditorCommon.h"
#include "PCGPoint.h"
#include "PCGSettings.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "DataVisualizations/PCGDataVisualizationHelpers.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include "AdvancedPreviewScene.h"
#include "EditorViewportClient.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/AssetManager.h"

#define LOCTEXT_NAMESPACE "PCGSpatialDataVisualization"

namespace PCGPointDataVisualizationConstants
{
	/** Special names of the columns in the attribute list. */
	const FName NAME_MetadataEntry = FName(TEXT("MetadataEntry"));
	const FName NAME_MetadataEntryParent = FName(TEXT("PointMetadataEntryParent"));

	/** Special labels of the columns. */
	const FText TEXT_MetadataEntry = LOCTEXT("MetadataEntry", "Entry Key");
	const FText TEXT_MetadataEntryParent = LOCTEXT("MetadataEntryParent", "Parent Key");
}

void IPCGSpatialDataVisualization::ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const
{
	if (!TargetActor)
	{
		PCGE_LOG_C(Error, GraphAndLog, Context, LOCTEXT("NoTargetActor", "Cannot execute debug display for spatial data with no target actor."));
		return;
	}

	if (!SettingsInterface || !SettingsInterface->GetSettings())
	{
		return;
	}

	ExecuteDebugDisplayHelper(Data, SettingsInterface->DebugSettings, Context, TargetActor, SettingsInterface->GetSettings()->GetSettingsCrc(), [](UInstancedStaticMeshComponent*) {});
}

void IPCGSpatialDataVisualization::ExecuteDebugDisplayHelper(
	const UPCGData* Data,
	const FPCGDebugVisualizationSettings& DebugSettings,
	FPCGContext* Context,
	AActor* TargetActor,
	const FPCGCrc& Crc,
	const TFunction<void(UInstancedStaticMeshComponent*)>& OnISMCCreatedCallback) const
{
	UStaticMesh* Mesh = DebugSettings.PointMesh.LoadSynchronous();

	if (!Mesh)
	{
		PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("UnableToLoadMesh", "Debug display was unable to load mesh '{0}'."),
			FText::FromString(DebugSettings.PointMesh.ToString())));
		return;
	}

	TArray<TSoftObjectPtr<UMaterialInterface>> Materials;
	Materials.Add(DebugSettings.GetMaterial());

	const UPCGBasePointData* PointData = CollapseToDebugBasePointData(Context, Data);

	if (!PointData)
	{
		return;
	}

	if (PointData->IsEmpty())
	{
		return;
	}

	const int NumCustomData = 8;
	const int32 NumPoints = PointData->GetNumPoints();

	TArray<FTransform> ForwardInstances;
	TArray<FTransform> ReverseInstances;
	TArray<float> InstanceCustomData;

	ForwardInstances.Reserve(NumPoints);
	InstanceCustomData.Reserve(NumCustomData);

	// First, create target instance transforms
	const float PointScale = DebugSettings.PointScale;
	const bool bIsAbsolute = DebugSettings.ScaleMethod == EPCGDebugVisScaleMethod::Absolute;
	const bool bIsRelative = DebugSettings.ScaleMethod == EPCGDebugVisScaleMethod::Relative;
	const bool bScaleWithExtents = DebugSettings.ScaleMethod == EPCGDebugVisScaleMethod::Extents;
	const FVector MeshExtents = Mesh->GetBoundingBox().GetExtent();

	/* Note: A re-used ISMC may have any number of pre-existing instances, so this won't prevent going over the max. But,
	 * the renderer is robust to over-instancing attempts and will not crash. However, MAX_INSTANCE_ID still serves as a
	 * good, scalable heuristic for a max limit.
	 */
	const int32 NumDesiredInstances = FMath::Min(NumPoints, static_cast<int32>(MAX_INSTANCE_ID));
	if (NumDesiredInstances != NumPoints)
	{
		PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT( "DebugPointsOverLimit", "Debug point display ({0}) surpassed the max instance limit ({1}) and will be clamped."), FText::AsNumber(NumPoints), FText::AsNumber(MAX_INSTANCE_ID)));
	}

	const FConstPCGPointValueRanges ValueRanges(PointData);
	bool bFoundNonNormalizedInstances = false;

	for (int i = 0; i < NumDesiredInstances; ++i)
	{
		TArray<FTransform>& Instances = ((bIsAbsolute || ValueRanges.TransformRange[i].GetDeterminant() >= 0) ? ForwardInstances : ReverseInstances);
		FTransform& InstanceTransform = Instances.Add_GetRef(ValueRanges.TransformRange[i]);
		if (bIsRelative)
		{
			InstanceTransform.SetScale3D(InstanceTransform.GetScale3D() * PointScale);
		}
		else if (bScaleWithExtents)
		{
			const FVector Extents = PCGPointHelpers::GetExtents(ValueRanges.BoundsMinRange[i], ValueRanges.BoundsMaxRange[i]);
			const FVector LocalCenter = PCGPointHelpers::GetLocalCenter(ValueRanges.BoundsMinRange[i], ValueRanges.BoundsMaxRange[i]);

			const FVector ScaleWithExtents = Extents / MeshExtents;
			const FVector TransformedBoxCenterWithOffset = InstanceTransform.TransformPosition(LocalCenter) - InstanceTransform.GetLocation();
			InstanceTransform.SetTranslation(InstanceTransform.GetTranslation() + TransformedBoxCenterWithOffset);
			InstanceTransform.SetScale3D(InstanceTransform.GetScale3D() * ScaleWithExtents);
		}
		else // absolute scaling only
		{
			InstanceTransform.SetScale3D(FVector(PointScale));
		}

		// If any instances have non-normalized rotations, normalize them to avoid crashing, but emit a warning.
		if (!InstanceTransform.IsRotationNormalized())
		{
			InstanceTransform.NormalizeRotation();
			bFoundNonNormalizedInstances = true;
		}
	}

	if (bFoundNonNormalizedInstances)
	{
		PCGE_LOG_C(Warning, GraphAndLog, Context, LOCTEXT("UnnormalizedRotation", "PCGSpatialDataVisualization: Encountered one or more transforms with unnormalized rotation. Rotations will be normalized for visualization."));
	}

	FPCGISMComponentBuilderParams Params[2];
	Params[0].SettingsCrc = Crc;
	Params[0].bTransient = true;
	Params[0].NumCustomDataFloats = NumCustomData;
	Params[0].Descriptor.StaticMesh = Mesh;
	Params[0].Descriptor.OverrideMaterials = Materials;
	Params[0].Descriptor.Mobility = EComponentMobility::Static;
	Params[0].Descriptor.BodyInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	// Note: In the future we may consider enabling culling for performance reasons, but for now culling disabled.
	Params[0].Descriptor.InstanceStartCullDistance = Params[0].Descriptor.InstanceEndCullDistance = 0;
	// Additional performance switches
	Params[0].Descriptor.bAffectDistanceFieldLighting = false;
	Params[0].Descriptor.bAffectDynamicIndirectLighting = false;
	Params[0].Descriptor.bAffectDynamicIndirectLightingWhileHidden = false;
	Params[0].Descriptor.bCastContactShadow = false;
	Params[0].Descriptor.bCastDynamicShadow = false;
	Params[0].Descriptor.bCastShadow = false;	
	Params[0].Descriptor.bCastStaticShadow = false;
	Params[0].Descriptor.bGenerateOverlapEvents = false;
	Params[0].Descriptor.bIncludeInHLOD = false;
	Params[0].Descriptor.bReceivesDecals = false;
	Params[0].Descriptor.bVisibleInRayTracing = false;
	
	// If the root actor we're binding to is movable, then the ISMC should be movable by default
	USceneComponent* SceneComponent = TargetActor ? TargetActor->GetRootComponent() : nullptr;
	if (SceneComponent)
	{
		Params[0].Descriptor.Mobility = SceneComponent->Mobility;
	}

	Params[1] = Params[0];
	Params[1].Descriptor.bReverseCulling = true;

	// Since the instance count is global, track the current instances applied and previously belong to the ISMCs.
	int32 NumCurrentInstances = 0;
	for (int32 Direction = 0; Direction < 2; ++Direction)
	{
		TArray<FTransform>& Instances = (Direction == 0 ? ForwardInstances : ReverseInstances);

		if (Instances.IsEmpty())
		{
			continue;
		}

		UInstancedStaticMeshComponent* ISMC = nullptr;
		
		if (TargetActor && Context && Context->ExecutionSource.Get())
		{
			ISMC = UPCGActorHelpers::GetOrCreateISMC(TargetActor, Context->ExecutionSource.Get(), Params[Direction], Context);
		}
		else
		{
			// If no target actor/source component were provided, create an ISMC directly instead.
			ISMC = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);

			FISMComponentDescriptor Descriptor(Params[Direction].Descriptor);
			Descriptor.InitComponent(ISMC);
			ISMC->SetNumCustomDataFloats(Params[Direction].NumCustomDataFloats);
		}

		check(ISMC && ISMC->NumCustomDataFloats == NumCustomData);

		ISMC->ComponentTags.AddUnique(PCGHelpers::DefaultPCGDebugTag);
		const int32 PreExistingInstanceCount = ISMC->GetInstanceCount();
		NumCurrentInstances += PreExistingInstanceCount;

		// The renderer is robust to going over the instance count, so it's okay not to account for other scene instances here.
		if (NumCurrentInstances + Instances.Num() > MAX_INSTANCE_ID)
		{
			// Drop instances to stay at the max.
			// Account for less than 0 if, for example, the forward was over the limit and the reverse had less than the PreExisting.
			Instances.SetNum(FMath::Max(0, static_cast<int32>(MAX_INSTANCE_ID) - NumCurrentInstances));
			if (Instances.IsEmpty())
			{
				continue;
			}
		}

		NumCurrentInstances += Instances.Num();
		ISMC->AddInstances(Instances, /*bShouldReturnIndices=*/false, /*bWorldSpace=*/true);

		// Scan all points looking for points that match current direction and add their custom data.
		int32 PointCounter = 0;
		for (int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			const int32 PointDirection = ((bIsAbsolute || ValueRanges.TransformRange[PointIndex].GetDeterminant() >= 0) ? 0 : 1);
			if (PointDirection != Direction)
			{
				continue;
			}

			const FVector4& Color = ValueRanges.ColorRange[PointIndex];
			const FVector Extents = PCGPointHelpers::GetExtents(ValueRanges.BoundsMinRange[PointIndex], ValueRanges.BoundsMaxRange[PointIndex]);
			InstanceCustomData.Add(ValueRanges.DensityRange[PointIndex]);
			
			InstanceCustomData.Add(Extents[0]);
			InstanceCustomData.Add(Extents[1]);
			InstanceCustomData.Add(Extents[2]);
			InstanceCustomData.Add(Color[0]);
			InstanceCustomData.Add(Color[1]);
			InstanceCustomData.Add(Color[2]);
			InstanceCustomData.Add(Color[3]);

			ISMC->SetCustomData(PreExistingInstanceCount + PointCounter, InstanceCustomData);

			InstanceCustomData.Reset();

			++PointCounter;
		}

		ISMC->UpdateBounds();
		OnISMCCreatedCallback(ISMC);
	}
}

FPCGTableVisualizerInfo IPCGSpatialDataVisualization::GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
{
	using namespace PCGDataVisualizationHelpers;
	using namespace PCGPointDataVisualizationConstants;

	// Collapse to point representation for visualization.
	const UPCGBasePointData* PointData = CollapseToDebugBasePointData(/*Context=*/nullptr, Data);

	if (DomainID == PCGMetadataDomainID::Data)
	{
		return CreateDefaultMetadataColumnInfos(PointData, DomainID);
	}

	FPCGTableVisualizerInfo Info;
	Info.Data = PointData;

	// Column Sorting
	AddColumnInfo(Info, PointData, FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index));
	Info.SortingColumn = Info.ColumnInfos.Last().Id;

	const EPCGPointNativeProperties AllocatedProperties = PointData->GetAllocatedProperties();

	AddPropertyEnumColumnInfo<FVector>(Info, PointData, EPCGPointProperties::Position, { .bIsConstantValueCompressed = !(AllocatedProperties & EPCGPointNativeProperties::Transform) });
	AddPropertyEnumColumnInfo<FRotator>(Info, PointData, EPCGPointProperties::Rotation, { .bIsConstantValueCompressed = !(AllocatedProperties & EPCGPointNativeProperties::Transform) });
	AddPropertyEnumColumnInfo<FVector>(Info, PointData, EPCGPointProperties::Scale, { .bIsConstantValueCompressed = !(AllocatedProperties & EPCGPointNativeProperties::Transform) });
	AddPropertyEnumColumnInfo<FVector>(Info, PointData, EPCGPointProperties::BoundsMin, { .bIsConstantValueCompressed = !(AllocatedProperties & EPCGPointNativeProperties::BoundsMin) });
	AddPropertyEnumColumnInfo<FVector>(Info, PointData, EPCGPointProperties::BoundsMax, { .bIsConstantValueCompressed = !(AllocatedProperties & EPCGPointNativeProperties::BoundsMax) });
	AddPropertyEnumColumnInfo<FLinearColor>(Info, PointData, EPCGPointProperties::Color, { .bIsConstantValueCompressed = !(AllocatedProperties & EPCGPointNativeProperties::Color) });
	AddPropertyEnumColumnInfo<float>(Info, PointData, EPCGPointProperties::Density, { .bIsConstantValueCompressed = !(AllocatedProperties & EPCGPointNativeProperties::Density) });
	AddPropertyEnumColumnInfo<float>(Info, PointData, EPCGPointProperties::Steepness, { .bIsConstantValueCompressed = !(AllocatedProperties & EPCGPointNativeProperties::Steepness) });
	AddPropertyEnumColumnInfo<int32>(Info, PointData, EPCGPointProperties::Seed, { .bIsConstantValueCompressed = !(AllocatedProperties & EPCGPointNativeProperties::Seed) });

	if (FPCGEditorCommon::CVarShowAdvancedAttributesFields.GetValueOnAnyThread())
	{
		FColumnInfoOverrides Overrides;
		Overrides.LabelOverride = TEXT_MetadataEntry;
		Overrides.CreateAccessorFuncOverride = [PointData]()
		{
			FPCGAttributePropertySelector MetadataEntrySelector;
			MetadataEntrySelector.SetPropertyName(*StaticEnum<EPCGPointNativeProperties>()->GetNameStringByValue((int64)EPCGPointNativeProperties::MetadataEntry));
			return TSharedPtr<const IPCGAttributeAccessor>(PCGAttributeAccessorHelpers::CreateConstAccessor(PointData, MetadataEntrySelector).Release());
		};

		Overrides.bIsConstantValueCompressed = !(AllocatedProperties & EPCGPointNativeProperties::MetadataEntry);
		
		AddTypedColumnInfo<int64>(Info, PointData, FPCGAttributePropertySelector{}, Overrides);

		Overrides.LabelOverride = TEXT_MetadataEntryParent;
		Overrides.CreateAccessorFuncOverride = [PointData]()
		{
			return MakeShared<FPCGCustomPointPropertyAccessor<int64, TConstPCGValueRange<int64>>>(PointData, [Metadata = PointData->ConstMetadata()](int32 Index, int64& OutValue, const TConstPCGValueRange<int64>& MetaDataEntryRange)
			{
				if (Metadata)
				{
					OutValue = Metadata->GetParentKey(MetaDataEntryRange[Index]);
					return true;
				}
				return false;
			},
			PointData->GetConstMetadataEntryValueRange());
		};
		
		AddTypedColumnInfo<int64>(Info, PointData, FPCGAttributePropertySelector{}, Overrides);
	}

	// Add Metadata Columns
	CreateMetadataColumnInfos(PointData, Info, PCGMetadataDomainID::Elements);

	// Focus on data behavior
	Info.FocusOnDataCallback = [](const UPCGData* Data, TArrayView<const int> Indices)
	{
		if (const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Data))
		{
			const UPCGBasePointData* PointData = SpatialData->ToBasePointData(nullptr);

			if (!PointData)
			{
				return;
			}

			FBox BoundingBox(EForceInit::ForceInit);
			if (Indices.IsEmpty())
			{
				BoundingBox = PointData->GetBounds();
			}
			else
			{
				FTransform TempTransform;

				const FConstPCGPointValueRanges ValueRanges(PointData);
				for (const int& Index : Indices)
				{
					const FBox LocalBounds = PCGPointHelpers::GetLocalBounds(ValueRanges.BoundsMinRange[Index], ValueRanges.BoundsMaxRange[Index]);
					const FTransform& Transform = ValueRanges.TransformRange[Index];

					const bool bRotationNormalized = Transform.IsRotationNormalized();
					if (!bRotationNormalized) // just null out the rotation if it's invalid - we'll still show the 'right sized' box at the right position
					{
						TempTransform = Transform;
						TempTransform.SetRotation(FQuat::Identity);
					}

					const FTransform& BoxTransform = (bRotationNormalized ? Transform : TempTransform);
					const FBox PointBoundingBox = LocalBounds.TransformBy(BoxTransform.ToMatrixWithScale());
					
					BoundingBox += PointBoundingBox;
				}
			}

			if (GEditor && BoundingBox.IsValid)
			{
				GEditor->MoveViewportCamerasToBox(BoundingBox, /*bActiveViewportOnly=*/true, /*DrawDebugBoxTimeInSeconds=*/2.5f);
			}
		}
	};

	return Info;
}

const UPCGPointData* IPCGSpatialDataVisualization::CollapseToDebugPointData(FPCGContext* Context, const UPCGData* Data) const
{
	if (const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Data))
	{
		return SpatialData->ToPointData(Context);
	}

	return nullptr;
}

const UPCGBasePointData* IPCGSpatialDataVisualization::CollapseToDebugBasePointData(FPCGContext* Context, const UPCGData* Data) const
{	
	if (const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Data))
	{
		if (CVarPCGEnablePointArrayData.GetValueOnAnyThread())
		{
			return SpatialData->ToPointArrayData(Context);
		}
		else
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return CollapseToDebugPointData(Context, Data);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	return nullptr;
}

FString IPCGSpatialDataVisualization::GetDomainDisplayNameForInspection(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
{
	if (DomainID != PCGMetadataDomainID::Elements || Data->IsSupportedMetadataDomainID(DomainID))
	{
		return IPCGDataVisualization::GetDomainDisplayNameForInspection(Data, DomainID);
	}
	else
	{
		// For sampled points, clearly indicate that it is the default sampled points and not just "points"
		return TEXT("Default Sampled Points");
	}
}

TArray<FPCGMetadataDomainID> IPCGSpatialDataVisualization::GetAllSupportedDomainsForInspection(const UPCGData* Data) const
{
	return GetDefault<UPCGBasePointData>()->GetAllSupportedMetadataDomainIDs();
}

FPCGSetupSceneFunc IPCGSpatialDataVisualization::GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const
{
	return [this, WeakData=TWeakObjectPtr<const UPCGData>(Data), 
		WeakSettingsInterface=TWeakObjectPtr<const UPCGSettingsInterface>(SettingsInterface)](FPCGSceneSetupParams& InOutParams)
	{
		check(InOutParams.Scene);
		check(InOutParams.EditorViewportClient);

		if (!WeakData.IsValid())
		{
			UE_LOGF(LogPCG, Error, "Failed to setup data viewport, the data was lost or invalid.");
			return;
		}

		FVector BoundsMin, BoundsMax;
		bool bInitializedBounds = false;

		ExecuteDebugDisplayHelper(
			WeakData.Get(),
			WeakSettingsInterface.IsValid() ? WeakSettingsInterface->DebugSettings : FPCGDebugVisualizationSettings(),
			nullptr,
			nullptr,
			FPCGCrc(),
			[&InOutParams, &bInitializedBounds, &BoundsMin, &BoundsMax](UInstancedStaticMeshComponent* ISMC)
			{
				check(ISMC);

				InOutParams.ManagedResources.Add(ISMC);
				InOutParams.Scene->AddComponent(ISMC, FTransform::Identity);

				const FVector CurrentBoundsMin = ISMC->Bounds.Origin - ISMC->Bounds.BoxExtent;
				const FVector CurrentBoundsMax = ISMC->Bounds.Origin + ISMC->Bounds.BoxExtent;

				if (!bInitializedBounds)
				{
					BoundsMin = CurrentBoundsMin;
					BoundsMax = CurrentBoundsMax;
					bInitializedBounds = true;
				}
				else
				{
					BoundsMin = BoundsMin.ComponentMin(CurrentBoundsMin);
					BoundsMax = BoundsMax.ComponentMax(CurrentBoundsMax);
				}
			}
		);

		if (bInitializedBounds)
		{
			InOutParams.FocusBounds = FBoxSphereBounds(FBox(BoundsMin, BoundsMax));
		}
	};
}

#undef LOCTEXT_NAMESPACE
