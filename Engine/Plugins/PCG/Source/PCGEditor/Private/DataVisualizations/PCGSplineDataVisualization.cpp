// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGSplineDataVisualization.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGSplineStruct.h"
#include "DataVisualizations/PCGDataVisualizationHelpers.h"

#include "AdvancedPreviewScene.h"
#include "EditorViewportClient.h"
#include "Components/SplineComponent.h"

#define LOCTEXT_NAMESPACE "PCGSplineDataVisualization"

namespace PCGSplineDataVisualizationConstants
{
	const FVector HalfExtents = FVector(50.0); // Set the points to fill up 1 meter of space by default.
}

FPCGTableVisualizerInfo IPCGSplineDataVisualization::GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
{
	using namespace PCGDataVisualizationHelpers;

	const UPCGSplineData* SplineData = CastChecked<const UPCGSplineData>(Data);

	FPCGTableVisualizerInfo Info;
	Info.Data = SplineData;

	if (DomainID == PCGMetadataDomainID::Data)
	{
		// Column Sorting
		FPCGAttributePropertySelector IndexSelector = FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index, PCGDataConstants::DataDomainName);
		AddColumnInfo(Info, SplineData, IndexSelector);
		Info.SortingColumn = Info.ColumnInfos.Last().Id;

		// Columns
		AddPropertyEnumColumnInfo<FTransform>(Info, SplineData, EPCGSplineDataProperties::SplineTransform);
		AddPropertyEnumColumnInfo<bool>(Info, SplineData, EPCGSplineDataProperties::IsClosed);
		AddPropertyEnumColumnInfo<double>(Info, SplineData, EPCGSplineDataProperties::Length);
		
		// Add Metadata Columns
		CreateMetadataColumnInfos(SplineData, Info, PCGMetadataDomainID::Data);

		return Info;
	}
	
	if (DomainID == PCGMetadataDomainID::Elements)
	{
		// Column Sorting
		FPCGAttributePropertySelector IndexSelector = FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index, PCGSplineData::ControlPointDomainName);
		AddColumnInfo(Info, SplineData, IndexSelector);
		Info.SortingColumn = Info.ColumnInfos.Last().Id;
	
		// Columns
		AddPropertyEnumColumnInfo<FVector>(Info, SplineData, EPCGSplineStructProperties::Position);
		AddPropertyEnumColumnInfo<FRotator>(Info, SplineData, EPCGSplineStructProperties::Rotation);
		AddPropertyEnumColumnInfo<FVector>(Info, SplineData, EPCGSplineStructProperties::Scale);
		AddPropertyEnumColumnInfo<FVector>(Info, SplineData, EPCGSplineStructProperties::LocalPosition);
		AddPropertyEnumColumnInfo<FRotator>(Info, SplineData, EPCGSplineStructProperties::LocalRotation);
		AddPropertyEnumColumnInfo<FVector>(Info, SplineData, EPCGSplineStructProperties::LocalScale);
		AddPropertyEnumColumnInfo<FVector>(Info, SplineData, EPCGSplineStructProperties::ArriveTangent);
		AddPropertyEnumColumnInfo<FVector>(Info, SplineData, EPCGSplineStructProperties::LeaveTangent);
		AddPropertyEnumColumnInfo<int32>(Info, SplineData, EPCGSplineStructProperties::InterpType);
	
		// Add Metadata Columns
		CreateMetadataColumnInfos(SplineData, Info, PCGMetadataDomainID::Elements);

		// Focus on data behavior
		Info.FocusOnDataCallback = [](const UPCGData* Data, TArrayView<const int> Indices)
		{
			if (const UPCGSplineData* SplineData = Cast<UPCGSplineData>(Data))
			{
				FBox BoundingBox(EForceInit::ForceInit);

				if (Indices.IsEmpty())
				{
					BoundingBox = SplineData->GetBounds();
				}
				else
				{
					const TArray<FInterpCurvePointVector>& Positions = SplineData->SplineStruct.GetSplinePointsPosition().Points;
					const TArray<FInterpCurvePointVector>& Scales = SplineData->SplineStruct.GetSplinePointsScale().Points;

					for (const int& Index : Indices)
					{
						check(Positions.IsValidIndex(Index) && Scales.IsValidIndex(Index));

						const FVector& Position = SplineData->GetTransform().TransformPosition(Positions[Index].OutVal);
						const FVector HalfExtent = Scales[Index].OutVal * PCGSplineDataVisualizationConstants::HalfExtents;

						FBox PointBoundingBox(Position + HalfExtent, Position - HalfExtent);
						BoundingBox += PointBoundingBox;
					}
				}

				if (GEditor && BoundingBox.IsValid)
				{
					GEditor->MoveViewportCamerasToBox(BoundingBox, /*bActiveViewportOnly=*/true, /*DrawDebugBoxTimeInSeconds=*/2.5f);
				}
			}
		};
	}

	return Info;
}

const UPCGBasePointData* IPCGSplineDataVisualization::CollapseToDebugBasePointData(FPCGContext* Context, const UPCGData* Data) const
{
	if (const UPCGSplineData* SplineData = Cast<UPCGSplineData>(Data))
	{
		UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(Context);
		FPCGInitializeFromDataParams InitializeFromDataParams(SplineData);
		PointData->InitializeFromDataWithParams(InitializeFromDataParams);

		const int32 NumControlPoints = SplineData->SplineStruct.GetSplinePointsPosition().Points.Num();
		TConstArrayView<PCGMetadataEntryKey> EntryKeys = SplineData->SplineStruct.GetConstControlPointsEntryKeys();

		PointData->SetNumPoints(NumControlPoints);
		PointData->SetExtents(PCGSplineDataVisualizationConstants::HalfExtents);
		PointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::MetadataEntry);

		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange(/*bAllocate=*/false);
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange(/*bAllocate=*/false);

		for (int ControlPointIndex = 0; ControlPointIndex < NumControlPoints; ++ControlPointIndex)
		{
			TransformRange[ControlPointIndex] = SplineData->SplineStruct.GetTransformAtSplineInputKey(ControlPointIndex, ESplineCoordinateSpace::World);
			MetadataEntryRange[ControlPointIndex] = EntryKeys.IsValidIndex(ControlPointIndex) ? EntryKeys[ControlPointIndex] : PCGInvalidEntryKey;
		}

		if (SplineData->HasCachedLastSelector())
		{
			PointData->SetLastSelector(SplineData->GetCachedLastSelector());
		}

		return PointData;
	}

	return nullptr;
}

FPCGSetupSceneFunc IPCGSplineDataVisualization::GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const
{
	return[this, Data](FPCGSceneSetupParams& InOutParams)
	{
		check(InOutParams.Scene);
		check(InOutParams.EditorViewportClient);

		const UPCGSplineData* SplineData = Cast<UPCGSplineData>(Data);

		if (!SplineData)
		{
			return;
		}

		TObjectPtr<USplineComponent> SplineComponent = NewObject<USplineComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		SplineData->ApplyTo(SplineComponent);

		if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() <= ERHIFeatureLevel::ES3_1)
		{
			SplineComponent->SetMobility(EComponentMobility::Static);
		}

		InOutParams.ManagedResources.Add(SplineComponent);
		InOutParams.Scene->AddComponent(SplineComponent, SplineComponent->GetComponentToWorld());
		InOutParams.FocusBounds = SplineComponent->CalcBounds(SplineComponent->GetComponentToWorld());
	};
}

#undef LOCTEXT_NAMESPACE
