// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGStaticMeshDataVisualization.h"

#include "Data/PCGStaticMeshResourceData.h"
#include "DataVisualizations/PCGDataVisualizationHelpers.h"

#include "AdvancedPreviewScene.h"
#include "EditorViewportClient.h"
#include "MaterialEditor/MaterialEditorMeshComponent.h"

FPCGTableVisualizerInfo IPCGStaticMeshDataVisualization::GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
{
	FPCGTableVisualizerInfo Info;
	Info.Data = Data;
	// @todo_pcg: Add row for mesh path? Support metadata as well?
 
	return Info;
}

TArray<TSharedPtr<FStreamableHandle>> IPCGStaticMeshDataVisualization::LoadRequiredResources(const UPCGData* Data) const
{
	TArray<TSharedPtr<FStreamableHandle>> LoadHandles;

	if (const UPCGStaticMeshResourceData* StaticMeshData = Cast<UPCGStaticMeshResourceData>(Data))
	{
		LoadHandles.Add(StaticMeshData->RequestResourceLoad());
	}

	return LoadHandles;
}

FPCGSetupSceneFunc IPCGStaticMeshDataVisualization::GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const
{
	return [this](FPCGSceneSetupParams& InOutParams)
	{
		check(InOutParams.Scene);
		check(InOutParams.EditorViewportClient);

		if (InOutParams.Resources.IsEmpty())
		{
			return;
		}

		// Note: Using UMaterialEditorMeshComponent subclass for more accurate mesh bounds.
		TObjectPtr<UStaticMeshComponent> MeshComponent = NewObject<UMaterialEditorMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		InOutParams.ManagedResources.Add(MeshComponent);

		if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() <= ERHIFeatureLevel::ES3_1)
		{
			MeshComponent->SetMobility(EComponentMobility::Static);
		}

		InOutParams.Scene->AddComponent(MeshComponent, FTransform::Identity);

		UStaticMesh* StaticMesh = Cast<UStaticMesh>(InOutParams.Resources[0]);
		MeshComponent->SetStaticMesh(StaticMesh);

		// Bounds will be updated already by SetStaticMesh() call.
		InOutParams.FocusBounds = MeshComponent->Bounds;
	};
}
