// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGVisualizationTexture2D.h"

#include "RHIStaticStates.h"
#include "RenderCaptureInterface.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGVisualizationTexture2D)

namespace PCGVisTexture2DHelpers
{
	static int32 GTriggerGPUCaptureDispatches = 0;
	static FAutoConsoleVariableRef CVarTriggerGPUCapture(
		TEXT("pcg.GPU.TriggerRenderCaptures.VisTexture2DInit"),
		GTriggerGPUCaptureDispatches,
		TEXT("Trigger GPU captures for this many of the subsequent visualization texture resource initializations."));

	/** Creates a black 1x1 dummy texture. */
	FTextureRHIRef CreateDummyTexture(FRHICommandListImmediate& RHICmdList)
	{
		const FRHITextureCreateDesc DummyTextureDesc =
			FRHITextureCreateDesc::Create2D(TEXT("PCGDummyTexture"), 1, 1, PF_G8)
			.SetFlags(ETextureCreateFlags::ShaderResource);

		FTextureRHIRef DummyTexture = RHICmdList.CreateTexture(DummyTextureDesc);

		{
			uint32 DestStride;
			uint8* DestBuffer = (uint8*)RHICmdList.LockTexture2D(DummyTexture, 0, RLM_WriteOnly, DestStride, false);
			*DestBuffer = 0;
			RHICmdList.UnlockTexture2D(DummyTexture, 0, false);
		}

		return DummyTexture;
	}
}

/**
 * RT Resource for UPCGVisualizationTexture2D. Wraps and manages the texture/sampler RHI for the UTexture.
 * Created from UPCGVisualizationTexture2D::CreateResource(), but otherwise managed in the UTexture base class.
 */
class FPCGVisualizationTexture2DResource : public FTextureResource
{
public:
	FPCGVisualizationTexture2DResource(UPCGVisualizationTexture2D* InOwner)
		: Owner(InOwner)
	{}

	virtual uint32 GetSizeX() const override { return ensure(Owner) ? Owner->GetSurfaceWidth() : 1u; }
	virtual uint32 GetSizeY() const override { return ensure(Owner) ? Owner->GetSurfaceHeight() : 1u; }

	/** Called when the resource is initialized. This is only called by the rendering thread. */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Called when the resource is released. This is only called by the rendering thread. */
	virtual void ReleaseRHI() override;

protected:
	/** The texture owning this resource. */
	UPCGVisualizationTexture2D* Owner;
};

void FPCGVisualizationTexture2DResource::InitRHI(FRHICommandListBase& RHICmdListBase)
{
	if (!Owner)
	{
		return;
	}

	FRHICommandListImmediate& RHICmdList = RHICmdListBase.GetAsImmediate();

	FTextureRHIRef SourceTextureRHI = Owner->GetTextureRHI();

	if (SourceTextureRHI.IsValid())
	{
		RenderCaptureInterface::FScopedCapture RenderCapture(PCGVisTexture2DHelpers::GTriggerGPUCaptureDispatches > 0, &RHICmdList, TEXT("FPCGVisualizationTextureResource::InitRHI::CopySource"));
		PCGVisTexture2DHelpers::GTriggerGPUCaptureDispatches = FMath::Max(PCGVisTexture2DHelpers::GTriggerGPUCaptureDispatches - 1, 0);

		// Create TextureRHI to match our SourceTextureRHI
		const FRHITextureDesc& SourceDesc = SourceTextureRHI->GetDesc();

		const FRHITextureCreateDesc CreateDesc =
			FRHITextureCreateDesc::Create2D(TEXT("PCGVisualizationTexture2D"))
			.SetExtent(SourceDesc.Extent)
			.SetFormat(SourceDesc.Format)
			.SetFlags(SourceDesc.Flags)
			.SetInitialState(ERHIAccess::CopyDest);

		TextureRHI = RHICmdList.CreateTexture(CreateDesc);

		// Prepare to copy the source texture.
		RHICmdList.Transition(FRHITransitionInfo(SourceTextureRHI, ERHIAccess::Unknown, ERHIAccess::CopySrc));

		// Copy the source texture.
		FRHICopyTextureInfo CopyInfo;
		CopyInfo.SourceSliceIndex = Owner->GetSliceIndex();
		RHICmdList.CopyTexture(SourceTextureRHI, TextureRHI, CopyInfo);

		RHICmdList.Transition(FRHITransitionInfo(TextureRHI, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
	}
	else
	{
		// If there is no source texture RHI, just use a black dummy texture.
		TextureRHI = PCGVisTexture2DHelpers::CreateDummyTexture(RHICmdList);
	}

	SamplerStateRHI = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);
}

void FPCGVisualizationTexture2DResource::ReleaseRHI()
{
	if (Owner)
	{
		RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
	}

	FTextureResource::ReleaseRHI();
	TextureRHI.SafeRelease();
}

FTextureResource* UPCGVisualizationTexture2D::CreateResource()
{
	return new FPCGVisualizationTexture2DResource(this);
}

float UPCGVisualizationTexture2D::GetSurfaceWidth() const
{
	return Resolution.X;
}

float UPCGVisualizationTexture2D::GetSurfaceHeight() const
{
	return Resolution.Y;
}

FTextureRHIRef UPCGVisualizationTexture2D::GetTextureRHI() const
{
	return TextureRHI;
}

uint16 UPCGVisualizationTexture2D::GetSliceIndex() const
{
	return SliceIndex;
}

void UPCGVisualizationTexture2D::Init(const FPCGVisualizationTexture2DParams& InParams)
{
	TextureRHI = InParams.TextureRHI;
	Resolution = InParams.Resolution;
	SliceIndex = InParams.SliceIndex;

	UpdateResource();
}

UPCGVisualizationTexture2D* UPCGVisualizationTexture2D::Create(const FPCGVisualizationTexture2DParams& InParams)
{
	UPCGVisualizationTexture2D* NewTexture = NewObject<UPCGVisualizationTexture2D>(GetTransientPackage(), NAME_None, RF_Transient);

	NewTexture->Init(InParams);

	return NewTexture;
}
