// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture.h"

#include "RHIFwd.h"

#include "PCGVisualizationTexture2D.generated.h"

namespace PCGTextureVisualizationConstants
{
	static const FSoftObjectPath PlaneMeshPath(TEXT("/Engine/BasicShapes/Plane.Plane"));
	static const FSoftObjectPath DebugMaterialPath(TEXT("Material'/PCG/DebugObjects/PCG_DebugMaterialTexture.PCG_DebugMaterialTexture'"));
}

struct FPCGVisualizationTexture2DParams
{
	FTextureRHIRef TextureRHI = nullptr;
	FIntPoint Resolution = FIntPoint::ZeroValue;
	uint16 SliceIndex = 0;
};

/**
 * UTexture implementation for visualizing PCG texture data in the data viewport.
 * This is necessary because some texture data do not have their own UTexture, so one needs
 * to be created in order to override the texture parameter for the debug vis material.
 */
UCLASS(MinimalAPI)
class UPCGVisualizationTexture2D : public UTexture
{
	GENERATED_BODY()

public:
	//~ Begin UTexture Interface.
	virtual ETextureClass GetTextureClass() const { return ETextureClass::TwoDDynamic; }
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_Texture2D; }
	virtual float GetSurfaceWidth() const override;
	virtual float GetSurfaceHeight() const override;
	virtual float GetSurfaceDepth() const override { return 0; }
	virtual uint32 GetSurfaceArraySize() const override { return 0; }
	//~ End UTexture Interface.
	
	void Init(const FPCGVisualizationTexture2DParams& InParams);

	/** Creates and initializes a new UPCGVisualizationTexture2D from the texture data. */
	static UPCGVisualizationTexture2D* Create(const FPCGVisualizationTexture2DParams& InParams);

	FTextureRHIRef GetTextureRHI() const;
	uint16 GetSliceIndex() const;

protected:
	FTextureRHIRef TextureRHI = nullptr;
	FIntPoint Resolution = FIntPoint::ZeroValue;
	uint16 SliceIndex;
};
