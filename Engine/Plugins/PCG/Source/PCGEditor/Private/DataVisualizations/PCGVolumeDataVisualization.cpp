// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualizations/PCGVolumeDataVisualization.h"

#include "Data/PCGCollisionShapeData.h"
#include "Data/PCGCollisionWrapperData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPrimitiveData.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGVolumeData.h"
#include "Elements/PCGVolumeSampler.h"

#include "PCGEditorSettings.h"
#include "DataVisualizations/PCGCollisionVisComponent.h"

#include "AdvancedPreviewScene.h"
#include "EditorViewportClient.h"
#include "Chaos/ChaosEngineInterface.h"
#include "Components/BoxComponent.h"
#include "Components/CapsuleComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SphereComponent.h"

const UPCGBasePointData* FPCGPrimitiveDataVisualization::CollapseToDebugBasePointData(FPCGContext* Context, const UPCGData* Data) const
{
	const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Data);
	
	if (!SpatialData)
	{
		return nullptr;
	}
	
	// Have special cases for primitive data that would potentially generate too many points if they were collapsed with default sizes.
	// We'll change the size of the extents of the points to make sure we can't get much higher than our upper bound.
	if (SpatialData->IsA<UPCGVolumeData>() || SpatialData->IsA<UPCGPrimitiveData>() || SpatialData->IsA<UPCGCollisionShapeData>() || SpatialData->IsA<UPCGCollisionWrapperData>())
	{
		const int32 NumPointsUpperBound = FMath::Max(1, GetDefault<UPCGEditorSettings>()->TargetNumPointsForDebug);
		
		const FBox Bounds = SpatialData->GetBounds();
		const double BoundsVolume = Bounds.GetVolume();
		// Pow of 1/3 is the cubic root.
		const double VoxelSize = FMath::Max(1.0, FMath::Pow(BoundsVolume / NumPointsUpperBound, 1.0 / 3.0));
		
		PCGVolumeSampler::FVolumeSamplerParams SamplerParams;
		SamplerParams.VoxelSize = FVector(VoxelSize, VoxelSize, VoxelSize);
		SamplerParams.Bounds = Bounds;

		return PCGVolumeSampler::SampleVolume(Context, SpatialData, SamplerParams);
	}
	else
	{
		return nullptr;
	}
}

FPCGSetupSceneFunc FPCGPrimitiveDataVisualization::GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const
{
	return [this, Data](FPCGSceneSetupParams& InOutParams)
	{
		check(InOutParams.Scene);
		check(InOutParams.EditorViewportClient);

		TArray<TObjectPtr<UPrimitiveComponent>> Components;
		TArray<FTransform> ComponentTransforms;

		GetComponentsAndTransforms(Data, Components, ComponentTransforms);

		if (Components.IsEmpty() || Components.Num() != ComponentTransforms.Num())
		{
			return;
		}

		FBoxSphereBounds Bounds(EForceInit::ForceInit);
		bool bHasBounds = false;

		for(int Index = 0; Index < Components.Num(); ++Index)
		{
			TObjectPtr<UPrimitiveComponent> Component = Components[Index];
			const FTransform& Transform = ComponentTransforms[Index];

			if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() <= ERHIFeatureLevel::ES3_1)
			{
				Component->SetMobility(EComponentMobility::Static);
			}	

			InOutParams.ManagedResources.Add(Component);
			InOutParams.Scene->AddComponent(Component, Transform);

			if (bHasBounds)
			{
				Bounds = Bounds + Component->CalcBounds(Transform);
			}
			else
			{
				bHasBounds = true;
				Bounds = Component->CalcBounds(Transform);
			}
		}

		InOutParams.FocusBounds = Bounds;
	};
}

void FPCGPrimitiveDataVisualization::GetComponentsAndTransforms(const UPCGData* InData, TArray<TObjectPtr<UPrimitiveComponent>>& OutComponents, TArray<FTransform>& OutComponentTransforms) const
{
	const UPCGPrimitiveData* PrimitiveData = Cast<UPCGPrimitiveData>(InData);

	if (!PrimitiveData)
	{
		return;
	}

	if (TStrongObjectPtr<UPrimitiveComponent> PrimitiveComponent = PrimitiveData->GetComponent().Pin())
	{
		TObjectPtr<UPCGCollisionVisComponent> VisComponent = NewObject<UPCGCollisionVisComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		VisComponent->SetBodySetup(PrimitiveComponent->GetBodySetup());

		OutComponents.Add(VisComponent);
		OutComponentTransforms.Add(PrimitiveComponent->GetComponentToWorld());
	}
}

void FPCGVolumeDataVisualization::GetComponentsAndTransforms(const UPCGData* InData, TArray<TObjectPtr<UPrimitiveComponent>>& OutComponents, TArray<FTransform>& OutComponentTransforms) const
{
	const UPCGVolumeData* VolumeData = Cast<UPCGVolumeData>(InData);

	if (!VolumeData)
	{
		return;
	}

	if (FBodyInstance* Instance = VolumeData->VolumeBodyInstance)
	{
		TObjectPtr<UPCGCollisionVisComponent> VisComponent = NewObject<UPCGCollisionVisComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		VisComponent->SetBodySetup(Instance->GetBodySetup());

		OutComponents.Add(VisComponent);
		OutComponentTransforms.Add(Instance->GetUnrealWorldTransform());
	}
	else
	{
		// Create a box component
		UBoxComponent* BoxComponent = NewObject<UBoxComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		BoxComponent->SetBoxExtent(VolumeData->Bounds.GetExtent());
		BoxComponent->SetSimulatePhysics(false);

		OutComponents.Add(BoxComponent);
		OutComponentTransforms.Add(FTransform(VolumeData->Bounds.GetCenter()));
	}
}

void FPCGCollisionShapeDataVisualization::GetComponentsAndTransforms(const UPCGData* InData, TArray<TObjectPtr<UPrimitiveComponent>>& OutComponents, TArray<FTransform>& OutComponentTransforms) const
{
	const UPCGCollisionShapeData* ShapeData = Cast<UPCGCollisionShapeData>(InData);

	if (!ShapeData)
	{
		return;
	}

	const FCollisionShape& Shape = ShapeData->Shape;

	if (Shape.IsLine()) // not supported in PCG
	{
		return;
	}

	if (Shape.IsSphere())
	{
		USphereComponent* SphereComponent = NewObject<USphereComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		SphereComponent->SetSphereRadius(Shape.Sphere.Radius);

		OutComponents.Add(SphereComponent);
	}
	else if (Shape.IsCapsule())
	{
		UCapsuleComponent* CapsuleComponent = NewObject<UCapsuleComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		CapsuleComponent->SetCapsuleRadius(Shape.Capsule.Radius);
		CapsuleComponent->SetCapsuleHalfHeight(Shape.Capsule.HalfHeight);

		OutComponents.Add(CapsuleComponent);
	}
	else if (Shape.IsBox())
	{
		UBoxComponent* BoxComponent = NewObject<UBoxComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		BoxComponent->SetBoxExtent(FVector(Shape.Box.HalfExtentX, Shape.Box.HalfExtentY, Shape.Box.HalfExtentZ));

		OutComponents.Add(BoxComponent);
	}

	OutComponentTransforms.Add(ShapeData->Transform);
}

void FPCGCollisionWrapperDataVisualization::GetComponentsAndTransforms(const UPCGData* InData, TArray<TObjectPtr<UPrimitiveComponent>>& OutComponents, TArray<FTransform>& OutComponentTransforms) const
{
	const UPCGCollisionWrapperData* WrapperData = Cast<UPCGCollisionWrapperData>(InData);

	if (!WrapperData || !WrapperData->CollisionWrapper.bInitialized)
	{
		return;
	}

	const FPCGCollisionWrapper& CollisionWrapper = WrapperData->CollisionWrapper;
	TConstPCGValueRange<FTransform> Transforms = WrapperData->GetPointData()->GetConstTransformValueRange();
	TConstPCGValueRange<FVector> BoundsMin = WrapperData->GetPointData()->GetConstBoundsMinValueRange();
	TConstPCGValueRange<FVector> BoundsMax = WrapperData->GetPointData()->GetConstBoundsMaxValueRange();

	// Prepare components
	UPCGCollisionVisComponent* VisComponent = NewObject<UPCGCollisionVisComponent>(GetTransientPackage(), NAME_None, RF_Transient);

	for (int Index = 0; Index < Transforms.Num(); ++Index)
	{
		const FTransform& Transform = Transforms[Index];
		FBodyInstance* BodyInstance = CollisionWrapper.IndexToBodyInstance[Index] == INDEX_NONE ? nullptr : CollisionWrapper.BodyInstances[CollisionWrapper.IndexToBodyInstance[Index]];
		if (BodyInstance && BodyInstance->GetBodySetup())
		{
			VisComponent->AddBodySetup(BodyInstance->GetBodySetup(), Transform);
		}
		else
		{
			// Default to creating a box component
			UBoxComponent* BoxComponent = NewObject<UBoxComponent>(GetTransientPackage(), NAME_None, RF_Transient);

			const FVector Center = (BoundsMax[Index] + BoundsMin[Index]) * 0.5;
			const FVector Extent = (BoundsMax[Index] - BoundsMin[Index]) * 0.5;

			BoxComponent->SetBoxExtent(Extent);
			BoxComponent->SetSimulatePhysics(false);

			OutComponents.Add(BoxComponent);
			OutComponentTransforms.Add(FTransform(Center) * Transform);
		}
	}

	if (!VisComponent->BodyTransforms.IsEmpty())
	{
		OutComponents.Add(VisComponent);
		OutComponentTransforms.Add(FTransform::Identity);
	}
}