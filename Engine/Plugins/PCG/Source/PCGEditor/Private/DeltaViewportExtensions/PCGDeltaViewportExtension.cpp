// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeltaViewportExtensions/PCGDeltaViewportExtension.h"

#include "Graph/DataOverride/PCGDataOverride.h"

void FPCGDeltaViewportExtensionRegistry::RegisterInternalExtension(const UScriptStruct* DeltaStruct, TUniquePtr<IPCGDeltaViewportExtension> Extension)
{
	if (ensureMsgf(!InternalRegistry.Find(DeltaStruct), TEXT("Cannot register multiple internal viewport extensions for the same struct type.")))
	{
		InternalRegistry.Add(DeltaStruct, MoveTemp(Extension));
	}
}

void FPCGDeltaViewportExtensionRegistry::UnregisterAllInternalExtensions()
{
	InternalRegistry.Empty();
}

void FPCGDeltaViewportExtensionRegistry::RegisterExtension(const UScriptStruct* DeltaStruct, TUniquePtr<IPCGDeltaViewportExtension> Extension)
{
	if (ensureMsgf(!ExternalRegistry.Find(DeltaStruct), TEXT("Cannot register multiple viewport extensions for the same struct type.")))
	{
		ExternalRegistry.Add(DeltaStruct, MoveTemp(Extension));
	}
}

void FPCGDeltaViewportExtensionRegistry::UnregisterExtension(const UScriptStruct* DeltaStruct)
{
	ExternalRegistry.Remove(DeltaStruct);
}

IPCGDeltaViewportExtension* FPCGDeltaViewportExtensionRegistry::GetExtension(const UScriptStruct* DeltaStruct) const
{
	const UScriptStruct* Current = DeltaStruct;
	const UScriptStruct* DeltaBaseStruct = FPCGDeltaBase::StaticStruct();

	// Walk the struct hierarchy from the concrete type up through FPCGDeltaBase (inclusive).
	while (Current)
	{
		// Always try external implementations first so users can override built-in extensions.
		if (const TUniquePtr<IPCGDeltaViewportExtension>* ExtensionPtr = ExternalRegistry.Find(Current))
		{
			return ExtensionPtr->Get();
		}

		if (const TUniquePtr<IPCGDeltaViewportExtension>* ExtensionPtr = InternalRegistry.Find(Current))
		{
			return ExtensionPtr->Get();
		}

		// Stop after checking FPCGDeltaBase -- don't walk into non-delta struct territory.
		if (Current == DeltaBaseStruct)
		{
			break;
		}

		Current = Cast<UScriptStruct>(Current->GetSuperStruct());
	}

	return nullptr;
}

TArray<const UScriptStruct*> FPCGDeltaViewportExtensionRegistry::GetRegisteredDeltaTypes() const
{
	TMap<const UScriptStruct*, int32> TypePriorities;

	for (const auto& [DeltaStruct, Extension] : InternalRegistry)
	{
		TypePriorities.Add(DeltaStruct, Extension ? Extension->GetSortPriority() : std::numeric_limits<int32>::max());
	}

	for (const auto& [DeltaStruct, Extension] : ExternalRegistry)
	{
		TypePriorities.Add(DeltaStruct, Extension ? Extension->GetSortPriority() : std::numeric_limits<int32>::max());
	}

	TypePriorities.ValueSort([](const int32 A, const int32 B) { return A < B; });

	TArray<const UScriptStruct*> Result;
	TypePriorities.GenerateKeyArray(Result);
	return Result;
}
