// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeltaViewportExtensions/PCGDeltaViewportExtensionHelpers.h"
#include "DeltaViewportExtensions/PCGDeltaViewportExtension.h"

#include "PCGComponent.h"
#include "PCGGraph.h"
#include "Graph/DataOverride/PCGDataOverride.h"
#include "Graph/PCGSourceDataContainer.h"

FPCGDeltaCollection* PCG::DeltaViewportExtension::Helpers::GetMutableCollection(const FPCGDeltaViewportContext& Context)
{
	UPCGComponent* PCGComponent = Context.ActivePCGComponent.Get();
	if (!PCGComponent)
	{
		return nullptr;
	}

	FPCGSourceDataContainer* DataContainer = PCGComponent->GetExecutionState().GetSourceDataContainer();
	if (!DataContainer)
	{
		return nullptr;
	}

	FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(Context.ActiveStorageKey);
	return SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
}

// @todo_pcg: DataContainer can be derived from PCGComponent->GetExecutionState().GetSourceDataContainer()
void PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(UPCGComponent* PCGComponent, FPCGSourceDataContainer* DataContainer)
{
	if (DataContainer)
	{
		DataContainer->MarkDirty();
	}

	if (UPCGGraphInstance* GraphInstance = PCGComponent ? PCGComponent->GetGraphInstance() : nullptr)
	{
		if (UPCGGraph* Graph = GraphInstance->GetMutablePCGGraph())
		{
			Graph->ForceNotificationForEditor(EPCGChangeType::ExternalModification);
		}
	}
}
