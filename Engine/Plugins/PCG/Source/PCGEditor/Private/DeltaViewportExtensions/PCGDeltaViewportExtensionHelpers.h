// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UPCGComponent;
struct FPCGDeltaCollection;
struct FPCGDeltaViewportContext;
struct FPCGSourceDataContainer;

namespace PCG::DeltaViewportExtension::Helpers
{
	/** Returns the mutable delta collection for the context's active storage key, or nullptr if any step in the chain is invalid. */
	FPCGDeltaCollection* GetMutableCollection(const FPCGDeltaViewportContext& Context);

	/** Mark the PCG component and its source data container as dirty, then notify the graph editor. */
	void MarkComponentDirty(UPCGComponent* PCGComponent, FPCGSourceDataContainer* DataContainer);
}
