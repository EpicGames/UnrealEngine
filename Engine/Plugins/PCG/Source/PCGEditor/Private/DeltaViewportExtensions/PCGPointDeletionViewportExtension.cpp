// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeltaViewportExtensions/PCGPointDeletionViewportExtension.h"

#include "DeltaViewportExtensions/PCGDeltaViewportExtensionHelpers.h"
#include "PCGComponent.h"

#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"

#include "SPCGManualEditPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "PCGPointDeletionViewportExtension"

namespace PCGPointDeletionDelta::Constants
{
	constexpr FLinearColor SelectedColor(1.0f, 0.4f, 0.4f, 0.5f);
	constexpr FLinearColor NormalColor(0.65f, 0.05f, 0.05f, 0.3f);
}

namespace PCGPointDeletionDelta
{
	/** Outcome of attempting to remove a single inserted instance from a collection. */
	enum class ERemoveInsertedInstanceResult : uint8
	{
		NotFound,        // Key/index did not resolve to an inserted instance.
		RemovedInstance, // The instance was removed.
		RemovedDelta     // The instance was the final entry, so the whole insertion delta was removed.
	};

	/** Removes a single inserted point from the insertion delta identified by Key. */
	ERemoveInsertedInstanceResult RemoveInsertedInstance(FPCGDeltaCollection& Collection, const FPCGDeltaKey& Key, const int32 InstanceIndex)
	{
		TInstancedStruct<FPCGDeltaBase>* DeltaStruct = Collection.Find(Key);
		if (!DeltaStruct)
		{
			return ERemoveInsertedInstanceResult::NotFound;
		}

		FPCGPointInsertionDelta* InsertionDelta = DeltaStruct->GetMutablePtr<FPCGPointInsertionDelta>();
		if (!InsertionDelta || !InsertionDelta->InsertedPoints.IsValidIndex(InstanceIndex))
		{
			return ERemoveInsertedInstanceResult::NotFound;
		}

		// RemoveAt preserves order so remaining entries keep stable relative indices.
		InsertionDelta->InsertedPoints.RemoveAt(InstanceIndex);

		if (InsertionDelta->InsertedPoints.IsEmpty())
		{
			Collection.Remove(Key);
			return ERemoveInsertedInstanceResult::RemovedDelta;
		}

		return ERemoveInsertedInstanceResult::RemovedInstance;
	}
}

TSharedRef<SWidget> FPCGPointDeletionViewportExtension::CreateWidget(FPCGDeltaViewportCallbacks Callbacks)
{
	CachedCallbacks = MoveTemp(Callbacks);
	DeletionEntries.Empty();

	TSharedRef<SVerticalBox> BaseBox = SNew(SVerticalBox)
		// Delete / Restore Selected button
		+ SVerticalBox::Slot()
	   .AutoHeight()
	   .Padding(0.0f, 4.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
		   .HAlign(HAlign_Center)
		   .IsEnabled_Lambda([this]() -> bool
			{
				return CachedContext.bSelectionActive;
			})
		   .OnClicked_Lambda([this]() -> FReply
			{
				HandleDeletePressed();
				return FReply::Handled();
			})
			[
				SNew(STextBlock)
			   .Text_Lambda([this]() -> FText
				{
					return IsSelectionDeletionDelta()
							   ? LOCTEXT("RestoreSelectedButton", "Restore Selected")
							   : LOCTEXT("DeleteSelectedButton", "Delete Selected");
				})
			   .Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
			]
		]
		// Deletions list section
		+ SVerticalBox::Slot()
	   .AutoHeight()
	   .Padding(0.0f, 8.0f, 0.0f, 2.0f)
		[
			SAssignNew(DeletionSection, SVerticalBox)
		   .Visibility(EVisibility::Collapsed)
			// Header row: "Deletions" + "Restore All"
			+ SVerticalBox::Slot()
		   .AutoHeight()
		   .Padding(0.0f, 0.0f, 0.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
			   .FillWidth(1.0f)
			   .VAlign(VAlign_Center)
				[
					SNew(STextBlock)
				   .Text(LOCTEXT("DeletionsHeader", "Deletions"))
				   .Font(FCoreStyle::GetDefaultFontStyle("Bold", 8))
				]
				+ SHorizontalBox::Slot()
			   .AutoWidth()
			   .VAlign(VAlign_Center)
				[
					SNew(SButton)
				   .OnClicked_Lambda([this]() -> FReply
					{
						if (CachedCallbacks.RevertAllRestorableDeltas)
						{
							CachedCallbacks.RevertAllRestorableDeltas();
						}
						RefreshLists(CachedContext);
						return FReply::Handled();
					})
					[
						SNew(STextBlock)
					   .Text(LOCTEXT("RestoreAllButton", "Restore All"))
					   .Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
					]
				]
			]
			// Deletion list view
			+ SVerticalBox::Slot()
		   .AutoHeight()
		   .MaxHeight(150.0f)
			[
				SAssignNew(DeletionListView, SListView<TSharedPtr<FPCGPointDeletionDeltaEntry>>)
			   .ListItemsSource(&DeletionEntries)
			   .OnGenerateRow_Lambda([this](TSharedPtr<FPCGPointDeletionDeltaEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
				{
					return OnGenerateDeletionRow(Entry, OwnerTable);
				})
			   .SelectionMode(ESelectionMode::None)
			]
		];

	return BaseBox;
}

void FPCGPointDeletionViewportExtension::RefreshLists(const FPCGDeltaViewportContext& Context)
{
	CachedContext = Context;

	TArray<TSharedPtr<FPCGPointDeletionDeltaEntry>> NewEntries;

	if (UPCGComponent* PCGComponent = Context.ActivePCGComponent.Get())
	{
		if (const FPCGSourceDataContainer* DataContainer = PCGComponent->GetExecutionState().GetSourceDataContainer())
		{
			const FConstSharedStruct SharedStruct = DataContainer->Get<FPCGDeltaCollection>(Context.ActiveStorageKey);
			if (const FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<const FPCGDeltaCollection>() : nullptr)
			{
				Collection->ForEachDelta([&NewEntries](const FPCGDeltaKey& DeltaKey, const TInstancedStruct<FPCGDeltaBase>& DeltaValue) -> bool
				{
					if (const FPCGPointDeletionDelta* DeleteDelta = DeltaValue.GetPtr<FPCGPointDeletionDelta>())
					{
						TSharedPtr<FPCGPointDeletionDeltaEntry> Entry = MakeShared<FPCGPointDeletionDeltaEntry>();
						Entry->Key = DeltaKey;
						Entry->OriginalElementIndex = DeleteDelta->ElementIndex;
						Entry->Location = DeleteDelta->OriginalTransform.GetLocation();
						NewEntries.Add(Entry);
					}

					return true;
				});
			}
		}
	}

	bool bListChanged = NewEntries.Num() != DeletionEntries.Num();
	if (!bListChanged)
	{
		for (int32 i = 0; i < NewEntries.Num(); ++i)
		{
			if (NewEntries[i]->Key != DeletionEntries[i]->Key)
			{
				bListChanged = true;
				break;
			}

			DeletionEntries[i]->Location = NewEntries[i]->Location;
		}
	}

	if (bListChanged)
	{
		DeletionEntries = MoveTemp(NewEntries);

		if (DeletionListView.IsValid())
		{
			DeletionListView->RequestListRefresh();
		}
	}

	const bool bHasDeletions = DeletionEntries.Num() > 0;
	if (DeletionSection.IsValid())
	{
		DeletionSection->SetVisibility(bHasDeletions ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed);
	}
}

FLinearColor FPCGPointDeletionViewportExtension::GetDisplayColor(const bool bIsSelected) const
{
	return bIsSelected ? PCGPointDeletionDelta::Constants::SelectedColor : PCGPointDeletionDelta::Constants::NormalColor;
}

FTransform FPCGPointDeletionViewportExtension::GetDisplayTransform(const FTransform& SourceElementTransform, const FConstStructView DeltaStruct) const
{
	if (const FPCGPointDeletionDelta* DeleteDelta = DeltaStruct.GetPtr<const FPCGPointDeletionDelta>())
	{
		return DeleteDelta->OriginalTransform;
	}

	return SourceElementTransform;
}

bool FPCGPointDeletionViewportExtension::MatchesSourceElement(const FTransform& SourceElementTransform, const FConstStructView DeltaStruct, const double SpatialTolerance) const
{
	const FPCGPointDeletionDelta* DeleteDelta = DeltaStruct.GetPtr<const FPCGPointDeletionDelta>();
	if (!DeleteDelta)
	{
		return false;
	}

	if (!CachedContext.NodeConfiguration)
	{
		return false;
	}

	const FPCGDeltaKey SourceKey = PCG::DataOverride::Keys::FPCGCompositeTransformDeltaKey(
		SourceElementTransform,
		SpatialTolerance,
		CachedContext.NodeConfiguration->bSignaturePosition,
		CachedContext.NodeConfiguration->bSignatureRotation,
		CachedContext.NodeConfiguration->bSignatureScale);

	const FPCGDeltaKey DeltaKey = PCG::DataOverride::Keys::FPCGCompositeTransformDeltaKey(
		DeleteDelta->OriginalTransform,
		SpatialTolerance,
		CachedContext.NodeConfiguration->bSignaturePosition,
		CachedContext.NodeConfiguration->bSignatureRotation,
		CachedContext.NodeConfiguration->bSignatureScale);

	return SourceKey == DeltaKey;
}

FTransform FPCGPointDeletionViewportExtension::GetOriginalTransform(const FConstStructView DeltaStruct) const
{
	if (const FPCGPointDeletionDelta* DeleteDelta = DeltaStruct.GetPtr<const FPCGPointDeletionDelta>())
	{
		return DeleteDelta->OriginalTransform;
	}

	return FTransform::Identity;
}

void FPCGPointDeletionViewportExtension::CollectRestorableKeys(const FPCGDeltaCollection& Collection, TArray<FPCGDeltaKey>& OutKeys) const
{
	Collection.ForEachDelta([&OutKeys](const FPCGDeltaKey& DeltaKey, const TInstancedStruct<FPCGDeltaBase>& DeltaValue) -> bool
	{
		if (DeltaValue.GetPtr<FPCGPointDeletionDelta>())
		{
			OutKeys.Add(DeltaKey);
		}

		return true;
	});
}

void FPCGPointDeletionViewportExtension::HandleDeletePressed()
{
	if (!CachedContext.bSelectionActive || CachedContext.SelectionDeltaKey.Hash == 0)
	{
		RefreshLists(CachedContext);
		return;
	}

	UPCGComponent* PCGComponent = CachedContext.ActivePCGComponent.Get();
	if (!PCGComponent)
	{
		RefreshLists(CachedContext);
		return;
	}

	FPCGSourceDataContainer* DataContainer = PCGComponent->GetExecutionState().GetSourceDataContainer();
	if (!DataContainer)
	{
		RefreshLists(CachedContext);
		return;
	}

	FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(CachedContext.ActiveStorageKey);
	FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
	if (!Collection)
	{
		RefreshLists(CachedContext);
		return;
	}

	if (const TInstancedStruct<FPCGDeltaBase>* SelectedDelta = Collection->Find(CachedContext.SelectionDeltaKey))
	{
		if (SelectedDelta->GetPtr<FPCGPointInsertionDelta>())
		{
			// No specific inserted instance is resolved -— nothing to delete.
			if (CachedContext.SelectedElementIndex == INDEX_NONE)
			{
				RefreshLists(CachedContext);
				return;
			}

			FScopedTransaction Transaction(LOCTEXT("DeleteInsertedPoint", "PCG Manual Edit: Delete Inserted Point"));
			PCGComponent->Modify();

			PCGPointDeletionDelta::RemoveInsertedInstance(*Collection, CachedContext.SelectionDeltaKey, CachedContext.SelectedElementIndex);

			PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);

			if (CachedCallbacks.ClearSelection)
			{
				CachedCallbacks.ClearSelection();
			}

			RefreshLists(CachedContext);

			return;
		}
	}

	if (IsSelectionDeletionDelta())
	{
		// RevertDelta creates its own FScopedTransaction.
		if (CachedCallbacks.RevertDelta)
		{
			CachedCallbacks.RevertDelta(CachedContext.SelectionDeltaKey);
		}
	}
	else
	{
		FScopedTransaction Transaction(LOCTEXT("CreateDeletionDelta", "PCG Manual Edit: Create Deletion"));
		PCGComponent->Modify();

		FPCGPointDeletionDelta DeletionDelta;
		DeletionDelta.OriginalTransform = CachedContext.SelectionTransform;
		DeletionDelta.Bounds = PCGComponent->GetTotalBounds();
		DeletionDelta.ElementIndex = CachedContext.OriginalElementIndex;

		if (!CachedContext.NodeConfiguration)
		{
			return;
		}

		const FPCGDeltaKey NewKey = PCG::DataOverride::Keys::FPCGCompositeTransformDeltaKey(
			CachedContext.SelectionTransform,
			CachedContext.NodeConfiguration->SignatureTolerance,
			CachedContext.NodeConfiguration->bSignaturePosition,
			CachedContext.NodeConfiguration->bSignatureRotation,
			CachedContext.NodeConfiguration->bSignatureScale,
			FPCGPointDeletionDelta::GetDeltaNameStatic());

		Collection->Add(NewKey, TInstancedStruct<FPCGPointDeletionDelta>::Make(MoveTemp(DeletionDelta)));

		PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);

		if (CachedCallbacks.ClearSelection)
		{
			CachedCallbacks.ClearSelection();
		}
	}

	RefreshLists(CachedContext);
}

TSharedRef<ITableRow> FPCGPointDeletionViewportExtension::OnGenerateDeletionRow(TSharedPtr<FPCGPointDeletionDeltaEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable)
{
	FPCGDeltaKey EntryKey = Entry->Key;

	return SNew(STableRow<TSharedPtr<FPCGPointDeletionDeltaEntry>>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		   .AutoWidth()
		   .VAlign(VAlign_Center)
		   .Padding(4.0f, 1.0f, 2.0f, 1.0f)
			[
				SNew(STextBlock)
			   .Text(FText::AsNumber(Entry->OriginalElementIndex))
			   .Font(FCoreStyle::GetDefaultFontStyle("Regular", 7))
			]
			+ SHorizontalBox::Slot()
		   .FillWidth(1.0f)
		   .VAlign(VAlign_Center)
		   .Padding(2.0f, 1.0f)
			[
				SNew(STextBlock)
			   .Text(FText::Format(LOCTEXT("DeletionLocationFormat", "({0}, {1}, {2})"),
								   FText::AsNumber(FMath::RoundToInt(Entry->Location.X)),
								   FText::AsNumber(FMath::RoundToInt(Entry->Location.Y)),
								   FText::AsNumber(FMath::RoundToInt(Entry->Location.Z))))
			   .Font(FCoreStyle::GetDefaultFontStyle("Regular", 7))
			]
			+ SHorizontalBox::Slot()
		   .AutoWidth()
		   .VAlign(VAlign_Center)
		   .Padding(2.0f, 0.0f)
			[
				SNew(SButton)
			   .ButtonStyle(FAppStyle::Get(), "SimpleButton")
			   .ContentPadding(FMargin(1, 0))
			   .ToolTipText(LOCTEXT("SelectDeletionTooltip", "Select this deleted point"))
			   .OnClicked_Lambda([this, EntryKey]() -> FReply
				{
					if (CachedCallbacks.SelectElement)
					{
						CachedCallbacks.SelectElement(EntryKey, INDEX_NONE);
					}
					return FReply::Handled();
				})
				[
					SNew(SImage)
				   .Image(FAppStyle::Get().GetBrush("Icons.Search"))
				   .ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
		   .AutoWidth()
		   .VAlign(VAlign_Center)
		   .Padding(2.0f, 0.0f)
			[
				SNew(SButton)
			   .ButtonStyle(FAppStyle::Get(), "SimpleButton")
			   .ContentPadding(FMargin(1, 0))
			   .ToolTipText(LOCTEXT("RestoreButtonTooltip", "Restore this deleted point"))
			   .OnClicked_Lambda([this, EntryKey]() -> FReply
				{
					if (CachedCallbacks.RevertDelta)
					{
						CachedCallbacks.RevertDelta(EntryKey);
					}
					RefreshLists(CachedContext);

					return FReply::Handled();
				})
				[
					SNew(SImage)
				   .Image(FAppStyle::Get().GetBrush("GenericCommands.Undo"))
				   .ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
}

bool FPCGPointDeletionViewportExtension::IsSelectionDeletionDelta() const
{
	if (!CachedContext.bSelectionActive || CachedContext.SelectionDeltaKey.Hash == 0)
	{
		return false;
	}

	UPCGComponent* PCGComponent = CachedContext.ActivePCGComponent.Get();
	if (!PCGComponent)
	{
		return false;
	}

	const FPCGSourceDataContainer* DataContainer = PCGComponent->GetExecutionState().GetSourceDataContainer();
	if (!DataContainer)
	{
		return false;
	}

	const FConstSharedStruct SharedStruct = DataContainer->Get<FPCGDeltaCollection>(CachedContext.ActiveStorageKey);
	const FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<const FPCGDeltaCollection>() : nullptr;
	if (!Collection)
	{
		return false;
	}

	const TInstancedStruct<FPCGDeltaBase>* Delta = Collection->Find(CachedContext.SelectionDeltaKey);
	return Delta && Delta->GetPtr<FPCGPointDeletionDelta>() != nullptr;
}

#undef LOCTEXT_NAMESPACE
