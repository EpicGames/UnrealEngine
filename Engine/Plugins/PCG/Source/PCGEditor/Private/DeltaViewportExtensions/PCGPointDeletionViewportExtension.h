// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeltaViewportExtensions/PCGDeltaViewportExtension.h"

#include "Graph/DataOverride/PCGDataOverrideHelpers.h"
#include "Graph/DataOverride/PCGDataOverridePoints.h"

#include "Widgets/Views/SListView.h"

struct FPCGPointDeletionDeltaEntry
{
	FPCGDeltaKey Key;
	int32 OriginalElementIndex = INDEX_NONE;
	FVector Location = FVector::ZeroVector;
};

/** Viewport extension for FPCGPointDeletionDelta. */
class FPCGPointDeletionViewportExtension : public IPCGDeltaViewportExtension
{
public:
	// ~Begin IPCGDeltaViewportExtension interface
	virtual FText GetDisplayName() const override { return NSLOCTEXT("PCGPointDeletionViewportExtension", "PointDeletionDisplayName", "Point Deletion"); }
	virtual FName GetDeltaName() const override { return FPCGPointDeletionDelta::GetDeltaNameStatic(); }
	virtual int32 GetSortPriority() const override { return 3; }
	virtual TSharedRef<SWidget> CreateWidget(FPCGDeltaViewportCallbacks Callbacks) override;
	virtual void UpdateContext(const FPCGDeltaViewportContext& Context) override { CachedContext = Context; }
	virtual void RefreshLists(const FPCGDeltaViewportContext& Context) override;
	virtual TArray<FPCGDeltaKeyBinding> GetKeyBindings() override { return {{EKeys::Delete, [this]() { HandleDeletePressed(); }}}; }
	virtual FLinearColor GetDisplayColor(const bool bIsSelected) const override;
	virtual FTransform GetDisplayTransform(const FTransform& SourceElementTransform, const FConstStructView DeltaStruct) const override;
	virtual bool MatchesSourceElement(const FTransform& SourceElementTransform, const FConstStructView DeltaStruct, const double SpatialTolerance) const override;
	virtual bool UsesTRSGizmo() const override { return false; }
	virtual FTransform GetOriginalTransform(const FConstStructView DeltaStruct) const override;
	virtual void CollectRestorableKeys(const FPCGDeltaCollection& Collection, TArray<FPCGDeltaKey>& OutKeys) const override;
	// ~End IPCGDeltaViewportExtension interface

private:
	TSharedRef<ITableRow> OnGenerateDeletionRow(TSharedPtr<FPCGPointDeletionDeltaEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable);

	/** Returns true if the current selection points to a deletion delta in the collection. */
	bool IsSelectionDeletionDelta() const;

	/** Toggles deletion on the selected point: creates a deletion delta or restores an existing one. */
	void HandleDeletePressed();

	FPCGDeltaViewportContext CachedContext;
	FPCGDeltaViewportCallbacks CachedCallbacks;

	TArray<TSharedPtr<FPCGPointDeletionDeltaEntry>> DeletionEntries;
	TSharedPtr<SVerticalBox> DeletionSection;
	TSharedPtr<SListView<TSharedPtr<FPCGPointDeletionDeltaEntry>>> DeletionListView;
};
