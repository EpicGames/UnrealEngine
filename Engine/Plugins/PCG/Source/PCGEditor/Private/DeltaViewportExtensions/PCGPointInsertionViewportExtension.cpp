// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeltaViewportExtensions/PCGPointInsertionViewportExtension.h"
#include "DeltaViewportExtensions/PCGDeltaViewportExtensionHelpers.h"
#include "PCGComponent.h"
#include "Graph/DataOverride/PCGDataOverrideHelpers.h"

#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#include "KismetPins/SVectorSlider.h"

#define LOCTEXT_NAMESPACE "PCGPointInsertionViewportExtension"

namespace PCGPointInsertionDelta::Constants
{
	constexpr FLinearColor SelectedColor(1.0f, 0.8f, 0.3f, 0.5f);
	constexpr FLinearColor NormalColor(0.75f, 0.45f, 0.0f, 0.3f);
}

namespace PCGPointInsertionDelta::Helpers
{
	// Inserted points inherit location and rotation from the parent context, but always start at unit scale.
	FPCGPoint& InsertNewPoint(FPCGPointInsertionDelta& Delta, const FTransform& ParentTransform)
	{
		FPCGPoint& NewPoint = Delta.InsertedPoints.Emplace_GetRef();
		NewPoint.Transform = ParentTransform;
		NewPoint.Transform.SetScale3D(FVector::OneVector);
		return NewPoint;
	}
}

TSharedRef<SWidget> FPCGPointInsertionViewportExtension::CreateWidget(FPCGDeltaViewportCallbacks Callbacks)
{
	CachedCallbacks = MoveTemp(Callbacks);
	InsertionEntries.Empty();

	TSharedRef<SVerticalBox> BaseBox = SNew(SVerticalBox)
		// Insert Point button
		+ SVerticalBox::Slot()
	   .AutoHeight()
	   .Padding(0.0f, 4.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
		   .HAlign(HAlign_Center)
		   .OnClicked(FOnClicked::CreateRaw(this, &FPCGPointInsertionViewportExtension::OnInsertPointClicked))
			[
				SNew(STextBlock)
			   .Text(LOCTEXT("InsertPointButton", "Insert Point"))
			   .Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
			]
		]
		// Insertions list section
		+ SVerticalBox::Slot()
	   .AutoHeight()
	   .Padding(0.0f, 8.0f, 0.0f, 2.0f)
		[
			SAssignNew(InsertionListSection, SVerticalBox)
		   .Visibility(EVisibility::Collapsed)
			// Header row: "Insertions" + "Remove All"
			+ SVerticalBox::Slot()
		   .AutoHeight()
		   .Padding(0.0f, 0.0f, 0.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
			   .FillWidth(1.0f)
			   .VAlign(VAlign_Center)
				[
					SNew(STextBlock)
				   .Text(LOCTEXT("InsertionsHeader", "Insertions"))
				   .Font(FCoreStyle::GetDefaultFontStyle("Bold", 8))
				]
				+ SHorizontalBox::Slot()
			   .AutoWidth()
			   .VAlign(VAlign_Center)
				[
					SNew(SButton)
				   .OnClicked(FOnClicked::CreateRaw(this, &FPCGPointInsertionViewportExtension::OnRemoveAllClicked))
					[
						SNew(STextBlock)
					   .Text(LOCTEXT("RemoveAllInsertionsButton", "Remove All"))
					   .Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
					]
				]
			]
			// Insertion list view
			+ SVerticalBox::Slot()
		   .AutoHeight()
		   .MaxHeight(150.0f)
			[
				SAssignNew(InsertionListView, SListView<TSharedPtr<FPCGPointInsertionDeltaEntry>>)
			   .ListItemsSource(&InsertionEntries)
			   .OnGenerateRow_Lambda([this](TSharedPtr<FPCGPointInsertionDeltaEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
				{
					return OnGenerateInsertionRow(Entry, OwnerTable);
				})
			   .SelectionMode(ESelectionMode::None)
			]
		];

	return BaseBox;
}

void FPCGPointInsertionViewportExtension::RefreshLists(const FPCGDeltaViewportContext& Context)
{
	CachedContext = Context;

	// Build new entries into a temporary array and compare against existing to avoid unnecessary list rebuilds
	// that destroy widget focus (SVectorSlider text fields lose focus on RequestListRefresh).
	TArray<TSharedPtr<FPCGPointInsertionDeltaEntry>> NewEntries;

	if (UPCGComponent* PCGComponent = Context.ActivePCGComponent.Get())
	{
		if (FPCGSourceDataContainer* DataContainer = PCGComponent->GetExecutionState().GetSourceDataContainer())
		{
			FConstSharedStruct SharedStruct = DataContainer->Get<FPCGDeltaCollection>(Context.ActiveStorageKey);
			if (const FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<const FPCGDeltaCollection>() : nullptr)
			{
				Collection->ForEachDelta([&NewEntries](const FPCGDeltaKey& DeltaKey, const TInstancedStruct<FPCGDeltaBase>& DeltaValue) -> bool
				{
					if (const FPCGPointInsertionDelta* InsertDelta = DeltaValue.GetPtr<FPCGPointInsertionDelta>())
					{
						for (int32 i = 0; i < InsertDelta->InsertedPoints.Num(); ++i)
						{
							TSharedPtr<FPCGPointInsertionDeltaEntry> Entry = MakeShared<FPCGPointInsertionDeltaEntry>();
							Entry->DeltaKey = DeltaKey;
							Entry->Index = i;
							Entry->Location = InsertDelta->InsertedPoints[i].Transform.GetLocation();
							NewEntries.Add(Entry);
						}
					}
					return true;
				});
			}
		}
	}

	// Only rebuild the list if the entry count or keys changed. Location updates are reflected via VisibleText lambdas.
	bool bListChanged = NewEntries.Num() != InsertionEntries.Num();
	if (!bListChanged)
	{
		for (int32 i = 0; i < NewEntries.Num(); ++i)
		{
			if (NewEntries[i]->DeltaKey != InsertionEntries[i]->DeltaKey || NewEntries[i]->Index != InsertionEntries[i]->Index)
			{
				bListChanged = true;
				break;
			}

			// Update cached location in-place so VisibleText lambdas reflect the latest values.
			InsertionEntries[i]->Location = NewEntries[i]->Location;
		}
	}

	if (bListChanged)
	{
		InsertionEntries = MoveTemp(NewEntries);

		if (InsertionListView.IsValid())
		{
			InsertionListView->RequestListRefresh();
		}
	}

	const bool bHasInsertions = InsertionEntries.Num() > 0;
	if (InsertionListSection.IsValid())
	{
		InsertionListSection->SetVisibility(bHasInsertions ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed);
	}
}

FLinearColor FPCGPointInsertionViewportExtension::GetDisplayColor(const bool bIsSelected) const
{
	return bIsSelected ? PCGPointInsertionDelta::Constants::SelectedColor : PCGPointInsertionDelta::Constants::NormalColor;
}

void FPCGPointInsertionViewportExtension::GetInsertedElements(const FPCGDeltaCollection& Collection, TArray<FInsertedViewportElement>& OutElements) const
{
	Collection.ForEachDelta([&OutElements](const FPCGDeltaKey& DeltaKey, const TInstancedStruct<FPCGDeltaBase>& DeltaValue) -> bool
	{
		if (const FPCGPointInsertionDelta* InsertDelta = DeltaValue.GetPtr<FPCGPointInsertionDelta>())
		{
			for (int32 Index = 0; Index < InsertDelta->InsertedPoints.Num(); ++Index)
			{
				OutElements.Add({DeltaKey, Index, InsertDelta->InsertedPoints[Index].Transform});
			}
		}
		return true;
	});
}

int32 FPCGPointInsertionViewportExtension::OnAltDrag(FPCGDeltaCollection& Collection, const FTransform& SourceTransform, FPCGDeltaKey& OutDeltaKey) const
{
	FPCGPointInsertionDelta* InsertionDelta = FindOrCreateInsertionDelta(&Collection, &OutDeltaKey);
	if (!InsertionDelta)
	{
		return INDEX_NONE;
	}

	PCGPointInsertionDelta::Helpers::InsertNewPoint(*InsertionDelta, SourceTransform);

	return InsertionDelta->InsertedPoints.Num() - 1;
}

bool FPCGPointInsertionViewportExtension::ApplyGizmoTransform(const FTransform& NewTransform, TInstancedStruct<FPCGDeltaBase>& DeltaStruct, const int32 ElementIndex) const
{
	FPCGPointInsertionDelta* InsertDelta = DeltaStruct.GetMutablePtr<FPCGPointInsertionDelta>();
	if (!InsertDelta)
	{
		return false;
	}

	if (InsertDelta->InsertedPoints.IsValidIndex(ElementIndex))
	{
		InsertDelta->InsertedPoints[ElementIndex].Transform = NewTransform;
		return true;
	}

	return false;
}

FPCGDeltaCollection* FPCGPointInsertionViewportExtension::GetOrCreateMutableCollection(FPCGSourceDataContainer* DataContainer)
{
	FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(CachedContext.ActiveStorageKey);
	FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;

	if (!Collection)
	{
		FPCGDeltaCollection NewCollection;
		NewCollection.Settings.KeyPolicy = EPCGDataOverrideKeyPolicy::Spatial;
		NewCollection.Settings.KeyTarget = EPCGSpatialKeyTarget::Position;
		NewCollection.Settings.SpatialTolerance = PCG::DataOverride::Constants::SpatialToleranceDefault;
		DataContainer->Store(CachedContext.ActiveStorageKey, NewCollection);

		SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(CachedContext.ActiveStorageKey);
		Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
	}

	return Collection;
}

FPCGPointInsertionDelta* FPCGPointInsertionViewportExtension::FindOrCreateInsertionDelta(FPCGDeltaCollection* Collection, FPCGDeltaKey* OutKey)
{
	FPCGPointInsertionDelta* InsertionDelta = nullptr;

	Collection->ForEachDelta([&InsertionDelta, OutKey](const FPCGDeltaKey& IterKey, TInstancedStruct<FPCGDeltaBase>& DeltaValue) -> bool
	{
		InsertionDelta = DeltaValue.GetMutablePtr<FPCGPointInsertionDelta>();
		if (InsertionDelta && OutKey)
		{
			*OutKey = IterKey;
		}

		return !InsertionDelta;
	});

	if (!InsertionDelta)
	{
		const FPCGDeltaKey NewKey(PCGInsertionConstants::DefaultInsertionKeyHash, FPCGPointInsertionDelta::GetDeltaNameStatic());
		TInstancedStruct<FPCGDeltaBase>& AddedRef = Collection->Add_GetRef(NewKey, TInstancedStruct<FPCGPointInsertionDelta>::Make(FPCGPointInsertionDelta()));
		InsertionDelta = AddedRef.GetMutablePtr<FPCGPointInsertionDelta>();
		if (OutKey)
		{
			*OutKey = NewKey;
		}
	}

	return InsertionDelta;
}

FReply FPCGPointInsertionViewportExtension::OnInsertPointClicked()
{
	AddInsertedElement();
	RefreshLists(CachedContext);

	return FReply::Handled();
}

FReply FPCGPointInsertionViewportExtension::OnRemoveAllClicked()
{
	RemoveAllInsertedElements();
	RefreshLists(CachedContext);

	return FReply::Handled();
}

void FPCGPointInsertionViewportExtension::HandleDeletePressed()
{
	if (CachedContext.bSelectionActive && CachedContext.SelectedElementIndex != INDEX_NONE)
	{
		RemoveInsertedElement(CachedContext.SelectedElementIndex);

		if (CachedCallbacks.ClearSelection)
		{
			CachedCallbacks.ClearSelection();
		}
	}

	RefreshLists(CachedContext);
}

void FPCGPointInsertionViewportExtension::AddInsertedElement()
{
	UPCGComponent* PCGComponent = CachedContext.ActivePCGComponent.Get();
	if (!PCGComponent)
	{
		return;
	}

	FPCGSourceDataContainer* DataContainer = PCGComponent->GetExecutionState().GetSourceDataContainer();
	if (!DataContainer)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddInsertedElement", "PCG Manual Edit: Add Inserted Point"));
	PCGComponent->Modify();

	FPCGDeltaCollection* Collection = GetOrCreateMutableCollection(DataContainer);
	if (!Collection)
	{
		return;
	}

	FPCGDeltaKey InsertionDeltaKey;
	FPCGPointInsertionDelta* InsertionDelta = FindOrCreateInsertionDelta(Collection, &InsertionDeltaKey);
	if (!InsertionDelta)
	{
		return;
	}

	FTransform NewTransform = FTransform::Identity;
	if (CachedContext.bSelectionActive)
	{
		NewTransform = CachedContext.SelectionTransform;
	}
	else if (AActor* Owner = PCGComponent->GetOwner())
	{
		NewTransform = Owner->GetActorTransform();
	}

	PCGPointInsertionDelta::Helpers::InsertNewPoint(*InsertionDelta, NewTransform);

	const int32 NewElementIndex = InsertionDelta->InsertedPoints.Num() - 1;

	PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);

	if (CachedCallbacks.SelectElement)
	{
		CachedCallbacks.SelectElement(InsertionDeltaKey, NewElementIndex);
	}
}

void FPCGPointInsertionViewportExtension::UpdateInsertedElementLocation(const FPCGDeltaKey& DeltaKey, const int32 Index, const FVector& NewLocation)
{
	UPCGComponent* PCGComponent = CachedContext.ActivePCGComponent.Get();
	if (!PCGComponent)
	{
		return;
	}

	FPCGSourceDataContainer* DataContainer = PCGComponent->GetExecutionState().GetSourceDataContainer();
	if (!DataContainer)
	{
		return;
	}

	FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(CachedContext.ActiveStorageKey);
	FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
	if (!Collection)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("UpdateInsertedElementLocation", "PCG Manual Edit: Update Inserted Point Location"));
	PCGComponent->Modify();

	if (TInstancedStruct<FPCGDeltaBase>* DeltaStruct = Collection->Find(DeltaKey))
	{
		if (FPCGPointInsertionDelta* InsertDelta = DeltaStruct->GetMutablePtr<FPCGPointInsertionDelta>())
		{
			if (InsertDelta->InsertedPoints.IsValidIndex(Index))
			{
				InsertDelta->InsertedPoints[Index].Transform.SetLocation(NewLocation);
			}
		}
	}

	PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);

	if (CachedCallbacks.SelectElement)
	{
		CachedCallbacks.SelectElement(DeltaKey, Index);
	}
}

void FPCGPointInsertionViewportExtension::RemoveInsertedElement(int32 Index)
{
	UPCGComponent* PCGComponent = CachedContext.ActivePCGComponent.Get();
	if (!PCGComponent)
	{
		return;
	}

	FPCGSourceDataContainer* DataContainer = PCGComponent->GetExecutionState().GetSourceDataContainer();
	if (!DataContainer)
	{
		return;
	}

	FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(CachedContext.ActiveStorageKey);
	FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
	if (!Collection)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveInsertedElement", "PCG Manual Edit: Remove Inserted Point"));
	PCGComponent->Modify();

	Collection->ForEachDelta([Index](const FPCGDeltaKey& Key, TInstancedStruct<FPCGDeltaBase>& DeltaValue) -> bool
	{
		if (FPCGPointInsertionDelta* InsertDelta = DeltaValue.GetMutablePtr<FPCGPointInsertionDelta>())
		{
			if (InsertDelta->InsertedPoints.IsValidIndex(Index))
			{
				InsertDelta->InsertedPoints.RemoveAt(Index);
			}

			return false;
		}

		return true;
	});

	PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);
}

void FPCGPointInsertionViewportExtension::RemoveAllInsertedElements()
{
	UPCGComponent* PCGComponent = CachedContext.ActivePCGComponent.Get();
	if (!PCGComponent)
	{
		return;
	}

	FPCGSourceDataContainer* DataContainer = PCGComponent->GetExecutionState().GetSourceDataContainer();
	if (!DataContainer)
	{
		return;
	}

	FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(CachedContext.ActiveStorageKey);
	FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
	if (!Collection)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveAllInsertedElements", "PCG Manual Edit: Remove All Inserted Points"));
	PCGComponent->Modify();

	TArray<FPCGDeltaKey> InsertionKeys;
	Collection->ForEachDelta([&InsertionKeys](const FPCGDeltaKey& DeltaKey, const TInstancedStruct<FPCGDeltaBase>& DeltaValue) -> bool
	{
		if (DeltaValue.GetPtr<FPCGPointInsertionDelta>())
		{
			InsertionKeys.Add(DeltaKey);
		}
		return true;
	});

	for (const FPCGDeltaKey& Key : InsertionKeys)
	{
		Collection->Remove(Key);
	}

	PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);
}

void FPCGPointInsertionViewportExtension::OnInsertionComponentCommitted(const int32 EntryIndex, const int32 AxisIndex, const double NewValue)
{
	for (const TSharedPtr<FPCGPointInsertionDeltaEntry>& Entry : InsertionEntries)
	{
		if (Entry && Entry->Index == EntryIndex)
		{
			Entry->Location[AxisIndex] = NewValue;
			UpdateInsertedElementLocation(Entry->DeltaKey, EntryIndex, Entry->Location);
			return;
		}
	}
}

TSharedRef<ITableRow> FPCGPointInsertionViewportExtension::OnGenerateInsertionRow(TSharedPtr<FPCGPointInsertionDeltaEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable)
{
	TWeakPtr<FPCGPointInsertionDeltaEntry> WeakEntry = Entry;
	int32 EntryIndex = Entry->Index;

	return SNew(STableRow<TSharedPtr<FPCGPointInsertionDeltaEntry>>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		   .FillWidth(1.0f)
		   .VAlign(VAlign_Center)
		   .Padding(4.0f, 1.0f)
			[
				SNew(SVectorSlider<double>, /*bIsRotator=*/false, /*InProperty=*/nullptr)
			   .VisibleText_0_Lambda([WeakEntry]() -> FString
				{
					TSharedPtr<FPCGPointInsertionDeltaEntry> Pinned = WeakEntry.Pin();
					return Pinned ? FString::SanitizeFloat(Pinned->Location.X) : FString();
				})
			   .VisibleText_1_Lambda([WeakEntry]() -> FString
				{
					TSharedPtr<FPCGPointInsertionDeltaEntry> Pinned = WeakEntry.Pin();
					return Pinned ? FString::SanitizeFloat(Pinned->Location.Y) : FString();
				})
			   .VisibleText_2_Lambda([WeakEntry]() -> FString
				{
					TSharedPtr<FPCGPointInsertionDeltaEntry> Pinned = WeakEntry.Pin();
					return Pinned ? FString::SanitizeFloat(Pinned->Location.Z) : FString();
				})
			   .OnNumericCommitted_Box_0(SVectorSlider<double>::FOnNumericValueCommitted::CreateLambda(
					[this, EntryIndex](const double V, ETextCommit::Type) { OnInsertionComponentCommitted(EntryIndex, 0, V); }))
			   .OnNumericCommitted_Box_1(SVectorSlider<double>::FOnNumericValueCommitted::CreateLambda(
					[this, EntryIndex](const double V, ETextCommit::Type) { OnInsertionComponentCommitted(EntryIndex, 1, V); }))
			   .OnNumericCommitted_Box_2(SVectorSlider<double>::FOnNumericValueCommitted::CreateLambda(
					[this, EntryIndex](const double V, ETextCommit::Type) { OnInsertionComponentCommitted(EntryIndex, 2, V); }))
			]
			+ SHorizontalBox::Slot()
		   .AutoWidth()
		   .VAlign(VAlign_Center)
		   .Padding(2.0f, 0.0f)
			[
				SNew(SButton)
			   .ButtonStyle(FAppStyle::Get(), "SimpleButton")
			   .ContentPadding(FMargin(1, 0))
			   .ToolTipText(LOCTEXT("SelectInsertionTooltip", "Select this inserted point"))
			   .OnClicked_Lambda([this, WeakEntry]() -> FReply
				{
					if (CachedCallbacks.SelectElement)
					{
						if (TSharedPtr<FPCGPointInsertionDeltaEntry> Pinned = WeakEntry.Pin())
						{
							CachedCallbacks.SelectElement(Pinned->DeltaKey, Pinned->Index);
						}
					}
					return FReply::Handled();
				})
				[
					SNew(SImage)
				   .Image(FAppStyle::Get().GetBrush("Icons.Search"))
				   .ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
		   .AutoWidth()
		   .VAlign(VAlign_Center)
		   .Padding(2.0f, 0.0f)
			[
				SNew(SButton)
			   .ButtonStyle(FAppStyle::Get(), "SimpleButton")
			   .ContentPadding(FMargin(1, 0))
			   .ToolTipText(LOCTEXT("RemoveInsertionTooltip", "Remove this inserted point"))
			   .OnClicked_Lambda([this, EntryIndex]() -> FReply
				{
					RemoveInsertedElement(EntryIndex);
					RefreshLists(CachedContext);
					return FReply::Handled();
				})
				[
					SNew(SImage)
				   .Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				   .ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
