// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeltaViewportExtensions/PCGDeltaViewportExtension.h"

#include "Graph/DataOverride/PCGDataOverridePoints.h"

class ITableRow;
class STableViewBase;
class SVerticalBox;
template <typename ItemType> class SListView;

struct FPCGPointInsertionDeltaEntry
{
	FPCGDeltaKey DeltaKey;
	int32 Index = INDEX_NONE;
	FVector Location = FVector::ZeroVector;
};

namespace PCGInsertionConstants
{
	// @todo_pcg: Find an appropriate hash value for the insertion delta key. Currently an arbitrary non-zero constant.
	constexpr FXxHash64 DefaultInsertionKeyHash{1};
}

/** Viewport extension for FPCGPointInsertionDelta: Insert button + insertions list + viewport policy. */
class FPCGPointInsertionViewportExtension : public IPCGDeltaViewportExtension
{
public:
	// ~Begin IPCGDeltaViewportExtension interface
	virtual FText GetDisplayName() const override { return NSLOCTEXT("PCGPointInsertionViewportExtension", "PointInsertionDisplayName", "Point Insertion"); }
	virtual FName GetDeltaName() const override { return FPCGPointInsertionDelta::GetDeltaNameStatic(); }
	virtual int32 GetSortPriority() const override { return 2; }
	virtual TSharedRef<SWidget> CreateWidget(FPCGDeltaViewportCallbacks Callbacks) override;
	virtual void UpdateContext(const FPCGDeltaViewportContext& Context) override { CachedContext = Context; }
	virtual void RefreshLists(const FPCGDeltaViewportContext& Context) override;
	virtual TArray<FPCGDeltaKeyBinding> GetKeyBindings() override { return {{EKeys::Delete, [this]() { HandleDeletePressed(); }}}; }
	virtual FLinearColor GetDisplayColor(const bool bIsSelected) const override;
	virtual void GetInsertedElements(const FPCGDeltaCollection& Collection, TArray<FInsertedViewportElement>& OutElements) const override;
	virtual int32 OnAltDrag(FPCGDeltaCollection& Collection, const FTransform& SourceTransform, FPCGDeltaKey& OutDeltaKey) const override;
	virtual bool ApplyGizmoTransform(const FTransform& NewTransform, TInstancedStruct<FPCGDeltaBase>& DeltaStruct, const int32 ElementIndex) const override;
	// ~End IPCGDeltaViewportExtension interface

private:
	/** Get or create the mutable delta collection from the cached context. */
	FPCGDeltaCollection* GetOrCreateMutableCollection(FPCGSourceDataContainer* DataContainer);

	/** Find or create the single insertion delta within the collection. */
	static FPCGPointInsertionDelta* FindOrCreateInsertionDelta(FPCGDeltaCollection* Collection, FPCGDeltaKey* OutKey = nullptr);

	FReply OnInsertPointClicked();
	FReply OnRemoveAllClicked();

	void HandleDeletePressed();
	void AddInsertedElement();
	void UpdateInsertedElementLocation(const FPCGDeltaKey& DeltaKey, const int32 Index, const FVector& NewLocation);
	void RemoveInsertedElement(int32 Index);
	void RemoveAllInsertedElements();
	void OnInsertionComponentCommitted(const int32 EntryIndex, const int32 AxisIndex, const double NewValue);
	TSharedRef<ITableRow> OnGenerateInsertionRow(TSharedPtr<FPCGPointInsertionDeltaEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable);

	FPCGDeltaViewportContext CachedContext;
	FPCGDeltaViewportCallbacks CachedCallbacks;

	TArray<TSharedPtr<FPCGPointInsertionDeltaEntry>> InsertionEntries;
	TSharedPtr<SVerticalBox> InsertionListSection;
	TSharedPtr<SListView<TSharedPtr<FPCGPointInsertionDeltaEntry>>> InsertionListView;
};
