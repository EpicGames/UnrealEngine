// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeltaViewportExtensions/PCGPointTransformOffsetViewportExtension.h"

#define LOCTEXT_NAMESPACE "PCGPointTransformOffsetViewportExtension"

namespace PCG::PointTransformOffsetDelta::Constants
{
	constexpr FLinearColor SelectedColor(0.3f, 0.8f, 1.0f, 0.5f);
	constexpr FLinearColor NormalColor(0.0f, 0.45f, 0.75f, 0.3f);
}

FLinearColor FPCGPointTransformOffsetViewportExtension::GetDisplayColor(const bool bIsSelected) const
{
	return bIsSelected ? PCG::PointTransformOffsetDelta::Constants::SelectedColor : PCG::PointTransformOffsetDelta::Constants::NormalColor;
}

FTransform FPCGPointTransformOffsetViewportExtension::GetDisplayTransform(const FTransform& SourceElementTransform, const FConstStructView DeltaStruct) const
{
	if (const FPCGPointTransformOffsetDelta* TransformDelta = DeltaStruct.GetPtr<const FPCGPointTransformOffsetDelta>())
	{
		FTransform DisplayTransform = TransformDelta->OriginalTransform;

		if (TransformDelta->bOffsetPosition)
		{
			DisplayTransform.AddToTranslation(TransformDelta->TransformOffset.GetLocation());
		}

		if (TransformDelta->bOffsetRotation)
		{
			DisplayTransform.SetRotation(TransformDelta->TransformOffset.GetRotation() * DisplayTransform.GetRotation());
		}

		if (TransformDelta->bOffsetScale)
		{
			DisplayTransform.SetScale3D(DisplayTransform.GetScale3D() * TransformDelta->TransformOffset.GetScale3D());
		}

		return DisplayTransform;
	}

	return SourceElementTransform;
}

bool FPCGPointTransformOffsetViewportExtension::ApplyGizmoTransform(const FTransform& NewTransform, TInstancedStruct<FPCGDeltaBase>& DeltaStruct, int32 ElementIndex) const
{
	if (FPCGPointTransformOffsetDelta* TransformDelta = DeltaStruct.GetMutablePtr<FPCGPointTransformOffsetDelta>())
	{
		const FTransform& Original = TransformDelta->OriginalTransform;

		TransformDelta->TransformOffset.SetLocation(NewTransform.GetLocation() - Original.GetLocation());
		TransformDelta->TransformOffset.SetRotation(NewTransform.GetRotation() * Original.GetRotation().Inverse());

		const FVector OriginalScale = Original.GetScale3D();
		const FVector NewScale = NewTransform.GetScale3D();
		TransformDelta->TransformOffset.SetScale3D(FVector(
			FMath::IsNearlyZero(OriginalScale.X) ? 1.0 : NewScale.X / OriginalScale.X,
			FMath::IsNearlyZero(OriginalScale.Y) ? 1.0 : NewScale.Y / OriginalScale.Y,
			FMath::IsNearlyZero(OriginalScale.Z) ? 1.0 : NewScale.Z / OriginalScale.Z));

		return true;
	}

	return false;
}

void FPCGPointTransformOffsetViewportExtension::CreateNewDelta(
	const FPCGDeltaKey& DeltaKey,
	FPCGDeltaCollection& Collection,
	const FTransform& OriginalTransform,
	const FTransform& NewTransform,
	const FPCGDeltaCreateContext& Context) const
{
	FPCGPointTransformOffsetDelta NewDelta;
	NewDelta.OriginalTransform = OriginalTransform;
	NewDelta.Bounds = Context.ComponentBounds;
	NewDelta.bOffsetPosition = bOverridePosition;
	NewDelta.bOffsetRotation = bOverrideRotation;
	NewDelta.bOffsetScale = bOverrideScale;

	NewDelta.TransformOffset.SetLocation(NewTransform.GetLocation() - OriginalTransform.GetLocation());
	NewDelta.TransformOffset.SetRotation(NewTransform.GetRotation() * OriginalTransform.GetRotation().Inverse());

	const FVector OrigScale = OriginalTransform.GetScale3D();
	const FVector NewScale = NewTransform.GetScale3D();
	NewDelta.TransformOffset.SetScale3D(FVector(
		FMath::IsNearlyZero(OrigScale.X) ? 1.0 : NewScale.X / OrigScale.X,
		FMath::IsNearlyZero(OrigScale.Y) ? 1.0 : NewScale.Y / OrigScale.Y,
		FMath::IsNearlyZero(OrigScale.Z) ? 1.0 : NewScale.Z / OrigScale.Z));

	NewDelta.ElementIndex = Context.OriginalElementIndex;

	Collection.Add(DeltaKey, TInstancedStruct<FPCGPointTransformOffsetDelta>::Make(MoveTemp(NewDelta)));
}

FText FPCGPointTransformOffsetViewportExtension::GetCheckboxGroupLabel() const
{
	return LOCTEXT("TransformLabel", "Transform:");
}

FText FPCGPointTransformOffsetViewportExtension::GetListHeaderLabel() const
{
	return LOCTEXT("TransformsHeader", "Transforms");
}

FVector FPCGPointTransformOffsetViewportExtension::GetDisplayVector(const TInstancedStruct<FPCGDeltaBase>& Delta) const
{
	if (const FPCGPointTransformOffsetDelta* TransformDelta = Delta.GetPtr<FPCGPointTransformOffsetDelta>())
	{
		return TransformDelta->TransformOffset.GetLocation();
	}

	return FVector::ZeroVector;
}

void FPCGPointTransformOffsetViewportExtension::SetDisplayVector(TInstancedStruct<FPCGDeltaBase>& Delta, const FVector& NewValue) const
{
	if (FPCGPointTransformOffsetDelta* TransformDelta = Delta.GetMutablePtr<FPCGPointTransformOffsetDelta>())
	{
		TransformDelta->TransformOffset.SetLocation(NewValue);
	}
}

bool FPCGPointTransformOffsetViewportExtension::IsManagedDelta(const TInstancedStruct<FPCGDeltaBase>& Delta) const
{
	return Delta.GetPtr<FPCGPointTransformOffsetDelta>() != nullptr;
}

void FPCGPointTransformOffsetViewportExtension::SetDeltaOverrideFlags(TInstancedStruct<FPCGDeltaBase>& Delta, const bool bPosition, const bool bRotation, const bool bScale) const
{
	if (FPCGPointTransformOffsetDelta* TransformDelta = Delta.GetMutablePtr<FPCGPointTransformOffsetDelta>())
	{
		TransformDelta->bOffsetPosition = bPosition;
		TransformDelta->bOffsetRotation = bRotation;
		TransformDelta->bOffsetScale = bScale;
	}
}

bool FPCGPointTransformOffsetViewportExtension::DeltaOverrideFlagsMatch(const TInstancedStruct<FPCGDeltaBase>& Delta, const bool bPosition, const bool bRotation, const bool bScale) const
{
	if (const FPCGPointTransformOffsetDelta* TransformDelta = Delta.GetPtr<FPCGPointTransformOffsetDelta>())
	{
		return TransformDelta->bOffsetPosition == bPosition
			&& TransformDelta->bOffsetRotation == bRotation
			&& TransformDelta->bOffsetScale == bScale;
	}

	return true;
}

int32 FPCGPointTransformOffsetViewportExtension::GetElementIndex(const TInstancedStruct<FPCGDeltaBase>& Delta) const
{
	if (const FPCGPointTransformOffsetDelta* TransformDelta = Delta.GetPtr<FPCGPointTransformOffsetDelta>())
	{
		return TransformDelta->ElementIndex;
	}

	return INDEX_NONE;
}

#undef LOCTEXT_NAMESPACE
