// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeltaViewportExtensions/PCGPointTransformViewportExtensionBase.h"

/** Viewport extension for FPCGPointTransformOffsetDelta. */
class FPCGPointTransformOffsetViewportExtension : public FPCGPointTransformViewportExtensionBase
{
public:
	// ~Begin IPCGDeltaViewportExtension interface
	virtual FText GetDisplayName() const override { return NSLOCTEXT("PCGPointTransformOffsetViewportExtension", "PointTransformOffsetDisplayName", "Point Transform Offset"); }
	virtual FName GetDeltaName() const override { return FPCGPointTransformOffsetDelta::GetDeltaNameStatic(); }
	virtual int32 GetSortPriority() const override { return 1; }
	virtual FLinearColor GetDisplayColor(bool bIsSelected) const override;
	virtual FTransform GetDisplayTransform(const FTransform& SourceElementTransform, FConstStructView DeltaStruct) const override;
	virtual bool ApplyGizmoTransform(const FTransform& NewTransform, TInstancedStruct<FPCGDeltaBase>& DeltaStruct, int32 ElementIndex) const override;
	virtual void CreateNewDelta(
		const FPCGDeltaKey& DeltaKey,
		FPCGDeltaCollection& Collection,
		const FTransform& OriginalTransform,
		const FTransform& NewTransform,
		const FPCGDeltaCreateContext& Context) const override;
	// ~End IPCGDeltaViewportExtension interface

protected:
	// ~Begin FPCGPointTransformViewportExtensionBase interface
	virtual FText GetCheckboxGroupLabel() const override;
	virtual FText GetListHeaderLabel() const override;
	virtual FVector GetDisplayVector(const TInstancedStruct<FPCGDeltaBase>& Delta) const override;
	virtual void SetDisplayVector(TInstancedStruct<FPCGDeltaBase>& Delta, const FVector& NewValue) const override;
	virtual bool IsManagedDelta(const TInstancedStruct<FPCGDeltaBase>& Delta) const override;
	virtual void SetDeltaOverrideFlags(TInstancedStruct<FPCGDeltaBase>& Delta, bool bPosition, bool bRotation, bool bScale) const override;
	virtual bool DeltaOverrideFlagsMatch(const TInstancedStruct<FPCGDeltaBase>& Delta, bool bPosition, bool bRotation, bool bScale) const override;
	virtual int32 GetElementIndex(const TInstancedStruct<FPCGDeltaBase>& Delta) const override;
	// ~End FPCGPointTransformViewportExtensionBase interface
};
