// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeltaViewportExtensions/PCGPointTransformViewportExtension.h"

#define LOCTEXT_NAMESPACE "PCGPointTransformViewportExtension"

namespace PCG::PointTransformDelta::Constants
{
	constexpr FLinearColor SelectedColor(1.0f, 0.8f, 0.3f, 0.5f);
	constexpr FLinearColor NormalColor(0.75f, 0.45f, 0.0f, 0.3f);
}

FLinearColor FPCGPointTransformViewportExtension::GetDisplayColor(const bool bIsSelected) const
{
	return bIsSelected ? PCG::PointTransformDelta::Constants::SelectedColor : PCG::PointTransformDelta::Constants::NormalColor;
}

FTransform FPCGPointTransformViewportExtension::GetDisplayTransform(const FTransform& SourceElementTransform, const FConstStructView DeltaStruct) const
{
	if (const FPCGPointTransformDelta* TransformDelta = DeltaStruct.GetPtr<const FPCGPointTransformDelta>())
	{
		return TransformDelta->TransformOverride;
	}

	return SourceElementTransform;
}

bool FPCGPointTransformViewportExtension::ApplyGizmoTransform(const FTransform& NewTransform, TInstancedStruct<FPCGDeltaBase>& DeltaStruct, int32 ElementIndex) const
{
	if (FPCGPointTransformDelta* TransformDelta = DeltaStruct.GetMutablePtr<FPCGPointTransformDelta>())
	{
		TransformDelta->TransformOverride = NewTransform;
		return true;
	}

	return false;
}

void FPCGPointTransformViewportExtension::CreateNewDelta(
	const FPCGDeltaKey& DeltaKey,
	FPCGDeltaCollection& Collection,
	const FTransform& OriginalTransform,
	const FTransform& NewTransform,
	const FPCGDeltaCreateContext& Context) const
{
	FPCGPointTransformDelta NewDelta;
	NewDelta.OriginalTransform = OriginalTransform;
	NewDelta.TransformOverride = NewTransform;
	NewDelta.Bounds = Context.ComponentBounds;
	NewDelta.bOverridePosition = bOverridePosition;
	NewDelta.bOverrideRotation = bOverrideRotation;
	NewDelta.bOverrideScale = bOverrideScale;

	NewDelta.ElementIndex = Context.OriginalElementIndex;

	Collection.Add(DeltaKey, TInstancedStruct<FPCGPointTransformDelta>::Make(MoveTemp(NewDelta)));
}

FText FPCGPointTransformViewportExtension::GetCheckboxGroupLabel() const
{
	return LOCTEXT("OverrideLabel", "Override:");
}

FText FPCGPointTransformViewportExtension::GetListHeaderLabel() const
{
	return LOCTEXT("TransformsHeader", "Transforms");
}

FVector FPCGPointTransformViewportExtension::GetDisplayVector(const TInstancedStruct<FPCGDeltaBase>& Delta) const
{
	if (const FPCGPointTransformDelta* TransformDelta = Delta.GetPtr<FPCGPointTransformDelta>())
	{
		return TransformDelta->TransformOverride.GetLocation();
	}

	return FVector::ZeroVector;
}

void FPCGPointTransformViewportExtension::SetDisplayVector(TInstancedStruct<FPCGDeltaBase>& Delta, const FVector& NewValue) const
{
	if (FPCGPointTransformDelta* TransformDelta = Delta.GetMutablePtr<FPCGPointTransformDelta>())
	{
		TransformDelta->TransformOverride.SetLocation(NewValue);
	}
}

bool FPCGPointTransformViewportExtension::IsManagedDelta(const TInstancedStruct<FPCGDeltaBase>& Delta) const
{
	return Delta.GetPtr<FPCGPointTransformDelta>() != nullptr;
}

void FPCGPointTransformViewportExtension::SetDeltaOverrideFlags(TInstancedStruct<FPCGDeltaBase>& Delta, bool bPosition, bool bRotation, bool bScale) const
{
	if (FPCGPointTransformDelta* TransformDelta = Delta.GetMutablePtr<FPCGPointTransformDelta>())
	{
		TransformDelta->bOverridePosition = bPosition;
		TransformDelta->bOverrideRotation = bRotation;
		TransformDelta->bOverrideScale = bScale;
	}
}

bool FPCGPointTransformViewportExtension::DeltaOverrideFlagsMatch(const TInstancedStruct<FPCGDeltaBase>& Delta, bool bPosition, bool bRotation, bool bScale) const
{
	if (const FPCGPointTransformDelta* TransformDelta = Delta.GetPtr<FPCGPointTransformDelta>())
	{
		return TransformDelta->bOverridePosition == bPosition
			&& TransformDelta->bOverrideRotation == bRotation
			&& TransformDelta->bOverrideScale == bScale;
	}

	return true;
}

int32 FPCGPointTransformViewportExtension::GetElementIndex(const TInstancedStruct<FPCGDeltaBase>& Delta) const
{
	if (const FPCGPointTransformDelta* TransformDelta = Delta.GetPtr<FPCGPointTransformDelta>())
	{
		return TransformDelta->ElementIndex;
	}

	return INDEX_NONE;
}

#undef LOCTEXT_NAMESPACE
