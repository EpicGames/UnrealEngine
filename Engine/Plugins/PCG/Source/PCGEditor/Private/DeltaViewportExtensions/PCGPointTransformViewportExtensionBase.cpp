// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeltaViewportExtensions/PCGPointTransformViewportExtensionBase.h"
#include "DeltaViewportExtensions/PCGDeltaViewportExtensionHelpers.h"
#include "SPCGManualEditPanel.h"

#include "PCGComponent.h"
#include "Graph/DataOverride/PCGDataOverrideHelpers.h"
#include "Graph/DataOverride/PCGDataOverridePoints.h"

#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#include "KismetPins/SVectorSlider.h"

#define LOCTEXT_NAMESPACE "PCGPointTransformViewportExtensionBase"

TSharedRef<SWidget> FPCGPointTransformViewportExtensionBase::CreateWidget(FPCGDeltaViewportCallbacks Callbacks)
{
	CachedCallbacks = MoveTemp(Callbacks);

	return SNew(SVerticalBox)
		// Override checkboxes for position, rotation, and scale
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MinDesiredWidth(60.0f)
				[
					SNew(STextBlock)
					.Text(GetCheckboxGroupLabel())
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() -> ECheckBoxState
				{
					return bOverridePosition ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](const ECheckBoxState NewState)
				{
					bOverridePosition = NewState == ECheckBoxState::Checked;
					UpdateFlags();
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PositionCheckbox", "Pos"))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() -> ECheckBoxState
				{
					return bOverrideRotation ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](const ECheckBoxState NewState)
				{
					bOverrideRotation = NewState == ECheckBoxState::Checked;
					UpdateFlags();
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("RotationCheckbox", "Rot"))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() -> ECheckBoxState
				{
					return bOverrideScale ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](const ECheckBoxState NewState)
				{
					bOverrideScale = NewState == ECheckBoxState::Checked;
					UpdateFlags();
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ScaleCheckbox", "Sca"))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				]
			]
		]
		// List section
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f, 0.0f, 2.0f)
		[
			SAssignNew(ListSection, SVerticalBox)
			.Visibility(EVisibility::Collapsed)
			// Header row
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 0.0f, 0.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(GetListHeaderLabel())
					.Font(FCoreStyle::GetDefaultFontStyle("Bold", 8))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.OnClicked_Lambda([this]() -> FReply
					{
						RemoveAllEntries();
						RefreshLists(CachedContext);
						return FReply::Handled();
					})
					[
						SNew(STextBlock)
						.Text(LOCTEXT("RemoveAllButton", "Remove All"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
					]
				]
			]
			// List view
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(150.0f)
			[
				SAssignNew(ListView, SListView<TSharedPtr<FPCGPointTransformDeltaEntry>>)
				.ListItemsSource(&Entries)
				.OnGenerateRow_Lambda([this](TSharedPtr<FPCGPointTransformDeltaEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
				{
					return OnGenerateRow(Entry, OwnerTable);
				})
				.SelectionMode(ESelectionMode::None)
			]
		];
}

void FPCGPointTransformViewportExtensionBase::RefreshLists(const FPCGDeltaViewportContext& Context)
{
	CachedContext = Context;

	TArray<TSharedPtr<FPCGPointTransformDeltaEntry>> NewEntries;

	if (UPCGComponent* PCGComponent = Context.ActivePCGComponent.Get())
	{
		if (FPCGSourceDataContainer* DataContainer = PCGComponent->GetExecutionState().GetSourceDataContainer())
		{
			const FConstSharedStruct SharedStruct = DataContainer->Get<FPCGDeltaCollection>(Context.ActiveStorageKey);
			if (const FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<const FPCGDeltaCollection>() : nullptr)
			{
				Collection->ForEachDelta([this, &NewEntries](const FPCGDeltaKey& DeltaKey, const TInstancedStruct<FPCGDeltaBase>& DeltaValue) -> bool
				{
					if (IsManagedDelta(DeltaValue))
					{
						TSharedPtr<FPCGPointTransformDeltaEntry> Entry = MakeShared<FPCGPointTransformDeltaEntry>();
						Entry->Key = DeltaKey;
						Entry->OriginalElementIndex = GetElementIndex(DeltaValue);
						Entry->DisplayVector = GetDisplayVector(DeltaValue);
						NewEntries.Add(Entry);
					}

					return true;
				});
			}
		}
	}

	// Compare against current entries to avoid unnecessary list rebuilds. Counts must match,
	// and each entry must have the same key, element index, and display vector.
	bool bEntriesMatch = NewEntries.Num() == Entries.Num();
	if (bEntriesMatch)
	{
		for (int32 i = 0; i < NewEntries.Num(); ++i)
		{
			if (NewEntries[i]->Key != Entries[i]->Key
				|| NewEntries[i]->OriginalElementIndex != Entries[i]->OriginalElementIndex
				|| !NewEntries[i]->DisplayVector.Equals(Entries[i]->DisplayVector))
			{
				bEntriesMatch = false;
				break;
			}
		}
	}

	// @todo_pcg: Separate data diffing from UI updates. RefreshLists should return whether the list changed,
	// and the caller should handle widget visibility and RequestListRefresh.
	if (!bEntriesMatch)
	{
		Entries = MoveTemp(NewEntries);

		if (ListView.IsValid())
		{
			ListView->RequestListRefresh();
		}
	}

	const bool bHasEntries = Entries.Num() > 0;
	if (ListSection.IsValid())
	{
		ListSection->SetVisibility(bHasEntries ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed);
	}
}

bool FPCGPointTransformViewportExtensionBase::MatchesSourceElement(const FTransform& SourceElementTransform, const FConstStructView DeltaStruct, const double SpatialTolerance) const
{
	const FPCGPointDeltaBase* PointDelta = DeltaStruct.GetPtr<const FPCGPointDeltaBase>();
	if (!PointDelta)
	{
		return false;
	}

	if (!CachedContext.NodeConfiguration)
	{
		return false;
	}

	auto BuildKey = [this, SpatialTolerance](const FTransform& Transform)
	{
		return PCG::DataOverride::Keys::FPCGCompositeTransformDeltaKey(
			Transform,
			SpatialTolerance,
			CachedContext.NodeConfiguration->bSignaturePosition,
			CachedContext.NodeConfiguration->bSignatureRotation,
			CachedContext.NodeConfiguration->bSignatureScale);
	};

	const FPCGDeltaKey SourceKey = BuildKey(SourceElementTransform);

	// Compare the source to the original transform's key.
	if (SourceKey == BuildKey(PointDelta->OriginalTransform))
	{
		return true;
	}

	// Apply mutates the source array in place, so SourceElementTransform may be the override transform.
	const FTransform DisplayTransform = GetDisplayTransform(SourceElementTransform, DeltaStruct);
	if (!DisplayTransform.Equals(PointDelta->OriginalTransform) && SourceKey == BuildKey(DisplayTransform))
	{
		return true;
	}

	return false;
}

FTransform FPCGPointTransformViewportExtensionBase::GetOriginalTransform(const FConstStructView DeltaStruct) const
{
	if (const FPCGPointDeltaBase* PointDelta = DeltaStruct.GetPtr<const FPCGPointDeltaBase>())
	{
		return PointDelta->OriginalTransform;
	}

	return FTransform::Identity;
}

void FPCGPointTransformViewportExtensionBase::HandleDeletePressed()
{
	FPCGDeltaCollection* Collection = PCG::DeltaViewportExtension::Helpers::GetMutableCollection(CachedContext);
	if (!Collection)
	{
		return;
	}

	if (CachedContext.bSelectionActive && CachedContext.SelectionDeltaKey.Hash != 0)
	{
		if (!CachedContext.NodeConfiguration)
		{
			return;
		}

		UPCGComponent* PCGComponent = CachedContext.ActivePCGComponent.Get();
		FPCGSourceDataContainer* DataContainer = PCGComponent ? PCGComponent->GetExecutionState().GetSourceDataContainer() : nullptr;
		if (!DataContainer)
		{
			return;
		}

		FScopedTransaction Transaction(FText::Format(LOCTEXT("ConvertToDeletion", "PCG Manual Edit: Convert {0} to Deletion"), GetDisplayName()));
		PCGComponent->Modify();

		TInstancedStruct<FPCGDeltaBase>* ExistingDelta = Collection->Find(CachedContext.SelectionDeltaKey);

		FTransform OriginalTransform = CachedContext.SelectionTransform;
		if (ExistingDelta)
		{
			const FConstStructView DeltaView(ExistingDelta->GetScriptStruct(), ExistingDelta->GetMemory());
			OriginalTransform = GetOriginalTransform(DeltaView);
			Collection->Remove(CachedContext.SelectionDeltaKey);
		}

		FPCGPointDeletionDelta DeletionDelta;
		DeletionDelta.OriginalTransform = OriginalTransform;
		DeletionDelta.Bounds = PCGComponent->GetTotalBounds();
		DeletionDelta.ElementIndex = CachedContext.OriginalElementIndex;

		const FPCGDeltaKey DeletionKey = PCG::DataOverride::Keys::FPCGCompositeTransformDeltaKey(
			OriginalTransform,
			CachedContext.NodeConfiguration->SignatureTolerance,
			CachedContext.NodeConfiguration->bSignaturePosition,
			CachedContext.NodeConfiguration->bSignatureRotation,
			CachedContext.NodeConfiguration->bSignatureScale,
			FPCGPointDeletionDelta::GetDeltaNameStatic());

		Collection->Add(DeletionKey, TInstancedStruct<FPCGPointDeletionDelta>::Make(MoveTemp(DeletionDelta)));

		PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);

		if (CachedCallbacks.ClearSelection)
		{
			CachedCallbacks.ClearSelection();
		}
	}

	RefreshLists(CachedContext);
}

TSharedRef<ITableRow> FPCGPointTransformViewportExtensionBase::OnGenerateRow(TSharedPtr<FPCGPointTransformDeltaEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable)
{
	TWeakPtr<FPCGPointTransformDeltaEntry> WeakEntry = Entry;
	FPCGDeltaKey EntryKey = Entry->Key;

	return SNew(STableRow<TSharedPtr<FPCGPointTransformDeltaEntry>>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 1.0f, 2.0f, 1.0f)
			[
				SNew(STextBlock)
				.Text(FText::AsNumber(Entry->OriginalElementIndex))
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(2.0f, 1.0f)
			[
				SNew(SVectorSlider<double>, /*bIsRotator=*/false, /*InProperty=*/nullptr)
				.VisibleText_0_Lambda([WeakEntry]() -> FString
				{
					TSharedPtr<FPCGPointTransformDeltaEntry> Pinned = WeakEntry.Pin();
					return Pinned ? FString::SanitizeFloat(Pinned->DisplayVector.X) : FString();
				})
				.VisibleText_1_Lambda([WeakEntry]() -> FString
				{
					TSharedPtr<FPCGPointTransformDeltaEntry> Pinned = WeakEntry.Pin();
					return Pinned ? FString::SanitizeFloat(Pinned->DisplayVector.Y) : FString();
				})
				.VisibleText_2_Lambda([WeakEntry]() -> FString
				{
					TSharedPtr<FPCGPointTransformDeltaEntry> Pinned = WeakEntry.Pin();
					return Pinned ? FString::SanitizeFloat(Pinned->DisplayVector.Z) : FString();
				})
				.OnNumericCommitted_Box_0(SVectorSlider<double>::FOnNumericValueCommitted::CreateLambda(
					[this, EntryKey](const double V, ETextCommit::Type) { OnComponentCommitted(EntryKey, 0, V); }))
				.OnNumericCommitted_Box_1(SVectorSlider<double>::FOnNumericValueCommitted::CreateLambda(
					[this, EntryKey](const double V, ETextCommit::Type) { OnComponentCommitted(EntryKey, 1, V); }))
				.OnNumericCommitted_Box_2(SVectorSlider<double>::FOnNumericValueCommitted::CreateLambda(
					[this, EntryKey](const double V, ETextCommit::Type) { OnComponentCommitted(EntryKey, 2, V); }))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(FMargin(1, 0))
				.ToolTipText(LOCTEXT("SelectTooltip", "Select this point"))
				.OnClicked_Lambda([this, EntryKey]() -> FReply
				{
					if (CachedCallbacks.SelectElement)
					{
						CachedCallbacks.SelectElement(EntryKey, INDEX_NONE);
					}
					return FReply::Handled();
				})
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Search"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(FMargin(1, 0))
				.ToolTipText(LOCTEXT("RemoveTooltip", "Remove this override"))
				.OnClicked_Lambda([this, EntryKey]() -> FReply
				{
					RemoveEntry(EntryKey);
					RefreshLists(CachedContext);
					return FReply::Handled();
				})
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
}

void FPCGPointTransformViewportExtensionBase::OnComponentCommitted(const FPCGDeltaKey& EntryKey, const int32 AxisIndex, const double NewValue)
{
	for (const TSharedPtr<FPCGPointTransformDeltaEntry>& Entry : Entries)
	{
		if (Entry && Entry->Key == EntryKey)
		{
			Entry->DisplayVector[AxisIndex] = NewValue;
			UpdateDisplayVector(EntryKey, Entry->DisplayVector);
			return;
		}
	}
}

void FPCGPointTransformViewportExtensionBase::UpdateDisplayVector(const FPCGDeltaKey& DeltaKey, const FVector& NewValue)
{
	FPCGDeltaCollection* Collection = PCG::DeltaViewportExtension::Helpers::GetMutableCollection(CachedContext);
	if (!Collection)
	{
		return;
	}

	TInstancedStruct<FPCGDeltaBase>* DeltaStruct = Collection->Find(DeltaKey);
	if (!DeltaStruct)
	{
		return;
	}

	UPCGComponent* PCGComponent = CachedContext.ActivePCGComponent.Get();
	FPCGSourceDataContainer* DataContainer = PCGComponent ? PCGComponent->GetExecutionState().GetSourceDataContainer() : nullptr;
	if (!DataContainer)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("UpdateDisplayVector", "PCG Manual Edit: Update {0} Value"), GetDisplayName()));
	PCGComponent->Modify();

	SetDisplayVector(*DeltaStruct, NewValue);

	PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);

	if (CachedCallbacks.SelectElement)
	{
		CachedCallbacks.SelectElement(DeltaKey, INDEX_NONE);
	}
}

void FPCGPointTransformViewportExtensionBase::RemoveEntry(const FPCGDeltaKey& EntryKey)
{
	FPCGDeltaCollection* Collection = PCG::DeltaViewportExtension::Helpers::GetMutableCollection(CachedContext);
	if (!Collection)
	{
		return;
	}

	UPCGComponent* PCGComponent = CachedContext.ActivePCGComponent.Get();
	FPCGSourceDataContainer* DataContainer = PCGComponent ? PCGComponent->GetExecutionState().GetSourceDataContainer() : nullptr;
	if (!DataContainer)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("RemoveEntry", "PCG Manual Edit: Remove {0}"), GetDisplayName()));
	PCGComponent->Modify();

	Collection->Remove(EntryKey);
	PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);
}

void FPCGPointTransformViewportExtensionBase::RemoveAllEntries()
{
	FPCGDeltaCollection* Collection = PCG::DeltaViewportExtension::Helpers::GetMutableCollection(CachedContext);
	if (!Collection)
	{
		return;
	}

	UPCGComponent* PCGComponent = CachedContext.ActivePCGComponent.Get();
	FPCGSourceDataContainer* DataContainer = PCGComponent ? PCGComponent->GetExecutionState().GetSourceDataContainer() : nullptr;
	if (!DataContainer)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("RemoveAllEntries", "PCG Manual Edit: Remove All {0}"), GetListHeaderLabel()));
	PCGComponent->Modify();

	TArray<FPCGDeltaKey> Keys;
	Collection->ForEachDelta([this, &Keys](const FPCGDeltaKey& DeltaKey, const TInstancedStruct<FPCGDeltaBase>& DeltaValue) -> bool
	{
		if (IsManagedDelta(DeltaValue))
		{
			Keys.Add(DeltaKey);
		}
		return true;
	});

	for (const FPCGDeltaKey& Key : Keys)
	{
		Collection->Remove(Key);
	}

	PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);
}

void FPCGPointTransformViewportExtensionBase::UpdateFlags()
{
	FPCGDeltaCollection* Collection = PCG::DeltaViewportExtension::Helpers::GetMutableCollection(CachedContext);
	if (!Collection)
	{
		return;
	}

	const bool bPosition = bOverridePosition;
	const bool bRotation = bOverrideRotation;
	const bool bScale = bOverrideScale;

	bool bModified = false;
	Collection->ForEachDelta([this, &bModified, bPosition, bRotation, bScale](const FPCGDeltaKey& DeltaKey, TInstancedStruct<FPCGDeltaBase>& DeltaValue) -> bool
	{
		if (IsManagedDelta(DeltaValue) && !DeltaOverrideFlagsMatch(DeltaValue, bPosition, bRotation, bScale))
		{
			SetDeltaOverrideFlags(DeltaValue, bPosition, bRotation, bScale);
			bModified = true;
		}
		return true;
	});

	if (bModified)
	{
		UPCGComponent* PCGComponent = CachedContext.ActivePCGComponent.Get();
		FPCGSourceDataContainer* DataContainer = PCGComponent ? PCGComponent->GetExecutionState().GetSourceDataContainer() : nullptr;
		if (DataContainer)
		{
			FScopedTransaction Transaction(FText::Format(LOCTEXT("UpdateFlags", "PCG Manual Edit: Update {0} Overrides"), GetDisplayName()));
			PCGComponent->Modify();
			PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);
		}
	}
}

#undef LOCTEXT_NAMESPACE
