// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeltaViewportExtensions/PCGDeltaViewportExtension.h"

#include "Graph/DataOverride/PCGDataOverridePoints.h"

class ITableRow;
class STableViewBase;
class SVerticalBox;
template <typename ItemType> class SListView;

/**
 * Shared list entry for point-transform viewport extensions. Stores a display vector that might represents an absolute
 * position (ex. Point Transform), a local offset (ex. Point Transform Offset), etc. */
struct FPCGPointTransformDeltaEntry
{
	FPCGDeltaKey Key;
	int32 OriginalElementIndex = INDEX_NONE;
	FVector DisplayVector = FVector::ZeroVector;
};

/** Shared base class for viewport extensions that edit a point's transform via a TRS gizmo and display a vector in a list view. */
class FPCGPointTransformViewportExtensionBase : public IPCGDeltaViewportExtension
{
public:
	// ~Begin IPCGDeltaViewportExtension interface
	virtual TSharedRef<SWidget> CreateWidget(FPCGDeltaViewportCallbacks Callbacks) override;
	virtual void UpdateContext(const FPCGDeltaViewportContext& Context) override { CachedContext = Context; }
	virtual void RefreshLists(const FPCGDeltaViewportContext& Context) override;
	virtual TArray<FPCGDeltaKeyBinding> GetKeyBindings() override { return {{EKeys::Delete, [this]() { HandleDeletePressed(); }}}; }
	virtual bool MatchesSourceElement(const FTransform& SourceElementTransform, const FConstStructView DeltaStruct, const double SpatialTolerance) const override;
	virtual FTransform GetOriginalTransform(const FConstStructView DeltaStruct) const override;
	// ~End IPCGDeltaViewportExtension interface

protected:
	/** Returns the label for the checkbox group row. */
	virtual FText GetCheckboxGroupLabel() const = 0;

	/** Returns the header label for the list section. */
	virtual FText GetListHeaderLabel() const = 0;

	/** Returns the display vector from the given delta. */
	virtual FVector GetDisplayVector(const TInstancedStruct<FPCGDeltaBase>& Delta) const = 0;

	/** Sets the display vector on the given delta. */
	virtual void SetDisplayVector(TInstancedStruct<FPCGDeltaBase>& Delta, const FVector& NewValue) const = 0;

	/** Returns true if the instanced struct is the delta type managed by this extension. */
	virtual bool IsManagedDelta(const TInstancedStruct<FPCGDeltaBase>& Delta) const = 0;

	/** Sets the override flags on the given delta. */
	virtual void SetDeltaOverrideFlags(TInstancedStruct<FPCGDeltaBase>& Delta, bool bPosition, bool bRotation, bool bScale) const = 0;

	/** Returns true if the delta's override flags match the provided values. */
	virtual bool DeltaOverrideFlagsMatch(const TInstancedStruct<FPCGDeltaBase>& Delta, bool bPosition, bool bRotation, bool bScale) const = 0;

	/** Returns the editor-only element index from the given delta. */
	virtual int32 GetElementIndex(const TInstancedStruct<FPCGDeltaBase>& Delta) const = 0;

	bool bOverridePosition = true;
	bool bOverrideRotation = true;
	bool bOverrideScale = true;

	FPCGDeltaViewportContext CachedContext;
	FPCGDeltaViewportCallbacks CachedCallbacks;

	TArray<TSharedPtr<FPCGPointTransformDeltaEntry>> Entries;
	TSharedPtr<SVerticalBox> ListSection;
	TSharedPtr<SListView<TSharedPtr<FPCGPointTransformDeltaEntry>>> ListView;

private:
	void HandleDeletePressed();
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FPCGPointTransformDeltaEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable);
	void OnComponentCommitted(const FPCGDeltaKey& EntryKey, int32 AxisIndex, double NewValue);
	void UpdateDisplayVector(const FPCGDeltaKey& DeltaKey, const FVector& NewValue);
	void RemoveEntry(const FPCGDeltaKey& EntryKey);
	void RemoveAllEntries();
	void UpdateFlags();
};
