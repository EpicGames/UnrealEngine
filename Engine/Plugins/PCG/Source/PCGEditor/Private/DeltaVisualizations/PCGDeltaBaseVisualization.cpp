// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeltaVisualizations/PCGDeltaBaseVisualization.h"

#include "DeltaVisualizations/PCGDeltaVisualizationHelpers.h"

TArray<FPCGDeltaVisualizerColumnInfo> FPCGDeltaBaseVisualization::GetColumnInfos() const
{
	TArray<FPCGDeltaVisualizerColumnInfo> Columns;
	PCGDeltaVisualization::Helpers::AppendBaseDeltaColumns(Columns);
	return Columns;
}

FText FPCGDeltaBaseVisualization::GetCellText(const FName ColumnId, const FPCGDeltaKey& DeltaKey, const FConstStructView Delta) const
{
	return PCGDeltaVisualization::Helpers::GetBaseDeltaCellText(ColumnId, DeltaKey, Delta);
}
