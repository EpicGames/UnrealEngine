// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeltaVisualizations/PCGDeltaVisualization.h"

/** Default visualizer registered to FPCGDeltaBase. Shows common delta metadata columns. */
class FPCGDeltaBaseVisualization : public IPCGDeltaVisualization
{
public:
	virtual TArray<FPCGDeltaVisualizerColumnInfo> GetColumnInfos() const override;
	virtual FText GetCellText(FName ColumnId, const FPCGDeltaKey& DeltaKey, FConstStructView Delta) const override;
};
