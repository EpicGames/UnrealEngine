// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeltaVisualizations/PCGDeltaPointVisualization.h"

#include "DeltaVisualizations/PCGDeltaVisualizationHelpers.h"
#include "Graph/DataOverride/PCGDataOverride.h"
#include "Graph/DataOverride/PCGDataOverridePoints.h"

#define LOCTEXT_NAMESPACE "PCGDeltaVisualization"

namespace PCG::DeltaPoints::Constants
{
	static constexpr float TransformComponentColumnWidth = 120.f;
	static constexpr EPCGTableVisualizerCellAlignment TransformComponentColumnAlignment = EPCGTableVisualizerCellAlignment::Right;
} // namespace PCG::DeltaPoints::Constants

// SHeaderRow column identifiers for point delta visualizers.
namespace PCG::DeltaVisualization::Columns
{
	static const FName OriginalIndexId = TEXT("OriginalIndex");

	// Per-component transform columns — Previous (OriginalTransform)
	static const FName PreviousPosX = TEXT("Previous.Position.X");
	static const FName PreviousPosY = TEXT("Previous.Position.Y");
	static const FName PreviousPosZ = TEXT("Previous.Position.Z");
	static const FName PreviousRotRoll = TEXT("Previous.Rotation.Roll");
	static const FName PreviousRotPitch = TEXT("Previous.Rotation.Pitch");
	static const FName PreviousRotYaw = TEXT("Previous.Rotation.Yaw");
	static const FName PreviousScaleX = TEXT("Previous.Scale.X");
	static const FName PreviousScaleY = TEXT("Previous.Scale.Y");
	static const FName PreviousScaleZ = TEXT("Previous.Scale.Z");

	// Per-component transform columns — Override (TransformOverride)
	static const FName OverridePosX = TEXT("Override.Position.X");
	static const FName OverridePosY = TEXT("Override.Position.Y");
	static const FName OverridePosZ = TEXT("Override.Position.Z");
	static const FName OverrideRotRoll = TEXT("Override.Rotation.Roll");
	static const FName OverrideRotPitch = TEXT("Override.Rotation.Pitch");
	static const FName OverrideRotYaw = TEXT("Override.Rotation.Yaw");
	static const FName OverrideScaleX = TEXT("Override.Scale.X");
	static const FName OverrideScaleY = TEXT("Override.Scale.Y");
	static const FName OverrideScaleZ = TEXT("Override.Scale.Z");

	// Per-component transform columns — Offset (TransformOffset)
	static const FName OffsetPosX = TEXT("Offset.Position.X");
	static const FName OffsetPosY = TEXT("Offset.Position.Y");
	static const FName OffsetPosZ = TEXT("Offset.Position.Z");
	static const FName OffsetRotRoll = TEXT("Offset.Rotation.Roll");
	static const FName OffsetRotPitch = TEXT("Offset.Rotation.Pitch");
	static const FName OffsetRotYaw = TEXT("Offset.Rotation.Yaw");
	static const FName OffsetScaleX = TEXT("Offset.Scale.X");
	static const FName OffsetScaleY = TEXT("Offset.Scale.Y");
	static const FName OffsetScaleZ = TEXT("Offset.Scale.Z");

	// Point insertion delta column
	static const FName InsertedPointsId = TEXT("InsertedPoints");

	// Spacer (visual end-cap padding)
	static const FName SpacerId = TEXT("Spacer");
} // namespace PCG::DeltaVisualization::Columns

namespace PCG::DeltaPointVisualization::Helpers
{
	/** Bundles the 9 column IDs for one decomposed FTransform. */
	struct FTransformColumnIds
	{
		FName PosX, PosY, PosZ;
		FName RotRoll, RotPitch, RotYaw;
		FName ScaleX, ScaleY, ScaleZ;
	};

	static const FTransformColumnIds PreviousTransformColumnIds =
	{
		DeltaVisualization::Columns::PreviousPosX, DeltaVisualization::Columns::PreviousPosY, DeltaVisualization::Columns::PreviousPosZ,
		DeltaVisualization::Columns::PreviousRotRoll, DeltaVisualization::Columns::PreviousRotPitch, DeltaVisualization::Columns::PreviousRotYaw,
		DeltaVisualization::Columns::PreviousScaleX, DeltaVisualization::Columns::PreviousScaleY, DeltaVisualization::Columns::PreviousScaleZ
	};

	static const FTransformColumnIds OverrideTransformColumnIds =
	{
		DeltaVisualization::Columns::OverridePosX, DeltaVisualization::Columns::OverridePosY, DeltaVisualization::Columns::OverridePosZ,
		DeltaVisualization::Columns::OverrideRotRoll, DeltaVisualization::Columns::OverrideRotPitch, DeltaVisualization::Columns::OverrideRotYaw,
		DeltaVisualization::Columns::OverrideScaleX, DeltaVisualization::Columns::OverrideScaleY, DeltaVisualization::Columns::OverrideScaleZ
	};

	static const FTransformColumnIds OffsetTransformColumnIds =
	{
		DeltaVisualization::Columns::OffsetPosX, DeltaVisualization::Columns::OffsetPosY, DeltaVisualization::Columns::OffsetPosZ,
		DeltaVisualization::Columns::OffsetRotRoll, DeltaVisualization::Columns::OffsetRotPitch, DeltaVisualization::Columns::OffsetRotYaw,
		DeltaVisualization::Columns::OffsetScaleX, DeltaVisualization::Columns::OffsetScaleY, DeltaVisualization::Columns::OffsetScaleZ
	};

	/** Appends 9 per-component transform columns (Position X/Y/Z, Rotation Roll/Pitch/Yaw, Scale X/Y/Z). */
	static void AppendTransformColumns(TArray<FPCGDeltaVisualizerColumnInfo>& OutColumns, const FText& GroupLabel, const FText& TooltipBase, const FTransformColumnIds& Ids)
	{
		using namespace PCG::DeltaPoints::Constants;

		auto CreateLabel = [&GroupLabel](const FText& Component) { return FText::Format(INVTEXT("{0}.{1}"), GroupLabel, Component); };
		auto CreateToolTip = [&TooltipBase](const FText& Component) { return FText::Format(INVTEXT("{0} - {1}"), TooltipBase, Component); };

		const FText PosX = INVTEXT("$Position.X");
		const FText PosY = INVTEXT("$Position.Y");
		const FText PosZ = INVTEXT("$Position.Z");
		const FText RotRoll = INVTEXT("$Rotation.Roll");
		const FText RotPitch = INVTEXT("$Rotation.Pitch");
		const FText RotYaw = INVTEXT("$Rotation.Yaw");
		const FText ScaleX = INVTEXT("$Scale.X");
		const FText ScaleY = INVTEXT("$Scale.Y");
		const FText ScaleZ = INVTEXT("$Scale.Z");

		OutColumns.Add({Ids.PosX, CreateLabel(PosX), CreateToolTip(PosX), TransformComponentColumnWidth, TransformComponentColumnAlignment});
		OutColumns.Add({Ids.PosY, CreateLabel(PosY), CreateToolTip(PosY), TransformComponentColumnWidth, TransformComponentColumnAlignment});
		OutColumns.Add({Ids.PosZ, CreateLabel(PosZ), CreateToolTip(PosZ), TransformComponentColumnWidth, TransformComponentColumnAlignment});
		OutColumns.Add({Ids.RotRoll, CreateLabel(RotRoll), CreateToolTip(RotRoll), TransformComponentColumnWidth, TransformComponentColumnAlignment});
		OutColumns.Add({Ids.RotPitch, CreateLabel(RotPitch), CreateToolTip(RotPitch), TransformComponentColumnWidth, TransformComponentColumnAlignment});
		OutColumns.Add({Ids.RotYaw, CreateLabel(RotYaw), CreateToolTip(RotYaw), TransformComponentColumnWidth, TransformComponentColumnAlignment});
		OutColumns.Add({Ids.ScaleX, CreateLabel(ScaleX), CreateToolTip(ScaleX), TransformComponentColumnWidth, TransformComponentColumnAlignment});
		OutColumns.Add({Ids.ScaleY, CreateLabel(ScaleY), CreateToolTip(ScaleY), TransformComponentColumnWidth, TransformComponentColumnAlignment});
		OutColumns.Add({Ids.ScaleZ, CreateLabel(ScaleZ), CreateToolTip(ScaleZ), TransformComponentColumnWidth, TransformComponentColumnAlignment});
	}

	/** Returns cell text for a per-component transform column, or empty text if the column is not handled. */
	static FText GetTransformCellText(const FName ColumnId, const FTransform& Transform, const FTransformColumnIds& Ids)
	{
		const FVector Translation = Transform.GetTranslation();
		if (ColumnId == Ids.PosX) { return FText::AsNumber(Translation.X); }
		if (ColumnId == Ids.PosY) { return FText::AsNumber(Translation.Y); }
		if (ColumnId == Ids.PosZ) { return FText::AsNumber(Translation.Z); }

		const FRotator Rotation = Transform.Rotator();
		if (ColumnId == Ids.RotRoll) { return FText::AsNumber(Rotation.Roll); }
		if (ColumnId == Ids.RotPitch) { return FText::AsNumber(Rotation.Pitch); }
		if (ColumnId == Ids.RotYaw) { return FText::AsNumber(Rotation.Yaw); }

		const FVector Scale = Transform.GetScale3D();
		if (ColumnId == Ids.ScaleX) { return FText::AsNumber(Scale.X); }
		if (ColumnId == Ids.ScaleY) { return FText::AsNumber(Scale.Y); }
		if (ColumnId == Ids.ScaleZ) { return FText::AsNumber(Scale.Z); }

		return FText::GetEmpty();
	}

	/** Appends columns common to point-based deltas (Original Index, Previous Transform components). */
	static void AppendPointDeltaBaseColumns(TArray<FPCGDeltaVisualizerColumnInfo>& OutColumns)
	{
		using namespace PCG::DeltaVisualization::Columns;

		OutColumns.Add({OriginalIndexId, LOCTEXT("OriginalIndexLabel", "Original Index"), LOCTEXT("OriginalIndexTooltip", "Original element index before overrides"), 95.f, EPCGTableVisualizerCellAlignment::Center});
		AppendTransformColumns(OutColumns, LOCTEXT("PreviousGroupLabel_Base", "Previous"), LOCTEXT("PreviousTransformTooltip_Base", "The original transform of the point"), PreviousTransformColumnIds);
	}

	/** Returns cell text for point delta base columns. Returns empty text if the column is not handled. */
	static FText GetPointDeltaBaseCellText(const FName ColumnId, const FConstStructView Delta)
	{
		using namespace PCG::DeltaVisualization::Columns;

		if (ColumnId == OriginalIndexId)
		{
			// ElementIndex is on FPCGPointTransformDelta, FPCGPointTransformOffsetDelta, and FPCGPointDeletionDelta but not on FPCGPointDeltaBase itself.
			// Check each concrete type. @todo_pcg: Evaluate abstracting the index out for all deltas?
			if (const FPCGPointTransformDelta* TransformDelta = Delta.GetPtr<const FPCGPointTransformDelta>())
			{
				return FText::AsNumber(TransformDelta->ElementIndex);
			}
			else if (const FPCGPointTransformOffsetDelta* OffsetDelta = Delta.GetPtr<const FPCGPointTransformOffsetDelta>())
			{
				return FText::AsNumber(OffsetDelta->ElementIndex);
			}
			else if (const FPCGPointDeletionDelta* DeletionDelta = Delta.GetPtr<const FPCGPointDeletionDelta>())
			{
				return FText::AsNumber(DeletionDelta->ElementIndex);
			}

			return FText::GetEmpty();
		}

		if (const FPCGPointDeltaBase* PointDelta = Delta.GetPtr<const FPCGPointDeltaBase>())
		{
			FText TransformText = GetTransformCellText(ColumnId, PointDelta->OriginalTransform, PreviousTransformColumnIds);
			if (!TransformText.IsEmpty())
			{
				return TransformText;
			}
		}

		return FText::GetEmpty();
	}
} // namespace PCG::DeltaPointVisualization::Helpers

// --- FPCGPointTransformDeltaVisualization ---

TArray<FPCGDeltaVisualizerColumnInfo> FPCGPointTransformDeltaVisualization::GetColumnInfos() const
{
	using namespace PCG::DeltaVisualization::Columns;
	using namespace PCG::DeltaPointVisualization::Helpers;

	TArray<FPCGDeltaVisualizerColumnInfo> Columns;

	// Build columns directly in desired display order rather than using shared append helpers.
	Columns.Add({OriginalIndexId, LOCTEXT("OriginalIndexLabel", "Original Index"), LOCTEXT("OriginalIndexTooltip", "Original element index before overrides"), 85.f, EPCGTableVisualizerCellAlignment::Center});
	Columns.Add({PCGDeltaVisualization::Constants::DeltaTypeId, LOCTEXT("TransformDeltaTypeLabel", "Delta Type"), LOCTEXT("TransformDeltaTypeTooltip", "Name of the delta type"), 100.f, EPCGTableVisualizerCellAlignment::Left});
	PCGDeltaVisualization::Helpers::AppendSignatureColumn(Columns, LOCTEXT("TransformSignatureTooltip", "Which components are overridden (P=Position, R=Rotation, S=Scale)"));
	AppendTransformColumns(Columns, LOCTEXT("PreviousGroupLabel_Transform", "Previous"), LOCTEXT("PreviousTransformTooltip_Transform", "The original transform of the point"), PreviousTransformColumnIds);
	AppendTransformColumns(Columns, LOCTEXT("OverrideGroupLabel_Transform", "Override"), LOCTEXT("OverrideTransformTooltip_Transform", "The overridden transform value"), OverrideTransformColumnIds);
	Columns.Add({SpacerId, FText::GetEmpty(), FText::GetEmpty(), 16.f, EPCGTableVisualizerCellAlignment::Left});

	return Columns;
}

// Returns the text for any column this visualizer supports, or empty if unhandled.
FText FPCGPointTransformDeltaVisualization::GetCellText(const FName ColumnId, const FPCGDeltaKey& DeltaKey, const FConstStructView Delta) const
{
	using namespace PCG::DeltaVisualization::Columns;
	using namespace PCG::DeltaPointVisualization::Helpers;

	FText BaseText = PCGDeltaVisualization::Helpers::GetBaseDeltaCellText(ColumnId, DeltaKey, Delta);
	if (!BaseText.IsEmpty())
	{
		return BaseText;
	}

	FText PointBaseText = GetPointDeltaBaseCellText(ColumnId, Delta);
	if (!PointBaseText.IsEmpty())
	{
		return PointBaseText;
	}

	const FPCGPointTransformDelta* TransformDelta = Delta.GetPtr<const FPCGPointTransformDelta>();
	if (!TransformDelta)
	{
		return FText::GetEmpty();
	}

	FText OverrideTransformText = GetTransformCellText(ColumnId, TransformDelta->TransformOverride, OverrideTransformColumnIds);
	if (!OverrideTransformText.IsEmpty())
	{
		return OverrideTransformText;
	}

	// Compact signature string (P=Position, R=Rotation, S=Scale)
	if (ColumnId == PCGDeltaVisualization::Constants::SignatureId)
	{
		FString Flags;
		if (TransformDelta->bOverridePosition) { Flags += TEXT("P"); }
		if (TransformDelta->bOverrideRotation) { Flags += TEXT("R"); }
		if (TransformDelta->bOverrideScale) { Flags += TEXT("S"); }
		return FText::FromString(Flags);
	}

	return FText::GetEmpty();
}

// --- FPCGPointDeletionDeltaVisualization ---

TArray<FPCGDeltaVisualizerColumnInfo> FPCGPointDeletionDeltaVisualization::GetColumnInfos() const
{
	using namespace PCG::DeltaPointVisualization::Helpers;

	TArray<FPCGDeltaVisualizerColumnInfo> Columns;
	PCGDeltaVisualization::Helpers::AppendBaseDeltaColumns(Columns);
	AppendPointDeltaBaseColumns(Columns);
	return Columns;
}

FText FPCGPointDeletionDeltaVisualization::GetCellText(const FName ColumnId, const FPCGDeltaKey& DeltaKey, const FConstStructView Delta) const
{
	using namespace PCG::DeltaPointVisualization::Helpers;

	FText BaseText = PCGDeltaVisualization::Helpers::GetBaseDeltaCellText(ColumnId, DeltaKey, Delta);
	if (!BaseText.IsEmpty())
	{
		return BaseText;
	}

	return GetPointDeltaBaseCellText(ColumnId, Delta);
}

// --- FPCGPointInsertionDeltaVisualization ---

TArray<FPCGDeltaVisualizerColumnInfo> FPCGPointInsertionDeltaVisualization::GetColumnInfos() const
{
	using namespace PCG::DeltaVisualization::Columns;
	using namespace PCG::DeltaPointVisualization::Helpers;

	TArray<FPCGDeltaVisualizerColumnInfo> Columns;

	Columns.Add({PCGDeltaVisualization::Constants::DeltaTypeId, LOCTEXT("InsertDeltaTypeLabel", "Delta Type"), LOCTEXT("InsertDeltaTypeTooltip", "Name of the delta type"), 100.f, EPCGTableVisualizerCellAlignment::Left});
	PCGDeltaVisualization::Helpers::AppendSignatureColumn(Columns, LOCTEXT("InsertionSignatureTooltip", "Identifies this delta as a point insertion"));
	Columns.Add({InsertedPointsId, LOCTEXT("InsertedPointsLabel", "Inserted Points"), LOCTEXT("InsertedPointsTooltip", "Number of points to be inserted"), 105.f, EPCGTableVisualizerCellAlignment::Center});
	AppendTransformColumns(Columns, LOCTEXT("OverrideGroupLabel_Insert", "Override"), LOCTEXT("OverrideTransformTooltip_Insert", "The transform of the inserted point"), OverrideTransformColumnIds);
	Columns.Add({SpacerId, FText::GetEmpty(), FText::GetEmpty(), 16.f, EPCGTableVisualizerCellAlignment::Left});

	return Columns;
}

FText FPCGPointInsertionDeltaVisualization::GetCellText(const FName ColumnId, const FPCGDeltaKey& DeltaKey, const FConstStructView Delta) const
{
	using namespace PCG::DeltaVisualization::Columns;
	using namespace PCG::DeltaPointVisualization::Helpers;

	FText BaseText = PCGDeltaVisualization::Helpers::GetBaseDeltaCellText(ColumnId, DeltaKey, Delta);
	if (!BaseText.IsEmpty())
	{
		return BaseText;
	}

	const FPCGPointInsertionDelta* InsertionDelta = Delta.GetPtr<const FPCGPointInsertionDelta>();
	if (!InsertionDelta)
	{
		return FText::GetEmpty();
	}

	if (ColumnId == PCGDeltaVisualization::Constants::SignatureId)
	{
		return LOCTEXT("InsertionSignatureValue", "PointInsert");
	}

	if (ColumnId == InsertedPointsId)
	{
		return FText::AsNumber(InsertionDelta->InsertedPoints.Num());
	}

	if (!InsertionDelta->InsertedPoints.IsEmpty())
	{
		FText OverrideTransformText = GetTransformCellText(ColumnId, InsertionDelta->InsertedPoints[0].Transform, OverrideTransformColumnIds);
		if (!OverrideTransformText.IsEmpty())
		{
			return OverrideTransformText;
		}
	}

	return FText::GetEmpty();
}

TArray<FPCGDeltaVisualizerColumnInfo> FPCGPointTransformOffsetDeltaVisualization::GetColumnInfos() const
{
	using namespace PCG::DeltaVisualization::Columns;
	using namespace PCG::DeltaPointVisualization::Helpers;

	TArray<FPCGDeltaVisualizerColumnInfo> Columns;

	Columns.Add({OriginalIndexId, LOCTEXT("OriginalIndexLabel", "Original Index"), LOCTEXT("OriginalIndexTooltip", "Original element index before overrides"), 85.f, EPCGTableVisualizerCellAlignment::Center});
	Columns.Add({PCGDeltaVisualization::Constants::DeltaTypeId, LOCTEXT("TransformOffsetDeltaTypeLabel", "Delta Type"), LOCTEXT("TransformOffsetDeltaTypeTooltip", "Name of the delta type"), 100.f, EPCGTableVisualizerCellAlignment::Left});
	PCGDeltaVisualization::Helpers::AppendSignatureColumn(Columns, LOCTEXT("OffsetSignatureTooltip", "Which components are offset (P=Position, R=Rotation, S=Scale)"));
	AppendTransformColumns(Columns, LOCTEXT("PreviousGroupLabel_Offset", "Previous"), LOCTEXT("PreviousTransformTooltip_Offset", "The original transform of the point"), PreviousTransformColumnIds);
	AppendTransformColumns(Columns, LOCTEXT("OffsetGroupLabel_Offset", "Offset"), LOCTEXT("OffsetTransformTooltip_Offset", "The local offset applied to the transform"), OffsetTransformColumnIds);
	Columns.Add({SpacerId, FText::GetEmpty(), FText::GetEmpty(), 16.f, EPCGTableVisualizerCellAlignment::Left});

	return Columns;
}

FText FPCGPointTransformOffsetDeltaVisualization::GetCellText(const FName ColumnId, const FPCGDeltaKey& DeltaKey, const FConstStructView Delta) const
{
	using namespace PCG::DeltaVisualization::Columns;
	using namespace PCG::DeltaPointVisualization::Helpers;

	FText BaseText = PCGDeltaVisualization::Helpers::GetBaseDeltaCellText(ColumnId, DeltaKey, Delta);
	if (!BaseText.IsEmpty())
	{
		return BaseText;
	}

	FText PointBaseText = GetPointDeltaBaseCellText(ColumnId, Delta);
	if (!PointBaseText.IsEmpty())
	{
		return PointBaseText;
	}

	const FPCGPointTransformOffsetDelta* OffsetDelta = Delta.GetPtr<const FPCGPointTransformOffsetDelta>();
	if (!OffsetDelta)
	{
		return FText::GetEmpty();
	}

	FText OffsetTransformText = GetTransformCellText(ColumnId, OffsetDelta->TransformOffset, OffsetTransformColumnIds);
	if (!OffsetTransformText.IsEmpty())
	{
		return OffsetTransformText;
	}

	if (ColumnId == PCGDeltaVisualization::Constants::SignatureId)
	{
		FString Flags;
		if (OffsetDelta->bOffsetPosition) { Flags += TEXT("P"); }
		if (OffsetDelta->bOffsetRotation) { Flags += TEXT("R"); }
		if (OffsetDelta->bOffsetScale) { Flags += TEXT("S"); }
		return FText::FromString(Flags);
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
