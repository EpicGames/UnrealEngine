// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeltaVisualizations/PCGDeltaVisualization.h"

/** Visualizer for FPCGPointTransformDelta. Extends base columns with transform override info. */
class FPCGPointTransformDeltaVisualization : public IPCGDeltaVisualization
{
public:
	virtual TArray<FPCGDeltaVisualizerColumnInfo> GetColumnInfos() const override;
	virtual FText GetCellText(FName ColumnId, const FPCGDeltaKey& DeltaKey, FConstStructView Delta) const override;
};

/** Visualizer for FPCGPointTransformOffsetDelta. Extends base columns with offset transform info. */
class FPCGPointTransformOffsetDeltaVisualization : public IPCGDeltaVisualization
{
public:
	virtual TArray<FPCGDeltaVisualizerColumnInfo> GetColumnInfos() const override;
	virtual FText GetCellText(FName ColumnId, const FPCGDeltaKey& DeltaKey, FConstStructView Delta) const override;
};

/** Visualizer for FPCGPointDeletionDelta. Extends base columns with original transform. */
class FPCGPointDeletionDeltaVisualization : public IPCGDeltaVisualization
{
public:
	virtual TArray<FPCGDeltaVisualizerColumnInfo> GetColumnInfos() const override;
	virtual FText GetCellText(FName ColumnId, const FPCGDeltaKey& DeltaKey, FConstStructView Delta) const override;
};

/** Visualizer for FPCGPointInsertionDelta. Extends base columns with inserted point count. */
class FPCGPointInsertionDeltaVisualization : public IPCGDeltaVisualization
{
public:
	virtual TArray<FPCGDeltaVisualizerColumnInfo> GetColumnInfos() const override;
	virtual FText GetCellText(FName ColumnId, const FPCGDeltaKey& DeltaKey, FConstStructView Delta) const override;
};
