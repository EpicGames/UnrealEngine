// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeltaVisualizations/PCGDeltaVisualizationHelpers.h"

#include "Graph/DataOverride/PCGDataOverride.h"

#define LOCTEXT_NAMESPACE "PCGDeltaVisualization"

void PCGDeltaVisualization::Helpers::AppendBaseDeltaColumns(TArray<FPCGDeltaVisualizerColumnInfo>& OutColumns)
{
	using namespace PCGDeltaVisualization::Constants;

	OutColumns.Add({DeltaTypeId, LOCTEXT("DeltaTypeLabel", "Delta Type"), LOCTEXT("DeltaTypeTooltip", "Name of the delta type"), 100.f, EPCGTableVisualizerCellAlignment::Left});
}

void PCGDeltaVisualization::Helpers::AppendSignatureColumn(TArray<FPCGDeltaVisualizerColumnInfo>& OutColumns, const FText& Tooltip)
{
	using namespace PCGDeltaVisualization::Constants;

	OutColumns.Add({SignatureId, LOCTEXT("SignatureLabel", "Signature"), Tooltip, 100.f, EPCGTableVisualizerCellAlignment::Center});
}

FText PCGDeltaVisualization::Helpers::GetBaseDeltaCellText(const FName ColumnId, const FPCGDeltaKey& DeltaKey, const FConstStructView Delta)
{
	using namespace PCGDeltaVisualization::Constants;

	if (ColumnId == DeltaTypeId)
	{
		if (const FPCGDeltaBase* DeltaBase = Delta.GetPtr<const FPCGDeltaBase>())
		{
			return FText::FromName(DeltaBase->GetDeltaName());
		}
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
