// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeltaVisualizations/PCGDeltaVisualization.h"

struct FPCGDeltaKey;

namespace PCGDeltaVisualization::Helpers
{
	/** Appends the common base delta columns (Delta Type). */
	void AppendBaseDeltaColumns(TArray<FPCGDeltaVisualizerColumnInfo>& OutColumns);

	/** Appends the shared signature column with the given value. */
	void AppendSignatureColumn(TArray<FPCGDeltaVisualizerColumnInfo>& OutColumns, const FText& Tooltip);

	/** Returns cell text for the common base delta columns. Returns empty text if the column is not handled. */
	FText GetBaseDeltaCellText(FName ColumnId, const FPCGDeltaKey& DeltaKey, FConstStructView Delta);
} // namespace PCGDeltaVisualization::Helpers
