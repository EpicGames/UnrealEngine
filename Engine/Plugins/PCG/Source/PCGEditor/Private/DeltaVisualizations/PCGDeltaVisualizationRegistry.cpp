// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeltaVisualizations/PCGDeltaVisualizationRegistry.h"

#include "Graph/DataOverride/PCGDataOverride.h"

void FPCGDeltaVisualizationRegistry::RegisterDeltaVisualization(const UScriptStruct* DeltaStruct, TUniquePtr<const IPCGDeltaVisualization> Visualization)
{
	if (ensureMsgf(!ExternalRegistry.Find(DeltaStruct), TEXT("Cannot register multiple delta visualizations for the same struct type.")))
	{
		ExternalRegistry.Add(DeltaStruct, MoveTemp(Visualization));
	}
}

void FPCGDeltaVisualizationRegistry::UnregisterDeltaVisualization(const UScriptStruct* DeltaStruct)
{
	ExternalRegistry.Remove(DeltaStruct);
}

const IPCGDeltaVisualization* FPCGDeltaVisualizationRegistry::GetDeltaVisualization(const UScriptStruct* DeltaStruct) const
{
	const UScriptStruct* CurrentStruct = DeltaStruct;
	const UScriptStruct* DeltaBaseStruct = FPCGDeltaBase::StaticStruct();

	// Walk the struct hierarchy from the concrete type up through FPCGDeltaBase (inclusive).
	while (CurrentStruct)
	{
		// Always try external implementations first so users can override built-in visualizations.
		if (const TUniquePtr<const IPCGDeltaVisualization>* ExternalVisualizationPtr = ExternalRegistry.Find(CurrentStruct))
		{
			return ExternalVisualizationPtr->Get();
		}

		if (const TUniquePtr<const IPCGDeltaVisualization>* InternalVisualizationPtr = InternalRegistry.Find(CurrentStruct))
		{
			return InternalVisualizationPtr->Get();
		}

		// Stop after checking FPCGDeltaBase.
		if (CurrentStruct == DeltaBaseStruct)
		{
			break;
		}

		CurrentStruct = Cast<UScriptStruct>(CurrentStruct->GetSuperStruct());
	}

	return nullptr;
}
