// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/EnumSelectorDetails.h"

#include "Elements/ControlFlow/PCGControlFlow.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "EnumSelectorDetails"

namespace FEnumSelectorDetailsConstants
{
	inline static const FText SetSelectorClassTransaction = LOCTEXT("EditEnumSelectorClass", "Edit Enum Selector Class");
	inline static const FText SetEnumSelectorValueTransaction = LOCTEXT("EditEnumSelectorValue", "Edit Enum Selector Value");
}

void FEnumSelectorDetails::CustomizeHeader(const TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	check(PropertyHandle.IsValid());

	const FEnumSelector* Selector = GetEnumSelector();
	if (!Selector)
	{
		return;
	}

	if (EnumClassSourceOptions.IsEmpty())
	{
		InitializeClassComboBox();
	}

	if (!ClassSelection.IsValid())
	{
		InitializeClassComboBoxSelection();
	}

	HeaderRow
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SVerticalBox)
			// Enum Class Selection
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				// Enum Class Selection Title
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("EnumClassSelection", "Class"))
				]
				// Enum Class Selection Options
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.FillWidth(1.f)
				.Padding(12.f, 1.f, 3.f, 1.f)
				[
					// This is currently required to be TSharedPtr<FString>, which is not optimal
					// TODO: UE-201874
					SNew(SSearchableComboBox)
					.OptionsSource(&EnumClassSourceOptions)
					.OnGenerateWidget_Lambda([this](const TSharedPtr<FString> InItemString)
					{
						return SNew(STextBlock).Text(FText::FromString(*InItemString));
					})
					.OnSelectionChanged(this, &FEnumSelectorDetails::OnClassComboSelectionChanged)
					.InitiallySelectedItem(ClassSelection)
					.Content()
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(this, &FEnumSelectorDetails::GetSelectedEnumClassText)
					]
				]
			]
			// Enum Value Selection
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				.Visibility_Raw(this, &FEnumSelectorDetails::IsValueSelectionVisible)
				// Enum Value Selection Title
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("EnumValueSelection", "Value"))
				]
				// Enum Value Selection Options
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.FillWidth(1.f)
				.Padding(12.f, 1.f, 3.f, 1.f)
				[
					SAssignNew(EnumValueComboBox, SComboBox<FName>)
					.OptionsSource(&EnumValueSourceOptions)
					.OnSelectionChanged_Raw(this, &FEnumSelectorDetails::OnValueComboSelectionChanged)
					.InitiallySelectedItem(ValueSelection)
					.OnGenerateWidget_Lambda([this](const FName InItemName)
					{
						return SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(StripEnumClassName(InItemName));
					})
					.Content()
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(this, &FEnumSelectorDetails::GetSelectedEnumValueText)
					]
				]
			]
		]
		.OverrideResetToDefault(FResetToDefaultOverride::Hide());
}

FText FEnumSelectorDetails::StripEnumClassName(const FName EnumValueName)
{
	FString EnumValueString = EnumValueName.ToString();

	int32 DelimiterIndex;
	if (EnumValueString.FindLastChar(':', DelimiterIndex))
	{
		EnumValueString.MidInline(DelimiterIndex + 1);
	}

	return FText::FromString(EnumValueString);
}

FEnumSelector* FEnumSelectorDetails::GetEnumSelector()
{
	return const_cast<FEnumSelector*>(std::as_const(*this).GetEnumSelector());
}

const FEnumSelector* FEnumSelectorDetails::GetEnumSelector() const
{
	void* Data = nullptr;
	return (PropertyHandle->GetValueData(Data) == FPropertyAccess::Success) ? static_cast<const FEnumSelector*>(Data) : nullptr;
}

UEnum* FEnumSelectorDetails::GetSelectedEnumClass() const
{
	const FEnumSelector* Selector = GetEnumSelector();
	return Selector ? Selector->Class : nullptr;
}

int64 FEnumSelectorDetails::GetSelectedEnumValue() const
{
	const FEnumSelector* Selector = GetEnumSelector();
	return Selector ? Selector->Value : 0;
}

void FEnumSelectorDetails::SetSelectorEnumClass(UEnum* Class)
{
	if (FEnumSelector* Selector = GetEnumSelector())
	{
		FScopedTransaction Transaction(FEnumSelectorDetailsConstants::SetSelectorClassTransaction);
		PropertyHandle->NotifyPreChange();
		Selector->Class = Class;
		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();
	}
}

void FEnumSelectorDetails::SetSelectorEnumValue(const int64 Value)
{
	if (FEnumSelector* Selector = GetEnumSelector())
	{
		FScopedTransaction Transaction(FEnumSelectorDetailsConstants::SetEnumSelectorValueTransaction);
		PropertyHandle->NotifyPreChange();
		Selector->Value = Value;
		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();
	}
}

void FEnumSelectorDetails::UpdateEnumClass(const TSharedPtr<FString>& EnumString)
{
	if (EnumString.IsValid())
	{
		if (!NameStringToEnumClassMap.Contains(EnumString))
		{
			return;
		}

		SetSelectorEnumClass(NameStringToEnumClassMap[EnumString]);

		ReloadValueOptions();

		if (EnumValueComboBox)
		{
			EnumValueComboBox->RefreshOptions();
			EnumValueComboBox->SetSelectedItem(EnumValueSourceOptions[0]);
		}
	}
}

void FEnumSelectorDetails::UpdateEnumValue(const FName ValueName)
{
	const FEnumSelector* Selector = GetEnumSelector();

	if (Selector && Selector->Class)
	{
		SetSelectorEnumValue(Selector->Class->GetValueByName(ValueName));
	}
}

void FEnumSelectorDetails::InitializeClassComboBox()
{
	// Find all the enums
	for (TObjectIterator<UEnum> It; It; ++It)
	{
		if (!It->HasMetaData(TEXT("Hidden")))
		{
			EnumClassSourceOptions.Emplace(MakeShared<FString>(It->GetName()));
			NameStringToEnumClassMap.Emplace(EnumClassSourceOptions.Last(), *It);
		}
	}

	// Sort the options so they're alphabetical
	EnumClassSourceOptions.Sort([](const TSharedPtr<FString>& Lhs, const TSharedPtr<FString>& Rhs)
	{
		return *Lhs < *Rhs;
	});
}

void FEnumSelectorDetails::InitializeClassComboBoxSelection()
{
	bool bEnumFound = false;
	ClassSelection.Reset();
	ValueSelection = NAME_None;

	// The selector is already defined, so use the struct data. Note: this is costly, but is needed to get the pointer to the correct widget option
	if (UEnum* SelectedEnum = GetSelectedEnumClass())
	{
		const TSharedPtr<FString>* SelectedOption = EnumClassSourceOptions.FindByPredicate([SelectedEnum](const TSharedPtr<FString>& InString)
		{
			return SelectedEnum->GetName() == *InString;
		});

		// If an enum is selected, reload the options for this enum.
		// If the selected value is valid (name is different of None), we're good.
		if (SelectedOption)
		{
			ClassSelection = *SelectedOption;

			ReloadValueOptions();
			ValueSelection = SelectedEnum->GetNameByValue(GetSelectedEnumValue());
			bEnumFound = ValueSelection != NAME_None;
		}
	}

	// The selector is null or the enum is invalid, so default to the first option
	if (!bEnumFound)
	{
		if (!ClassSelection)
		{
			// Note: already verified this wasn't empty earlier
			UpdateEnumClass(EnumClassSourceOptions[0]);
			ClassSelection = EnumClassSourceOptions[0];
		}

		// In the off chance that the enum has no options or only a MAX value
		if (!EnumValueSourceOptions.IsEmpty())
		{
			UpdateEnumValue(EnumValueSourceOptions[0]);
			ValueSelection = EnumValueSourceOptions[0];
		}
	}
}

void FEnumSelectorDetails::OnClassComboSelectionChanged(TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo)
{
	UpdateEnumClass(InSelectedItem);
}

void FEnumSelectorDetails::OnValueComboSelectionChanged(const FName InSelectedItem, ESelectInfo::Type SelectInfo)
{
	UpdateEnumValue(InSelectedItem);
}

void FEnumSelectorDetails::ReloadValueOptions()
{
	const FEnumSelector* Selector = GetEnumSelector();

	// -1 to ignore the MAX
	const int Count = Selector->Class->NumEnums() - 1;

	EnumValueSourceOptions.Reset(Count);

	// Collect all the valid values in the enum
	for (int64 i = 0; i < Count; ++i)
	{
		bool const bHidden = Selector->Class->HasMetaData(TEXT("Hidden"), i) || Selector->Class->HasMetaData(TEXT("Spacer"), i);
		if (!bHidden)
		{
			EnumValueSourceOptions.Emplace(Selector->Class->GetNameByIndex(i));
		}
	}

	EnumValueSourceOptions.Shrink();
}

EVisibility FEnumSelectorDetails::IsValueSelectionVisible() const
{
	const FEnumSelector* Selector = GetEnumSelector();
	return (Selector && Selector->Class != nullptr) ? EVisibility::Visible : EVisibility::Hidden;
}

FText FEnumSelectorDetails::GetSelectedEnumClassText() const
{
	const FEnumSelector* Selector = GetEnumSelector();
	return Selector->Class ? Selector->Class->GetDisplayNameText() : FText::FromString(*EnumClassSourceOptions[0]);
}

FText FEnumSelectorDetails::GetSelectedEnumValueText() const
{
	const FEnumSelector* Selector = GetEnumSelector();
	return FText::AsCultureInvariant(Selector->GetCultureInvariantDisplayName());
}

#undef LOCTEXT_NAMESPACE
