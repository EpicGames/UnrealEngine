// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGAttributePropertySelectorDetails.h"

#include "PCGCommon.h"
#include "PCGModule.h"
#include "Metadata/Accessors/PCGAttributeExtractor.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataAttribute.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "PCGAttributePropertySelectorDetails"

namespace PCGAttributePropertySelectorDetails
{
	static const FText SetAttributePropertyNameTransaction = LOCTEXT("SetAttributePropertyName", "[PCG] Set Attribute/Property Name");

	template <typename EnumType>
	void BuildMenuSectionFromEnum(FMenuBuilder& InMenuBuilder, FPCGAttributePropertySelectorDetails* InDetailsObject, void(FPCGAttributePropertySelectorDetails::*InCallback)(EnumType), bool bIsInput)
	{
		if (const UEnum* EnumPtr = StaticEnum<EnumType>())
		{
			for (int32 i = 0; i < EnumPtr->NumEnums() - 1; ++i)
			{
				if (EnumPtr->HasMetaData(TEXT("Hidden"), i) || (!bIsInput && EnumPtr->HasMetaData(*PCGObjectMetadata::PropertyReadOnly.ToString(), i)))
				{
					continue;
				}
				
				FString EnumName = EnumPtr->GetDisplayNameTextByIndex(i).ToString();
				EnumType EnumValue = (EnumType)EnumPtr->GetValueByIndex(i);
				FText Tooltip = EnumPtr->GetToolTipTextByIndex(i);

				InMenuBuilder.AddMenuEntry(
					FText::FromString(EnumName),
					std::move(Tooltip),
					FSlateIcon(),
					FExecuteAction::CreateSP(InDetailsObject, InCallback, EnumValue),
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
		}
	}

	void BuildSubMenuSectionForExtractor(FMenuBuilder& InMenuBuilder, FPCGAttributePropertySelectorDetails* InDetailsObject, const FName ExtractorLabel, const FText& InTooltip = FText{})
	{
		InMenuBuilder.AddMenuEntry(
			FText::FromName(ExtractorLabel),
			InTooltip,
			FSlateIcon(),
			FExecuteAction::CreateSP(InDetailsObject, &FPCGAttributePropertySelectorDetails::AddExtractor, ExtractorLabel),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}

	void BuildSubMenuSectionVector(FMenuBuilder& InMenuBuilder, FPCGAttributePropertySelectorDetails* InDetailsObject, bool bIsInput)
	{
		InMenuBuilder.AddSubMenu(
			LOCTEXT("Vector", "Vector"),
			FText{},
			FNewMenuDelegate::CreateLambda([InDetailsObject, bIsInput](FMenuBuilder& InSubMenuBuilder)
		{
			static const FText TooltipComponents = LOCTEXT("TooltipComponents", "Components can be stacked like XYZ or ZYX.\nZ/B only for Vec3/Vec4, W/A only for Vec4.\nCan't use XYZW and RGBA at the same time.");

			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::VectorX, TooltipComponents);
			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::VectorY, TooltipComponents);
			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::VectorZ, TooltipComponents);
			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::VectorW, TooltipComponents);
			InSubMenuBuilder.AddSeparator();

			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::VectorR, TooltipComponents);
			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::VectorG, TooltipComponents);
			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::VectorB, TooltipComponents);
			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::VectorA, TooltipComponents);

			if (bIsInput)
			{
				InSubMenuBuilder.AddSeparator();
				BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::VectorLength, LOCTEXT("VectorLengthTooltip", "Get the length of the vector (double value). Read only."));
				BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::VectorSquaredLength, LOCTEXT("VectorSquaredLengthTooltip", "Get the squared length of the vector (double value). Read only."));
				BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::VectorNormalized, LOCTEXT("VectorNormalizedTooltip", "Get the vector normalized (double value). Read only."));
			}
		}));
	}

	void BuildSubMenuSectionRotator(FMenuBuilder& InMenuBuilder, FPCGAttributePropertySelectorDetails* InDetailsObject, bool bIsInput)
	{
		InMenuBuilder.AddSubMenu(
			LOCTEXT("Rotator", "Rotator"),
			FText{},
			FNewMenuDelegate::CreateLambda([InDetailsObject, bIsInput](FMenuBuilder& InSubMenuBuilder)
		{
			static const FText TooltipComponents = LOCTEXT("TooltipComponentsRotator", "Euler angles");

			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::RotatorRoll, TooltipComponents);
			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::RotatorPitch, TooltipComponents);
			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::RotatorYaw, TooltipComponents);

			InSubMenuBuilder.AddSeparator();
				
			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::RotatorEulerAngles, LOCTEXT("TooltipComponentsRotatorEuler", "Euler angles as a Vector"));

			if (bIsInput)
			{
				InSubMenuBuilder.AddSeparator();
				BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::RotatorForward, LOCTEXT("RotatorForwardTooltip", "X Axis of the rotation. Read only."));
				BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::RotatorRight, LOCTEXT("RotatorRightTooltip", "Y Axis of the rotation. Read only."));
				BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::RotatorUp, LOCTEXT("RotatorUpTooltip", "Z Axis of the rotation. Read only."));
			}
		}));
	}

	void BuildSubMenuSectionTransform(FMenuBuilder& InMenuBuilder, FPCGAttributePropertySelectorDetails* InDetailsObject)
	{
		InMenuBuilder.AddSubMenu(
			LOCTEXT("Transform", "Transform"),
			FText{},
			FNewMenuDelegate::CreateLambda([InDetailsObject](FMenuBuilder& InSubMenuBuilder)
		{

			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::TransformLocation);
			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::TransformRotation);
			BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::TransformScale);
		}));
	}

	void BuildSubMenuSectionString(FMenuBuilder& InMenuBuilder, FPCGAttributePropertySelectorDetails* InDetailsObject)
	{
		InMenuBuilder.AddSubMenu(
			LOCTEXT("String", "String"),
			FText{},
			FNewMenuDelegate::CreateLambda([InDetailsObject](FMenuBuilder& InSubMenuBuilder)
			{
				static const FText TooltipComponents = LOCTEXT("TooltipString", "Length of any string-based attribute.");

				BuildSubMenuSectionForExtractor(InSubMenuBuilder, InDetailsObject, PCGAttributeExtractorConstants::StringLength, TooltipComponents);
			}));
	}

	// Func signature: bool(FPCGAttributePropertySelector*, T)
	template <typename T, typename Func>
	void SetValue(const FPCGAttributePropertySelector* InSelector, TSharedPtr<IPropertyHandle>& InPropertyHandle, T&& InValue, Func InCallback)
	{
		if (!InSelector)
		{
			return;
		}

		FPCGAttributePropertySelector TempSelector = *InSelector;

		if (InCallback(&TempSelector, std::forward<T>(InValue)))
		{
			// Need to export text and import text to go through the normal flow (like update Blueprint instances)
			FString SelectorAsText{};
			FPCGAttributePropertySelector::StaticStruct()->ExportText(SelectorAsText, &TempSelector, /*Defaults=*/nullptr, nullptr, PPF_None, /*ExportRootScope=*/nullptr);
			if (!SelectorAsText.IsEmpty())
			{
				FScopedTransaction Transaction(SetAttributePropertyNameTransaction);

				for (uint32 i = 0; i < InPropertyHandle->GetNumOuterObjects(); ++i)
				{
					InPropertyHandle->SetPerObjectValue(i, SelectorAsText);
				}
			}
		}
	}
}

void FPCGAttributePropertySelectorDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	check(PropertyHandle.IsValid());

	static FText TooltipText = LOCTEXT("TooltipText", "Enter the name of the attribute. You can prefix it by '$' to get a property. Red text means that your attribute or property is invalid.\n"
		"You can also postfix with `.` to extract values. For example, $Position.X will extract the X component of the position. They can also be chained. You can see the list under the '+'.");

	auto Validation = [this]() -> FSlateColor
	{
		const TTuple<FPropertyAccess::Result, const FPCGAttributePropertySelector*> Result = GetStruct();
		return (Result.Get<0>() == FPropertyAccess::Success && Result.Get<1>() && !Result.Get<1>()->IsValid()) ? FStyleColors::AccentRed : FSlateColor::UseForeground();
	};

	TSharedRef<SWidget> PropertyNameWidget = PropertyHandle->CreatePropertyNameWidget();
	PropertyNameWidget->SetEnabled(TAttribute<bool>(this, &FPCGAttributePropertySelectorDetails::IsEnabled));

	HeaderRow
		.NameContent()
		[
			PropertyNameWidget
		]
		.ValueContent()
		.MinDesiredWidth(350.0f)
		[
			SNew(SHorizontalBox)
			.IsEnabled_Raw(this, &FPCGAttributePropertySelectorDetails::IsEnabled)
			+ SHorizontalBox::Slot()
			[
				SNew(SEditableTextBox)
				.Text(this, &FPCGAttributePropertySelectorDetails::GetText)
				.ToolTipText(TooltipText)
				.OnTextCommitted(this, &FPCGAttributePropertySelectorDetails::SetText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ForegroundColor_Lambda(Validation)
			]
			+ SHorizontalBox::Slot()
			.MaxWidth(20.0f)
			[
				SNew(SComboButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.Visibility_Raw(this, &FPCGAttributePropertySelectorDetails::ExtraMenuVisibility)
				.OnGetMenuContent(this, &FPCGAttributePropertySelectorDetails::GenerateExtraMenu)
				.HasDownArrow(false)
				.ButtonContent()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
				]
			]
		];
}

EVisibility FPCGAttributePropertySelectorDetails::ExtraMenuVisibility() const
{
	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(PropertyHandle->GetProperty());

	const bool bIsVisible = StructProperty &&
		(StructProperty->Struct == FPCGAttributePropertyInputSelector::StaticStruct() ||
			StructProperty->Struct == FPCGAttributePropertyOutputSelector::StaticStruct() ||
			!StructProperty->HasMetaData(PCGObjectMetadata::DiscardPropertySelection) ||
			!StructProperty->HasMetaData(PCGObjectMetadata::DiscardExtraSelection));

	return bIsVisible ? EVisibility::Visible : EVisibility::Hidden;
}

bool FPCGAttributePropertySelectorDetails::IsEnabled() const
{
	if (!PropertyHandle->IsEditable())
	{
		return false;
	}

	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(PropertyHandle->GetProperty());
	TArray<UObject*> Outers;
	PropertyHandle->GetOuterObjects(Outers);
	for (const UObject* Outer : Outers)
	{
		if (Outer && !Outer->CanEditChange(StructProperty))
		{
			return false;
		}
	}

	return true;
}

TSharedRef<SWidget> FPCGAttributePropertySelectorDetails::GenerateExtraMenu()
{
	// Clear the focus on the text box. It is preventing from updating the text.
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);

	FMenuBuilder MenuBuilder(true, nullptr);

	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(PropertyHandle->GetProperty());
	const bool bIsInput = StructProperty->Struct->IsChildOf<FPCGAttributePropertyInputSelector>();

	// @todo_pcg: This submenu could be context friendly to the property type an only show when relevant.
	// Can't add an extractor if we have different values
	if (GetStruct().Get<0>() == FPropertyAccess::Success)
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("AddAnExtractor", "Add an extractor"),
			LOCTEXT("AddAnExtractorTooltip", "List of all the extractor to add to the current attribute."),
			FNewMenuDelegate::CreateLambda([this, bIsInput](FMenuBuilder& InSubMenuBuilder)
			{
				PCGAttributePropertySelectorDetails::BuildSubMenuSectionVector(InSubMenuBuilder, this, bIsInput);
				PCGAttributePropertySelectorDetails::BuildSubMenuSectionRotator(InSubMenuBuilder, this, bIsInput);
				PCGAttributePropertySelectorDetails::BuildSubMenuSectionTransform(InSubMenuBuilder, this);
				PCGAttributePropertySelectorDetails::BuildSubMenuSectionString(InSubMenuBuilder, this);
			}));
	}

	MenuBuilder.BeginSection("Attributes", LOCTEXT("AttributesHeader", "Attributes"));
	{
		if (StructProperty->Struct == FPCGAttributePropertyInputSelector::StaticStruct())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("LastAttributeHeader", "Last Attribute"),
				LOCTEXT("LastAttributeTooltip", "Refer to the last modified attribute. You can check it in the input inspection data."),
				FSlateIcon(),
				FExecuteAction::CreateSP(this, &FPCGAttributePropertySelectorDetails::SetAttributeName, PCGMetadataAttributeConstants::LastAttributeName),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		if (StructProperty->Struct == FPCGAttributePropertyOutputSelector::StaticStruct())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("SourceAttributeHeader", "Source Attribute"),
				LOCTEXT("SourceAttributeTooltip", "Refer to the same attribute that will be used as input."),
				FSlateIcon(),
				FExecuteAction::CreateSP(this, &FPCGAttributePropertySelectorDetails::SetAttributeName, PCGMetadataAttributeConstants::SourceAttributeName),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}
	MenuBuilder.EndSection();

	if (!StructProperty->HasMetaData(PCGObjectMetadata::DiscardPropertySelection))
	{
		MenuBuilder.BeginSection(TEXT("Properties"), LOCTEXT("PropertiesHeader", "Properties"));

		TWeakPtr<FPCGAttributePropertySelectorDetails> ThisWeak = StaticCastWeakPtr<FPCGAttributePropertySelectorDetails>(AsWeak());

		auto BuildMenu = [ThisWeak, bIsInput](const FPCGAttributeSelectorMenu& InSelectorMenu, FMenuBuilder& MenuBuilder, auto RecurseCallback) -> void
		{
			for (const FPCGAttributeSelectorMenu& SubMenu : InSelectorMenu.SubMenus)
			{
				MenuBuilder.AddSubMenu(SubMenu.Label,
					SubMenu.Tooltip,
					FNewMenuDelegate::CreateLambda([SubMenu, RecurseCallback](FMenuBuilder& InSubMenuBuilder) -> void
					{
						RecurseCallback(SubMenu, InSubMenuBuilder, RecurseCallback);
					}));
			}

			for (const FPCGAttributeSelectorMenuEntry& MenuEntry : InSelectorMenu.Entries)
			{
				if (!bIsInput && MenuEntry.bReadOnly)
				{
					continue;
				}
				
				auto SetSelector = [ThisWeak, Selector = MenuEntry.Selector]()
				{
					if (FPCGAttributePropertySelectorDetails* This = ThisWeak.Pin().Get())
					{
						This->SetSelector(Selector);
					}
				};
				
				MenuBuilder.AddMenuEntry(
					MenuEntry.Label,
					MenuEntry.Tooltip,
					FSlateIcon(),
					FExecuteAction::CreateLambda(std::move(SetSelector)),
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
		};

		FPCGModule::GetConstAttributeAccessorFactory().ForEachSelectorMenu([&BuildMenu, &MenuBuilder](const FPCGAttributeSelectorMenu& InSelectorMenu) -> void
		{
			BuildMenu(InSelectorMenu, MenuBuilder, BuildMenu);
		});

		MenuBuilder.EndSection();
	}

	if (!StructProperty->HasMetaData(PCGObjectMetadata::DiscardExtraSelection))
	{
		MenuBuilder.BeginSection(TEXT("OtherProperties"), LOCTEXT("OtherPropertiesHeader", "Other Properties"));
		{
			PCGAttributePropertySelectorDetails::BuildMenuSectionFromEnum<EPCGExtraProperties>(MenuBuilder, this, &FPCGAttributePropertySelectorDetails::SetExtraProperty, bIsInput);
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

TTuple<FPropertyAccess::Result, const FPCGAttributePropertySelector*> FPCGAttributePropertySelectorDetails::GetStruct() const
{
	void* Data = nullptr;
	const FPropertyAccess::Result Result = PropertyHandle->GetValueData(Data);
	const FPCGAttributePropertySelector* OutStruct = nullptr;

	switch (Result)
	{
	case FPropertyAccess::MultipleValues:
		{
			TArray<void*> RawData;
			PropertyHandle->AccessRawData(RawData);
			OutStruct = !RawData.IsEmpty() ? static_cast<const FPCGAttributePropertySelector*>(RawData[0]) : nullptr;
		}
		break;
	case FPropertyAccess::Success:
		OutStruct = static_cast<const FPCGAttributePropertySelector*>(Data);
		break;
	case FPropertyAccess::Fail: // fall-through
	default:
		break;
	}

	return {Result, OutStruct};
}

FText FPCGAttributePropertySelectorDetails::GetText() const
{
	TTuple<FPropertyAccess::Result, const FPCGAttributePropertySelector*> Result = GetStruct();
	
	switch (Result.Get<0>())
	{
	case FPropertyAccess::MultipleValues:
		return LOCTEXT("MultipleValues", "Multiple Values");
	case FPropertyAccess::Success:
		check(Result.Get<1>());
		return Result.Get<1>()->GetDisplayText();
	case FPropertyAccess::Fail: // fall-through
	default:
		return FText{};
	}
}

void FPCGAttributePropertySelectorDetails::SetText(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::Type::OnCleared)
	{
		return;
	}

	PCGAttributePropertySelectorDetails::SetValue(GetStruct().Get<1>(), PropertyHandle, NewText.ToString(), [](FPCGAttributePropertySelector* InStruct, FString InValue) -> bool { return InStruct && InStruct->Update(std::move(InValue)); });
}

void FPCGAttributePropertySelectorDetails::SetPointProperty(EPCGPointProperties EnumValue)
{
	PCGAttributePropertySelectorDetails::SetValue(GetStruct().Get<1>(), PropertyHandle, EnumValue, [](FPCGAttributePropertySelector* InStruct, EPCGPointProperties InValue) -> bool { return InStruct && InStruct->SetPointProperty(InValue); });
}

void FPCGAttributePropertySelectorDetails::SetAttributeName(FName NewName)
{
	PCGAttributePropertySelectorDetails::SetValue(GetStruct().Get<1>(), PropertyHandle, NewName, [](FPCGAttributePropertySelector* InStruct, FName InValue) -> bool { return InStruct && InStruct->SetAttributeName(InValue); });
}

void FPCGAttributePropertySelectorDetails::SetExtraProperty(EPCGExtraProperties EnumValue)
{
	const UEnum* EnumPtr = StaticEnum<EPCGExtraProperties>();
	const int32 EnumIndex = EnumPtr ? EnumPtr->GetIndexByValue(static_cast<int64>(EnumValue)) : INDEX_NONE;
	const FName DomainName = EnumPtr ? FName(EnumPtr->GetMetaData(*PCGObjectMetadata::EnumMetadataDomain.ToString(), EnumIndex)) : FName(NAME_None);
	
	PCGAttributePropertySelectorDetails::SetValue(GetStruct().Get<1>(), PropertyHandle, EnumValue, [DomainName](FPCGAttributePropertySelector* InStruct, EPCGExtraProperties InValue) -> bool
		{
			if (!InStruct)
			{
				return false;
			}

			bool bHasChanged = InStruct->SetExtraProperty(InValue);
			bHasChanged |= InStruct->SetDomainName(DomainName);
			return bHasChanged;
		});
}

void FPCGAttributePropertySelectorDetails::SetSelector(FPCGAttributePropertySelector InSelector)
{
	PCGAttributePropertySelectorDetails::SetValue(GetStruct().Get<1>(), PropertyHandle, InSelector, [](FPCGAttributePropertySelector* InStruct, FPCGAttributePropertySelector InValue) -> bool
	{
		if (!InStruct)
		{
			return false;
		}

		const bool bIsDifferent = *InStruct != InValue;
		if (bIsDifferent)
		{
			*InStruct = InValue;
		}
		
		return bIsDifferent;
	});
}

void FPCGAttributePropertySelectorDetails::AddExtractor(FName InExtractor)
{
	PCGAttributePropertySelectorDetails::SetValue(GetStruct().Get<1>(), PropertyHandle, InExtractor, [](FPCGAttributePropertySelector* InStruct, FName InValue) -> bool 
	{
		if (!InStruct)
		{
			return false;
		}

		InStruct->GetExtraNamesMutable().Add(InValue.ToString());
		return true;
	});
}

#undef LOCTEXT_NAMESPACE
