// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGBlueprintSettingsDetails.h"

#include "DetailLayoutBuilder.h"
//#include "ObjectDetailsTools.h"

#include "DetailWidgetRow.h"
#include "EdGraphSchema_K2.h"
#include "Elements/PCGExecuteBlueprint.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWrapBox.h"

#define LOCTEXT_NAMESPACE "PCGSettingsDetails"

TSharedRef<IDetailCustomization> FPCGBlueprintSettingsDetails::MakeInstance()
{
	return MakeShareable(new FPCGBlueprintSettingsDetails());
}

void FPCGBlueprintSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	if (ObjectsBeingCustomized.Num() != 1)
	{
		return;
	}

	UPCGSettingsInterface* SettingsInterface = Cast<UPCGSettingsInterface>(ObjectsBeingCustomized[0]);

	if (!SettingsInterface)
	{
		return;
	}

	UPCGBlueprintSettings* BlueprintSettings = Cast<UPCGBlueprintSettings>(SettingsInterface->GetSettings());

	if (!BlueprintSettings)
	{
		return;
	}

	// Go through all the properties, if we find properties that have CallInEditor,
	// create buttons for them to be executed
	UPCGBlueprintBaseElement* BlueprintElement = BlueprintSettings->GetElementObject();
	if (BlueprintElement)
	{
		SelectedObject = TWeakObjectPtr<UPCGBlueprintBaseElement>(BlueprintElement);
		AddCallInEditorMethods(DetailBuilder);
		//FObjectDetailsTools::AddCallInEditorMethods(
		//	DetailBuilder,
		//	BlueprintElement->GetClass(),
		//	[this](TWeakObjectPtr<UFunction> WeakFunctionPtr) { return FOnClicked::CreateSP(this, &FPCGBlueprintSettingsDetails::OnExecuteCallInEditorFunction, WeakFunctionPtr); });
	}
}

void FPCGBlueprintSettingsDetails::AddCallInEditorMethods(IDetailLayoutBuilder& DetailBuilder)
{
	check(SelectedObject.IsValid());
	// NOTE: this is an adaptation of FObjectDetails::AddCallInEditorMethods
	// metadata tag for defining sort order of function buttons within a Category
	static const FName NAME_DisplayPriority("DisplayPriority");

	TArray<UFunction*, TInlineAllocator<8>> CallInEditorFunctions;

	for (TFieldIterator<UFunction> FunctionIter(SelectedObject->GetClass(), EFieldIteratorFlags::IncludeSuper); FunctionIter; ++FunctionIter)
	{
		UFunction* Function = *FunctionIter;
		if (Function->GetBoolMetaData(FBlueprintMetadata::MD_CallInEditor) && (Function->ParmsSize == 0))
		{
			const FName FunctionName = Function->GetFName();
			if (!CallInEditorFunctions.FindByPredicate([&FunctionName](const UFunction* Func) { return Func->GetFName() == FunctionName; }))
			{
				CallInEditorFunctions.Add(*FunctionIter);
			}
		}
	}

	if (CallInEditorFunctions.Num() > 0)
	{
		// Sort the functions by category and then by DisplayPriority meta tag, and then by name
		CallInEditorFunctions.Sort([](UFunction& A, UFunction& B)
		{
			const int32 CategorySort = A.GetMetaData(FBlueprintMetadata::MD_FunctionCategory).Compare(B.GetMetaData(FBlueprintMetadata::MD_FunctionCategory));
			if (CategorySort != 0)
			{
				return (CategorySort <= 0);
			}
			else
			{
				FString DisplayPriorityAStr = A.GetMetaData(NAME_DisplayPriority);
				int32 DisplayPriorityA = (DisplayPriorityAStr.IsEmpty() ? MAX_int32 : FCString::Atoi(*DisplayPriorityAStr));
				if (DisplayPriorityA == 0 && !FCString::IsNumeric(*DisplayPriorityAStr))
				{
					DisplayPriorityA = MAX_int32;
				}

				FString DisplayPriorityBStr = B.GetMetaData(NAME_DisplayPriority);
				int32 DisplayPriorityB = (DisplayPriorityBStr.IsEmpty() ? MAX_int32 : FCString::Atoi(*DisplayPriorityBStr));
				if (DisplayPriorityB == 0 && !FCString::IsNumeric(*DisplayPriorityBStr))
				{
					DisplayPriorityB = MAX_int32;
				}

				return (DisplayPriorityA == DisplayPriorityB) ? (A.GetName() <= B.GetName()) : (DisplayPriorityA <= DisplayPriorityB);
			}
		});

		struct FCategoryEntry
		{
			FName CategoryName;
			FName RowTag;
			TSharedPtr<SWrapBox> WrapBox;
			FTextBuilder FunctionSearchText;

			FCategoryEntry(FName InCategoryName)
				: CategoryName(InCategoryName)
			{
				WrapBox = SNew(SWrapBox).UseAllottedSize(true);
			}
		};

		// Build up a set of functions for each category, accumulating search text and buttons in a wrap box
		FName ActiveCategory;
		TArray<FCategoryEntry, TInlineAllocator<8>> CategoryList;
		for (UFunction* Function : CallInEditorFunctions)
		{
			FName FunctionCategoryName(NAME_Default);
			if (Function->HasMetaData(FBlueprintMetadata::MD_FunctionCategory))
			{
				FunctionCategoryName = FName(*Function->GetMetaData(FBlueprintMetadata::MD_FunctionCategory));
			}

			if (FunctionCategoryName != ActiveCategory)
			{
				ActiveCategory = FunctionCategoryName;
				CategoryList.Emplace(FunctionCategoryName);
			}
			FCategoryEntry& CategoryEntry = CategoryList.Last();

			//@TODO: Expose the code in UK2Node_CallFunction::GetUserFacingFunctionName / etc...
			const FText ButtonCaption = FText::FromString(FName::NameToDisplayString(*Function->GetName(), false));
			FText FunctionTooltip = Function->GetToolTipText();
			if (FunctionTooltip.IsEmpty())
			{
				FunctionTooltip = FText::FromString(Function->GetName());
			}

			TWeakObjectPtr<UFunction> WeakFunctionPtr(Function);
			CategoryEntry.WrapBox->AddSlot()
				.Padding(0.0f, 0.0f, 5.0f, 3.0f)
				[
					SNew(SButton)
					.Text(ButtonCaption)
					.OnClicked(FOnClicked::CreateSP(this, &FPCGBlueprintSettingsDetails::OnExecuteCallInEditorFunction, WeakFunctionPtr))
					.ToolTipText(FunctionTooltip.IsEmptyOrWhitespace() ? LOCTEXT("CallInEditorTooltip", "Call an event on the selected object(s)") : FunctionTooltip)
				];

			CategoryEntry.RowTag = Function->GetFName();
			CategoryEntry.FunctionSearchText.AppendLine(ButtonCaption);
			CategoryEntry.FunctionSearchText.AppendLine(FunctionTooltip);
		}

		// Now edit the categories, adding the button strips to the details panel
		for (FCategoryEntry& CategoryEntry : CategoryList)
		{
			IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(CategoryEntry.CategoryName);
			CategoryBuilder.AddCustomRow(CategoryEntry.FunctionSearchText.ToText())
				.RowTag(CategoryEntry.RowTag)
				[
					CategoryEntry.WrapBox.ToSharedRef()
				];
		}
	}
}

FReply FPCGBlueprintSettingsDetails::OnExecuteCallInEditorFunction(TWeakObjectPtr<UFunction> WeakFunctionPtr)
{
	if (UFunction* Function = WeakFunctionPtr.Get())
	{
		//@TODO: Consider naming the transaction scope after the fully qualified function name for better UX
		FScopedTransaction Transaction(LOCTEXT("ExecuteCallInEditorMethod", "Call In Editor Action"));

		FEditorScriptExecutionGuard ScriptGuard;
		if (UObject* Object = SelectedObject.Get())
		{
			Object->ProcessEvent(Function, nullptr);
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
