// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGComponentDetails.h"

#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGSubsystem.h"
#include "Widgets/SPCGExecutionSourceActionWidget.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Algo/NoneOf.h"
#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "PCGComponentDetails"

TArray<TWeakInterfacePtr<IPCGGraphExecutionSource>> FPCGComponentDetails::GatherExecutionSourcesFromSelection(const TArray<TWeakObjectPtr<UObject>>& InObjectSelected) const
{
	TArray<TWeakInterfacePtr<IPCGGraphExecutionSource>> ExecutionSources;
	for (const TWeakObjectPtr<UObject>& Object : InObjectSelected)
	{
		if (IPCGGraphExecutionSource* ExecutionSource = Cast<IPCGGraphExecutionSource>(Object.Get()))
		{
			ExecutionSources.Add(ExecutionSource);
		}
	}

	return ExecutionSources;
}

TSharedRef<IDetailCustomization> FPCGComponentDetails::MakeInstance()
{
	return MakeShareable(new FPCGComponentDetails());
}

void FPCGComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const FName PCGCategoryName("PCG");
	IDetailCategoryBuilder& PCGCategory = DetailBuilder.EditCategory(PCGCategoryName);

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	TArray<TWeakInterfacePtr<IPCGGraphExecutionSource>> ExecutionSources = GatherExecutionSourcesFromSelection(ObjectsBeingCustomized);

	if (AddDefaultProperties())
	{
		TArray<TSharedRef<IPropertyHandle>> AllProperties;
		bool bSimpleProperties = true;
		bool bAdvancedProperties = false;
		// Add all properties in the category in order
		PCGCategory.GetDefaultProperties(AllProperties, bSimpleProperties, bAdvancedProperties);

		for (TSharedRef<IPropertyHandle>& Property : AllProperties)
		{
			PCGCategory.AddProperty(Property);
		}
	}

	for (TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSource : ExecutionSources)
	{
		if (UPCGComponent* Component = Cast<UPCGComponent>(ExecutionSource.Get()))
		{
			AActor* ComponentOwner = Component->GetOwner();
			if (!ComponentOwner || (ComponentOwner->IsInLevelInstance() && !ComponentOwner->IsInEditLevelInstance()))
			{
				// Do not customize for non editing Level Instance
				return;
			}
		}
	}

	// Only promote the special view if customized objects are not a component themselves
	if (Algo::NoneOf(ObjectsBeingCustomized, [](const TWeakObjectPtr<UObject>& Object) { return !Object.IsValid() || Object.Get()->IsA<UPCGComponent>(); }))
	{
		TArray<UObject*> Objects;
		Objects.Reserve(ExecutionSources.Num());
		Algo::Transform(ExecutionSources, Objects, [](const TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSource) -> UObject*
		{
			return ExecutionSource.IsValid() ? CastChecked<UObject>(ExecutionSource.Get()->GetExecutionState().GetGraphInstance(), ECastCheckedType::NullAllowed) : nullptr;
		});
		PCGCategory.AddExternalObjectProperty(Objects, TEXT("Graph"));
		PCGCategory.AddExternalObjectProperty(Objects, TEXT("ParametersOverrides"));
	}

	FDetailWidgetRow& NewRow = PCGCategory.AddCustomRow(FText::GetEmpty()).RowTag("PCG_Component_Actions");

	NewRow.ValueContent()
		.MaxDesiredWidth(120.f)
		[
			SNew(SPCGExecutionSourceActionWidget)
			.ExecutionSources(ExecutionSources)
		];
}

#undef LOCTEXT_NAMESPACE
