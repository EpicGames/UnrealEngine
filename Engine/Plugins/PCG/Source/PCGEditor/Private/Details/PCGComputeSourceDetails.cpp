// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGComputeSourceDetails.h"

#include "Compute/PCGComputeSource.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "PCGHLSLSyntaxHighlighter.h"
#include "Widgets/SPCGCodeEditor.h"
#include "Widgets/SPCGCodeEditorTextBox.h"

#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "PCGComputeSourceDetails"

FPCGComputeSourceDetails::FPCGComputeSourceDetails()
	: SyntaxHighlighter(FPCGHLSLSyntaxHighlighter::Create())
{}

TSharedRef<IDetailCustomization> FPCGComputeSourceDetails::MakeInstance()
{
	return MakeShared<FPCGComputeSourceDetails>();
}

void FPCGComputeSourceDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	if (ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}

	ComputeSourceWeakPtr = Cast<UPCGComputeSource>(ObjectsBeingCustomized[0].Get());

	if (!ComputeSourceWeakPtr.Get())
	{
		return;
	}

	if (TSharedPtr<IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr())
	{
		DetailsView->HideFilterArea(true);
	}

	// @todo_pcg: This should use the SPCGCodeEditor widget.

	SExpandableArea::FArguments ExpandableAreaArgs;
	ExpandableAreaArgs.AreaTitleFont(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"));
	ExpandableAreaArgs.AreaTitle(LOCTEXT("PCGCodeEditor_ShaderText_Title", "Source"));

	TSharedRef<IPropertyHandle> SourcePropertyHandle = DetailBuilder.GetProperty(TEXT("Source"));
	DetailBuilder.EditDefaultProperty(SourcePropertyHandle)->CustomWidget()
	.OverrideResetToDefault(FResetToDefaultOverride::Hide())
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			//ConstructNonExpandableHeaderWidget(ExpandableAreaArgs.AreaTitle(LOCTEXT("PCGCodeEditor_ShaderText_Title", "Source")))
			SNew(SBorder)
				.BorderImage(FStyleDefaults::GetNoBrush())
				.BorderBackgroundColor(FLinearColor::Transparent)
				.Padding(0.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SButton)
						.ButtonStyle(FCoreStyle::Get(), "NoBorder")
						.ContentPadding(ExpandableAreaArgs._HeaderPadding)
						.ForegroundColor(FSlateColor::UseForeground())
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(ExpandableAreaArgs._AreaTitlePadding)
							.VAlign(VAlign_Center)
							[
								SNew(SSpacer)
								.Size(ExpandableAreaArgs._Style->CollapsedImage.GetImageSize())
							]
							+SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(ExpandableAreaArgs._AreaTitle)
								.Font(ExpandableAreaArgs._AreaTitleFont)
							]
						]
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					[
						SPCGCodeEditor::MakeMenuButton()
					]
				]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SBox)
			.MinDesiredHeight(200)
			.MaxDesiredHeight(800)
			[
				SAssignNew(SourceTextBox, SPCGCodeEditorTextBox)
				.Text(this, &FPCGComputeSourceDetails::GetSourceText)
				.IsReadOnly(false)
				.OnTextChanged(this, &FPCGComputeSourceDetails::OnSourceTextChanged)
				.OnTextCommitted(this, &FPCGComputeSourceDetails::OnSourceTextCommitted)
				.OnTextChangesApplied(this, &FPCGComputeSourceDetails::OnSourceTextChangesApplied)
				.Marshaller(SyntaxHighlighter)
			]
		]
	];

	SourceTextBox->SetTextProviderObject(ComputeSourceWeakPtr.Get());
}

FText FPCGComputeSourceDetails::GetSourceText() const
{
	UPCGComputeSource* ComputeSource = ComputeSourceWeakPtr.Get();
	return ComputeSource ? FText::FromString(ComputeSource->GetSourceText()) : FText::GetEmpty();
}

void FPCGComputeSourceDetails::OnSourceTextChanged(const FText& InText)
{
	SourceText = InText;
}

void FPCGComputeSourceDetails::OnSourceTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo)
{
	SourceText = InText;
	SetComputeSourceText();
}

void FPCGComputeSourceDetails::OnSourceTextChangesApplied() const
{
	SetComputeSourceText();
}

void FPCGComputeSourceDetails::SetComputeSourceText() const
{
	if (UPCGComputeSource* ComputeSource = ComputeSourceWeakPtr.Get())
	{
		const FString SourceString = SourceText.ToString();

		if (!SourceString.Equals(ComputeSource->GetSourceText(), ESearchCase::CaseSensitive))
		{
			ComputeSource->SetSourceText(SourceString);
		}
	}
}

#undef LOCTEXT_NAMESPACE
