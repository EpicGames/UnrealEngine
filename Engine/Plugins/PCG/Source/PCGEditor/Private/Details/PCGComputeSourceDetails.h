// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"

class FPCGHLSLSyntaxHighlighter;
class SPCGCodeEditorTextBox;
class UPCGComputeSource;

/**
 * Implements a details view customization for PCG Compute Sources.
 */
class FPCGComputeSourceDetails : public IDetailCustomization
{
public:
	FPCGComputeSourceDetails();

	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	FText GetSourceText() const;
	void OnSourceTextChanged(const FText& InText);
	void OnSourceTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo);
	void OnSourceTextChangesApplied() const;
	void SetComputeSourceText() const;

	TWeakObjectPtr<UPCGComputeSource> ComputeSourceWeakPtr;
	TSharedRef<FPCGHLSLSyntaxHighlighter> SyntaxHighlighter;
	TSharedPtr<SPCGCodeEditorTextBox> SourceTextBox;

	FText SourceText;
};
