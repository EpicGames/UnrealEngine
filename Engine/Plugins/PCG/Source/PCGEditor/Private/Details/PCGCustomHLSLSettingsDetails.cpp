// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGCustomHLSLSettingsDetails.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"

#include "Containers/StringConv.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "PCGCustomHLSLSettingsDetails"

void FPCGCustomHLSLSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	IDetailCategoryBuilder& SettingsCategory = DetailBuilder.EditCategory(TEXT("Settings"));

	if (!ObjectsBeingCustomized.IsEmpty() && ObjectsBeingCustomized[0].IsValid())
	{
		const UPCGSettings* FirstSettings = Cast<UPCGSettings>(ObjectsBeingCustomized[0].Get());
		const UPCGEditorGraph* PCGEditorGraph = FPCGEditor::GetPCGEditorGraph(FirstSettings);
		const TWeakPtr<const FPCGEditor> PCGEditorWeakPtr = PCGEditorGraph ? PCGEditorGraph->GetEditor() : nullptr;
		const TSharedPtr<const FPCGEditor> PCGEditorSharedPtr = PCGEditorWeakPtr.Pin();

		// Add a "open source editor" button if the source editor tab is not already visible.
		if (PCGEditorSharedPtr.IsValid() && !PCGEditorSharedPtr->IsPanelCurrentlyOpen(PCGEditorTabs::CodeEditorID))
		{
			SettingsCategory.AddCustomRow(FText::GetEmpty())
				.Visibility(
				TAttribute<EVisibility>::CreateLambda(
				[PCGEditorWeakPtr]
				{
					const TSharedPtr<const FPCGEditor> SharedPtr = PCGEditorWeakPtr.Pin();
					return (SharedPtr.IsValid() && SharedPtr->IsPanelCurrentlyOpen(PCGEditorTabs::CodeEditorID)) ? EVisibility::Collapsed : EVisibility::Visible;
				}))
				.ValueContent()
				.MaxDesiredWidth(120.f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f)
					.VAlign(VAlign_Fill)
					[
						SNew(SButton)
						.OnClicked_Lambda([PCGEditorWeakPtr]() -> FReply
						{
							if (const TSharedPtr<const FPCGEditor> SharedPtr = PCGEditorWeakPtr.Pin())
							{
								SharedPtr->BringFocusToPanel(PCGEditorTabs::CodeEditorID);
							}

							return FReply::Handled();
						})
						.ToolTipText(FText::FromString("Opens HLSL Source Editor Panel."))
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(LOCTEXT("ButtonOpenSourceEditor", "Open HLSL Editor"))
						]
					]
				];
		}
	}

	// Add default properties in the category in order
	TArray<TSharedRef<IPropertyHandle>> AllProperties;
	SettingsCategory.GetDefaultProperties(AllProperties);

	for (TSharedRef<IPropertyHandle>& Property : AllProperties)
	{
		SettingsCategory.AddProperty(Property);
	}
}

#undef LOCTEXT_NAMESPACE
