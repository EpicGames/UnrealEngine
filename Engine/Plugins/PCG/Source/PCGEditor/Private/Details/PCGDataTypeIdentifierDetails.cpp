// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGDataTypeIdentifierDetails.h"

#include "PCGModule.h"
#include "Helpers/PCGHelpers.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"

namespace PCGDataTypeIdentifierDetails
{
	/** Walks property parent hierarchy until a property is found that belongs to a UObject. */
	TSharedPtr<IPropertyHandle> GetFirstUObjectParentProperty(TSharedPtr<IPropertyHandle> InPropertyHandle)
	{
		while (InPropertyHandle.IsValid())
		{
			if (const FProperty* Property = InPropertyHandle->GetProperty())
			{
				const UStruct* OwnerStruct = Property->GetOwnerStruct();

				// OwnerStruct will be a UClass if this property is declared on a UObject.
				if (OwnerStruct && OwnerStruct->IsA<UClass>())
				{
					return InPropertyHandle;
				}
			}

			InPropertyHandle = InPropertyHandle->GetParentHandle();
		}

		return nullptr;
	}

	// Check if the identifier is a superset of other.
	bool IsSuperSet(const FPCGDataTypeIdentifier& Identifier, const FPCGDataTypeBaseId& Other)
	{
		return (Identifier & Other) == Other;
	}

	bool AreAllSameValues(TSharedPtr<IPropertyHandle> InPropertyHandle)
	{
		void* Data = nullptr;
		FPropertyAccess::Result Result = InPropertyHandle ? InPropertyHandle->GetValueData(Data) : FPropertyAccess::Fail;

		return Result == FPropertyAccess::Success;
	}

	/**
	 * Look for a function on the outer object of the property or a flag in the property metadata to check if the widget
	 * should support composition (have its bitflag-like behavior enabled) or not. 
	 */
	bool SupportComposition(TSharedRef<IPropertyHandle> InPropertyHandle)
	{
		const FString& SupportCompositionFunctionName = InPropertyHandle->GetMetaData(PCGObjectMetadata::DataTypeIdentifierSupportsComposition);
		if (!SupportCompositionFunctionName.IsEmpty())
		{
			TArray<UObject*> OutObjects;
			InPropertyHandle->GetOuterObjects(OutObjects);

			if (OutObjects.IsEmpty())
			{
				return false;
			}

			DECLARE_DELEGATE_RetVal(bool, FSupportComposition);

			// Strip `()` that could be at the end of the function name
			const FString FunctionName = SupportCompositionFunctionName.Replace(TEXT("()"), TEXT(""));

			for (UObject* OuterObject : OutObjects)
			{
				if (!OuterObject)
				{
					return false;
				}

				if (!OuterObject->GetClass() || !OuterObject->GetClass()->FindFunctionByName(*FunctionName))
				{
					return false;
				}
				
				auto Func = FSupportComposition::CreateUFunction(OuterObject, *FunctionName);
				const bool bResult = Func.IsBound() && Func.Execute();
				if (!bResult)
				{
					return false;
				}
			}

			return true;
		}
		else if (InPropertyHandle->HasMetaData(PCGObjectMetadata::DataTypeIdentifierSupportsComposition))
		{
			return true;
		}
		else
		{
			return false;
		}
	}

	/**
	 * Look for a function on the outer object of the property (or its parent properties) to check if the widget should filter out any types.
	 */
	TOptional<TArray<FPCGDataTypeIdentifier>> GetFilter(TSharedRef<IPropertyHandle> InPropertyHandle)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = GetFirstUObjectParentProperty(InPropertyHandle.ToSharedPtr());

		if (!PropertyHandle)
		{
			return {};
		}

		const FString& GetFilterFunctionName = PropertyHandle->GetMetaData(PCGObjectMetadata::DataTypeIdentifierFilter);

		if (!GetFilterFunctionName.IsEmpty())
		{
			TArray<UObject*> OutObjects;
			PropertyHandle->GetOuterObjects(OutObjects);

			TArray<FPCGDataTypeIdentifier> Result;
			bool bHasFilters = false;

			DECLARE_DELEGATE_RetVal(TArray<FPCGDataTypeIdentifier>, FGetFilters);

			// Strip `()` that could be at the end of the function name
			const FString FunctionName = GetFilterFunctionName.Replace(TEXT("()"), TEXT(""));
			
			for (UObject* OuterObject : OutObjects)
			{
				if (!OuterObject->GetClass() || !OuterObject->GetClass()->FindFunctionByName(*FunctionName))
				{
					continue;
				}

				auto Func = FGetFilters::CreateUFunction(OuterObject, *FunctionName);
			
				if (Func.IsBound())
				{
					TArray<FPCGDataTypeIdentifier> FuncResult = Func.Execute();
					
					if (!bHasFilters)
					{
						bHasFilters = true;
						Result = MoveTemp(FuncResult);
					}
					else
					{
						// We can only keep the filters that are in common.
						TBitArray<> Mask;
						Mask.Init(false, Result.Num());
						for (const FPCGDataTypeIdentifier& ID : FuncResult)
						{
							if (int32 Index = Result.IndexOfByKey(ID); Index != INDEX_NONE)
							{
								Mask[Index] = true;
							}
						}

						for (int32 i = Result.Num() - 1; i >= 0; --i)
						{
							if (!Mask[i])
							{
								Result.RemoveAtSwap(i);
							}
						}
					}
				}
			}

			return bHasFilters ? TOptional<TArray<FPCGDataTypeIdentifier>>{MoveTemp(Result)} : TOptional<TArray<FPCGDataTypeIdentifier>>{};
		}

		return {};
	}
}

void FPCGDataTypeIdentifierDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	check(PropertyHandle.IsValid());

	TSharedRef<SWidget> PropertyNameWidget = PropertyHandle->CreatePropertyNameWidget();
	
	const TOptional<TArray<FPCGDataTypeIdentifier>> Filter = PCGDataTypeIdentifierDetails::GetFilter(InPropertyHandle);
	bSupportComposition = PCGDataTypeIdentifierDetails::SupportComposition(InPropertyHandle);

	const FPCGDataTypeRegistry& Registry = FPCGModule::GetConstDataTypeRegistry();
	
	HiddenTypes.Empty();
	VisibleTypes.Empty();

	// Do a depth search so types are ordered by category.
	Registry.HierarchyDepthSearch([this, &Filter, &Registry](const FPCGDataTypeBaseId& DataType, int32 Depth) -> FPCGDataTypeRegistry::ESearchCommand
	{
		const FPCGDataTypeInfo* TypeInfo = Registry.GetTypeInfo(DataType);
		check(TypeInfo);

		const bool bInFilter = !Filter.IsSet() || Filter.GetValue().Contains(DataType);

		// Remove hidden or filtered types
		if (TypeInfo->Hidden() || !bInFilter)
		{
			HiddenTypes.Add(DataType);
		}
		else
		{
			VisibleTypes.Emplace(DataType, Depth);
		}

		return FPCGDataTypeRegistry::ESearchCommand::ExpandAndContinue;
	});

	HeaderRow
		.NameContent()
		[
			PropertyNameWidget
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FPCGDataTypeIdentifierDetails::OnGetMenuContent)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &FPCGDataTypeIdentifierDetails::GetText)
				.ToolTipText(this, &FPCGDataTypeIdentifierDetails::GetTooltip)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
}

TSharedRef<SWidget> FPCGDataTypeIdentifierDetails::OnGetMenuContent()
{
	const bool bCloseAfterSelection = !bSupportComposition;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr, nullptr, true);

	const FPCGDataTypeRegistry& Registry = FPCGModule::GetConstDataTypeRegistry();

	for (const auto& [DataType, Depth] : VisibleTypes)
	{
		const FPCGDataTypeInfo* TypeInfo = Registry.GetTypeInfo(DataType);
		check(TypeInfo);
		
		MenuBuilder.AddMenuEntry(
			FText::FromString(DataType.ToString()),
			FText{},
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateLambda([this, ThisDataType = DataType]()
				{
					TArray<void*> RawData;
					PropertyHandle->AccessRawData(RawData);
					if (RawData.IsEmpty())
					{
						return;
					}

					FPCGDataTypeIdentifier* FirstIdentifier = static_cast<FPCGDataTypeIdentifier*>(RawData[0]);

					// We can only support the composition if all the identifiers are the same.
					bool bCanSupportComposition = bSupportComposition && PCGDataTypeIdentifierDetails::AreAllSameValues(PropertyHandle);

					FPCGDataTypeIdentifier TempIdentifier{};

					if (bCanSupportComposition && FSlateApplication::Get().GetModifierKeys().IsControlDown())
					{
						if (!PCGDataTypeIdentifierDetails::IsSuperSet(*FirstIdentifier, ThisDataType))
						{
							TempIdentifier = FirstIdentifier->Compose(ThisDataType);
						}
						else
						{
							const FPCGDataTypeIdentifier DataTypeIdentifier{ ThisDataType };
							FPCGDataTypeRegistry::FGetIdentifiersDifferenceParams Params = {
								.SourceIdentifier = FirstIdentifier,
								.DifferenceIdentifiers = MakeConstArrayView<FPCGDataTypeIdentifier>(&DataTypeIdentifier, 1),
								.Filter = FPCGDataTypeRegistry::FGetIdentifiersDifferenceParams::ExcludeFilteredTypes,
								.FilteredTypes = HiddenTypes
							};
							
							TempIdentifier = FPCGModule::GetConstDataTypeRegistry().GetIdentifiersDifference(Params);
						}
					}
					else
					{
						TempIdentifier = ThisDataType;
					}

					if (TempIdentifier.IsValid())
					{
						// Need to export text and import text to go through the normal flow (like update Blueprint instances)
						FString IdentifierAsText{};
						FPCGDataTypeIdentifier::StaticStruct()->ExportText(IdentifierAsText, &TempIdentifier, /*Defaults=*/nullptr, /*OwnerObject*/nullptr, PPF_None, /*ExportRootScope=*/nullptr);
						if (!IdentifierAsText.IsEmpty())
						{
							FScopedTransaction Transaction(NSLOCTEXT("PCGDataTypeIdentifierDetails", "ModifyIdentifier", "Modify PCG Data Type Identifier."));
							for (int32 i = 0; i < RawData.Num(); ++i)
							{
								PropertyHandle->SetPerObjectValue(i, IdentifierAsText);
							}
						}
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, ThisDataType = DataType]() -> bool
				{
					const FPCGDataTypeIdentifier* Identifier = GetStruct();
					return Identifier ? PCGDataTypeIdentifierDetails::IsSuperSet(*Identifier, ThisDataType) : false;
				})
			),
			NAME_None,
			bSupportComposition ? EUserInterfaceActionType::Check : EUserInterfaceActionType::None);
	}

	return MenuBuilder.MakeWidget();
}

FText FPCGDataTypeIdentifierDetails::GetText() const
{
	static const FPCGDataTypeIdentifier DefaultInvalidDataType{};

	void* Data = nullptr;
	FPropertyAccess::Result Result = PropertyHandle->GetValueData(Data);

	switch (Result)
	{
	case FPropertyAccess::MultipleValues:
		return NSLOCTEXT("PCGDataTypeIdentifierDetails", "MultipleValues", "Multiple Values");
	case FPropertyAccess::Success:
		return static_cast<FPCGDataTypeIdentifier*>(Data)->ToDisplayText();
	case FPropertyAccess::Fail: // fall-through
	default:
		return DefaultInvalidDataType.ToDisplayText();
	}
}

FText FPCGDataTypeIdentifierDetails::GetTooltip() const
{
	static const FText CompositionTooltip = NSLOCTEXT("PCGDataTypeIdentifierDetails", "CompositionTooltip", "\n---\nTypes can be composed like bitflags.\n"
	"By default, selecting a type will remove all the others, no composition.\n"
	"A broader type will select all its subtypes (like BaseTexture selects Texture and RenderTarget, or Any selects everything)\n"
	"Use 'Ctrl + Click' to add/remove another type to the composition.");

	return FText::Format(INVTEXT("{0}{1}"), GetText(), bSupportComposition ? CompositionTooltip : FText{});
}

FPCGDataTypeIdentifier* FPCGDataTypeIdentifierDetails::GetStruct()
{
	void* Data = nullptr;
	FPropertyAccess::Result Result = PropertyHandle->GetValueData(Data);

	return (Result == FPropertyAccess::Success) ? static_cast<FPCGDataTypeIdentifier*>(Data) : nullptr;
}

const FPCGDataTypeIdentifier* FPCGDataTypeIdentifierDetails::GetStruct() const
{
	return const_cast<FPCGDataTypeIdentifierDetails&>(*this).GetStruct();
}
