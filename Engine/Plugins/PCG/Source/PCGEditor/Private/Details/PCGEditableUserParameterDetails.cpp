// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGEditableUserParameterDetails.h"

#include "PCGCommon.h"
#include "PCGGraph.h"
#include "Elements/PCGUserParameterGet.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "InstancedPropertyBagStructureDataProvider.h"
#include "UObject/StructOnScope.h"

#define LOCTEXT_NAMESPACE "PCGEditableUserParameterDetails"

namespace PCGEditableUserParameterDetailsConstants
{
	FName UserParametersCategory = TEXT("Instance");
	// Periodic duration for a forced refresh of the details.
	static constexpr float RefreshTime = 1.f;
}

TSharedRef<IDetailCustomization> FPCGEditableUserParameterDetails::MakeInstance()
{
	return MakeShareable(new FPCGEditableUserParameterDetails);
}

void FPCGEditableUserParameterDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Reset cached values
	ParentDetailsView = TWeakPtr<IDetailsView>();
	CachedGraphInterface = TWeakObjectPtr<UPCGGraphInterface>();
	CachedPropertyDesc = FPropertyBagPropertyDesc();

	TArray<TSharedPtr<FStructOnScope>> CustomizedStructs;
	DetailBuilder.GetStructsBeingCustomized(CustomizedStructs);

	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailBuilder.GetSelectedObjects();
	if (!ensure(!SelectedObjects.IsEmpty()))
	{
		return;
	}

	TWeakObjectPtr<UObject> SettingsObject = SelectedObjects[0];
	if (!ensure(SettingsObject.IsValid()))
	{
		return;
	}

	// Cache the parent details view to be used on tick
	ParentDetailsView = DetailBuilder.GetDetailsViewSharedPtr();

	if (UPCGUserParameterGetSettings* Settings = Cast<UPCGUserParameterGetSettings>(SettingsObject.Get()))
	{
		if (UObject* NodeObject = Settings->GetOuter())
		{
			if (UPCGGraphInterface* GraphInterface = Cast<UPCGGraphInterface>(NodeObject->GetOuter()))
			{
				// Cache the interface to be used to verify on tick
				CachedGraphInterface = MakeWeakObjectPtr(GraphInterface);

				// It is safe, because we hook pre/post edit changes that will trigger the callbacks
				if (FInstancedPropertyBag* UserParameters = GraphInterface->GetMutableUserParametersStruct_Unsafe())
				{
					const UPropertyBag* PropertyBag = UserParameters->GetPropertyBagStruct();
					if (!IsValid(PropertyBag))
					{
						return;
					}

					// Cache the property description to compare later in the tick against our property
					if (const FPropertyBagPropertyDesc* PropertyDesc = PropertyBag->FindPropertyDescByName(Settings->PropertyName))
					{
						CachedPropertyDesc = *PropertyDesc;
					}
					else
					{
						return;
					}

					IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(PCGEditableUserParameterDetailsConstants::UserParametersCategory);
					IDetailPropertyRow* DetailPropertyRow = CategoryBuilder.AddExternalStructureProperty(MakeShared<FInstancePropertyBagStructureDataProvider>(*UserParameters), CachedPropertyDesc.Name);
					TSharedPtr<IPropertyHandle> RowPropertyHandle = DetailPropertyRow->GetPropertyHandle();

					FSimpleDelegate OnPreChange = FSimpleDelegate::CreateLambda([this]()
					{
						check(GEditor);
						if (GEditor->CanTransact())
						{
							GEditor->BeginTransaction(LOCTEXT("EditGraphParameter", "Edit Graph Parameter"));
						}

						if (CachedGraphInterface.IsValid())
						{
							CachedGraphInterface->Modify();
						}
					});

					FSimpleDelegate OnPostChange = FSimpleDelegate::CreateLambda([this]()
					{
						check(GEditor);
						if (CachedGraphInterface.IsValid())
						{
							const FInstancedPropertyBag* CurrentUserParameters = CachedGraphInterface->GetMutableUserParametersStruct_Unsafe();
							if (CurrentUserParameters && CurrentUserParameters->FindPropertyDescByName(CachedPropertyDesc.Name))
							{
								CachedGraphInterface->OnGraphParametersChanged(EPCGGraphParameterEvent::ValueModifiedLocally, CachedPropertyDesc.Name);
							}
							else if (GEditor->IsTransactionActive())
							{
								GEditor->CancelTransaction(0);
								return;
							}
						}

						if (GEditor->IsTransactionActive())
						{
							GEditor->EndTransaction();
						}
					});

					if (RowPropertyHandle.IsValid())
					{
						RowPropertyHandle->SetOnPropertyValuePreChange(OnPreChange);
						RowPropertyHandle->SetOnPropertyValueChanged(OnPostChange);
						// We also need to react to child property changes, if the parameter is a struct or an array.
						RowPropertyHandle->SetOnChildPropertyValuePreChange(OnPreChange);
						RowPropertyHandle->SetOnChildPropertyValueChanged(OnPostChange);
					}
				}
			}
		}
	}
}

void FPCGEditableUserParameterDetails::Tick(float DeltaTime)
{
	if (!ensure(ParentDetailsView.IsValid()))
	{
		return;
	}

	// If nothing cached is valid, don't do anything
	if (!CachedGraphInterface.IsValid() || !CachedPropertyDesc.CachedProperty)
	{
		// If the property is deleted and this details panel is alive, it will remove it and cease checking
		// So, periodically force a refresh, in case the user issues a redo command or adds the same property back
		TimeUntilRefresh -= DeltaTime;
		if (TimeUntilRefresh < 0.f)
		{
			ParentDetailsView.Pin()->ForceRefresh();
			TimeUntilRefresh = PCGEditableUserParameterDetailsConstants::RefreshTime;
		}

		return;
	}

	// Check the property bag for the property. If it is no longer valid, refresh the details view
	if (const FInstancedPropertyBag* UserParameters = CachedGraphInterface->GetMutableUserParametersStruct_Unsafe())
	{
		const UPropertyBag* PropertyBag = UserParameters->GetPropertyBagStruct();
		if (IsValid(PropertyBag))
		{
			const FPropertyBagPropertyDesc* PropertyDesc = PropertyBag->FindPropertyDescByName(CachedPropertyDesc.Name);
			// Validate that these are the same property. If they are not, refresh
			if (!PropertyDesc ||
				PropertyDesc->ID != CachedPropertyDesc.ID ||
				!PropertyDesc->CompatibleType(CachedPropertyDesc) ||
				!PropertyDesc->CachedProperty ||
				!CachedPropertyDesc.CachedProperty ||
				PropertyDesc->CachedProperty->GetOffset_ForInternal() != CachedPropertyDesc.CachedProperty->GetOffset_ForInternal())
			{
				ParentDetailsView.Pin()->ForceRefresh();
			}
		}
	}
}

TStatId FPCGEditableUserParameterDetails::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FPCGEditableUserParameterDetails, STATGROUP_Tickables);
}

#undef LOCTEXT_NAMESPACE