// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGGraphDetails.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "PCGGraph.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "PropertyHandle.h"
#include "Algo/AnyOf.h"

#define LOCTEXT_NAMESPACE "PCGGraphDetails"

const FName FPCGGraphDetails::DiffViewIdentifier(TEXT("PCGGraphDiffView"));

TSharedRef<IDetailCustomization> FPCGGraphDetails::MakeInstance()
{
	return MakeShareable(new FPCGGraphDetails());
}

void FPCGGraphDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	SelectedGraphs.Empty();

	for (TWeakObjectPtr<UObject>& Object : ObjectsBeingCustomized)
	{
		UPCGGraph* Graph = Cast<UPCGGraph>(Object.Get());
		if (ensure(Graph))
		{
			SelectedGraphs.Add(Graph);
		}
	}

	TWeakPtr<const FPCGEditor> PCGEditorWeakPtr = nullptr;
	if (!ObjectsBeingCustomized.IsEmpty() && ObjectsBeingCustomized[0].IsValid())
	{
		const UPCGEditorGraph* PCGEditorGraph = nullptr;

		if (!SelectedGraphs.IsEmpty())
		{
			PCGEditorGraph = FPCGEditor::GetPCGEditorGraph(SelectedGraphs[0].Get());
		}
		else if (const UPCGSettings* FirstSettings = Cast<UPCGSettings>(ObjectsBeingCustomized[0].Get()))
		{
			PCGEditorGraph = FPCGEditor::GetPCGEditorGraph(FirstSettings);
		}

		PCGEditorWeakPtr = PCGEditorGraph ? PCGEditorGraph->GetEditor() : nullptr;
	}

	// In a diff context the host details view sets this identifier — the editor-action buttons below
	// (Open Graph Parameters, Run Determinism Test) target the live editor session and are meaningless
	// while inspecting a saved revision, so suppress them.
	const TSharedPtr<IDetailsView> HostView = DetailBuilder.GetDetailsViewSharedPtr();
	const bool bSuppressEditorActions = HostView.IsValid() && HostView->GetIdentifier() == FPCGGraphDetails::DiffViewIdentifier;

	if (PCGEditorWeakPtr.IsValid() && !bSuppressEditorActions)
	{
		// UE_DEPRECATED(5.6, "Added for familiarity and convenience to the Graph Parameters panel moving.")
		// Add an "Open Graph Parameters" button if the source editor tab is not already visible.
		IDetailCategoryBuilder& SettingsCategory = DetailBuilder.EditCategory(TEXT("Instance"));

		SettingsCategory.AddCustomRow(FText::GetEmpty())
			.Visibility(
			TAttribute<EVisibility>::CreateLambda(
			[PCGEditorWeakPtr]
			{
				const TSharedPtr<const FPCGEditor> PCGEditorSharedPtr = PCGEditorWeakPtr.Pin();
				return (PCGEditorSharedPtr.IsValid() && PCGEditorSharedPtr->IsPanelCurrentlyOpen(PCGEditorTabs::UserParamsID)) ? EVisibility::Collapsed : EVisibility::Visible;
			}))
			.ValueContent()
			.MaxDesiredWidth(120.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				.VAlign(VAlign_Fill)
				[
					SNew(SButton)
					.OnClicked_Lambda(
					[PCGEditorWeakPtr]() -> FReply
					{
						if (const TSharedPtr<const FPCGEditor> SharedPtr = PCGEditorWeakPtr.Pin())
						{
							SharedPtr->BringFocusToPanel(PCGEditorTabs::UserParamsID);
						}

						return FReply::Handled();
					})
					.ToolTipText(LOCTEXT("OpenGraphParamPanelTooltip", "Opens the Graph Parameters Panel."))
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(LOCTEXT("OpenGraphParamPanelButtonText", "Open Graph Parameters"))
					]
				]
			];

		// Add a "Run Graph Determinism Test" button in the debug category, if we're not a standalone graph
		if (!Algo::AnyOf(SelectedGraphs, [](const TWeakObjectPtr<UPCGGraph> Graph) { return !Graph.IsValid() || Graph->IsStandaloneGraph() || Graph->IsEmbeddedSubgraph(); }))
		{
			IDetailCategoryBuilder& DebugCategory = DetailBuilder.EditCategory(TEXT("Debug"));

			DebugCategory.AddCustomRow(FText::GetEmpty())
				.ValueContent()
				.MaxDesiredWidth(120.f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f)
					.VAlign(VAlign_Fill)
					[
						SNew(SButton)
						.OnClicked_Lambda(
						[PCGEditorWeakPtr]() -> FReply
						{
							if (const TSharedPtr<const FPCGEditor> PCGEditor = PCGEditorWeakPtr.Pin())
							{
								PCGEditor->OnDeterminismGraphTest();
							}

							return FReply::Handled();
						})
						.IsEnabled_Lambda(
						[PCGEditorWeakPtr]() -> bool
						{
							if (const TSharedPtr<const FPCGEditor> PCGEditor = PCGEditorWeakPtr.Pin())
							{
								return PCGEditor->CanRunDeterminismGraphTest();
							}

							return false;
						})
						.ToolTipText(LOCTEXT("RunGraphDeterminismTestTooltip", "Runs the graph-level determinism test on the currently selected debug object."))
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(LOCTEXT("RunGraphDeterminism", "Run Determinism Test"))
						]
					]
				];
		}
	}

	const FName PCGCategoryName("PCG");
	IDetailCategoryBuilder& PCGCategory = DetailBuilder.EditCategory(PCGCategoryName);

	TArray<TSharedRef<IPropertyHandle>> AllProperties;
	bool bSimpleProperties = true;
	bool bAdvancedProperties = false;
	// Add all properties in the category in order
	PCGCategory.GetDefaultProperties(AllProperties, bSimpleProperties, bAdvancedProperties);

	for (TSharedRef<IPropertyHandle>& Property : AllProperties)
	{
		PCGCategory.AddProperty(Property);
	}
}

#undef LOCTEXT_NAMESPACE
