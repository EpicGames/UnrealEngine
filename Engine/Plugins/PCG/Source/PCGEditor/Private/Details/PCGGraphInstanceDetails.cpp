// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGGraphInstanceDetails.h"

#include "PCGEditorModule.h"
#include "PCGEditorUtils.h"
#include "PCGGraph.h"
#include "PCGGraphFactory.h"

#include "AssetToolsModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorAssetLibrary.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "PCGGraphInstanceDetails"

TSharedRef<IDetailCustomization> FPCGGraphInstanceDetails::MakeInstance()
{
	return MakeShareable(new FPCGGraphInstanceDetails());
}

void FPCGGraphInstanceDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const FName InstanceCategoryName("Instance");
	IDetailCategoryBuilder& InstanceCategory = DetailBuilder.EditCategory(InstanceCategoryName);

	FDetailWidgetRow& NewRow = InstanceCategory.AddCustomRow(FText::GetEmpty());

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	for (const TWeakObjectPtr<UObject>& Object : ObjectsBeingCustomized)
	{
		UPCGGraphInstance* GraphInstance = Cast<UPCGGraphInstance>(Object.Get());
		if (ensure(GraphInstance))
		{
			SelectedGraphInstances.Add(GraphInstance);
		}
	}

	if (SelectedGraphInstances.Num() == 1 && SelectedGraphInstances[0]->CanSaveAsAsset())
	{
		NewRow.ValueContent()
		.MaxDesiredWidth(120.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f)
			[
				SNew(SButton)
				.OnClicked(this, &FPCGGraphInstanceDetails::OnSaveInstanceClicked)
				.IsEnabled_Raw(this, &FPCGGraphInstanceDetails::SaveInstanceButtonEnabled)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("SaveInstanceButton", "Save Instance"))
				]
			]
		];
	}
}

bool FPCGGraphInstanceDetails::SaveInstanceButtonEnabled() const
{
	const UPCGGraphInstance* GraphInstance = !SelectedGraphInstances.IsEmpty() ? SelectedGraphInstances[0].Get() : nullptr;
	return (GraphInstance && GraphInstance->Graph);
}

FReply FPCGGraphInstanceDetails::OnSaveInstanceClicked()
{
	const UPCGGraphInstance* GraphInstance = !SelectedGraphInstances.IsEmpty() ? SelectedGraphInstances[0].Get() : nullptr;

	if (!GraphInstance || !IsValid(GraphInstance) || !GraphInstance->Graph)
	{
		return FReply::Handled();
	}

	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	UPCGGraphInstanceFactory* Factory = NewObject<UPCGGraphInstanceFactory>();

	FString NewPackageName;
	FString NewAssetName;
	PCGEditorUtils::GetParentPackagePathAndUniqueName(GraphInstance, LOCTEXT("NewPCGGraphInstanceAsset", "NewPCGGraphInstance").ToString(), NewPackageName, NewAssetName);

	// Saving info about the graph instance, since there is nothing preventing the user to override the asset of the graph instance they want to save. In that case, GraphInstance is destroyed, so we need to keep the info alive.
	UPCGGraphInterface* Graph = GraphInstance->Graph;
	FPCGOverrideInstancedPropertyBag Overrides = GraphInstance->ParametersOverrides;

	UPCGGraphInstance* NewPCGGraphInstance = Cast<UPCGGraphInstance>(AssetTools.CreateAssetWithDialog(NewAssetName, NewPackageName, GraphInstance->GetClass(), Factory, "PCGEditor_SaveGraphInstance"));

	if (NewPCGGraphInstance == nullptr)
	{
		UE_LOGF(LogPCGEditor, Warning, "Graph instance asset creation was aborted or failed, abort.");
		return FReply::Handled();
	}

	NewPCGGraphInstance->SetGraph(Graph);
	NewPCGGraphInstance->ParametersOverrides = std::move(Overrides);

	// Save the new asset
	UEditorAssetLibrary::SaveLoadedAsset(NewPCGGraphInstance);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
