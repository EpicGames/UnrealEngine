// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGInstancedPropertyBagOverrideDetails.h"
#include "PCGCommon.h"
#include "PCGGraph.h"
#include "PCGSettings.h"

#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyBagDetails.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "StructUtilsMetadata.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "PCGOverrideInstancedPropertyBagDetails"

namespace PCGOverrideInstancedPropertyBagDataDetails
{
	/** @return true if the property is one of the known missing types. */
	bool HasMissingType(const TSharedPtr<IPropertyHandle>& PropertyHandle)
	{
		if (!PropertyHandle)
		{
			return false;
		}

		// Handles Struct
		if (FStructProperty* StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()))
		{
			return StructProperty->Struct == FPropertyBagMissingStruct::StaticStruct();
		}
		// Handles Object, SoftObject, Class, SoftClass.
		if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(PropertyHandle->GetProperty()))
		{
			return ObjectProperty->PropertyClass == UPropertyBagMissingObject::StaticClass();
		}
		// Handles Enum
		if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(PropertyHandle->GetProperty()))
		{
			return EnumProperty->GetEnum() == StaticEnum<EPropertyBagMissingEnum>();
		}

		return false;
	}
}

TSharedRef<IPropertyTypeCustomization> FPCGOverrideInstancedPropertyBagDetails::MakeInstance()
{
	return MakeShareable(new FPCGOverrideInstancedPropertyBagDetails);
}

void FPCGOverrideInstancedPropertyBagDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	InHeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		];
}

void FPCGOverrideInstancedPropertyBagDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> InstancedPropertyBagHandle = PropertyHandle.IsValid() ? PropertyHandle->GetChildHandle(TEXT("Parameters")) : nullptr;
	
	// We forward the schema metadata
	if (PropertyHandle.IsValid() && PropertyHandle->HasMetaData(UE::StructUtils::Metadata::PropertyBagSchemaClassName) && InstancedPropertyBagHandle.IsValid())
	{
		InstancedPropertyBagHandle->SetInstanceMetaData(UE::StructUtils::Metadata::PropertyBagSchemaClassName, *PropertyHandle->GetMetaData(UE::StructUtils::Metadata::PropertyBagSchemaClassName));
	}
	
	const TSharedRef<FPCGOverrideInstancedPropertyBagDataDetails> InstanceDetails = MakeShared<FPCGOverrideInstancedPropertyBagDataDetails>(InstancedPropertyBagHandle, InCustomizationUtils.GetPropertyUtilities());
	InChildrenBuilder.AddCustomBuilder(InstanceDetails);
}

FPCGOverrideInstancedPropertyBagDataDetails::FPCGOverrideInstancedPropertyBagDataDetails(TSharedPtr<IPropertyHandle> InStructProperty, const TSharedPtr<IPropertyUtilities>& InPropUtils)
	: FPropertyBagInstanceDataDetails(
		FConstructParams{
			.BagStructProperty = InStructProperty.IsValid() ? InStructProperty : nullptr,
			.PropUtils = InPropUtils,
			.ChildRowFeatures = EPropertyBagChildRowFeatures::Fixed
		})
{
	if (InStructProperty.IsValid())
	{
		// InStructProperty correspond to GraphInstance->ParameterOverrides->Parameters
		// We need GraphInstance->ParameterOverrides->PropertiesIDsOverridden. So look at the parent then the child.
		TSharedPtr<IPropertyHandle> ParentProperty = InStructProperty->GetParentHandle();
		if (ParentProperty.IsValid())
		{
			PropertiesIDsOverriddenHandle = ParentProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPCGOverrideInstancedPropertyBag, PropertiesIDsOverridden));
		}

		TArray<UObject*> OuterObjects;
		InStructProperty->GetOuterObjects(OuterObjects);

		if (!OuterObjects.IsEmpty())
		{
			Owner = Cast<UPCGGraphInstance>(OuterObjects[0]);
			if (Owner.IsValid())
			{
				SettingsOwner = Cast<UPCGSettings>(Owner->GetOuter());
			}
		}
	}
}

FPCGOverrideInstancedPropertyBagDataDetails::~FPCGOverrideInstancedPropertyBagDataDetails()
{
	PCGDelegates::OnInstancedPropertyBagLayoutChanged.RemoveAll(this);
}

void FPCGOverrideInstancedPropertyBagDataDetails::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	FInstancedStructDataDetails::GenerateHeaderRowContent(NodeRow);

	PCGDelegates::OnInstancedPropertyBagLayoutChanged.AddSP(this, &FPCGOverrideInstancedPropertyBagDataDetails::OnStructChange);
}

EVisibility FPCGOverrideInstancedPropertyBagDataDetails::IsResetVisible(TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	return (Owner.IsValid() ? Owner->IsPropertyOverriddenAndNotDefault(InPropertyHandle->GetProperty()) : false) ? EVisibility::Visible : EVisibility::Hidden;
}

FReply FPCGOverrideInstancedPropertyBagDataDetails::OnResetToDefaultValue(TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	if (!InPropertyHandle.IsValid() || !InPropertyHandle->GetProperty())
	{
		return FReply::Handled();
	}
	
	if (Owner.IsValid())
	{
		bool bHasChanged = false;
		FString DefaultValue = Owner->GetDefaultPropertyValueForEditor(InPropertyHandle->GetProperty(), bHasChanged);
		if (bHasChanged)
		{
			FScopedTransaction Transaction(FText::Format(LOCTEXT("OnResetToDefaultValue", "Reset Override for {0}"), FText::FromName(InPropertyHandle->GetProperty()->GetFName())));
			InPropertyHandle->SetPerObjectValues({std::move(DefaultValue)});
		}
	}

	return FReply::Handled();
}

bool FPCGOverrideInstancedPropertyBagDataDetails::IsPropertyOverriddenByPin(TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	return InPropertyHandle.IsValid() && SettingsOwner.IsValid() && SettingsOwner->IsPropertyOverriddenByPin(InPropertyHandle->GetProperty());
}

void FPCGOverrideInstancedPropertyBagDataDetails::OnChildRowAdded(IDetailPropertyRow& ChildRow)
{
	if (!Owner.IsValid() || !PropertiesIDsOverriddenHandle.IsValid())
	{
		return;
	}

	TSharedPtr<IPropertyHandle> ChildPropertyHandle = ChildRow.GetPropertyHandle();
	if (!ChildPropertyHandle.IsValid())
	{
		return;
	}

	ChildPropertyHandle->SetInstanceMetaData("NoResetToDefault", "true");

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	ChildRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	ChildRow
	.CustomWidget(/*bShowChildren*/true)
	.NameContent()
	[
		SNew(SHorizontalBox)
		// Error icon
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(0, 0, 2, 0)
		[
			SNew(SBox)
			.WidthOverride(12)
			.HeightOverride(12)
			[
				SNew(SImage)
				.ToolTipText(LOCTEXT("MissingType", "The property is missing type. The Struct, Enum, or Object may have been removed."))
				.Visibility_Lambda([ChildPropertyHandle]()
				{
					return PCGOverrideInstancedPropertyBagDataDetails::HasMissingType(ChildPropertyHandle) ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.Image(FAppStyle::GetBrush("Icons.Error"))
			]
		]
		// Name
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			NameWidget.ToSharedRef()
		]
	]
	.ValueContent()
	[
		ValueWidget.ToSharedRef()
	]
	.ExtensionContent()
	[
		SNew(SButton)
		.IsFocusable(false)
		.ToolTipText(LOCTEXT("ResetButtonTooltip", "Reset property value to its default value, defined by the parent."))
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ContentPadding(0)
		.Visibility_Lambda([this, ChildPropertyHandle]() { return IsResetVisible(ChildPropertyHandle); })
		.OnClicked_Lambda([this, ChildPropertyHandle]() { return OnResetToDefaultValue(ChildPropertyHandle); })
		.Content()
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	];

	// A property is mark overriden if the property is overridden and the property is not overridden by a pin (in the case of a subgraph).
	TAttribute<bool> EditConditionValue = TAttribute<bool>::CreateLambda([this, ChildPropertyHandle]() -> bool { return Owner.IsValid() && Owner->IsPropertyOverridden(ChildPropertyHandle->GetProperty()) && !IsPropertyOverriddenByPin(ChildPropertyHandle); });
	FOnBooleanValueChanged OnEditConditionChanged = FOnBooleanValueChanged::CreateLambda([this, ChildPropertyHandle](bool bNewValue)
	{
		//We have to go through ImportText for the whole propagation to happen
		if (Owner.IsValid())
		{
			bool bHasChanged = false;
			FString NewValue = Owner->ExportOverriddenPropertyIdsChangeForEditor(ChildPropertyHandle->GetProperty(), bNewValue, bHasChanged);
			if (bHasChanged)
			{
				FScopedTransaction Transaction(FText::Format(LOCTEXT("OnCheckStateChanged", "Change Override for {0}"), FText::FromName(ChildPropertyHandle->GetProperty()->GetFName())));
				PropertiesIDsOverriddenHandle->SetPerObjectValues({std::move(NewValue)});
			}
		}
	});

	ChildRow.EditCondition(std::move(EditConditionValue), std::move(OnEditConditionChanged));
}

#undef LOCTEXT_NAMESPACE