// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGraph.h"

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "InstancedStructDetails.h"
#include "PropertyBagDetails.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UPCGSettings;
struct FInstancedPropertyBag;

class FPCGOverrideInstancedPropertyBagDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** ~Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	/** ~End IPropertyTypeCustomization interface */
private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
};

class FPCGOverrideInstancedPropertyBagDataDetails : public FPropertyBagInstanceDataDetails
{
public:
	FPCGOverrideInstancedPropertyBagDataDetails(TSharedPtr<IPropertyHandle> InStructProperty, const TSharedPtr<IPropertyUtilities>& InPropUtils);

	virtual void OnChildRowAdded(IDetailPropertyRow& ChildRow) override;

private:
	FReply OnResetToDefaultValue(TSharedPtr<IPropertyHandle> InPropertyHandle) const;
	EVisibility IsResetVisible(TSharedPtr<IPropertyHandle> InPropertyHandle) const;
	bool IsPropertyOverriddenByPin(TSharedPtr<IPropertyHandle> InPropertyHandle) const;

	TWeakObjectPtr<UPCGGraphInstance> Owner;
	TWeakObjectPtr<UPCGSettings> SettingsOwner;
	TSharedPtr<IPropertyHandle> PropertiesIDsOverriddenHandle;

public:
	~FPCGOverrideInstancedPropertyBagDataDetails();

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	void OnStructChange(const FInstancedPropertyBag& Instance)
	{
		// Only react to changes that happened on this property bag.
		const UPCGGraphInstance* OwnerPtr = Owner.Get();
		if (IsValid(OwnerPtr) && OwnerPtr->GetUserParametersStruct() == &Instance)
		{
			OnStructLayoutChanges();
		}
	}
};