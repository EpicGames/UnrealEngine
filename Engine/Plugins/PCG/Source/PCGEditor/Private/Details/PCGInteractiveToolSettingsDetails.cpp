// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGInteractiveToolSettingsDetails.h"

#include "PCGEditorModule.h"
#include "PCGGraph.h"
#include "Data/Tool/PCGToolBaseData.h"
#include "EditorMode/Tools/PCGInteractiveToolSettings.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "PCGGraphInstanceDetails"

TSharedRef<IDetailCustomization> FPCGInteractiveToolSettingsDetails::MakeInstance()
{
	TSharedRef<FPCGInteractiveToolSettingsDetails> Customization = MakeShared<FPCGInteractiveToolSettingsDetails>();
	Customization->RegisterDelegates();
	return Customization;
}

FPCGInteractiveToolSettingsDetails::FPCGInteractiveToolSettingsDetails()
{
}

FPCGInteractiveToolSettingsDetails::~FPCGInteractiveToolSettingsDetails()
{
	UnregisterDelegates();
}

void FPCGInteractiveToolSettingsDetails::RegisterDelegates()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FPCGInteractiveToolSettingsDetails::Rebuild);
}

void FPCGInteractiveToolSettingsDetails::UnregisterDelegates()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void FPCGInteractiveToolSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{	
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	for(const TWeakObjectPtr<UObject>& Object : ObjectsBeingCustomized)
	{
		UPCGInteractiveToolSettings* InToolSettings = Cast<UPCGInteractiveToolSettings>(Object.Get());
		if(ensure(InToolSettings))
		{
			ToolSettings = InToolSettings;
		}
	}
	
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory("PCG");
	
	TArray<TSharedRef<IPropertyHandle>> PropertyHandles;
	CategoryBuilder.GetDefaultProperties(PropertyHandles);

	for(TSharedRef<IPropertyHandle> PropertyHandle : PropertyHandles)
	{
		IDetailPropertyRow& PropertyRow = CategoryBuilder.AddProperty(PropertyHandle);

		// We add the parameters right behind the graph property
		if(PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGInteractiveToolSettings, ToolGraph))
		{
			UPCGGraphInstance* GraphInstance = ToolSettings.IsValid() ? ToolSettings->GetGraphInstance() : nullptr;
			if(GraphInstance != nullptr && GraphInstance->GetUserParametersStruct()->GetNumPropertiesInBag() > 0)
			{
				IDetailPropertyRow* ParametersPropertyRow = CategoryBuilder.AddExternalObjectProperty(
				{GraphInstance}, 
				GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, ParametersOverrides),
				 EPropertyLocation::Default,
				 FAddPropertyParams());
		
				ParametersPropertyRow->ShouldAutoExpand(true);
			}
		}

		if(PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGInteractiveToolSettings, DataInstance))
		{
			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;
			PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, true);

			PropertyRow.CustomWidget()
			.NameContent()
			[
				NameWidget.ToSharedRef()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					ValueWidget.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f)
				[
					SNew(SButton)
					.Text(this, &FPCGInteractiveToolSettingsDetails::GetResetToolDataButtonText)
					.Visibility(this, &FPCGInteractiveToolSettingsDetails::IsResetToolDataButtonVisible)
					.OnClicked(this, &FPCGInteractiveToolSettingsDetails::ResetToolData)
				]	
			];
		}
	}	
}

void FPCGInteractiveToolSettingsDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	DetailLayoutBuilder = DetailBuilder;
	IDetailCustomization::CustomizeDetails(DetailBuilder);
}

void FPCGInteractiveToolSettingsDetails::Rebuild(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if(Object == ToolSettings)
	{
		if(PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGInteractiveToolSettings, ToolGraph))
		{
			if(DetailLayoutBuilder.IsValid())
			{
				DetailLayoutBuilder.Pin()->ForceRefreshDetails();
			}
		}
	}
}

FText FPCGInteractiveToolSettingsDetails::GetResetToolDataButtonText() const
{
	if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
	{
		return LOCTEXT("ResetAllToolDataButton_Text", "Reset All");	
	}
	
	return LOCTEXT("ResetToolDataButton_Text", "Reset");
}

FReply FPCGInteractiveToolSettingsDetails::ResetToolData() const
{
	if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
	{
		FPCGInteractiveToolWorkingDataContext Context;
		Context.PCGSettings = ToolSettings;
		Context.InteractiveTool = ToolSettings->GetTypedOuter<UInteractiveTool>();
		Context.OwningPCGComponent = ToolSettings->GetWorkingPCGComponent();
		Context.OwningActor = Context.OwningPCGComponent.IsValid() ? Context.OwningPCGComponent->GetOwner() : nullptr;
		for (const auto& WorkingDataTuple : ToolSettings->GetMutableTypedWorkingDataMap())
		{
			Context.DataInstanceIdentifier = WorkingDataTuple.Key;

			if(ToolSettings->CanResetToolData(Context.DataInstanceIdentifier))
			{
				WorkingDataTuple.Value->OnResetToolDataRequested(Context);
			}
		}
		
	}
	else
	{
		if (FPCGInteractiveToolWorkingData* WorkingData = ToolSettings->GetMutableTypedWorkingData(ToolSettings->DataInstance))
		{
			FPCGInteractiveToolWorkingDataContext Context;
			Context.DataInstanceIdentifier = ToolSettings->DataInstance;
			Context.PCGSettings = ToolSettings;
			Context.InteractiveTool = ToolSettings->GetTypedOuter<UInteractiveTool>();
			Context.OwningPCGComponent = ToolSettings->GetWorkingPCGComponent();
			Context.OwningActor = Context.OwningPCGComponent.IsValid() ? Context.OwningPCGComponent->GetOwner() : nullptr;
			
			if(ToolSettings->CanResetToolData(Context.DataInstanceIdentifier))
			{
				WorkingData->OnResetToolDataRequested(Context);
			}
		}
	}

	return FReply::Handled();
}

EVisibility FPCGInteractiveToolSettingsDetails::IsResetToolDataButtonVisible() const
{
	if (ToolSettings.IsValid())
	{
		if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
		{
			bool bAny = false;
			for (const auto& WorkingDataTuple : ToolSettings->GetMutableTypedWorkingDataMap())
			{			
				if(ToolSettings->CanResetToolData(WorkingDataTuple.Key))
				{
					bAny = true;
				}
			}

			return bAny ? EVisibility::Visible : EVisibility::Collapsed;
		}
		
		return ToolSettings->CanResetToolData(ToolSettings->DataInstance) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
