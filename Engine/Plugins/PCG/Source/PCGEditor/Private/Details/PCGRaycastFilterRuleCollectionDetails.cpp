// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGRaycastFilterRuleCollectionDetails.h"
#include "PCGModule.h"
#include "Helpers/PCGHelpers.h"
#include "EditorMode/Tools/Helpers/PCGEdModeSceneQueryHelpers.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "PCGEditorMode"

void FPCGRaycastFilterRuleCollectionDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	check(PropertyHandle.IsValid());

	HeaderRow.Visibility(EVisibility::Collapsed);
}

void FPCGRaycastFilterRuleCollectionDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if(TSharedPtr<IPropertyHandle> RaycastRuleHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPCGRaycastFilterRuleCollection, RaycastRules)))
	{
		TSharedRef<FDetailArrayBuilder> RaycastRuleArrayBuilder = MakeShared<FDetailArrayBuilder>(RaycastRuleHandle.ToSharedRef(), true, true, true);
		RaycastRuleArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FPCGRaycastFilterRuleCollectionDetails::OnGenerateRaycastRuleElement, CustomizationUtils.GetPropertyUtilities()));
		
		ChildBuilder.AddCustomBuilder(RaycastRuleArrayBuilder);
	}
}

void FPCGRaycastFilterRuleCollectionDetails::OnGenerateRaycastRuleElement(TSharedRef<IPropertyHandle> RaycastRuleArrayElementHandle, int32 ArrayIndex,	IDetailChildrenBuilder& DetailChildrenBuilder, TSharedPtr<IPropertyUtilities> PropertyUtilities) const
{
	IDetailPropertyRow& ArrayElementRow = DetailChildrenBuilder.AddProperty(RaycastRuleArrayElementHandle);
	ArrayElementRow.ShouldAutoExpand(true);
	
	FInstancedStruct* Instanced = nullptr;
	if (RaycastRuleArrayElementHandle->GetValueData((void*&)Instanced) != FPropertyAccess::Success || !Instanced || !Instanced->IsValid())
	{
		return;
	}

	const UScriptStruct* InnerScript = Instanced->GetScriptStruct();
	void* Memory = Instanced->GetMutableMemory();
	if (!InnerScript || !Memory)
	{
		return;
	}
	
	FBoolProperty* EnabledProp = FindFProperty<FBoolProperty>(InnerScript, GET_MEMBER_NAME_CHECKED(FPCGRaycastFilterRule, bEnabled));
	auto GetEnabled = [EnabledProp, Memory]() -> bool
	{
		bool bEnabled = true;
		if (EnabledProp)
		{
			bEnabled = EnabledProp->GetPropertyValue(EnabledProp->ContainerPtrToValuePtr<void>(Memory));
		}
			
		return bEnabled;
	};

	auto SetEnabled = [EnabledProp, Memory](bool bEnabled) -> void
	{
		if (EnabledProp)
		{
			EnabledProp->SetPropertyValue(EnabledProp->ContainerPtrToValuePtr<void>(Memory), bEnabled);
		}
	};

	TSharedPtr<SWidget> DefaultNameWidget;
	TSharedPtr<SWidget> DefaultValueWidget;	
	ArrayElementRow.GetDefaultWidgets(DefaultNameWidget, DefaultValueWidget, false);
	
	ArrayElementRow.CustomWidget(true)
	.NameContent()
	[
		DefaultValueWidget.ToSharedRef()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([GetEnabled]() -> ECheckBoxState
			{
				return GetEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([SetEnabled, RaycastRuleArrayElementHandle, PropertyUtilities](ECheckBoxState InCheckBoxState)
			{
				FScopedTransaction Transaction(LOCTEXT("ToggleRuleEnabled", "Toggle Rule Enabled"));
					
				RaycastRuleArrayElementHandle->NotifyPreChange();
				SetEnabled(InCheckBoxState == ECheckBoxState::Checked);
				RaycastRuleArrayElementHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

				PropertyUtilities->ForceRefresh();
			})
		]
	];
}

FPCGRaycastFilterRuleCollection* FPCGRaycastFilterRuleCollectionDetails::GetStruct()
{
	void* Data = nullptr;
	FPropertyAccess::Result Result = PropertyHandle->GetValueData(Data);

	return (Result == FPropertyAccess::Success) ? static_cast<FPCGRaycastFilterRuleCollection*>(Data) : nullptr;
}

const FPCGRaycastFilterRuleCollection* FPCGRaycastFilterRuleCollectionDetails::GetStruct() const
{
	return const_cast<FPCGRaycastFilterRuleCollectionDetails&>(*this).GetStruct();
}

#undef LOCTEXT_NAMESPACE
