// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/PCGVolumeDetails.h"

#include "PCGComponent.h"
#include "PCGVolume.h"

TSharedRef<IDetailCustomization> FPCGVolumeDetails::MakeInstance()
{
	return MakeShareable(new FPCGVolumeDetails());
}

TArray<TWeakInterfacePtr<IPCGGraphExecutionSource>> FPCGVolumeDetails::GatherExecutionSourcesFromSelection(const TArray<TWeakObjectPtr<UObject>>& InObjectSelected) const
{
	TArray<TWeakInterfacePtr<IPCGGraphExecutionSource>> ExecutionSources;

	for (const TWeakObjectPtr<UObject>& Object : InObjectSelected)
	{
		if (APCGVolume* Volume = Cast<APCGVolume>(Object))
		{
			TInlineComponentArray<UPCGComponent*, 1> Components;
			Volume->GetComponents<UPCGComponent>(Components);
			for (UPCGComponent* Component : Components)
			{
				if (Component)
				{
					ExecutionSources.Add(Component);
				}
			}
		}
	}

	return ExecutionSources;
}