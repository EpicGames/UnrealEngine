// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/PCGEdMode.h"

#include "PCGEditorModule.h"
#include "EditorMode/PCGEdModeCommands.h"
#include "EditorMode/PCGEdModeSettings.h"
#include "EditorMode/PCGEdModeStyle.h"
#include "EditorMode/PCGEdModeToolkit.h"
#include "EditorMode/Tools/PCGInteractiveToolSettings.h"
#include "EditorMode/Tools/Line/PCGDrawSplineTool.h"
#include "EditorMode/Tools/Paint/PCGPaintTool.h"
#include "EditorMode/Tools/Query/PCGQueryTool.h"
#include "EditorMode/Tools/Volume/PCGVolumeTool.h"

#include "ContextObjectStore.h"
#include "EngineAnalytics.h"
#include "IAnalyticsProviderET.h"
#include "ILevelEditor.h"
#include "InteractiveToolManager.h"
#include "Selection.h"
#include "UnrealEdGlobals.h" // GUnrealEd
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Components/StaticMeshComponent.h"
#include "Editor/UnrealEdEngine.h"
#include "Tools/EdModeInteractiveToolsContext.h"

// Commands
#include "EditorModeManager.h"
#include "InteractiveToolQueryInterfaces.h"
#include "LevelEditor.h"
#include "SLevelViewport.h"
#include "Application/ThrottleManager.h"

#include "ToolContextInterfaces.h"
#include "ToolTargetManager.h"
#include "Tools/ManualEdit/PCGManualEditTool.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEdMode)

#define LOCTEXT_NAMESPACE "UPCGEditorMode"

/** Widget helper that exposes the standard editor transform gizmo. */
class FPCGEditorModeWidgetHelper : public FLegacyEdModeWidgetHelper
{
public:
	virtual bool ShouldDrawWidget() const override
	{
		return ActiveToolUsesTransformGizmo() && FLegacyEdModeWidgetHelper::ShouldDrawWidget();
	}

	virtual bool UsesTransformWidget() const override
	{
		return ActiveToolUsesTransformGizmo() && FLegacyEdModeWidgetHelper::UsesTransformWidget();
	}

	virtual bool UsesTransformWidget(const UE::Widget::EWidgetMode WidgetMode) const override
	{
		return ActiveToolUsesTransformGizmo() && FLegacyEdModeWidgetHelper::UsesTransformWidget(WidgetMode);
	}

	virtual bool CanCycleWidgetMode() const override
	{
		return ActiveToolUsesTransformGizmo() && FLegacyEdModeWidgetHelper::CanCycleWidgetMode();
	}

	/** Keep the project's ITF TRS gizmo enabled. */
	virtual bool RequiresLegacyViewportInteractions() const override
	{
		return false;
	}

	TWeakObjectPtr<UPCGEditorMode> PCGEditorMode;

private:
	bool ActiveToolUsesTransformGizmo() const
	{
		const UPCGEditorMode* Mode = PCGEditorMode.Get();
		return Mode && Mode->ActiveToolUsesTransformGizmo();
	}
};

const FEditorModeID UPCGEditorMode::EM_PCGEditorModeId = TEXT("EM_PCGEditorMode");

namespace PCGEditorMode
{
	namespace Constants
	{
		const FName PCGEditorModeOwnerName = "PCGEditorModeViewportToolBar";
		const FText PCGEditorModeDisplayName = LOCTEXT("PCGEditorModeDisplayName", "PCG");
	}

	namespace Helpers
	{
		FString GetToolName(const UInteractiveTool& Tool)
		{
			const FString* ToolName = FTextInspector::GetSourceString(Tool.GetToolInfo().ToolDisplayName);
			return ToolName ? *ToolName : FString(TEXT("<Invalid ToolName>"));
		}
	}

	static FAutoConsoleCommand CommandEnablePCGTool(
		TEXT("pcg.EnableTool"),
		TEXT("Enables toolmode and selects the specified tool."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			GLevelEditorModeTools().ActivateMode(UPCGEditorMode::EM_PCGEditorModeId, /*bToggle=*/false);
			if (UEdMode* ThisMode = GLevelEditorModeTools().GetActiveScriptableMode(UPCGEditorMode::EM_PCGEditorModeId); ThisMode && !Args.IsEmpty())
			{
				ThisMode->GetInteractiveToolsContext()->StartTool(Args[0]);
			}
		}));

	static FAutoConsoleCommand CommandSetGraphInTool(
		TEXT("pcg.tool.SetGraph"),
		TEXT("Sets graph on the current tool."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			UEdMode* ThisMode = GLevelEditorModeTools().GetActiveScriptableMode(UPCGEditorMode::EM_PCGEditorModeId);
			UPCGEditorMode* EditorMode = Cast<UPCGEditorMode>(ThisMode);
			UPCGInteractiveToolSettings* ToolSettings = EditorMode ? Cast<UPCGInteractiveToolSettings>(EditorMode->GetCurrentToolSettings()) : nullptr;
			if (ToolSettings && !Args.IsEmpty())
			{
				// If the input path is empty, or not pointing to a graph, the tool graph will be set to nullptr.
				const FString& GraphPathString = Args[0];
				const FSoftObjectPath GraphPath(GraphPathString);
				UPCGGraphInterface* Graph = Cast<UPCGGraphInterface>(GraphPath.TryLoad());
				ToolSettings->SetToolGraph(Graph);

				if (Args.Num() > 1)
				{
					ToolSettings->SetActorLabel(FName(Args[1]));
				}
			}
		}));

	static FAutoConsoleCommand CommandIsolate(
		TEXT("pcg.tool.Isolate"),
		TEXT("Isolates (like clear PCG link) content from an actor with a PCG component into a separate actor."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if(Args.IsEmpty())
			{
				UE_LOGF(LogPCGEditor, Error, "Invalid arguments to the pcg.tool.Isolate command.");
				return;
			}

			// [0] - selected actor
			FSoftObjectPath ActorPath(Args[0]);
			UObject* ActorObject = ActorPath.TryLoad();

			AActor* Actor = Cast<AActor>(ActorObject);

			if (!Actor)
			{
				UE_LOGF(LogPCGEditor, Error, "Unable to retrieve the actor '%ls' in the pcg.tool.Isolate command.", *Args[0]);
				return;
			}

			// [1+] - tags conjunction
			TArray<FName> Tags;

			for (int ArgIndex = 1; ArgIndex < Args.Num(); ++ArgIndex)
			{
				Tags.Add(FName(Args[ArgIndex]));
			}

			FPCGMoveResourceParams Params
			{
				.RequiredTags = Tags
			};
			
			UPCGActorHelpers::IsolateFromActor(Actor, Params);
		}));
}

UPCGEditorMode::UPCGEditorMode()
{
	Info = FEditorModeInfo(
		EM_PCGEditorModeId,
		PCGEditorMode::Constants::PCGEditorModeDisplayName,
		FSlateIcon("PCGEditorModeStyle", "PCGEditorModeIcon"),
		GetDefault<UPCGEditorModeSettings>()->bEnableEditorMode,
		/*@todo_pcg: TBD...*/std::numeric_limits<int32>::max());
}

void UPCGEditorMode::RegisterEditorMode()
{
	FPCGEditorModeStyle::Register();
	FPCGEditorModeCommands::Register();
	FPCGEditorModePaletteCommands::Register();
	FPCGEditorModeToolCommands::Register();
	FPCGEditorModeToolCommands::RegisterAllToolCommands();
}

void UPCGEditorMode::UnregisterEditorMode()
{
	FPCGEditorModeToolCommands::UnregisterAllToolCommands();
	FPCGEditorModeToolCommands::Unregister();
	FPCGEditorModePaletteCommands::Unregister();
	FPCGEditorModeCommands::Unregister();
	FPCGEditorModeStyle::Unregister();
}

bool UPCGEditorMode::ProcessEditDelete()
{
	if (UEdMode::ProcessEditDelete())
	{
		return true;
	}

	UInteractiveToolManager* ToolManager = GetToolManager();
	// Disable deleting in an Accept-style tool which could be volatile.
	if (ToolManager && ToolManager->HasAnyActiveTool() && ToolManager->GetActiveTool(EToolSide::Mouse)->HasAccept())
	{
		ToolManager->DisplayMessage(
			LOCTEXT("CannotDeleteWarning", "Cannot delete objects while this Tool is active"), EToolMessageLevel::UserWarning);
		return true;
	}

	return false;
}

bool UPCGEditorMode::ProcessEditCut()
{
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		// Disable deleting in an Accept-style tool because it can result in crashes if we are deleting target object
		if (ToolManager->HasAnyActiveTool() && ToolManager->GetActiveTool(EToolSide::Mouse)->HasAccept())
		{
			ToolManager->DisplayMessage(
				LOCTEXT("CannotCutWarning", "Cannot cut objects while this Tool is active"), EToolMessageLevel::UserWarning);
			return true;
		}
	}

	return false;
}

bool UPCGEditorMode::CanAutoSave() const
{
	// @todo_pcg: Autosave should be fine, since all tools' artifacts are transient, but this needs to be verified.
	// For now, just disable autosave if a tool is active.
	return !GetToolManager() || (GetToolManager()->HasAnyActiveTool() == false);
}

bool UPCGEditorMode::GetPivotForOrbit(FVector& OutPivot) const
{
	// @todo_pcg: Some features may want to override this.

	if (GCurrentLevelEditingViewportClient)
	{
		OutPivot = GCurrentLevelEditingViewportClient->GetViewTransform().GetLookAt();
		return true;
	}

	return false;
}

void UPCGEditorMode::Enter()
{
	UEdMode::Enter();

	UInteractiveToolManager* ToolManager = GetToolManager();
	if (!ToolManager)
	{
		return;
	}

	UEditorInteractiveToolsContext* InteractiveToolsContext = GetInteractiveToolsContext();
	InteractiveToolsContext->TargetManager->AddTargetFactory(NewObject<UPrimitiveComponentToolTargetFactory>(InteractiveToolsContext->TargetManager));
	
	ToolManager->OnToolPostBuild.AddUObject(this, &UPCGEditorMode::OnToolPostBuild);
	ToolManager->OnToolShutdownRequest.BindLambda([this](UInteractiveToolManager*, UInteractiveTool*, const EToolShutdownType ShutdownType)
	{
		if (UEditorInteractiveToolsContext* ToolsContext = GetInteractiveToolsContext())
		{
			ToolsContext->EndTool(ShutdownType);
			return true;
		}
		else
		{
			return false;
		}
	});

	FPCGEditorModeToolCommands& ToolManagerCommands = FPCGEditorModeToolCommands::Get();
	
	{
		UPCGDrawSplineToolBuilder* DrawSplineToolBuilder = NewObject<UPCGDrawSplineToolBuilder>();
		DrawSplineToolBuilder->SetSplineToolClass(UPCGDrawSplineTool::StaticClass());
		RegisterTool(ToolManagerCommands.EnableDrawSplineTool, UPCGInteractiveToolSettings_Spline::StaticGetToolTag().ToString(), DrawSplineToolBuilder);
	}
	
	{
		UPCGDrawSplineToolBuilder* DrawSurfaceToolBuilder = NewObject<UPCGDrawSplineToolBuilder>();
		DrawSurfaceToolBuilder->SetSplineToolClass(UPCGDrawSplineSurfaceTool::StaticClass());
		RegisterTool(ToolManagerCommands.EnableDrawSurfaceTool, UPCGInteractiveToolSettings_SplineSurface::StaticGetToolTag().ToString(), DrawSurfaceToolBuilder);
	}
	
	{
		UPCGPaintToolBuilder* PaintToolBuilder = NewObject<UPCGPaintToolBuilder>();
		RegisterTool(ToolManagerCommands.EnablePaintTool, UPCGInteractiveToolSettings_PaintTool::StaticGetToolTag().ToString(), PaintToolBuilder);
	}
	
	{
		UPCGVolumeToolBuilder* VolumeToolBuilder = NewObject<UPCGVolumeToolBuilder>();
		RegisterTool(ToolManagerCommands.EnableVolumeTool, UPCGInteractiveToolSettings_Volume::StaticGetToolTag().ToString(), VolumeToolBuilder);
	}

	{
		UPCGQueryToolBuilder* QueryToolBuilder = NewObject<UPCGQueryToolBuilder>();
		QueryToolBuilder->SetToolClass(UPCGQueryTool::StaticClass());
		RegisterTool(ToolManagerCommands.EnableQueryTool, UPCGQueryTool::StaticGetToolTag().ToString(), QueryToolBuilder);
	}

	{
		UPCGQueryToolBuilder* QueryToolBuilder = NewObject<UPCGQueryToolBuilder>();
		QueryToolBuilder->SetToolClass(UPCGIsolateTool::StaticClass());
		RegisterTool(ToolManagerCommands.EnableIsolateTool, UPCGIsolateTool::StaticGetToolTag().ToString(), QueryToolBuilder);
	}

	{
		UPCGManualEditToolBuilder* ManualEditToolBuilder = NewObject<UPCGManualEditToolBuilder>();
		RegisterTool(ToolManagerCommands.EnableManualEditTool, UPCGManualEditTool::StaticGetToolTag().ToString(), ManualEditToolBuilder);
	}

	ToolManager->SelectActiveToolType(EToolSide::Left, "DrawSpline");

	ConfigureRealTimeViewportsOverride(/*bEnable=*/true);
}

void UPCGEditorMode::Exit()
{
	// Clear realtime viewport override
	ConfigureRealTimeViewportsOverride(/*bEnable=*/false);

	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		ToolManager->OnToolShutdownRequest.Unbind();
		ToolManager->OnToolPostBuild.RemoveAll(this);
	}

	// Call base Exit method to ensure proper cleanup
	UEdMode::Exit();
}

bool UPCGEditorMode::ShouldToolStartBeAllowed(const FString& ToolIdentifier) const
{
	UInteractiveToolManager* ToolManager = GetToolManager();
	if (ToolManager && ToolManager->HasActiveTool(EToolSide::Left) && ToolManager->GetActiveToolName(EToolSide::Left) == ToolIdentifier)
	{
		return false;
	}
	// @todo_pcg: As more tools are added, some may require scene compatibility.

	return Super::ShouldToolStartBeAllowed(ToolIdentifier);
}

bool UPCGEditorMode::HasCustomViewportFocus() const
{
	if (Super::HasCustomViewportFocus())
	{
		return true;
	}

	// @todo_pcg: Some features may want to override this.

	return false;
}

FBox UPCGEditorMode::ComputeCustomViewportFocus() const
{
	auto ProcessFocusBoxFunc = [](FBox& FocusBoxInOut)
	{
		const double MaxDimension = FocusBoxInOut.GetExtent().GetMax();
		FocusBoxInOut = FocusBoxInOut.ExpandBy(MaxDimension * 0.2);
	};

	FBox FocusBox = Super::ComputeCustomViewportFocus();
	if (FocusBox.IsValid)
	{
		ProcessFocusBoxFunc(FocusBox);
		return FocusBox;
	}

	// @todo_pcg: Some features may want to override this.

	// No custom focus box. Return a default (invalid) box
	return FBox();
}

bool UPCGEditorMode::IsSelectionAllowed(AActor* InActor, bool bInSelection) const
{
	if (!bIsToolActive)
	{
		return true;
	}

	if (const UPCGInteractiveToolBaseSettings* ToolSettings = GetCurrentToolSettings())
	{
		return ToolSettings->IsSelectionAllowed(InActor, bInSelection);
	}

	return false;
}

void UPCGEditorMode::CreateToolkit()
{
	Toolkit = MakeShared<FPCGEditorModeToolkit>();
}

TSharedRef<FLegacyEdModeWidgetHelper> UPCGEditorMode::CreateWidgetHelper()
{
	TSharedRef<FPCGEditorModeWidgetHelper> Helper = MakeShared<FPCGEditorModeWidgetHelper>();
	Helper->PCGEditorMode = this;

	return Helper;
}

bool UPCGEditorMode::ActiveToolUsesTransformGizmo() const
{
	UInteractiveToolManager* ToolManager = GetToolManager();
	if (!ToolManager || !ToolManager->HasActiveTool(EToolSide::Left))
	{
		return false;
	}

	// Generic tool opt-in. Any PCG tool that overrides UsesTransformGizmo() gets the standard transform gizmo.
	const UPCGInteractiveTool* ActiveTool = Cast<UPCGInteractiveTool>(ToolManager->GetActiveTool(EToolSide::Left));
	return ActiveTool && ActiveTool->UsesTransformGizmo();
}

void UPCGEditorMode::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	// Disable slate throttling so that Tool background computes responding to sliders can properly be processed
	// on Tool Tick. Otherwise, when a Tool kicks off a background update in a background thread, the computed
	// result will be ignored until the user moves the slider, ie you cannot hold down the mouse and wait to see
	// the result. This apparently broken behavior is currently by-design.
	FSlateThrottleManager::Get().DisableThrottle(true);

	bIsToolActive = true;
	bEnteredToolSinceLastTick = true;
}

void UPCGEditorMode::ModeTick(float DeltaTime)
{
	Super::ModeTick(DeltaTime);

	if (bEnteredToolSinceLastTick)
	{
		bEnteredToolSinceLastTick = false;

		if (!GetDefault<UPCGEditorModeSettings>()->bDisableTemporalAntiAliasingWhenEnteringTool)
		{
			// Undo setting the TAA to false as it doesn't look good in the PCG case.
			// This is an override on top of what's done in UEditorInteractiveToolsContext::SetEditorStateForTool()
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
			if (LevelEditor.IsValid())
			{
				TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
				for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
				{
					if (ViewportWindow.IsValid())
					{
						FEditorViewportClient& Viewport = ViewportWindow->GetAssetViewportClient();
						Viewport.EnableOverrideEngineShowFlags([](FEngineShowFlags& Flags)
						{
							//Flags.SetTemporalAA(false); // <- purposefully not disabled
							Flags.SetMotionBlur(false);
							// disable this as depending on fixed exposure settings the entire scene may turn black
							//Flags.SetEyeAdaptation(false);
						});
					}
				}
			}
		}
	}
}

void UPCGEditorMode::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	bIsToolActive = false;

	// Give the editor back control over pivot location
	GUnrealEd->UpdatePivotLocationForSelection();

	// Re-enable slate throttling (see OnToolStarted)
	FSlateThrottleManager::Get().DisableThrottle(false);
}

void UPCGEditorMode::OnToolPostBuild(
	UInteractiveToolManager* InToolManager,
	EToolSide InSide,
	UInteractiveTool* InBuiltTool,
	UInteractiveToolBuilder* InToolBuilder,
	const FToolBuilderState& ToolState)
{
	// @todo_pcg: Anything the editor mode might need to track after a tool is built can go here.
}

void UPCGEditorMode::OnEditorClosed() const
{
	// On editor close, Exit() should run to clean up, but this happens very late.
	// Close out any active Tools or Selections to mitigate any late-destruction issues.

	UInteractiveToolManager* ToolManager = GetToolManager();

	if (GetModeManager() != nullptr
		&& GetInteractiveToolsContext() != nullptr
		&& ToolManager != nullptr
		&& ToolManager->HasAnyActiveTool())
	{
		ToolManager->DeactivateTool(EToolSide::Mouse, EToolShutdownType::Cancel);
	}
}

void UPCGEditorMode::AcceptActiveToolActionOrTool() const
{
	UInteractiveToolManager* ToolManager = GetToolManager();

	if (ToolManager && ToolManager->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = ToolManager->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolNestedAcceptCancelAPI* CancelAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
		if (CancelAPI && CancelAPI->SupportsNestedAcceptCommand() && CancelAPI->CanCurrentlyNestedAccept())
		{
			if (CancelAPI->ExecuteNestedAcceptCommand())
			{
				return;
			}
		}
	}

	UEditorInteractiveToolsContext* ToolsContext = GetInteractiveToolsContext();
	const EToolShutdownType ShutdownType = ToolsContext->CanAcceptActiveTool() ? EToolShutdownType::Accept : EToolShutdownType::Completed;
	ToolsContext->EndTool(ShutdownType);
}

void UPCGEditorMode::CancelActiveToolActionOrTool() const
{
	UInteractiveToolManager* ToolManager = GetToolManager();

	if (ToolManager && ToolManager->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = ToolManager->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolNestedAcceptCancelAPI* CancelAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
		if (CancelAPI && CancelAPI->SupportsNestedCancelCommand() && CancelAPI->CanCurrentlyNestedCancel())
		{
			if (CancelAPI->ExecuteNestedCancelCommand())
			{
				return;
			}
		}
	}

	UEditorInteractiveToolsContext* ToolsContext = GetInteractiveToolsContext();
	const EToolShutdownType ShutdownType = ToolsContext->CanCancelActiveTool() ? EToolShutdownType::Cancel : EToolShutdownType::Completed;
	ToolsContext->EndTool(ShutdownType);
}

void UPCGEditorMode::ConfigureRealTimeViewportsOverride(const bool bEnable)
{
	const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
		for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
		{
			if (ViewportWindow.IsValid())
			{
				FEditorViewportClient& Viewport = ViewportWindow->GetAssetViewportClient();
				if (bEnable)
				{
					Viewport.AddRealtimeOverride(bEnable, PCGEditorMode::Constants::PCGEditorModeDisplayName);
				}
				else
				{
					Viewport.RemoveRealtimeOverride(PCGEditorMode::Constants::PCGEditorModeDisplayName, false);
				}
			}
		}
	}
}

UPCGInteractiveToolBaseSettings* UPCGEditorMode::GetCurrentToolSettings() const
{
	UInteractiveToolManager* ToolManager = GetToolManager();

	if (ToolManager && ToolManager->HasAnyActiveTool())
	{
		if (UInteractiveTool* Tool = ToolManager->GetActiveTool(EToolSide::Mouse))
		{
			TArray<UObject*> AllToolProperties = Tool->GetToolProperties(false);

			UObject** FoundProperties = AllToolProperties.FindByPredicate([](const UObject* Candidate)
			{
				return Candidate->IsA<UPCGInteractiveToolBaseSettings>();
			});

			if (FoundProperties)
			{
				return Cast<UPCGInteractiveToolBaseSettings>(*FoundProperties);
			}
		}
	}

	return nullptr;
}

// @todo_pcg: Activate when custom tools are supported.
// void UPCGEditorMode::RegisterCustomTool(
// 	TSharedPtr<FUICommandInfo> UICommand,
// 	FString ToolIdentifier,
// 	UInteractiveToolBuilder* Builder,
// 	const TFunction<bool(UInteractiveToolManager*, EToolSide)>& ExecuteAction,
// 	const TFunction<bool(UInteractiveToolManager*, EToolSide)>& CanExecuteAction,
// 	const TFunction<bool(UInteractiveToolManager*, EToolSide)>& IsActionChecked) const
// {
// 	if (!Toolkit.IsValid())
// 	{
// 		return;
// 	}
//
// 	constexpr EToolsContextScope ToolScope = EToolsContextScope::EdMode;
// 	UEditorInteractiveToolsContext* UseToolsContext = GetInteractiveToolsContext(ToolScope);
// 	if (ensure(UseToolsContext != nullptr) == false)
// 	{
// 		return;
// 	}
//
// 	constexpr EToolSide ToolSide = EToolSide::Left;
// 	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
// 	UseToolsContext->ToolManager->RegisterToolType(ToolIdentifier, Builder);
// 	CommandList->MapAction(UICommand,
// 		FExecuteAction::CreateWeakLambda(UseToolsContext, [this, ToolIdentifier, UseToolsContext, ExecuteAction]
// 		{
// 			if (!ExecuteAction || !ExecuteAction(UseToolsContext->ToolManager, ToolSide))
// 			{
// 				UseToolsContext->StartTool(ToolIdentifier);
// 			}
// 		}),
// 		FCanExecuteAction::CreateWeakLambda(UseToolsContext, [this, ToolIdentifier, UseToolsContext, CanExecuteAction]()
// 		{
// 			bool bResult = false;
// 			if (ShouldToolStartBeAllowed(ToolIdentifier))
// 			{
// 				bResult = CanExecuteAction ? CanExecuteAction(UseToolsContext->ToolManager, ToolSide) : UseToolsContext->ToolManager->CanActivateTool(ToolSide, ToolIdentifier);
// 			}
// 			return bResult;
// 		}),
// 		FIsActionChecked::CreateWeakLambda(UseToolsContext, [this, ToolIdentifier, UseToolsContext, IsActionChecked]()
// 		{
// 			if (IsActionChecked)
// 			{
// 				return IsActionChecked(UseToolsContext->ToolManager, ToolSide);
// 			}
//
// 			if (UseToolsContext->ToolManager->GetActiveTool(ToolSide))
// 			{
// 				// Read the ActiveToolType rather than the ActiveToolName so that clients can leverage
// 				// that storage on the ToolsContext through SelectActiveToolType to notify of changes
// 				// in the active tool without actually requiring a tool change.
// 				const FString& ActiveToolIdentifier = UseToolsContext->ToolManager->GetActiveToolType(ToolSide);
// 				return ActiveToolIdentifier == ToolIdentifier;
// 			}
// 			return false;
// 		}),
// 		EUIActionRepeatMode::RepeatDisabled);
// }

#undef LOCTEXT_NAMESPACE
