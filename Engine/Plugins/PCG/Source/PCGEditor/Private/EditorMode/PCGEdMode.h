// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdMode.h"
#include "InteractiveToolManager.h"
#include "Tools/LegacyEdModeWidgetHelpers.h"

#include "PCGEdMode.generated.h"

class UPCGInteractiveToolBaseSettings;

/** The unique state for the editor to be in, specific to PCG's toolkits. */
UCLASS(Transient)
class UPCGEditorMode : public UBaseLegacyWidgetEdMode
{
	GENERATED_BODY()

public:
	/** Unique name for PCG Editor Mode */
	const static FEditorModeID EM_PCGEditorModeId;

	UPCGEditorMode();

	static void RegisterEditorMode();
	static void UnregisterEditorMode();

	//~Begin UEdMode interface
	virtual bool ProcessEditDelete() override;
	virtual bool ProcessEditCut() override;
	virtual bool CanAutoSave() const override;
	/** Override the viewport pivot, if needed. */
	virtual bool GetPivotForOrbit(FVector& OutPivot) const override;
	/** Focus event when the Editor enters PCG Editor Mode. */
	virtual void Enter() override;
	/** Focus event when the Editor leaves PCG Editor Mode. */
	virtual void Exit() override;
	/** Used to restrict starting a tool, if a certain state is required beforehand. */
	virtual bool ShouldToolStartBeAllowed(const FString& ToolIdentifier) const override;
	virtual bool HasCustomViewportFocus() const override;
	virtual FBox ComputeCustomViewportFocus() const override;
	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const override;
	virtual void ModeTick(float DeltaTime) override;
	//~End UEdMode interface

	UPCGInteractiveToolBaseSettings* GetCurrentToolSettings() const;

	/** True when the active PCG tool opts into the standard transform gizmo. */
	bool ActiveToolUsesTransformGizmo() const;

protected:
	virtual void CreateToolkit() override;
	virtual TSharedRef<FLegacyEdModeWidgetHelper> CreateWidgetHelper() override;

	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolPostBuild(UInteractiveToolManager* InToolManager, EToolSide InSide, UInteractiveTool* InBuiltTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& ToolState);

	void OnEditorClosed() const;
	void AcceptActiveToolActionOrTool() const;
	void CancelActiveToolActionOrTool() const;

	void ConfigureRealTimeViewportsOverride(bool bEnable);

private:
	// @todo_pcg: Activate when custom tools are supported.
	// void RegisterCustomTool(
	// 	TSharedPtr<FUICommandInfo> UICommand,
	// 	FString ToolIdentifier,
	// 	UInteractiveToolBuilder* Builder,
	// 	const TFunction<bool(UInteractiveToolManager*, EToolSide)>& ExecuteAction,
	// 	const TFunction<bool(UInteractiveToolManager*, EToolSide)>& CanExecuteAction,
	// 	const TFunction<bool(UInteractiveToolManager*, EToolSide)>& IsActionChecked) const;

	bool bIsToolActive = false;
	bool bEnteredToolSinceLastTick = false;
};
