// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/PCGEdModeCommands.h"

#include "PCGEditorModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorMode/PCGEdModeSettings.h"
#include "EditorMode/PCGEdModeStyle.h"
#include "EditorMode/Tools/Helpers/PCGEdModeEditorUtilities.h"
#include "EditorMode/Tools/ManualEdit/PCGManualEditTool.h"
#include "EditorMode/Tools/Line/PCGDrawSplineTool.h"
#include "EditorMode/Tools/Paint/PCGPaintTool.h"
#include "EditorMode/Tools/Query/PCGQueryTool.h"
#include "EditorMode/Tools/Volume/PCGVolumeTool.h"
#include "Logging/StructuredLog.h"
#include "Misc/OutputDeviceNull.h"

#define LOCTEXT_NAMESPACE "PCGEditorMode"

struct FSlateIcon;

FPCGEditorModeCommands::FPCGEditorModeCommands()
	: TCommands(/*InContextName=*/TEXT("PCGEditorModeCommands")
							  , /*InContextDesc=*/LOCTEXT("PCGEditorModeCommands", "PCG Editor Mode")
							  , /*InContextParent=*/NAME_None
							  , /*InStyleSetName=*/FPCGEditorModeStyle::Get().GetStyleSetName()) {}

void FPCGEditorModeCommands::RegisterCommands() {}

FPCGEditorModePaletteCommands::FPCGEditorModePaletteCommands()
	: TCommands(TEXT("PCGEditorModePaletteCommands")
			, LOCTEXT("PCGEditorModePaletteCommands", "PCG Editor Mode - Palette")
			, /*Parent=*/NAME_None
			, /*IconStyleSet=*/FPCGEditorModeStyle::Get().GetStyleSetName()) {}

/**
 * @param CommandInfoSP The shared pointer for the FUICommandInfo
 * @param Name The name of the tool
 * @param IconStyleName The style name for the icon to be used for the button
 */
#define PCG_CONTEXT_COMMAND_INFO(CommandInfoSP, CommandName, CommandLabel, ToolTip, IconStyleName)\
FUICommandInfo::MakeCommandInfo(AsShared()\
	, CommandInfoSP\
	, CommandName\
	, CommandLabel\
	, ToolTip\
	, FSlateIcon(FPCGEditorModeStyle::Get().GetStyleSetName(), IconStyleName)\
	, EUserInterfaceActionType::ToggleButton\
	, FInputChord())

void FPCGEditorModePaletteCommands::RegisterCommands()
{
	/*** PCG EDITOR CONTEXTS **/
	PCG_CONTEXT_COMMAND_INFO(LoadSplineContextPalette, TEXT("EnterSplineContext"), LOCTEXT("SplineContextLabel", "Spline"), LOCTEXT("SplineContextToolTip", "Enter 'Spline' context and open corresponding toolkit"), "PCGEditorMode.Context.DrawSpline");
	PCG_CONTEXT_COMMAND_INFO(LoadPaintContextPalette, TEXT("EnterPaintContext"), LOCTEXT("PaintContextLabel", "Paint"), LOCTEXT("PaintContextToolTip", "Enter 'Paint' context and open corresponding toolkit"), "PCGEditorMode.Context.Paint");
	PCG_CONTEXT_COMMAND_INFO(LoadVolumeContextPalette, TEXT("EnterVolumeContext"), LOCTEXT("VolumeContextLabel", "Volume"), LOCTEXT("VolumeContextToolTip", "Enter 'Volume' context and open corresponding toolkit"), "PCGEditorMode.Context.Volume");
	PCG_CONTEXT_COMMAND_INFO(LoadQueryContextPalette, TEXT("EnterQueryContext"), LOCTEXT("QueryContextLabel", "Query"), LOCTEXT("QueryContextToolTip", "Enter 'Query' context and open corresponding toolkit"), "PCGEditorMode.Context.Query");
	PCG_CONTEXT_COMMAND_INFO(LoadManualContextPalette, TEXT("EnterManualContext"), LOCTEXT("ManualContextLabel", "Manual"), LOCTEXT("ManualContextToolTip", "Enter 'Manual' context and open corresponding toolkit"), "PCGEditorMode.Context.Manual");
}

#undef PCG_CONTEXT_COMMAND_INFO

FPCGEditorModeToolCommands::FPCGEditorModeToolCommands()
	: TInteractiveToolCommands(TEXT("PCGEditorModeToolCommands")
						   , LOCTEXT("PCGEditorModeToolCommands", "PCG Editor Mode - Tool Commands")
						   , /*Parent=*/NAME_None
						   , /*IconStyleSet=*/FPCGEditorModeStyle::Get().GetStyleSetName()) {}

/**
  * @param CommandInfoSP The shared pointer for the FUICommandInfo
 * @param Name The name of the tool
 * @param Label The user-forward label for the tool
 * @param ToolTip The tooltip for the tool: Format can include {ToolTag}.
 * @param ToolTag The tag for the tool: can be used in the tooltip.
 * @param IconStyleName The style name for the icon to be used for the button
 */
#define PCG_TOOL_COMMAND_INFO(CommandInfoSP, CommandName, CommandLabel, ToolTip, ToolTag, IconStyleName)\
FUICommandInfo::MakeCommandInfo(AsShared()\
	, CommandInfoSP\
	, CommandName\
	, CommandLabel\
	, FText::Format(ToolTip, FFormatNamedArguments({{TEXT("ToolTag"), FText::FromName(ToolTag)}}))\
	, FSlateIcon(FPCGEditorModeStyle::Get().GetStyleSetName(), IconStyleName)\
	, EUserInterfaceActionType::ToggleButton\
	, FInputChord())

void FPCGEditorModeToolCommands::RegisterCommands()
{
	TInteractiveToolCommands::RegisterCommands();

	/*** PCG EDITOR MODE TOOLS **/

	PCG_TOOL_COMMAND_INFO(EnableDrawSplineTool, "DrawSplineTool", LOCTEXT("DrawSplineToolLabel", "Draw Spline"), LOCTEXT("DrawSplineToolTip", "Use the '{ToolTag}' tool to build a spline for PCG."), UPCGInteractiveToolSettings_Spline::StaticGetToolTag(), "PCGEditorMode.Tools.DrawSpline");
	PCG_TOOL_COMMAND_INFO(EnableDrawSurfaceTool, "DrawSurfaceTool", LOCTEXT("DrawSurfaceToolLabel", "Draw Spline Surface"), LOCTEXT("DrawSurfaceToolTip", "Use the '{ToolTag}' tool to build a closed spline representing an area for PCG."), UPCGInteractiveToolSettings_SplineSurface::StaticGetToolTag(), "PCGEditorMode.Tools.DrawSurface");
	PCG_TOOL_COMMAND_INFO(EnablePaintTool, "PaintTool", LOCTEXT("PaintToolLabel", "Paint"), LOCTEXT("PaintToolTip", "Use the '{ToolTag}' tool to apply PCG points with a brush."), UPCGInteractiveToolSettings_PaintTool::StaticGetToolTag(), "PCGEditorMode.Context.Paint");
	PCG_TOOL_COMMAND_INFO(EnableVolumeTool, "VolumeTool", LOCTEXT("VolumeToolLabel", "Volume"), LOCTEXT("VolumeToolTip", "Create or updates a volume in the scene using the '{ToolTag}' tool."), UPCGInteractiveToolSettings_Volume::StaticGetToolTag(), "PCGEditorMode.Tools.Volume");
	PCG_TOOL_COMMAND_INFO(EnableQueryTool, "QueryTool", LOCTEXT("QueryToolLabel", "Query"), LOCTEXT("QueryToolTip", "Queries scene data from raycasts."), UPCGQueryTool::StaticGetToolTag(), "PCGEditorMode.Tools.Query");
	PCG_TOOL_COMMAND_INFO(EnableIsolateTool, "IsolateTool", LOCTEXT("IsolateToolLabel", "Isolate"), LOCTEXT("IsolateToolTip", "Isolates content from the actor on a tag basis."), UPCGIsolateTool::StaticGetToolTag(), "PCGEditorMode.Tools.Isolate");
	PCG_TOOL_COMMAND_INFO(EnableManualEditTool, "ManualEditTool", LOCTEXT("ManualEditToolLabel", "Manual Edit"), LOCTEXT("ManualEditToolTip", "Use the '{ToolTag}' tool to manually edit Static Mesh Spawner instances on the selected actor's PCG components."), UPCGInteractiveToolSettings_ManualEdit::StaticGetToolTag(), "PCGEditorMode.Tools.ManualEdit");
}

void FPCGEditorModeToolCommands::RegisterAllToolCommands()
{
	FPCGDrawSplineCommands::Register();
	FPCGPaintCommands::Register();
	FPCGVolumeCommands::Register();
	FPCGQueryCommands::Register();
	FPCGIsolateCommands::Register();
	FPCGManualEditCommands::Register();
}

void FPCGEditorModeToolCommands::UnregisterAllToolCommands()
{
	FPCGManualEditCommands::Unregister();
	FPCGIsolateCommands::Unregister();
	FPCGQueryCommands::Unregister();
	FPCGVolumeCommands::Unregister();
	FPCGPaintCommands::Unregister();
	FPCGDrawSplineCommands::Unregister();
}

void FPCGEditorModeToolCommands::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs)
{
	// @todo_pcg: To be populated with tool commands for bindings.
}

TArray<TSharedPtr<FUICommandInfo>> FPCGEditorModeToolCommands::GetLineContextToolCommands()
{
	return {EnableDrawSplineTool, EnableDrawSurfaceTool};
}

TArray<TSharedPtr<FUICommandInfo>> FPCGEditorModeToolCommands::GetPaintContextToolCommands()
{
	return {EnablePaintTool};
}

TArray<TSharedPtr<FUICommandInfo>> FPCGEditorModeToolCommands::GetVolumeContextToolCommands()
{
	return {EnableVolumeTool};
}

TArray<TSharedPtr<FUICommandInfo>> FPCGEditorModeToolCommands::GetQueryContextToolCommands()
{
	return { EnableQueryTool, EnableIsolateTool };
}

TArray<TSharedPtr<FUICommandInfo>> FPCGEditorModeToolCommands::GetManualEditContextToolCommands()
{
	return { EnableManualEditTool };
}

/**
 * Macro for defining a constructor and a helper function to retrieve
 * @param ToolCommandClassName The tool's class name
 * @param ToolNameLabel A name string for the command context
 * @param ToolClassName The class name of the tool
 */
#define PCG_DEFINE_TOOL_COMMANDS(ToolCommandClassName, Label, Name, ToolClassName )\
ToolCommandClassName::ToolCommandClassName()\
	: TInteractiveToolCommands<ToolCommandClassName>(\
		Name,\
		NSLOCTEXT("PCGEditorModeTools", "PCG" Label "ToolCommands", "PCG Editor Tools - '" Name "' Tool"),\
		NAME_None,\
		FPCGEditorModeStyle::Get().GetStyleSetName()) {}\
void ToolCommandClassName::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs)\
{\
ToolCDOs.Add(GetMutableDefault<ToolClassName>());\
}
PCG_DEFINE_TOOL_COMMANDS(FPCGDrawSplineCommands, "DrawSpline", "Draw Spline", UPCGDrawSplineToolBase);
PCG_DEFINE_TOOL_COMMANDS(FPCGDrawSurfaceCommands, "DrawSurface", "Draw Surface", UPCGDrawSplineToolBase);
PCG_DEFINE_TOOL_COMMANDS(FPCGPaintCommands, "Paint", "Paint", UPCGPaintTool);
PCG_DEFINE_TOOL_COMMANDS(FPCGVolumeCommands, "Volume", "Volume", UPCGVolumeTool);
PCG_DEFINE_TOOL_COMMANDS(FPCGQueryCommands, "Query", "Query", UPCGQueryTool);
PCG_DEFINE_TOOL_COMMANDS(FPCGIsolateCommands, "Isolate", "Isolate", UPCGIsolateTool);
PCG_DEFINE_TOOL_COMMANDS(FPCGManualEditCommands, "ManualEdit", "Manual Edit", UPCGManualEditTool);

#undef PCG_DEFINE_TOOL_COMMANDS

#undef PCG_TOOL_COMMAND_INFO

#undef LOCTEXT_NAMESPACE
