// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Tools/InteractiveToolsCommands.h"

/** General actions related to the PCG Editor Mode. */
class FPCGEditorModeCommands : public TCommands<FPCGEditorModeCommands>
{
public:
	FPCGEditorModeCommands();

	virtual void RegisterCommands() override;

	// @todo_pcg: Any file menu, hotkeys, etc related to the editor mode in general can go here.
};

/** Actions related to the PCG Editor Mode Toolkit Palette. */
class FPCGEditorModePaletteCommands : public TCommands<FPCGEditorModePaletteCommands>
{
public:
	FPCGEditorModePaletteCommands();

	/*** PCG EDITOR CONTEXTS **/
	TSharedPtr<FUICommandInfo> LoadSplineContextPalette;
	TSharedPtr<FUICommandInfo> LoadPaintContextPalette;
	TSharedPtr<FUICommandInfo> LoadVolumeContextPalette;
	TSharedPtr<FUICommandInfo> LoadQueryContextPalette;
	TSharedPtr<FUICommandInfo> LoadManualContextPalette;

	// @todo_pcg: Selection/Filtering actions.

	virtual void RegisterCommands() override;

	// @todo_pcg: Allow extension for custom tools. See Modeling Tools as an example.
};

/** Tool specific actions. */
class FPCGEditorModeToolCommands : public TInteractiveToolCommands<FPCGEditorModeToolCommands>
{
public:
	FPCGEditorModeToolCommands();

	virtual void RegisterCommands() override;

	/** Register all Tool command sets. Call this in module startup */
	static void RegisterAllToolCommands();

	/** Unregister all Tool command sets. Call this from module shutdown. */
	static void UnregisterAllToolCommands();

	/** Needed to retrieve the CDOs to call GetActionSet directly on the tool. */
	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override;

	// @todo_pcg: Implement if needed for custom tool commands.
	/**
	 * Add the tool-specific commands from an interactive tool to a specified UICommandList 
	 * Call this when the active tool changes (eg on ToolManager.OnToolStarted / OnToolEnded)
	 */
	// static void AddToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList);
	// static void RemoveToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList);

	/*** PCG EDITOR MODE TOOLS - Line Context **/
	TSharedPtr<FUICommandInfo> EnableDrawSplineTool;
	TSharedPtr<FUICommandInfo> EnableDrawSurfaceTool;
	TArray<TSharedPtr<FUICommandInfo>> GetLineContextToolCommands();

	/*** PCG EDITOR MODE TOOLS - Paint Context **/
	TSharedPtr<FUICommandInfo> EnablePaintTool;
	TArray<TSharedPtr<FUICommandInfo>> GetPaintContextToolCommands();

	/*** PCG EDITOR MODE TOOLS - Volume Context **/
	TSharedPtr<FUICommandInfo> EnableVolumeTool;
	TArray<TSharedPtr<FUICommandInfo>> GetVolumeContextToolCommands();

	/*** PCG EDITOR MODE TOOLS - Query Context **/
	TSharedPtr<FUICommandInfo> EnableQueryTool;
	TSharedPtr<FUICommandInfo> EnableIsolateTool;
	TArray<TSharedPtr<FUICommandInfo>> GetQueryContextToolCommands();

	/*** PCG EDITOR MODE TOOLS - Manual Edit Context **/
	TSharedPtr<FUICommandInfo> EnableManualEditTool;
	TArray<TSharedPtr<FUICommandInfo>> GetManualEditContextToolCommands();
};

/**
 * Macro for the class declaration for each tool as a TInteractiveToolCommands class
 * @todo_pcg: This is basically done elsewhere for other toolkits, but can probably be modernized using C++ Concepts 
 * @param ToolCommandClassName The tool's class name
 */
#define DECLARE_TOOL_COMMANDS(ToolCommandClassName) \
class ToolCommandClassName : public TInteractiveToolCommands<ToolCommandClassName> \
{\
public:\
ToolCommandClassName();\
\
virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override;\
};\

// Define all tools commands here
DECLARE_TOOL_COMMANDS(FPCGDrawSplineCommands);
DECLARE_TOOL_COMMANDS(FPCGDrawSurfaceCommands);
DECLARE_TOOL_COMMANDS(FPCGPaintCommands);
DECLARE_TOOL_COMMANDS(FPCGVolumeCommands);
DECLARE_TOOL_COMMANDS(FPCGQueryCommands);
DECLARE_TOOL_COMMANDS(FPCGIsolateCommands);
DECLARE_TOOL_COMMANDS(FPCGManualEditCommands);

#undef DECLARE_TOOL_COMMANDS
