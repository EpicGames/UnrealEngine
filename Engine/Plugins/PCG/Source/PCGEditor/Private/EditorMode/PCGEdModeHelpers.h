// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

namespace PCG::EditorMode
{
	static constexpr float DefaultNotificationDuration = 2.5f;

	inline void DispatchEditorNotification(const FText& Text, const FText& SubText = FText{}, const float Duration = DefaultNotificationDuration)
	{
		FNotificationInfo NotificationInfo(std::move(Text));
		NotificationInfo.ExpireDuration = Duration;
		NotificationInfo.bFireAndForget = true;
		NotificationInfo.SubText = std::move(SubText);
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}
}
