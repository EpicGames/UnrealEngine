// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/PCGEdModeSettings.h"

#include "PCGGraph.h"
#include "EditorMode/Tools/Line/PCGDrawSplineTool.h"
#include "EditorMode/Tools/Paint/PCGPaintTool.h"
#include "EditorMode/Tools/Volume/PCGVolumeTool.h"

UPCGEditorModeSettings::UPCGEditorModeSettings(const FObjectInitializer& ObjectInitializer)
	: UDeveloperSettings(ObjectInitializer)
{
	FPCGPerInteractiveToolSettingSettings& DrawSpline = InteractiveToolSettings.Emplace_GetRef();
	DrawSpline.SettingsClass = UPCGInteractiveToolSettings_Spline::StaticClass();
	DrawSpline.DefaultGraph = TSoftObjectPtr<UPCGGraphInterface>(FSoftObjectPath(TEXT("/PCG/EdMode/DrawSpline/SpacingTool.SpacingTool")));

	FPCGPerInteractiveToolSettingSettings& DrawSplineSurface = InteractiveToolSettings.Emplace_GetRef();
	DrawSplineSurface.SettingsClass = UPCGInteractiveToolSettings_SplineSurface::StaticClass();
	DrawSplineSurface.DefaultGraph = TSoftObjectPtr<UPCGGraphInterface>(FSoftObjectPath(TEXT("/PCG/EdMode/DrawSplineSurface/AreaTool.AreaTool")));

	FPCGPerInteractiveToolSettingSettings& PaintTool = InteractiveToolSettings.Emplace_GetRef();
	PaintTool.SettingsClass = UPCGInteractiveToolSettings_PaintTool::StaticClass();
	PaintTool.DefaultGraph = TSoftObjectPtr<UPCGGraphInterface>(FSoftObjectPath(TEXT("/PCG/EdMode/Paint/PlaceTool.PlaceTool")));

	FPCGPerInteractiveToolSettingSettings& CreateVolume = InteractiveToolSettings.Emplace_GetRef();
	CreateVolume.SettingsClass = UPCGInteractiveToolSettings_Volume::StaticClass();
	CreateVolume.DefaultGraph = TSoftObjectPtr<UPCGGraphInterface>(FSoftObjectPath(TEXT("/PCG/EdMode/Volume/ScatterTool.ScatterTool")));
}