// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "PCGEdModeSettings.generated.h"

class UPCGInteractiveToolSettings;
class UPCGGraphInterface;

USTRUCT()
struct FPCGPerInteractiveToolSettingSettings
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Config, Category = "Default")
	TSoftClassPtr<UPCGInteractiveToolSettings> SettingsClass;

	// @todo_pcg GetAssetFilter only works on UClasses due to need for UFUNCTION. Will reenable this via details customization instead.
	//UPROPERTY(EditAnywhere, Config, Category = "Default", meta=(GetAssetFilter = "FilterBySettingsClass"))
	UPROPERTY(EditAnywhere, Config, Category = "Default")
	TSoftObjectPtr<UPCGGraphInterface> DefaultGraph;
};

/** User preferences related to the PCG Editor Mode. */
UCLASS(Config = EditorPerProjectUserSettings, DisplayName = "PCG Editor Mode Settings")
class UPCGEditorModeSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPCGEditorModeSettings(const FObjectInitializer& ObjectInitializer);

	virtual FName GetContainerName() const override { return FName("Editor"); }
	virtual FName GetCategoryName() const override { return FName("Plugins"); }
	virtual FName GetSectionName() const override { return FName("PCGEditorMode"); }

	/** Editor Mode user configurations. */
	UPROPERTY(EditAnywhere, Config, Category = "General", meta = (DisplayName = "Enable the PCG Editor Mode Toolkit (requires restart)", ConfigRestartRequired = true))
	bool bEnableEditorMode = true;

	/** The rate at which PCG Graphs will be refreshed while actively editing with a tool. */
	UPROPERTY(EditAnywhere, Category = "Component", meta = (
		ForceUnits = "Seconds",
		ConsoleVariable = "pcg.editormode.GraphRefreshRate"))
	float GraphRefreshRate = 0.5f;

	/** Hide the tool buttons from the UI when a tool is actively being used. */
	UPROPERTY(EditAnywhere, Config, Category = "Tools", meta = (
		EditCondition = "bEnableEditorMode",
		EditConditionHides,
		ConsoleVariable = "pcg.editormode.HideToolButtonsDuringActiveTool"))
	bool bHideToolButtonsDuringActiveTool = false;

	/** Enable an editor toast issued when an Editor Mode tool has an error. */
	UPROPERTY(EditAnywhere, Config, Category = "Tools", meta = (
		EditCondition = "bEnableEditorMode",
		EditConditionHides,
		ConsoleVariable = "pcg.editormode.ShowEditorToastOnToolErrors"))
	bool bShowEditorToastOnToolErrors = true;

	UPROPERTY(EditAnywhere, Config, Category = "Tools", meta = (EditCondition = "bEnableEditorMode", EditConditionHides))
	bool bDisableTemporalAntiAliasingWhenEnteringTool = false;

	UPROPERTY(EditAnywhere, Category = "Tools|Output", meta = (EditCondition = "bEnableEditorMode", EditConditionHides))
	FName DefaultNewActorName = "PCGToolActor";

	UPROPERTY(EditAnywhere, Category = "Tools|Output", meta = (EditCondition = "bEnableEditorMode", EditConditionHides))
	FName DefaultNewPCGComponentName = "PCGToolComponent";

	UPROPERTY(EditAnywhere, Category = "Tools|Output|Spline", meta = (EditCondition = "bEnableEditorMode", EditConditionHides))
	FName DefaultNewSplineComponentName = "PCGToolSplineComponent";

	UPROPERTY(EditAnywhere, Config, Category = "Tools")
	TArray<FPCGPerInteractiveToolSettingSettings> InteractiveToolSettings;
};
