// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/PCGEdModeStyle.h"
#include "SlateOptMacros.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"
#include "Util/ColorConstants.h"

#include "Textures/SlateIcon.h"

namespace PCGEdModeStyle::Constants
{
	constexpr TCHAR StyleSet[] = TEXT("PCGEditorModeStyle");
	constexpr TCHAR DefaultBrushStyle[] = TEXT("PCGEditorMode.Warning");
}

void FPCGEditorModeStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FPCGEditorModeStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

FPCGEditorModeStyle::FPCGEditorModeStyle()
	: FSlateStyleSet(PCGEdModeStyle::Constants::StyleSet)
{
	using namespace PCGEdModeStyle::Constants;
	// Const icon sizes
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);

	// Default back to CoreStyle
	SetParentStyleName("CoreStyle");

	FSlateStyleSet::SetContentRoot(FPaths::EngineDir() / TEXT("Plugins/PCG/Content"));
	FSlateStyleSet::SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	// General
	{
		// Set a default brush to indicate the brush is missing
		DefaultBrush = new IMAGE_BRUSH_SVG("Icons/EdMode/Warning", Icon40x40);
		Set(DefaultBrushStyle, DefaultBrush);

		// @todo_pcg: Get a 20x20 version
		Set("PCGEditorModeIcon", new IMAGE_BRUSH_SVG("Icons/PCG_64", Icon20x20));
	}

	// Editor Mode Contexts
	{
		Set("PCGEditorMode.Context.DrawSpline", new IMAGE_BRUSH_SVG("Icons/EdMode/DrawSpline", Icon20x20));
		Set("PCGEditorMode.Context.Paint", new IMAGE_BRUSH_SVG("Icons/EdMode/Paint", Icon20x20));
		Set("PCGEditorMode.Context.Volume", new IMAGE_BRUSH_SVG("Icons/EdMode/Volume", Icon20x20));
		Set("PCGEditorMode.Context.Select", new IMAGE_BRUSH_SVG("Icons/EdMode/Select", Icon20x20));
		Set("PCGEditorMode.Context.Query", new IMAGE_BRUSH_SVG("Icons/EdMode/Select", Icon20x20));
		Set("PCGEditorMode.Context.Manual", new IMAGE_BRUSH_SVG("Icons/EdMode/Manual", Icon20x20));
	}

	// Tools
	{
		Set("PCGEditorMode.Tools.DrawSpline", new IMAGE_BRUSH_SVG("Icons/EdMode/DrawSpline", Icon20x20));
		Set("PCGEditorMode.Tools.DrawSurface", new IMAGE_BRUSH_SVG("Icons/EdMode/DrawSurface", Icon20x20));
		Set("PCGEditorMode.Tools.Paint", new IMAGE_BRUSH_SVG("Icons/EdMode/Paint", Icon20x20));
		Set("PCGEditorMode.Tools.Volume", new IMAGE_BRUSH_SVG("Icons/EdMode/Volume", Icon20x20));
		Set("PCGEditorMode.Tools.Query", new IMAGE_BRUSH_SVG("Icons/EdMode/Select", Icon20x20));
		Set("PCGEditorMode.Tools.ManualEdit", new IMAGE_BRUSH_SVG("Icons/EdMode/Manual", Icon20x20));
	}

	// Backdrops and backgrounds
	{
		FLinearColor ViewportOverlayBackgroundColor = FStyleColors::Foreground.GetSpecifiedColor();
		ViewportOverlayBackgroundColor.A = 0.1f;
		Set("PCGEditorMode.OverlayBackgroundBrush", new FSlateRoundedBoxBrush(ViewportOverlayBackgroundColor, 8.0f, FStyleColors::Dropdown, 0.0));

		const FLinearColor ContextPaletteColor = FStyleColors::Dropdown.GetSpecifiedColor();
		Set("PCGEditorMode.ContextPaletteBrush",
			new FSlateRoundedBoxBrush(ContextPaletteColor, FVector4(8.0f, 0.0f, 0.0f, 8.0f), FStyleColors::Dropdown, 0.0));

		const FLinearColor ContextToolsPaletteColor = FStyleColors::Panel.GetSpecifiedColor();
		Set("PCGEditorMode.ContextToolsPaletteBrush",
			new FSlateRoundedBoxBrush(ContextToolsPaletteColor, FVector4(0.0f, 8.0f, 8.0f, 0.0f), FStyleColors::Dropdown, 0.0));
	}

	// Button styles
	{
		FCheckBoxStyle ToggleButtonStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
		ToggleButtonStyle.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f));
		ToggleButtonStyle.SetPadding(FMargin(4.0f, 2.0f));
		Set("PaletteToggleButton", ToggleButtonStyle);
	}
}

const FPCGEditorModeStyle& FPCGEditorModeStyle::Get()
{
	static FPCGEditorModeStyle Instance;
	return Instance;
}

