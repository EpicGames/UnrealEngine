// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

struct FSlateIcon;

/** Style details specific to the PCG Editor Mode. */
class FPCGEditorModeStyle : public FSlateStyleSet
{
public:
	static void Register();
	static void Unregister();

	static const FPCGEditorModeStyle& Get();

private:
	FPCGEditorModeStyle();
};
