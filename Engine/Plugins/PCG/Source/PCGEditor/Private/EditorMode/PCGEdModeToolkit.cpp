// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/PCGEdModeToolkit.h"

#include "EditorMode/PCGEdMode.h"
#include "EditorMode/PCGEdModeCommands.h"
#include "EditorMode/PCGEdModeSettings.h"
#include "EditorMode/Tools/PCGInteractiveToolSettings.h"
#include "EditorMode/Tools/Helpers/PCGEdModeEditorUtilities.h"

#include "DetailLayoutBuilder.h"
#include "EditorModeManager.h"
#include "LevelEditorViewport.h"
#include "PropertyEditorModule.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "Toolkits/BaseToolkit.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Tools/UEdMode.h"

#include "SPrimaryButton.h"
#include "Layout/SeparatorTemplates.h"
#include "Styling/SlateIconFinder.h"
#include "Tools/Line/PCGDrawSplineTool.h"
#include "Tools/ManualEdit/PCGManualEditTool.h"
#include "Tools/Paint/PCGPaintTool.h"
#include "Tools/Query/PCGQueryTool.h"
#include "Tools/Volume/PCGVolumeTool.h"
#include "Widgets/SPCGToolPresetSection.h"
#include "Widgets/Input/STextComboBox.h"

#define LOCTEXT_NAMESPACE "PCGEditorModeToolkit"

namespace PCGEdModeToolkit
{
	static TAutoConsoleVariable<bool> CVarEnableManualEditPalette(
		TEXT("pcg.Editor.EnableManualEditPalette"),
		true,
		TEXT("Whether the Manual Edit tool palette is shown in the PCG Editor Mode toolkit."));
}

FPCGEditorModeToolkitBuilder::FPCGEditorModeToolkitBuilder(FToolkitBuilderArgs& Args) : FToolkitBuilder(Args)
{
}

void FPCGEditorModeToolkitBuilder::UpdateContentForCategory(FName InActiveCategoryName, FText InActiveCategoryText)
{
	if ( !MainContentVerticalBox.IsValid() )
	{
		return;
	}
	
	if (ToolkitSections->ModeWarningArea)
	{
		MainContentVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(5)
		[
			ToolkitSections->ModeWarningArea->AsShared()
		];
	}

	if (ToolkitSections->Header)
	{
		MainContentVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(0)
		[
			ToolkitSections->Header->AsShared()
		];
	}

	MainContentVerticalBox->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Fill)
	.Padding(0)
	[
		GetToolPaletteWidget()
	];

	if (ToolkitSections->ToolPresetArea)
	{
		MainContentVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(0)
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.Padding(Style.ActiveToolTitleBorderPadding)
			[
				ToolkitSections->ToolPresetArea->AsShared()
			]
		];
	}
	
	MainContentVerticalBox->AddSlot()
	.AutoHeight()
	[
		*FSeparatorTemplates::SmallHorizontalBackgroundNoBorder()
		.BindVisibility(TAttribute<EVisibility>::CreateLambda([this]()
		 { 
		 	return ActivePaletteButtonVisibility; 
		 }))
	];

	if (ToolkitSections->ToolWarningArea)
	{
		MainContentVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(5)
		[
			ToolkitSections->ToolWarningArea->AsShared()
		];
	}

	if (ToolkitSections->ToolHeader)
	{
		MainContentVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(0)
		[
			ToolkitSections->ToolHeader->AsShared()
		];
	}

	if (ToolkitSections->DetailsView)
	{
		MainContentVerticalBox->AddSlot()
		.HAlign(HAlign_Fill)
		.FillHeight(1.f)
		[
		SNew(SBorder)
		.BorderImage(&Style.ToolDetailsBackgroundBrush)
		.Padding(0.f, 2.f, 0.f, 2.f)
			[
				ToolkitSections->DetailsView->AsShared()
			]
		];
	}
	
	if (ToolkitSections->Footer)
	{
		MainContentVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Bottom)
		.Padding(0)
		[
			ToolkitSections->Footer->AsShared()
		];
	}
}

FPCGToolPalette::FPCGToolPalette(TSharedPtr<FUICommandInfo> InLoadToolPaletteAction, const TArray<TSharedPtr<FUICommandInfo>>& InPaletteActions)
	: FToolPalette(InLoadToolPaletteAction, InPaletteActions)
{
}

void FPCGEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, const TWeakObjectPtr<UEdMode> OwningMode)
{
	// @todo_pcg: To be removed with UI overhaul (shared with Modeling Tools) in the future
	bUsesToolkitBuilder = true;

	FModeToolkit::Init(InitToolkitHost, OwningMode);

	RegisterToolkitCallbacks();

	RegisterPalettes();

	CreateWidgets();

	ClearToolNotification();
	ClearToolWarning();

	SAssignNew(ToolkitWidget, SBorder)
	.HAlign(HAlign_Fill)
	.Padding(0)
	.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
	[
		ToolkitBuilder->GenerateWidget()->AsShared()
	];

	ShowModeWarnings();
}

FName FPCGEditorModeToolkit::GetToolkitFName() const
{
	return FName("PCGEditorMode");
}

FText FPCGEditorModeToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("DisplayName", "PCGEditorMode Tool");
}

void FPCGEditorModeToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames.Emplace("PCG Toolkit");
}

FText FPCGEditorModeToolkit::GetToolPaletteDisplayName(const FName PaletteName) const
{
	return FText::FromName(PaletteName);
}

void FPCGEditorModeToolkit::OnToolPaletteChanged(const FName PaletteName)
{
	// There is currently only one palette
	ensure(false);
}

void FPCGEditorModeToolkit::BuildToolPalette(FName Palette, class FToolBarBuilder& ToolbarBuilder)
{
	if (!OwningEditorMode.IsValid())
	{
		return;
	}

	FModeToolkit::BuildToolPalette(Palette, ToolbarBuilder);	
}

FText FPCGEditorModeToolkit::GetActiveToolDisplayName() const
{
	return CachedActiveToolInfo.ToolDisplayName;
}

FText FPCGEditorModeToolkit::GetActiveToolMessage() const
{
	return CachedActiveToolInfo.ToolDisplayMessage;
}

void FPCGEditorModeToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	ClearToolNotification();
	ClearToolWarning();
	
	UpdateActiveTool();

	if (IsHosted())
	{
		GetToolkitHost()->AddViewportOverlayWidget(AcceptToolWidget.ToSharedRef());
		// Force a refresh of the tool widget with the new tool properties.
		AcceptToolWidget->Invalidate(EInvalidateWidgetReason::Volatility);
	}

	if (ToolPresetSection.IsValid() && GetActivePCGToolPalette().IsValid())
	{
		ToolPresetSection->HidePresets(false);
		
		LastPaletteWithActiveTool = GetActivePCGToolPalette()->LoadToolPaletteAction->GetCommandName();
	}
	
	UpdatePresets();
}

void FPCGEditorModeToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	if (IsHosted())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(AcceptToolWidget.ToSharedRef());
	}
	
	if (ToolPresetSection.IsValid())
	{
		LastPaletteWithActiveTool.Reset();
		UpdatePresets();
	}
	
	ModeDetailsView->SetObject(nullptr);
	
	ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Visible);

	ClearToolNotification();
	ClearToolWarning();

	UnregisterToolCallbacks(Tool);
}

void FPCGEditorModeToolkit::PostToolNotification(const FText& Message)
{
	ClearToolNotification();

	UStatusBarSubsystem* StatusBarSubsystem = GEditor->GetEditorSubsystem<UStatusBarSubsystem>();
	const TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
	if (StatusBarSubsystem && ModeUILayerPtr)
	{
		ActiveToolMessageHandle = StatusBarSubsystem->PushStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), Message);
	}
}

void FPCGEditorModeToolkit::ClearToolNotification()
{
	UStatusBarSubsystem* StatusBarSubsystem = GEditor->GetEditorSubsystem<UStatusBarSubsystem>();
	const TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
	if (StatusBarSubsystem && ModeUILayerPtr)
	{
		StatusBarSubsystem->PopStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), ActiveToolMessageHandle);
	}

	ActiveToolMessageHandle.Reset();
}

void FPCGEditorModeToolkit::PostToolWarning(const FText& Message)
{
	check(ToolkitSections && ToolkitSections->ToolWarningArea);
	ToolkitSections->ToolWarningArea->SetText(Message);
	ToolkitSections->ToolWarningArea->SetVisibility(EVisibility::Visible);
}

void FPCGEditorModeToolkit::ClearToolWarning()
{
	check(ToolkitSections && ToolkitSections->ToolWarningArea);
	ToolkitSections->ToolWarningArea->SetText(FText());
	ToolkitSections->ToolWarningArea->SetVisibility(EVisibility::Collapsed);
}

void FPCGEditorModeToolkit::ShowModeWarnings()
{
	FText WarningText{};
	if (GEditor->bIsSimulatingInEditor)
	{
		WarningText = LOCTEXT("SimulatingWarning", "Cannot use PCG Editor Mode while simulating.");
	}
	else if (GEditor->PlayWorld != nullptr)
	{
		WarningText = LOCTEXT("PIEWarning", "Cannot use PCG Editor Mode in PIE.");
	}

	if (!WarningText.IdenticalTo(ActiveWarning))
	{
		ActiveWarning = WarningText;
		ToolkitSections->ModeWarningArea->SetVisibility(ActiveWarning.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible);
		ToolkitSections->ModeWarningArea->SetText(ActiveWarning);
	}
}

void FPCGEditorModeToolkit::OnActiveViewportChanged(TSharedPtr<IAssetViewport> OldViewport, TSharedPtr<IAssetViewport> NewViewport) const
{
	// @todo_pcg: Needs to be confirmed in the long run once the toolkit is fleshed out. For now, just cancel the tool.
	if (ToolkitBuilder->HasActivePalette())
	{
		GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel);
	}
}

const FSlateBrush* FPCGEditorModeToolkit::GetActiveToolIcon() const
{
	return CachedActiveToolInfo.ToolIcon;
}

bool FPCGEditorModeToolkit::IsAcceptButtonEnabled() const
{
	return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanAcceptActiveTool();
}

bool FPCGEditorModeToolkit::IsCancelButtonEnabled() const
{
	return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCancelActiveTool();
}

EVisibility FPCGEditorModeToolkit::GetAcceptButtonVisibility() const
{
	return GetScriptableEditorMode()->GetInteractiveToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FPCGEditorModeToolkit::GetCancelButtonVisibility() const
{
	return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCancelActiveTool() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply FPCGEditorModeToolkit::OnAcceptButtonClicked() const
{
	GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept);
	return FReply::Handled();
}

FReply FPCGEditorModeToolkit::OnCancelButtonClicked() const
{
	GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel);
	return FReply::Handled();
}

void FPCGEditorModeToolkit::UpdatePresets()
{
	if (ToolPresetSection.IsValid())
	{
		ToolPresetSection->UpdatePresets();
	}
}

TSharedPtr<FPCGToolPalette> FPCGEditorModeToolkit::GetActivePCGToolPalette() const
{
	if (ToolkitBuilder->ActivePalette.IsValid())
	{
		return StaticCastSharedPtr<FPCGToolPalette>(ToolkitBuilder->ActivePalette);
	}

	return nullptr;
}

void FPCGEditorModeToolkit::RegisterToolkitCallbacks()
{
	// Subscribe to receive settings change updates.
	UPCGEditorModeSettings* PCGEditorModeSettings = GetMutableDefault<UPCGEditorModeSettings>();
	PCGEditorModeSettings->OnSettingChanged().AddSP(SharedThis(this), &FPCGEditorModeToolkit::OnEditorModeSettingsChanged);

	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolNotificationMessage.AddSP(this, &FPCGEditorModeToolkit::PostToolNotification);
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolWarningMessage.AddSP(this, &FPCGEditorModeToolkit::PostToolWarning);

	GetToolkitHost()->OnActiveViewportChanged().AddSP(this, &FPCGEditorModeToolkit::OnActiveViewportChanged);	
}

void FPCGEditorModeToolkit::UnregisterToolkitCallbacks()
{
	GetToolkitHost()->OnActiveViewportChanged().RemoveAll(this);

	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolNotificationMessage.RemoveAll(this);
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolWarningMessage.RemoveAll(this);

	UPCGEditorModeSettings* PCGEditorModeSettings = GetMutableDefault<UPCGEditorModeSettings>();
	PCGEditorModeSettings->OnSettingChanged().RemoveAll(this);
}

void FPCGEditorModeToolkit::RegisterToolCallbacks(UInteractiveTool* CurrentTool)
{
	if (CurrentTool)
	{
		CurrentTool->OnPropertySetsModified.AddSP(this, &FPCGEditorModeToolkit::UpdateActiveTool);
		CurrentTool->OnPropertyModifiedDirectlyByTool.AddSP(this, &FPCGEditorModeToolkit::InvalidateCachedDetailPanelState);
	}
}

void FPCGEditorModeToolkit::UnregisterToolCallbacks(UInteractiveTool* CurrentTool) const
{
	if (CurrentTool)
	{
		CurrentTool->OnPropertySetsModified.RemoveAll(this);
		CurrentTool->OnPropertyModifiedDirectlyByTool.RemoveAll(this);
	}
}

void FPCGEditorModeToolkit::OnEditorModeSettingsChanged(UObject* InObject, FPropertyChangedEvent& InChangedEvent)
{
	// @todo_pcg: A safety precaution, but this may not be needed.
	if (InChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGEditorModeSettings, bEnableEditorMode))
	{
		GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->DeactivateTool(EToolSide::Left, EToolShutdownType::Cancel);
	}
}

void FPCGEditorModeToolkit::UpdateActiveTool()
{
	UInteractiveTool* CurrentTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
	
	CachedActiveToolInfo = CurrentTool->GetToolInfo();

	if (const UPCGEditorModeSettings* Settings = GetDefault<UPCGEditorModeSettings>())
	{
		if (Settings->bHideToolButtonsDuringActiveTool)
		{
			ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Collapsed);
		}
	}

	// Before actually changing the detail panel, we need to see where the current keyboard focus is, because
	// if it's inside the detail panel, we'll need to reset it to the detail panel as a whole, else we might
	// lose it entirely when that detail panel element gets destroyed (which would make us unable to receive any
	// hotkey presses until the user clicks somewhere).
	const TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();
	if (FocusedWidget != ModeDetailsView)
	{
		// Search upward from the currently focused widget
		TSharedPtr<SWidget> CurrentWidget = FocusedWidget;
		while (CurrentWidget.IsValid())
		{
			if (CurrentWidget == ModeDetailsView)
			{
				// Reset focus to the detail panel as a whole to avoid losing it when the inner elements change.
				FSlateApplication::Get().SetKeyboardFocus(ModeDetailsView);
				break;
			}

			CurrentWidget = CurrentWidget->GetParentWidget();
		}
	}
	
	ModeDetailsView->SetObjects(CurrentTool->GetToolProperties(true));
	RegisterToolCallbacks(CurrentTool);

	// @todo_pcg: If a tool property needs to invalidate the cached details panel, do it here. 
	// InvalidateCachedDetailPanelState(nullptr);
}

void FPCGEditorModeToolkit::InvalidateCachedDetailPanelState(UObject* ChangedObject) const
{
	ModeDetailsView->InvalidateCachedState();
}

void FPCGEditorModeToolkit::RegisterPalettes()
{
	const FPCGEditorModePaletteCommands& PaletteCommands = FPCGEditorModePaletteCommands::Get();
	FPCGEditorModeToolCommands& ToolCommands = FPCGEditorModeToolCommands::Get();

	ToolkitSections = MakeShared<FToolkitSections>();
	ToolkitSections->DetailsView = ModeDetailsView;

	FToolkitBuilderArgs ToolkitBuilderArgs(GetScriptableEditorMode()->GetModeInfo().ToolbarCustomizationName);
	ToolkitBuilderArgs.ToolkitCommandList = GetToolkitCommands();
	ToolkitBuilderArgs.ToolkitSections = ToolkitSections;
	ToolkitBuilderArgs.SelectedCategoryTitleVisibility = EVisibility::Collapsed;

	ToolkitBuilder = MakeShared<FPCGEditorModeToolkitBuilder>(ToolkitBuilderArgs);
		
	{
		FString ToolIdentifier = UPCGInteractiveToolSettings_Spline::StaticGetToolTag().ToString();
		TSharedPtr<FPCGToolPalette> DrawSplineToolSection = MakeShared<FPCGToolPalette>(PaletteCommands.LoadSplineContextPalette, ToolCommands.GetLineContextToolCommands());
		ToolkitBuilder->AddPalette(MoveTemp(DrawSplineToolSection));
	}

	{
		FString ToolIdentifier = UPCGInteractiveToolSettings_PaintTool::StaticGetToolTag().ToString();
		TSharedPtr<FPCGToolPalette> PaintToolPalette = MakeShared<FPCGToolPalette>(PaletteCommands.LoadPaintContextPalette, ToolCommands.GetPaintContextToolCommands());
		ToolkitBuilder->AddPalette(MoveTemp(PaintToolPalette));
	}

	{
		FString ToolIdentifier = UPCGInteractiveToolSettings_Volume::StaticGetToolTag().ToString();
		TSharedPtr<FPCGToolPalette> VolumeToolPalette = MakeShared<FPCGToolPalette>(PaletteCommands.LoadVolumeContextPalette, ToolCommands.GetVolumeContextToolCommands());
		ToolkitBuilder->AddPalette(MoveTemp(VolumeToolPalette));
	}

	{
		FString ToolIdentifier = UPCGQueryTool::StaticGetToolTag().ToString();
		TSharedPtr<FPCGToolPalette> QueryToolPalette = MakeShared<FPCGToolPalette>(PaletteCommands.LoadQueryContextPalette, ToolCommands.GetQueryContextToolCommands());
		ToolkitBuilder->AddPalette(MoveTemp(QueryToolPalette));
	}

	if (PCGEdModeToolkit::CVarEnableManualEditPalette.GetValueOnAnyThread())
	{
		FString ToolIdentifier = UPCGInteractiveToolSettings_ManualEdit::StaticGetToolTag().ToString();
		TSharedPtr<FPCGToolPalette> ManualEditPalette = MakeShared<FPCGToolPalette>(PaletteCommands.LoadManualContextPalette, ToolCommands.GetManualEditContextToolCommands());
		ToolkitBuilder->AddPalette(MoveTemp(ManualEditPalette));
	}

	ToolkitBuilder->SetActivePaletteOnLoad(PaletteCommands.LoadSplineContextPalette.Get());

	ToolkitBuilder->UpdateWidget();

	// If selected palette changes, make sure we are showing the palette command buttons, which may be hidden by active Tool.
	ActivePaletteChangedHandle = ToolkitBuilder->OnActivePaletteChanged.AddSP(this, &FPCGEditorModeToolkit::OnActivePaletteChanged);
}

void FPCGEditorModeToolkit::CreateWidgets()
{
	SAssignNew(ToolkitSections->ModeWarningArea, STextBlock)
	.Text(FText::GetEmpty())
	.AutoWrapText(true)
	.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
	.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)))
	.Visibility(EVisibility::Collapsed);

	SAssignNew(ToolkitSections->ToolWarningArea, STextBlock)
	.Text(FText::GetEmpty())
	.AutoWrapText(true)
	.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
	.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));

	const TSharedPtr<SVerticalBox> ToolkitWidgetVBox = SNew(SVerticalBox);
	ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
	[
		ToolkitSections->ModeWarningArea->AsShared()
	];

	ToolPresetSection = SNew(SPCGToolPresetSection, *OwningEditorMode.Get());
	ToolkitSections->ToolPresetArea = ToolPresetSection;

	SAssignNew(AcceptToolWidget, SPCGEdModeAcceptToolOverlay)
	.ActiveToolDisplayName(this, &FPCGEditorModeToolkit::GetActiveToolDisplayName)
	.ActiveToolIcon(this, &FPCGEditorModeToolkit::GetActiveToolIcon)
	.AcceptButtonLabel(LOCTEXT("AcceptButtonLabel", "Accept"))
	.CancelButtonLabel(LOCTEXT("CancelButtonLabel", "Cancel"))
	.AcceptButtonTooltip(LOCTEXT("AcceptButtonTooltip", "Confirm the tool operation and apply the result."))
	.CancelButtonTooltip(LOCTEXT("CancelButtonTooltip", "Cancel the current tool operation."))
	.IsAcceptButtonEnabled(this, &FPCGEditorModeToolkit::IsAcceptButtonEnabled)
	.IsCancelButtonEnabled(this, &FPCGEditorModeToolkit::IsCancelButtonEnabled)
	.GetAcceptButtonVisibility(this, &FPCGEditorModeToolkit::GetAcceptButtonVisibility)
	.GetCancelButtonVisibility(this, &FPCGEditorModeToolkit::GetCancelButtonVisibility)
	.OnAcceptButtonClicked(this, &FPCGEditorModeToolkit::OnAcceptButtonClicked)
	.OnCancelButtonClicked(this, &FPCGEditorModeToolkit::OnCancelButtonClicked);
}

void FPCGEditorModeToolkit::OnActivePaletteChanged()
{
	ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Visible);

	FName CurrentlyActivePaletteName = GetActivePCGToolPalette()->LoadToolPaletteAction->GetCommandName();
	
	// If the palette changes, hide the preset widget.
	if (ToolPresetSection.IsValid() && LastPaletteWithActiveTool.IsSet())
	{
		bool bPaletteChanged = !LastPaletteWithActiveTool.GetValue().IsEqual(CurrentlyActivePaletteName); 
		ToolPresetSection->HidePresets(bPaletteChanged);
	}
}

#undef LOCTEXT_NAMESPACE
