// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGraph.h"

#include "InteractiveTool.h"
#include "StatusBarSubsystem.h"
#include "Toolkits/BaseToolkit.h"

#include "Widgets/SPCGEdModeAcceptToolOverlay.h"
#include "Widgets/Views/STileView.h"

class SPCGToolPresetSection;
class UPCGInteractiveTool;

/** A customized Toolkit Builder used for the PCG Editor Mode  */
class FPCGEditorModeToolkitBuilder : public FToolkitBuilder
{
public:
	explicit FPCGEditorModeToolkitBuilder(FToolkitBuilderArgs& Args);

protected:
	virtual void UpdateContentForCategory(FName InActiveCategoryName, FText InActiveCategoryText) override;
};

struct FPCGToolPalette : FToolPalette
{
	FPCGToolPalette(TSharedPtr<FUICommandInfo> InLoadToolPaletteAction, const TArray<TSharedPtr<FUICommandInfo>>& InPaletteActions);

	/** Future custom properties go here. */
};

/** The basic toolkit needed to show the docked editor mode panel and set up the tool shelves and details views. */
class FPCGEditorModeToolkit : public FModeToolkit
{
public:
	// ~Begin FModeToolkit interface
	virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> OwningMode) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;

	/** Returns the PCG Editor Mode specific tabs in the mode toolbar */
	virtual void GetToolPaletteNames(TArray<FName>& PaletteNames) const override;
	virtual FText GetToolPaletteDisplayName(FName PaletteName) const override;
	virtual void OnToolPaletteChanged(FName PaletteName) override;
	virtual bool HasIntegratedToolPalettes() const override { return false; }
	virtual void BuildToolPalette(FName Palette, class FToolBarBuilder& ToolbarBuilder) override;

	virtual FText GetActiveToolDisplayName() const override;
	virtual FText GetActiveToolMessage() const override;

	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;

	// Keep track of the widget for customization.
	virtual TSharedPtr<SWidget> GetInlineContent() const override { return ToolkitWidget; }
	// ~End FModeToolkit interface

	virtual void PostToolNotification(const FText& Message);
	virtual void ClearToolNotification();

	virtual void PostToolWarning(const FText& Message);
	virtual void ClearToolWarning();

	virtual void ShowModeWarnings();

	/*** UI Events ***/
	void OnActiveViewportChanged(TSharedPtr<IAssetViewport>, TSharedPtr<IAssetViewport>) const;

	/*** Accept/Cancel Window ***/
	const FSlateBrush* GetActiveToolIcon() const;
	bool IsAcceptButtonEnabled() const;
	bool IsCancelButtonEnabled() const;
	EVisibility GetAcceptButtonVisibility() const;
	EVisibility GetCancelButtonVisibility() const;
	FReply OnAcceptButtonClicked() const;
	FReply OnCancelButtonClicked() const;

	void UpdatePresets();

	TSharedPtr<FPCGToolPalette> GetActivePCGToolPalette() const;
private:
	void RegisterToolkitCallbacks();
	void UnregisterToolkitCallbacks();

	void RegisterToolCallbacks(UInteractiveTool* CurrentTool);
	void UnregisterToolCallbacks(UInteractiveTool* CurrentTool) const;

	void OnEditorModeSettingsChanged(UObject* InObject, FPropertyChangedEvent& InChangedEvent);

	void UpdateActiveTool();
	void InvalidateCachedDetailPanelState(UObject* ChangedObject) const;

	/** A utility function to register the tool palettes with the ToolkitBuilder. */
	void RegisterPalettes();
	/** A utility function to create additional widgets for the Toolkit. */
	void CreateWidgets();

	void OnActivePaletteChanged();

	/** UI overlay in the viewport to accept or cancel the current tool. */
	TSharedPtr<SPCGEdModeAcceptToolOverlay> AcceptToolWidget;

	TSharedPtr<SPCGToolPresetSection> ToolPresetSection;
	
	/** Currently only used to cancel the current tool. */
	FDelegateHandle ActivePaletteChangedHandle;

	FStatusBarMessageHandle ActiveToolMessageHandle;
	FText ActiveWarning{};
	FInteractiveToolInfo CachedActiveToolInfo;

	TOptional<FName> LastPaletteWithActiveTool;
};
