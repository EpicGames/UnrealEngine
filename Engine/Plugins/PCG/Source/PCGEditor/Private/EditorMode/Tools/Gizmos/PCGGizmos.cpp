// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/Gizmos/PCGGizmos.h"

#include "InteractiveGizmoManager.h"
#include "InteractiveToolManager.h"
#include "PreviewMesh.h"
#include "ToolDataVisualizer.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Generators/MinimalBoxMeshGenerator.h"
#include "Generators/SphereGenerator.h"

namespace UE::PCG::EditorMode
{
	namespace Gizmos
	{
		UMaterialInstanceDynamic* GetDefaultVolumeBrushMaterial(UInteractiveToolManager* ToolManager)
		{
			if (UMaterial* Material = LoadObject<UMaterial>(nullptr, TEXT("/PCG/Materials/BrushIndicatorMaterial")))
			{
				UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
				return MaterialInstance;
			}
			
			return nullptr;
		}
	}
}

UInteractiveGizmo* UVolumetricBoxBrushStampIndicatorBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	return NewObject<UPCGVolumetricBrushStampIndicator>(SceneState.GizmoManager);
}

void UPCGVolumetricBrushStampIndicator::Setup()
{
	bDrawIndicatorLines = true;
	bDrawRadiusCircle = true;
}

void UPCGVolumetricBrushStampIndicator::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (bVisible == false)
	{
		return;
	}
	
	if (BrushMode == EPCGBrushMode::Sphere)
	{
		Super::Render(RenderAPI);
	}
	else if (BrushMode == EPCGBrushMode::Box)
	{
		if (bDrawIndicatorLines)
		{
			FToolDataVisualizer Draw;
			Draw.SetTransform(GetPreviewMesh()->GetTransform());
			Draw.BeginFrame(RenderAPI);

			if (bDrawSecondaryLines)
			{
				const bool bCirclesCoincident = bDrawRadiusCircle && FMath::IsNearlyEqual(BrushFalloff, 1.0f);
				if (!bCirclesCoincident)
				{
					// SideA and SideB are 2 units long due to extents being twice the brush radius, which is the scale in the transform.
					// Then scaled down by the BrushFalloff
					Draw.DrawSquare(FVector::ZeroVector, FVector(2 * BrushFalloff, 0, 0), FVector(0, 2 * BrushFalloff, 0), LineColor, LineThickness, bDepthTested);
				}
			}
		
			if (bDrawExtents)
			{
				Draw.DrawSquare(FVector::ZeroVector, FVector(2, 0, 0), FVector(0, 2, 0), LineColor, LineThickness, bDepthTested);
			}
		
			Draw.EndFrame();
		}
	}

	if (bDrawDirections)
	{
		FToolDataVisualizer Draw;
		
		FTransform Transform = GetPreviewMesh()->GetTransform();
		Draw.SetTransform(Transform);
		
		Draw.BeginFrame(RenderAPI);

		// Unit vectors are fine because scale takes care of appropriate sizing.
		FVector DirectionUp = FVector(0, 0, 1);
		FVector DirectionForward = FVector(1, 0, 0);
		
		Draw.DrawDirectionalArrow(FVector::ZeroVector, DirectionUp, DirectionUp, FLinearColor::Blue, 10.f, 2.f);
		Draw.DrawDirectionalArrow(FVector::ZeroVector, DirectionForward, DirectionForward, FLinearColor::Red, 10.f, 2.f);

		Draw.EndFrame();
	}
}

void UPCGVolumetricBrushStampIndicator::Shutdown()
{
	if (PreviewMesh)
	{
		PreviewMesh->Disconnect();
	}
	
	PreviewMesh = nullptr;
}

void UPCGVolumetricBrushStampIndicator::SetVolumetricBrushVisible(bool bInVisible)
{
	if (PreviewMesh)
	{
		PreviewMesh->SetVisible(bInVisible);
	}
}

bool UPCGVolumetricBrushStampIndicator::IsVolumetricBrushVisible() const
{
	if (PreviewMesh)
	{
		return PreviewMesh->IsVisible();
	}
	
	return false;
}

void UPCGVolumetricBrushStampIndicator::MakeBrushIndicatorMesh(UInteractiveTool* ParentTool, UWorld* World)
{
	if (ensure(ParentTool))
	{
		PreviewMesh = NewObject<UPreviewMesh>(ParentTool);
		if (PreviewMesh)
		{
			PreviewMesh->CreateInWorld(World, FTransform::Identity);
			
			GenerateMesh(BrushMode);
			
			if (UMaterialInstanceDynamic* BrushIndicatorMaterial = UE::PCG::EditorMode::Gizmos::GetDefaultVolumeBrushMaterial(ParentTool->GetToolManager()))
			{
				PreviewMesh->SetMaterial(BrushIndicatorMaterial);
			}

			// make sure raytracing is disabled on the brush indicator
			if (UDynamicMeshComponent* DynamicComponent = Cast<UDynamicMeshComponent>(PreviewMesh->GetRootComponent()))
			{
				DynamicComponent->SetEnableRaytracing(false);
				PreviewMesh->SetShadowsEnabled(false);
			}
			
			AttachedComponent = PreviewMesh->GetRootComponent();
		}
	}
}

UPreviewMesh* UPCGVolumetricBrushStampIndicator::GetPreviewMesh() const
{
	if (PreviewMesh)
	{
		return PreviewMesh;
	}

	return nullptr;
}

void UPCGVolumetricBrushStampIndicator::AddYawRotation(float AddYawRotation)
{
	if (PreviewMesh)
	{
		FRotator AdditionalRotator(0.f, AddYawRotation, 0.f);
		PreviewMesh->GetActor()->AddActorLocalRotation(AdditionalRotator);
	}
}

void UPCGVolumetricBrushStampIndicator::SetBrushMode(EPCGBrushMode InBrushMode)
{
	if(BrushMode == InBrushMode)
	{
		return;
	}

	BrushMode = InBrushMode;

	GenerateMesh(BrushMode);
}

void UPCGVolumetricBrushStampIndicator::GenerateMesh(EPCGBrushMode InBrushMode)
{
	if(PreviewMesh == nullptr)
	{
		return;
	}
	
	FDynamicMesh3 Mesh;

	if( BrushMode == EPCGBrushMode::Sphere)
	{
		UE::Geometry::FSphereGenerator SphereGenerator;
		SphereGenerator.NumPhi = SphereGenerator.NumTheta = 32;
		SphereGenerator.Generate();
		Mesh = FDynamicMesh3(&SphereGenerator);
	}
	else if (BrushMode == EPCGBrushMode::Box)
	{
		UE::Geometry::FMinimalBoxMeshGenerator BoxGenerator;
		BoxGenerator.Box = UE::Geometry::FOrientedBox3d(FVector3d::Zero(), FVector3d::One());
		BoxGenerator.Generate();
		Mesh = FDynamicMesh3(&BoxGenerator);
	}

	if(Mesh.CheckValidity(FDynamicMesh3::FValidityOptions(), UE::Geometry::EValidityCheckFailMode::ReturnOnly))
	{
		PreviewMesh->UpdatePreview(&Mesh);
	}
}
