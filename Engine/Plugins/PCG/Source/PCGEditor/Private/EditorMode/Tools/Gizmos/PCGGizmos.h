// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/BrushStampIndicator.h"
#include "InteractiveTool.h"
#include "InteractiveGizmoBuilder.h"
#include "PCGGizmos.generated.h"

class UPreviewMesh;
class UMaterialInstanceDynamic;

namespace UE::PCG::EditorMode::Gizmos
{
	namespace Constants
	{
		const FString BrushIdentifier_VolumetricBox = "VolumetricBoxBrushGizmoType";
	}
	
	UMaterialInstanceDynamic* GetDefaultVolumeBrushMaterial(UInteractiveToolManager* ToolManager);
}

UENUM()
enum class EPCGBrushMode
{
	Sphere,
	Box
};

UCLASS(MinimalAPI)
class UVolumetricBoxBrushStampIndicatorBuilder : public UInteractiveGizmoBuilder
{
	GENERATED_BODY()
public:
	virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
};

UCLASS(Transient, MinimalAPI)
class UPCGVolumetricBrushStampIndicator : public UBrushStampIndicator
{
	GENERATED_BODY()

public:
	virtual void Setup() override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void Shutdown() override;

	void SetVolumetricBrushVisible(bool bInVisible);

	bool IsVolumetricBrushVisible() const;

	void MakeBrushIndicatorMesh(UInteractiveTool* ParentTool, UWorld* World);

	UPreviewMesh* GetPreviewMesh() const;

	void AddYawRotation(float AddYawRotation);

	void SetBrushMode(EPCGBrushMode InBrushMode);
protected:
	void GenerateMesh(EPCGBrushMode InBrushMode);
private:
	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	UPROPERTY()
	bool bDrawExtents = true;

	UPROPERTY()
	bool bDrawDirections = true;
	
	UPROPERTY()
	EPCGBrushMode BrushMode = EPCGBrushMode::Sphere;
};