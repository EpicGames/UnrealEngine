// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/Helpers/PCGEdModeActorHelpers.h"

#include "PCGComponent.h"
#include "PCGGraph.h"
#include "ActorFactories/ActorFactoryEmptyActor.h"
#include "Helpers/PCGActorHelpers.h"

#define LOCTEXT_NAMESPACE "PCGEditorMode"

namespace UE::PCG::EditorMode
{
	namespace Actor
	{
		static const FLazyName DefaultWorkingActorName = "PCGToolWorkingActor";

		AActor* Spawn(UWorld* World, const FName ActorName, const FTransform& Transform)
		{
			if (!ensure(World && !ActorName.IsNone()))
			{
				return nullptr;
			}

			FActorSpawnParameters SpawnParams;
			SpawnParams.Name = ActorName;
			SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;

			UPCGActorHelpers::FSpawnDefaultActorParams SpawnDefaultActorParams(World, AActor::StaticClass(), Transform, SpawnParams);
			SpawnDefaultActorParams.bForceStaticMobility = false;

			return UPCGActorHelpers::SpawnDefaultActor(SpawnDefaultActorParams);
		}

		AActor* SpawnWorking(UWorld* World, const FTransform& Transform, TSubclassOf<AActor> ActorClass)
		{
			FActorSpawnParameters SpawnParams;
			SpawnParams.Name = DefaultWorkingActorName;
			SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
			// Actor needs to be transactional, otherwise it can't be deleted.
			SpawnParams.ObjectFlags = RF_Transient | RF_Transactional;

			UPCGActorHelpers::FSpawnDefaultActorParams SpawnDefaultActorParams(World, ActorClass, Transform, SpawnParams);

			return UPCGActorHelpers::SpawnDefaultActor(SpawnDefaultActorParams);
		}
	}

	namespace PCGComponent
	{
		UPCGComponent* Find(const AActor* Owner, const FName ToolTag)
		{
			if (!Owner)
			{
				return nullptr;
			}

			TArray<UPCGComponent*, TInlineAllocator<8>> Components;
			Owner->GetComponents<UPCGComponent>(Components);

			// Priority order:
			// 1. A tool-marked pcg component with a graph that has the same tool tags
			// 2. A pcg component with a graph that has the same tool tags ("tentative").
			// 3. Any pcg component with no graph - especially useful to pickup PCG components on PCG Volumes ("fallback").
			UPCGComponent* TentativeComponent = nullptr;
			UPCGComponent* FallbackComponent = nullptr;

			for (UPCGComponent* Component : Components)
			{
				if (Component)
				{
					const bool bComponentHasToolTag = Component->ComponentHasTag(Tags::ToolGeneratedTag);
					const bool bComponentHasCompatibleGraph = Component->GetGraph() && Component->GetGraph()->ToolData.CompatibleToolTags.Contains(ToolTag);
					const bool bComponentHasNoGraph = !Component->GetGraph();

					if (bComponentHasToolTag && bComponentHasCompatibleGraph)
					{
						return Component;
					}
					else if (bComponentHasCompatibleGraph && !TentativeComponent)
					{
						TentativeComponent = Component;
					}
					else if (bComponentHasNoGraph && !FallbackComponent)
					{
						FallbackComponent = Component;
					}
				}
			}

			return TentativeComponent ? TentativeComponent : FallbackComponent;
		}

		UPCGComponent* Create(AActor* Owner, const FName ComponentName)
		{
			if (!ensure(Owner && !ComponentName.IsNone()))
			{
				return nullptr;
			}

			UPCGComponent* PCGComponent = NewObject<UPCGComponent>(Owner, MakeUniqueObjectName(Owner, UPCGComponent::StaticClass(), ComponentName));
			PCGComponent->ComponentTags.Add(Tags::ToolGeneratedTag);
			PCGComponent->RegisterComponent();
			Owner->AddInstanceComponent(PCGComponent);
			return PCGComponent;
		}

		UPCGComponent* CreateWorking(AActor* Owner)
		{
			return Create(Owner, Actor::DefaultWorkingActorName);
		}
	}
}

#undef LOCTEXT_NAMESPACE
