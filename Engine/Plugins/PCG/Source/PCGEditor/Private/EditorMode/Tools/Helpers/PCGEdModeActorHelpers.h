// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers/PCGActorHelpers.h"

namespace UE::PCG::EditorMode
{
	namespace Tags
	{
		const FLazyName ToolGeneratedTag = "PCG Tool";
	}

	namespace Actor
	{
		/** Spawn an actor for the toolkit with spawn parameters. */
		AActor* Spawn(const UPCGActorHelpers::FSpawnDefaultActorParams& SpawnParams);
		/** Spawn an actor for the toolkit with explicit parameters. */
		AActor* Spawn(UWorld* World, FName ActorName, const FTransform& Transform = FTransform::Identity);
		/** Helper to spawn a temporary working actor for a tool. */
		AActor* SpawnWorking(UWorld* World, const FTransform& Transform = FTransform::Identity, TSubclassOf<AActor> ActorClass = AActor::StaticClass());
	}

	namespace PCGComponent
	{
		/** Find a PCG Component on an actor if it exists. */
		UPCGComponent* Find(const AActor* Owner, const FName ToolTag);
		/** Spawn a PCG Component on an actor. */
		UPCGComponent* Create(AActor* Owner, FName ComponentName);
		/** Helper to spawn a temporary working PCG component for a tool. */
		UPCGComponent* CreateWorking(AActor* Owner);
	}
}
