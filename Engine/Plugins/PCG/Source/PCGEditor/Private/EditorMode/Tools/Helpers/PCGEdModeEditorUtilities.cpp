// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/Helpers/PCGEdModeEditorUtilities.h"

#include "PCGGraph.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "GameFramework/Actor.h"
#include "Misc/OutputDeviceNull.h"
#include "HAL/IConsoleManager.h"
#include "Framework/Commands/InputBindingManager.h"


static TAutoConsoleVariable<bool> CVarClearToolAssetCache(
	TEXT("pcg.debug.tools.cleartoolassetscache"),
	true,                                
	TEXT("Enable to always regather the tool commands for the PCG Mode."),
	ECVF_Default
);

TArray<UE::PCG::EditorMode::Utility::FPCGGraphToolEditorData> UE::PCG::EditorMode::Utility::GetGraphToolsWithToolTag(FName ToolTag)
{
	TArray<FPCGGraphToolEditorData> Result;
	
	TArray<TSharedPtr<FUICommandInfo>> GraphAssetCommands;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	FARFilter Filter;
	Filter.ClassPaths.Add(UPCGGraphInterface::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;

	TArray<FAssetData> GraphAssets;
	AssetRegistryModule.Get().GetAssets(Filter, GraphAssets);

	for(auto It = GraphAssets.CreateIterator(); It; ++It)
	{
		if (DoesPCGGraphInterfaceHaveToolTag(ToolTag, *It))
		{
			TOptional<FPCGGraphToolData> GraphToolData = GetGraphToolDataFromPCGGraphInterface(*It);
			FPCGGraphToolEditorData GraphToolEditorData;
			GraphToolEditorData.AssetData = (*It);
			GraphToolEditorData.GraphToolData = GraphToolData.Get(FPCGGraphToolData());
			Result.Add(GraphToolEditorData);
		}
	}

	return Result;
}

bool UE::PCG::EditorMode::Utility::DoesPCGGraphInterfaceHaveToolTag(FName ToolTag, const FAssetData& AssetData)
{	
	if ( ToolTag.IsNone() || !AssetData.IsValid() || !AssetData.GetClass()->IsChildOf(UPCGGraphInterface::StaticClass()))
	{
		return false;
	}

	TOptional<FPCGGraphToolData> GraphToolData = GetGraphToolDataFromPCGGraphInterface(AssetData);

	if (GraphToolData.IsSet())
	{
		return GraphToolData->CompatibleToolTags.Contains(ToolTag);
	}

	return false;
}

bool UE::PCG::EditorMode::Utility::IsPCGGraphInterfaceCompatibleWithActor(AActor* Actor, const FAssetData& AssetData)
{
	if (!ensure(Actor))
	{
		return false;
	}

	if (!AssetData.IsValid() || !AssetData.GetClass()->IsChildOf(UPCGGraphInterface::StaticClass()))
	{
		return false;
	}

	TOptional<FPCGGraphToolData> GraphToolData = GetGraphToolDataFromPCGGraphInterface(AssetData);

	if (GraphToolData.IsSet())
	{
		return Actor->IsA(GraphToolData->InitialActorClassToSpawn);
	}
	
	return false;
}

TOptional<FPCGGraphToolData> UE::PCG::EditorMode::Utility::GetGraphToolDataFromPCGGraphInterface(const FAssetData& AssetData)
{
	if (!AssetData.IsValid() || !AssetData.GetClass()->IsChildOf(UPCGGraphInterface::StaticClass()))
	{
		return {};
	}

	// Prefer the loaded data, if available
	if (AssetData.IsAssetLoaded())
	{
		if (UPCGGraphInterface* LoadedAsset = Cast<UPCGGraphInterface>(AssetData.GetAsset()))
		{
			return LoadedAsset->GetGraphToolData();
		}
	}

	// Otherwise, retrieve the tool data from the Asset Registry
	if (AssetData.GetClass()->IsChildOf(UPCGGraph::StaticClass()))
	{
		return GetARGraphToolDataFromPCGGraph(AssetData);
	}
	else if (AssetData.GetClass()->IsChildOf(UPCGGraphInstance::StaticClass()))
	{
		return GetARGraphToolDataFromPCGGraphInstance(AssetData);
	}

	return {};
}

TOptional<FPCGGraphToolData> UE::PCG::EditorMode::Utility::GetARGraphToolDataFromPCGGraph(const FAssetData& AssetData)
{
	if (AssetData.GetClass()->IsChildOf(UPCGGraph::StaticClass()))
	{
		FPCGGraphToolData GraphToolData;
		
		FString GraphToolDataText;
		if (AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(UPCGGraph, ToolData), GraphToolDataText))
		{
			FOutputDeviceNull NullOut;
			StaticStruct<FPCGGraphToolData>()->ImportText(*GraphToolDataText, &GraphToolData, nullptr, PPF_None, &NullOut,
				FPCGGraphToolData::StaticStruct()->GetName(), true);

			return GraphToolData;
		}
	}

	return {};
}

TOptional<FPCGGraphToolData> UE::PCG::EditorMode::Utility::GetARGraphToolDataFromPCGGraphInstance(const FAssetData& AssetData)
{
	if (AssetData.GetClass()->IsChildOf(UPCGGraphInstance::StaticClass()))
	{
		FString PCGGraphText;
		if (AssetData.GetTagValue("Graph", PCGGraphText))
		{
			FSoftObjectPath PCGGraphPath(PCGGraphText);

			if(PCGGraphPath.IsAsset())
			{
				FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
				FAssetData LinkedPCGGraphAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(PCGGraphPath);

				if (LinkedPCGGraphAssetData.IsValid() && LinkedPCGGraphAssetData.GetClass()->IsChildOf(UPCGGraph::StaticClass()))
				{
					TOptional<FPCGGraphToolData> GraphToolData = GetARGraphToolDataFromPCGGraph(LinkedPCGGraphAssetData);

					TOptional<FPCGGraphInstanceToolDataOverrides> GraphInstanceToolDataOverrides = GetARGraphToolDataOverridesFromPCGGraphInstance(AssetData);

					if (GraphToolData.IsSet() && GraphInstanceToolDataOverrides.IsSet())
					{
						GraphToolData->ApplyOverrides(GraphInstanceToolDataOverrides.GetValue());
					}
					
					return GraphToolData;
				}
			}
		}
	}

	return {};
}

TOptional<FPCGGraphInstanceToolDataOverrides> UE::PCG::EditorMode::Utility::GetARGraphToolDataOverridesFromPCGGraphInstance(const FAssetData& AssetData)
{
	if (AssetData.GetClass()->IsChildOf(UPCGGraphInstance::StaticClass()))
	{
		FPCGGraphInstanceToolDataOverrides GraphToolDataOverrides;
		
		FString GraphToolDataOverridesText;
		if (AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, ToolDataOverrides), GraphToolDataOverridesText))
		{
			FOutputDeviceNull NullOut;
			StaticStruct<FPCGGraphInstanceToolDataOverrides>()->ImportText(*GraphToolDataOverridesText, &GraphToolDataOverrides, nullptr, PPF_None, &NullOut,
				FPCGGraphInstanceToolDataOverrides::StaticStruct()->GetName(), true);

			return GraphToolDataOverrides;
		}
	}

	return {};
}

TArray<TSharedPtr<FUICommandInfo>> UE::PCG::EditorMode::Utility::GetOrCreateUICommandsWithToolTag(TSharedRef<FBindingContext> BindingContext, FName ToolTag)
{
	static TMap<FName, TOptional<TArray<TSharedPtr<FUICommandInfo>>>> CachedGraphAssetCommands;

	TOptional<TArray<TSharedPtr<FUICommandInfo>>>& CommandInfos = CachedGraphAssetCommands.FindOrAdd(ToolTag);
	if(CommandInfos.IsSet() && CVarClearToolAssetCache.GetValueOnGameThread())
	{
		for(TSharedPtr<FUICommandInfo> CommandInfo : CommandInfos.GetValue())
		{
			FInputBindingManager::Get().RemoveInputCommand(BindingContext, CommandInfo.ToSharedRef());
		}
		
		CommandInfos.Reset();
	}

	if(CommandInfos.IsSet() == false)
	{
		TArray<TSharedPtr<FUICommandInfo>> GraphAssetCommands;

		TArray<FPCGGraphToolEditorData> GraphToolEditorData = GetGraphToolsWithToolTag(ToolTag);

		for(const FPCGGraphToolEditorData& Data : GraphToolEditorData)
		{
			TSharedPtr<FUICommandInfo>& AssetCommandInfo = GraphAssetCommands.AddDefaulted_GetRef();
			FString ObjectPath = Data.AssetData.GetObjectPathString();

			FText DisplayName = Data.GraphToolData.DisplayName;
			if(DisplayName.IsEmpty())
			{
				DisplayName = FText::AsCultureInvariant(Data.AssetData.AssetName.ToString());
			}

			FName CommandName = FName(ObjectPath);
			FUICommandInfo::MakeCommandInfo(BindingContext,
				AssetCommandInfo,
				CommandName,
				DisplayName,
				FText::GetEmpty(),
				FSlateIcon(),
				EUserInterfaceActionType::ToggleButton,
				FInputChord());
		}

		CommandInfos = GraphAssetCommands;
	}

	return CommandInfos.Get({});
}
