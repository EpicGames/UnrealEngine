// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGToolData.h"

#include "AssetRegistry/AssetData.h"
#include "Templates/SharedPointer.h"

class AActor;
class FUICommandInfo;
class FBindingContext;
 
namespace UE::PCG::EditorMode::Utility
{
	struct FPCGGraphToolEditorData
	{
		FAssetData AssetData;
		FPCGGraphToolData GraphToolData;
	};
	
	TArray<FPCGGraphToolEditorData> GetGraphToolsWithToolTag(FName ToolTag);
	
	bool DoesPCGGraphInterfaceHaveToolTag(FName ToolTag, const FAssetData& AssetData);
	bool IsPCGGraphInterfaceCompatibleWithActor(AActor* Actor, const FAssetData& AssetData);
	
	TOptional<FPCGGraphToolData> GetGraphToolDataFromPCGGraphInterface(const FAssetData& AssetData);
	
	TOptional<FPCGGraphToolData> GetARGraphToolDataFromPCGGraph(const FAssetData& AssetData);
	TOptional<FPCGGraphToolData> GetARGraphToolDataFromPCGGraphInstance(const FAssetData& AssetData);

	TOptional<FPCGGraphInstanceToolDataOverrides> GetARGraphToolDataOverridesFromPCGGraphInstance(const FAssetData& AssetData);
	
	TArray<TSharedPtr<FUICommandInfo>> GetOrCreateUICommandsWithToolTag(TSharedRef<FBindingContext> BindingContext, FName ToolTag);
}
