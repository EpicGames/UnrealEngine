// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/Helpers/PCGEdModeSceneQueryHelpers.h"

#include "CollisionQueryParams.h"
#include "Landscape.h"
#include "BaseGizmos/GizmoMath.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "Helpers/PCGHelpers.h"

namespace UE::PCG::EditorMode::Scene
{
	bool IsHitTargetVisible(const FHitResult& HitResult)
	{
		if (const AActor* Actor = HitResult.GetActor())
		{
			if (Actor->IsHiddenEd())
			{
				return false;
			}
		}

		if (const UPrimitiveComponent* Component = HitResult.GetComponent())
		{
			if (!Component->IsVisibleInEditor())
			{
				return false;
			}
		}

		return true;
	}

	TOptional<FHitResult> TraceToNearestObject(const FRay& WorldRay, const FViewRayParams& Params)
	{
		if (!Params.World)
		{
			return {};
		}

		// @todo_pcg: make this configurable
		const FCollisionObjectQueryParams ObjectQueryParams(ECC_TO_BITFIELD(ECC_WorldStatic) | ECC_TO_BITFIELD(ECC_WorldDynamic));
		FCollisionQueryParams QueryParams = FCollisionQueryParams::DefaultQueryParam;
		QueryParams.bTraceComplex = true;

		TArray<FHitResult> OutHits;
		const FVector& Origin = WorldRay.Origin;
		const FVector End = Origin + WorldRay.Direction * Params.Distance;
		if (!Params.World->LineTraceMultiByObjectType(OutHits, Origin, End, ObjectQueryParams, QueryParams))
		{
			return {};
		}

		for (const FHitResult& Hit : OutHits)
		{
			UPrimitiveComponent* HitComponent = Hit.GetComponent();

			// Skip "No collision" type actors (non-physical components, esp. brush components, like PCG volumes).
			if (!HitComponent || !HitComponent->IsQueryCollisionEnabled())
			{
				continue;
			}

			if (HitComponent->GetCollisionResponseToChannel(ECC_WorldStatic) != ECR_Block && HitComponent->GetCollisionResponseToChannel(ECC_WorldDynamic) != ECR_Block)
			{
				continue;
			}

			if (Params.FilterRuleCollection != nullptr)
			{
				TOptional<bool> Result = Params.FilterRuleCollection->IsRaycastHitValid(Hit);
				if (Result.IsSet() == false || Result.GetValue() == false)
				{
					continue;
				}
			}
			
			// Check for a visible result
			bool bHitIsValid = IsHitTargetVisible(Hit);
			// If invisible, check if it's explicitly allowed.
			bHitIsValid |= Params.AllowedInvisibleComponents && Params.AllowedInvisibleComponents->Contains(HitComponent);
			// Check that it's not explicitly ignored.
			bHitIsValid &= !Params.ComponentsToIgnore || !Params.ComponentsToIgnore->Contains(HitComponent);

			if (bHitIsValid)
			{
				return Hit;
			}
		}

		return {};
	}
	
	TOptional<FHitResult> TraceToNearestObject(const FInputDeviceRay& Ray, const FViewRayParams& Params)
	{
		return TraceToNearestObject(Ray.WorldRay, Params);
	}

	TOptional<FViewHitResult> ViewportRay(const FRay& WorldRay, const FViewRayParams& Params)
	{
		if (TOptional<FHitResult> Result = TraceToNearestObject(WorldRay, Params))
		{
			return FViewHitResult(Result->ImpactPoint, Result->ImpactNormal, Result->Distance);
		}
		else if (Params.bHitGroundPlanes != nullptr && *Params.bHitGroundPlanes)
		{
			const FVector PlaneOrigin = FVector::ZeroVector;
			const FVector PlaneNormal = (Params.CameraState && Params.CameraState->bIsOrthographic) ? -WorldRay.Direction : FVector::ZAxisVector;

			// Implementation note - not testing against world because we've already done it & it would create an infinite recursion.
			return ViewportRayAgainstPlane(WorldRay, Params, FPlane(PlaneOrigin, PlaneNormal), /*bTestAgainstWorld=*/false);
		}
		else
		{
			return {};
		}
	}
	
	TOptional<FViewHitResult> ViewportRay(const FInputDeviceRay& Ray, const FViewRayParams& Params)
	{
		return ViewportRay(Ray.WorldRay, Params);
	}

	TOptional<FViewHitResult> ViewportRayAgainstPlane(const FInputDeviceRay& Ray, const FViewRayParams& Params, const FPlane& Plane, bool bTestAgainstWorld)
	{
		return ViewportRayAgainstPlane(Ray.WorldRay, Params, Plane, bTestAgainstWorld);
	}

	TOptional<FViewHitResult> ViewportRayAgainstPlane(const FRay& Ray, const FViewRayParams& Params, const FPlane& Plane, bool bTestAgainstWorld)
	{
		bool bHitPlane = false;
		FVector IntersectionPoint;
		GizmoMath::RayPlaneIntersectionPoint(Plane.GetOrigin(), Plane.GetNormal(), Ray.Origin, Ray.Direction, bHitPlane, IntersectionPoint);

		TOptional<FViewHitResult> HitResult;
		if (bTestAgainstWorld)
		{
			HitResult = ViewportRay(Ray, Params);
		}

		if (HitResult)
		{
			// If the plane wasn't hit or the hit result was closer, return it.
			if (!bHitPlane || FVector::DistSquared(Ray.Origin, HitResult->ImpactPosition) < FVector::DistSquared(Ray.Origin, IntersectionPoint))
			{
				return HitResult;
			}
		}

		if (bHitPlane)
		{
			return FViewHitResult(IntersectionPoint, Plane.GetNormal(), FVector::Distance(Ray.Origin, IntersectionPoint));
		}

		return {};
	}

	TOptional<FViewHitResult> ViewportRayAgainstCameraPlane(const FInputDeviceRay& Ray, const FViewRayParams& Params, const FViewCameraState& CameraState)
	{
		const FVector PlaneOrigin = CameraState.Position + CameraState.Forward() * Params.Distance;
		const FVector PlaneNormal = -CameraState.Forward();
		return ViewportRayAgainstPlane(Ray, Params, FPlane(PlaneOrigin, PlaneNormal));
	}
}

FPCGRaycastFilterRule::FPCGRaycastFilterRule()
{
}

FPCGRaycastFilterRule::~FPCGRaycastFilterRule()
{
}

TOptional<bool> FPCGRaycastFilterRule_Landscape::IsRaycastHitValid(const FHitResult& HitResult) const
{
	if (HitResult.bBlockingHit)
	{
		if (ALandscapeProxy* LandscapeProxyActor = Cast<ALandscapeProxy>(HitResult.GetActor()))
		{
			return true;
		}
	}

	return {};
}

TOptional<bool> FPCGRaycastFilterRule_Meshes::IsRaycastHitValid(const FHitResult& HitResult) const
{
	if (HitResult.bBlockingHit)
	{
		UActorComponent* HitComponent = HitResult.GetComponent();
		if (HitComponent && (HitComponent->IsA<UStaticMeshComponent>() || HitComponent->IsA<USkeletalMeshComponent>()))
		{
			return true;
		}
	}

	return {};
}

TOptional<bool> FPCGRaycastFilterRule_IgnorePCGGeneratedComponents::IsRaycastHitValid(const FHitResult& HitResult) const
{
	if (HitResult.bBlockingHit)
	{
		UActorComponent* HitComponent = HitResult.GetComponent();
		if (HitComponent && HitComponent->ComponentHasTag(PCGHelpers::DefaultPCGTag))
		{
			return false;
		}
	}

	return {};
}

TOptional<bool> FPCGRaycastFilterRule_ConstrainToActor::IsRaycastHitValid(const FHitResult& HitResult) const
{
	if (SelectedActor == nullptr)
	{
		return {};
	}
	
	AActor* HitActor = HitResult.GetActor();
	if (HitActor && HitActor == SelectedActor)
	{
		return true;
	}

	return false;
}

TOptional<bool> FPCGRaycastFilterRule_AllowedClasses::IsRaycastHitValid(const FHitResult& HitResult) const
{
	if (AllowedClasses.Num() == 0)
	{
		return {};
	}

	AActor* HitActor = HitResult.GetActor();
	if (UClass* HitActorClass = HitActor ? HitActor->GetClass() : nullptr)
	{
		if (AllowedClasses.Contains(HitActorClass))
		{
			return true;
		}
		else
		{
			for (UClass* Class : AllowedClasses)
			{
				if (HitActorClass->IsChildOf(Class))
				{
					return true;
				}
			}
		}
	}

	return false;
}

TOptional<bool> FPCGRaycastFilterRuleCollection::IsRaycastHitValid(const FHitResult& RaycastHit) const
{
	TOptional<bool> Result;

	for (const TInstancedStruct<FPCGRaycastFilterRule>& Rule : RaycastRules)
	{
		const FPCGRaycastFilterRule* RulePtr = Rule.GetPtr<FPCGRaycastFilterRule>();
		if (RulePtr && RulePtr->bEnabled)
		{
			TOptional<bool> RuleResult = RulePtr->IsRaycastHitValid(RaycastHit);

			// If there haven't been concrete results yet, we use the current rule's result 
			if (Result.IsSet() == false)
			{
				Result = RuleResult;
			}
			// If there have been concrete results, we have to make sure all subsequent rules return true
			else
			{
				if (Result.GetValue() == true)
				{
					if (RuleResult.IsSet())
					{
						Result = Result.GetValue() && RuleResult.GetValue();
					}
				}
			}

			// Early out if any rule has returned false
			if (Result.IsSet() && Result.GetValue() == false)
			{
				return false;
			}
		}
	}

	return Result;
}
