// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputState.h"
#include "Math/Vector.h"
#include "Misc/Optional.h"
#include "StructUtils/InstancedStruct.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "PCGEdModeSceneQueryHelpers.generated.h"

struct FViewCameraState;
struct FHitResult;
class ALandscape;

USTRUCT()
struct FPCGRaycastFilterRule
{
	GENERATED_BODY()

	FPCGRaycastFilterRule();
	virtual ~FPCGRaycastFilterRule();
	
	/** If this returns false, the raycast hit is filtered out.
	 *  If it returns true, this rule has determined is valid under this given rule.
	 *  If it returns an empty TOptional, this rule doesn't apply to the hit.
	 */
	virtual TOptional<bool> IsRaycastHitValid(const FHitResult& HitResult) const { return {}; }

	/** Whether this rule is enabled or not. Not exposed as editing is done via custom UI. */
	UPROPERTY()
	bool bEnabled = true;
};

USTRUCT(DisplayName="Landscape")
struct FPCGRaycastFilterRule_Landscape : public FPCGRaycastFilterRule
{
	GENERATED_BODY()

	virtual TOptional<bool> IsRaycastHitValid(const FHitResult& HitResult) const override;
};

USTRUCT(DisplayName="Meshes")
struct FPCGRaycastFilterRule_Meshes : public FPCGRaycastFilterRule
{
	GENERATED_BODY()

	virtual TOptional<bool> IsRaycastHitValid(const FHitResult& HitResult) const override;
};

USTRUCT(DisplayName="Ignore PCG Components")
struct FPCGRaycastFilterRule_IgnorePCGGeneratedComponents : public FPCGRaycastFilterRule
{
	GENERATED_BODY()

	virtual TOptional<bool> IsRaycastHitValid(const FHitResult& HitResult) const override;
};

USTRUCT(DisplayName="Constrain to Actor")
struct FPCGRaycastFilterRule_ConstrainToActor : public FPCGRaycastFilterRule
{
	GENERATED_BODY()

	virtual TOptional<bool> IsRaycastHitValid(const FHitResult& HitResult) const override;

	UPROPERTY(EditAnywhere, Category= "Raycast")
	TObjectPtr<AActor> SelectedActor;
};

/** The Ray will only be valid when the hit actor belongs to any of the specified classes. */
USTRUCT(DisplayName="Allowed Classes")
struct FPCGRaycastFilterRule_AllowedClasses : public FPCGRaycastFilterRule
{
	GENERATED_BODY()

	virtual TOptional<bool> IsRaycastHitValid(const FHitResult& HitResult) const override;

	UPROPERTY(EditAnywhere, Category= "Raycast")
	TSet<TSubclassOf<AActor>> AllowedClasses;
};

/** A collection struct used for details customization purposes. This allows us to add the collection to any PCGInteractiveToolSettings class where required. */
USTRUCT()
struct FPCGRaycastFilterRuleCollection
{
	GENERATED_BODY()

	TOptional<bool> IsRaycastHitValid(const FHitResult& RaycastHit) const;

	/** The raycast rules for a given tool. */
	UPROPERTY(EditAnywhere, Category = "Raycast", meta=(ExcludeBaseStruct))
	TArray<TInstancedStruct<FPCGRaycastFilterRule>> RaycastRules;
};

namespace UE::PCG::EditorMode::Scene
{
	/** Custom parameters for UI ray casts from the current view into the scene. */
	struct FViewRayParams
	{
		const UWorld* World = nullptr;
		double Distance = UE_DOUBLE_BIG_NUMBER;
		const TArray<const UPrimitiveComponent*>* ComponentsToIgnore = nullptr;
		const TArray<const UPrimitiveComponent*>* AllowedInvisibleComponents = nullptr;
		const FPCGRaycastFilterRuleCollection* FilterRuleCollection = nullptr;
		const FViewCameraState* CameraState = nullptr;
		bool* bHitGroundPlanes = nullptr;
	};

	/** The resulting data from a UI ray cast hit. */
	struct FViewHitResult
	{
		FVector ImpactPosition;
		FVector ImpactNormal;
		double Depth;

		FInputRayHit ToInputRayHit() const
		{
			FInputRayHit OutRayHit;
			OutRayHit.bHit = true;
			OutRayHit.bHasHitNormal = true;
			OutRayHit.HitNormal = ImpactNormal;
			OutRayHit.HitDepth = Depth;
			OutRayHit.HitObject = nullptr;
			OutRayHit.HitOwner = nullptr;

			return OutRayHit;
		}
	};

	/** The hit actor or primitive component is visible. */
	bool IsHitTargetVisible(const FHitResult& HitResult);

	/** Cast a ray and if a hit is recorded, return the hit result nearest object. */
	TOptional<FHitResult> TraceToNearestObject(const FRay& WorldRay, const FViewRayParams& Params);
	TOptional<FHitResult> TraceToNearestObject(const FInputDeviceRay& Ray, const FViewRayParams& Params);

	/** Cast a ray from the view and if a hit is recorded, return a condensed view-related hit result. */
	TOptional<FViewHitResult> ViewportRay(const FRay& WorldRay, const FViewRayParams& Params);
	TOptional<FViewHitResult> ViewportRay(const FInputDeviceRay& Ray, const FViewRayParams& Params);

	/** Cast a ray from the view to a plane in space. If a hit is recorded, return a condensed view-related hit result. */
	TOptional<FViewHitResult> ViewportRayAgainstPlane(const FRay& Ray, const FViewRayParams& Params, const FPlane& Plane, bool bTestAgainstWorld = true);
	TOptional<FViewHitResult> ViewportRayAgainstPlane(const FInputDeviceRay& Ray, const FViewRayParams& Params, const FPlane& Plane, bool bTestAgainstWorld = true);

	/** Cast a ray from the view to a plane parallel to the user view. If a hit is recorded, return a condensed view-related hit result. */
	TOptional<FViewHitResult> ViewportRayAgainstCameraPlane(const FInputDeviceRay& Ray, const FViewRayParams& Params, const FViewCameraState& CameraState);
}
