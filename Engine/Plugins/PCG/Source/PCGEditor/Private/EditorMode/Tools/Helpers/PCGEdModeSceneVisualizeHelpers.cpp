// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/Helpers/PCGEdModeSceneVisualizeHelpers.h"

#include "PrimitiveDrawInterface.h"
#include "ToolContextInterfaces.h"
#include "Components/SplineComponent.h"
#include "Settings/LevelEditorViewportSettings.h"

namespace PCG::EditorMode::Scene::Visualize
{
	// Taken in large directly from SplineComponentVisualizer::DrawVisualization in SplineComponentVisualizer.h
	// Note: this currently overdraws the new spline visualizer, but can be used for additional things like scale, normals, etc.
	void Spline(const USplineComponent& Spline, IToolsContextRenderAPI& RenderAPI, int32 LatestIndex, bool bShouldVisualizeScale, bool bShouldVisualizeTangents)
	{
		FPrimitiveDrawInterface* PDI = RenderAPI.GetPrimitiveDrawInterface();
		const FInterpCurveVector& SplineInfo = Spline.GetSplinePointsPosition();

		const bool bIsSplineEditable = !Spline.bModifiedByConstructionScript;

		// @todo_pcg: Special color for read-only splines
		const FColor NormalColor = FColor::White;
		const FColor SelectedColor = FColor::White;
		const FColor TangentColor = FColor::White;
		const float PointSize = 10.0f + (bIsSplineEditable ? GetDefault<ULevelEditorViewportSettings>()->SelectedSplinePointSizeAdjustment : 0.0f);

		const float VisualizationWidthScale = Spline.ScaleVisualizationWidth;
		const float TangentScale = GetDefault<ULevelEditorViewportSettings>()->SplineTangentScale;

		FVector PreviousKeyPosition = !SplineInfo.Points.IsEmpty() ? Spline.GetLocationAtSplinePoint(0, ESplineCoordinateSpace::World) : FVector::ZeroVector;
		FVector PreviousKeyRightVector = !SplineInfo.Points.IsEmpty() ? Spline.GetRightVectorAtSplinePoint(0, ESplineCoordinateSpace::World) : FVector::ZeroVector;
		FVector PreviousKeyScale = !SplineInfo.Points.IsEmpty() ? Spline.GetUpVectorAtSplinePoint(0, ESplineCoordinateSpace::World) : FVector::ZeroVector;

		const int32 NumPoints = SplineInfo.Points.Num();
		const int32 NumSegments = SplineInfo.bIsLooped ? NumPoints : NumPoints - 1;
		for (int32 KeyIndex = 0; KeyIndex < NumSegments + 1; ++KeyIndex)
		{
			const FVector KeyPosition = Spline.GetLocationAtSplinePoint(KeyIndex, ESplineCoordinateSpace::World);
			const FVector KeyRightVector = Spline.GetRightVectorAtSplinePoint(KeyIndex, ESplineCoordinateSpace::World);
			const FVector KeyUpVector = Spline.GetUpVectorAtSplinePoint(KeyIndex, ESplineCoordinateSpace::World);
			const FVector KeyScale = Spline.GetScaleAtSplinePoint(KeyIndex) * VisualizationWidthScale;
			const FVector ArriveTangent = Spline.GetArriveTangentAtSplinePoint(KeyIndex, ESplineCoordinateSpace::World) * TangentScale;
			const FVector LeaveTangent = Spline.GetLeaveTangentAtSplinePoint(KeyIndex, ESplineCoordinateSpace::World) * TangentScale;

			const FColor KeyColor = (KeyIndex == LatestIndex) ? SelectedColor : NormalColor;

			// Draw the keypoint and up/right vectors
			if (KeyIndex < NumPoints)
			{
				// Draws the tangents separately with translucent lines.
				// @todo_pcg: Separate visualization into arrive and leave tangents
				if (bShouldVisualizeTangents)
				{
					PDI->DrawTranslucentLine(KeyPosition, KeyPosition - ArriveTangent, TangentColor, SDPG_Foreground);
					PDI->DrawTranslucentLine(KeyPosition, KeyPosition + LeaveTangent, TangentColor, SDPG_Foreground);
				}

				// If true, will create a thickness of lines around the spline
				if (bShouldVisualizeScale)
				{
					PDI->DrawLine(KeyPosition, KeyPosition - KeyRightVector * KeyScale.Y, KeyColor, SDPG_Foreground);
					PDI->DrawLine(KeyPosition, KeyPosition + KeyRightVector * KeyScale.Y, KeyColor, SDPG_Foreground);
					PDI->DrawLine(KeyPosition, KeyPosition + KeyUpVector * KeyScale.Z, KeyColor, SDPG_Foreground);

					// Points used to visualize the radial dimension
					constexpr int32 ArcPoints = 20;
					FVector OldArcPos = KeyPosition + KeyRightVector * KeyScale.Y;
					for (int32 ArcIndex = 1; ArcIndex <= ArcPoints; ++ArcIndex)
					{
						float Sin;
						float Cos;
						FMath::SinCos(&Sin, &Cos, ArcIndex * PI / ArcPoints);
						const FVector NewArcPos = KeyPosition + Cos * KeyRightVector * KeyScale.Y + Sin * KeyUpVector * KeyScale.Z;
						PDI->DrawLine(OldArcPos, NewArcPos, KeyColor, SDPG_Foreground);
						OldArcPos = NewArcPos;
					}
				}

				PDI->DrawPoint(KeyPosition, KeyColor, PointSize, SDPG_Foreground);
			}

			// If not the first keypoint, draw a line to the previous keypoint.
			if (KeyIndex > 0)
			{
				const FColor LineColor = NormalColor;

				const int32 PreviousIndex = KeyIndex - 1;
				const FColor BeginColor = (LatestIndex == PreviousIndex) ? SelectedColor : NormalColor;
				const FColor EndColor = (LatestIndex == KeyIndex) ? SelectedColor : NormalColor;

				// Find position on first keyframe.
				FVector PreviousPosition = PreviousKeyPosition;
				FVector PreviousRightVector = PreviousKeyRightVector;
				FVector PreviousScale = PreviousKeyScale;

				// Then draw a line for each substep.
				constexpr int32 NumSteps = 20;
				float SegmentLineThickness = GetDefault<ULevelEditorViewportSettings>()->SplineLineThicknessAdjustment;

				for (int32 StepIndex = 1; StepIndex <= NumSteps; ++StepIndex)
				{
					const float StepRatio = static_cast<float>(StepIndex) / static_cast<float>(NumSteps);
					const float Key = PreviousIndex + StepRatio;

					// Find the step position
					const FVector StepPosition = Spline.GetLocationAtSplinePoint(Key, ESplineCoordinateSpace::World);
					const FVector StepRightVector = Spline.GetRightVectorAtSplinePoint(Key, ESplineCoordinateSpace::World);
					const FVector StepScale = Spline.GetScaleAtSplinePoint(Key);

					FColor StepColor;
					// Grade the lerp to full over 80%, then clamp.
					constexpr float PartialGradientProportion = 0.8f;
					const float LerpRatio = (1.0f - StepRatio) / PartialGradientProportion;
					StepColor = FMath::Lerp(BeginColor.ReinterpretAsLinear(), EndColor.ReinterpretAsLinear(), LerpRatio).ToFColor(false);

					// Draw the segment.
					PDI->DrawLine(PreviousPosition, StepPosition, StepColor, SDPG_Foreground, SegmentLineThickness);

					if (bShouldVisualizeScale)
					{
						PDI->DrawLine(PreviousPosition - PreviousRightVector * PreviousScale.Y, StepPosition - StepRightVector * StepScale.Y, LineColor, SDPG_Foreground);
						PDI->DrawLine(PreviousPosition + PreviousRightVector * PreviousScale.Y, StepPosition + StepRightVector * StepScale.Y, LineColor, SDPG_Foreground);
					}

					// Visualize Up Vector
					const float HalfVisualizationWidthScale = VisualizationWidthScale * 0.5f;
					const FVector StepUpVector = Spline.GetUpVectorAtSplineInputKey(Key, ESplineCoordinateSpace::World);
					PDI->DrawLine(StepPosition, StepPosition + StepUpVector * HalfVisualizationWidthScale, LineColor, SDPG_Foreground);
					PDI->DrawLine(StepPosition, StepPosition + StepRightVector * HalfVisualizationWidthScale, LineColor, SDPG_Foreground);

					PreviousPosition = StepPosition;
					PreviousRightVector = StepRightVector;
					PreviousScale = StepScale;
				}

				PreviousKeyPosition = KeyPosition;
				PreviousKeyRightVector = KeyRightVector;
				PreviousKeyScale = KeyScale;
			}
		}
	}

	void SplineTangent(const USplineComponent& Spline, IToolsContextRenderAPI& RenderAPI, const int32 SplinePointIndex)
	{
		if (!ensure(SplinePointIndex >= 0 && SplinePointIndex < Spline.GetNumberOfSplinePoints()))
		{
			return;
		}

		FPrimitiveDrawInterface* PDI = RenderAPI.GetPrimitiveDrawInterface();
		check(PDI);

		const float TangentScale = FMath::Min(0.5f, GetDefault<ULevelEditorViewportSettings>()->SplineTangentScale);
		const float TangentHandleSize = 8.0f + GetDefault<ULevelEditorViewportSettings>()->SplineTangentHandleSizeAdjustment;

		const FVector Location = Spline.GetLocationAtSplinePoint(SplinePointIndex, ESplineCoordinateSpace::World);
		const FVector LeaveTangent = Spline.GetLeaveTangentAtSplinePoint(SplinePointIndex, ESplineCoordinateSpace::World) * TangentScale;
		const FVector ArriveTangent = Spline.bAllowDiscontinuousSpline ? Spline.GetArriveTangentAtSplinePoint(SplinePointIndex, ESplineCoordinateSpace::World) * TangentScale : LeaveTangent;

		const FColor Color = FColor::White;

		PDI->DrawLine(Location, Location - ArriveTangent, Color, SDPG_Foreground);
		PDI->DrawLine(Location, Location + LeaveTangent, Color, SDPG_Foreground);

		PDI->DrawPoint(Location + LeaveTangent, Color, TangentHandleSize, SDPG_Foreground);
		PDI->DrawPoint(Location - ArriveTangent, Color, TangentHandleSize, SDPG_Foreground);
	}
}
