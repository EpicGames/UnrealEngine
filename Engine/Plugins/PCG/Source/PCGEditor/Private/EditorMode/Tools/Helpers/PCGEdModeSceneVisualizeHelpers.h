// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IToolsContextRenderAPI;
class USplineComponent;

namespace PCG::EditorMode::Scene::Visualize
{
	// Visualize a spline within the editor viewport.
	void Spline(const USplineComponent& Spline, IToolsContextRenderAPI& RenderAPI, int32 LatestIndex = INDEX_NONE, bool bShouldVisualizeScale = false, bool bShouldVisualizeTangents = true);

	// Visualize a spline tangent within the editor viewport.
	void SplineTangent(const USplineComponent& Spline, IToolsContextRenderAPI& RenderAPI, const int32 SplinePointIndex);
}
