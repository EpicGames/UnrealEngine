// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/Helpers/PCGEdModeSplineHelpers.h"

#include "Components/SplineComponent.h"
#include "EditorMode/Tools/Helpers/PCGEdModeActorHelpers.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "PCGEditorMode"

namespace UE::PCG::EditorMode::Spline
{
	static const FLazyName DefaultWorkingSplineName = "PCGToolWorkingSpline";

	USplineComponent* CreateComponent(
		AActor* Actor,
		const FName ComponentName,
		const USplineComponent* TemplateComponent,
		const bool bTransactional)
	{
		if (!ensure(Actor && !ComponentName.IsNone()))
		{
			return nullptr;
		}

		if (bTransactional)
		{
			Actor->Modify();
		}

		const FName Name = MakeUniqueObjectName(Actor, TemplateComponent ? TemplateComponent->StaticClass() : USplineComponent::StaticClass(), ComponentName);

		USplineComponent* Spline = nullptr;
		const EObjectFlags Flags = RF_Transient | (bTransactional ? RF_Transactional : RF_NoFlags);
		if (TemplateComponent)
		{
			Spline = DuplicateObject<USplineComponent>(TemplateComponent, Actor, Name);
			Spline->SetFlags(Flags);
		}
		else
		{
			Spline = NewObject<USplineComponent>(Actor, USplineComponent::StaticClass(), Name, Flags);
		}

		Spline->SetFlags(RF_Transient);
		Spline->ComponentTags.Add(Tags::ToolGeneratedTag);
		Actor->AddInstanceComponent(Spline);
		Spline->RegisterComponent();
		Spline->ResetRelativeTransform();
		Actor->PostEditChange();

		return Spline;
	}

	USplineComponent* CreateWorkingComponent(AActor* Actor)
	{
		return CreateComponent(Actor, DefaultWorkingSplineName, /*TemplateComponent=*/nullptr, /*bTransactional=*/false);
	}

	USplineComponent* Find(const AActor* Actor, const int32 TargetIndex)
	{
		if (!ensure(Actor))
		{
			return nullptr;
		}

		TInlineComponentArray<USplineComponent*> SplineComponents;
		Actor->GetComponents<USplineComponent>(SplineComponents);
		if (SplineComponents.IsValidIndex(TargetIndex))
		{
			return SplineComponents[TargetIndex];
		}
		else if (!SplineComponents.IsEmpty())
		{
			return SplineComponents[0];
		}

		return nullptr;
	}

	void CopySplinePoints(const USplineComponent& Source, USplineComponent& Destination, const bool bTransactional)
	{
		if (bTransactional)
		{
			Destination.Modify();
		}

		Destination.ClearSplinePoints();
		Destination.bSplineHasBeenEdited = true;

		/**
		 * Iterate here (rather than just copying over the SplineCurves data) to transform the data properly into the
		 * coordinate space of the target component.
		 */
		const int32 NumSplinePoints = Source.GetNumberOfSplinePoints();
		for (int32 SplinePointIndex = 0; SplinePointIndex < NumSplinePoints; ++SplinePointIndex)
		{
			Destination.AddSplinePoint(Source.GetLocationAtSplinePoint(SplinePointIndex, ESplineCoordinateSpace::World),
									   ESplineCoordinateSpace::World, false);
			Destination.SetUpVectorAtSplinePoint(SplinePointIndex, Source.GetUpVectorAtSplinePoint(SplinePointIndex, ESplineCoordinateSpace::World),
												 ESplineCoordinateSpace::World, false);
			Destination.SetTangentsAtSplinePoint(SplinePointIndex,
												 Source.GetArriveTangentAtSplinePoint(SplinePointIndex, ESplineCoordinateSpace::World),
												 Source.GetLeaveTangentAtSplinePoint(SplinePointIndex, ESplineCoordinateSpace::World),
												 ESplineCoordinateSpace::World, false);
			Destination.SetSplinePointType(SplinePointIndex, Source.GetSplinePointType(SplinePointIndex), false);
		}

		Destination.SetClosedLoop(Source.IsClosedLoop());
		Destination.UpdateSpline();
	}
}

#undef LOCTEXT_NAMESPACE
