// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class USplineComponent;
class AActor;

namespace UE::PCG::EditorMode::Spline
{	
	USplineComponent* CreateComponent(
		AActor* Actor,
		FName ComponentName,
		const USplineComponent* TemplateComponent = nullptr,
		const bool bTransactional = false);

	USplineComponent* CreateWorkingComponent(AActor* Actor);

	USplineComponent* Find(const AActor* Actor, const int32 TargetIndex = INDEX_NONE);

	void CopySplinePoints(const USplineComponent& Source, USplineComponent& Destination, const bool bTransactional = false);
}
