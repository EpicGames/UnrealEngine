// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/Line/PCGDrawSplineTool.h"

#include "PCGComponent.h"
#include "Data/Tool/PCGToolSplineData.h"
#include "EditorMode/PCGEdModeSettings.h"
#include "EditorMode/PCGEdModeStyle.h"
#include "EditorMode/Tools/Helpers/PCGEdModeSceneVisualizeHelpers.h"

#include "InteractiveToolManager.h"
#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "Components/SplineComponent.h"
#include "Settings/LevelEditorViewportSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDrawSplineTool)

#define LOCTEXT_NAMESPACE "UPCGDrawSplineTool"

namespace PCGDrawSplineTool
{
	using namespace UE::PCG::EditorMode;

	// Helper classes for making undo/redo transactions
	namespace CommandChange
	{
		class FSplineCommandChange : public FToolCommandChange
		{
		public:
			FSplineCommandChange(USplineComponent* InWorkingSpline)
				: WorkingSpline(InWorkingSpline) {}

			// These pass the working spline to the overloads below
			virtual void Apply(UObject* Object) override
			{
				if (!ensure(Object && Object->IsA<UPCGDrawSplineToolBase>() && WorkingSpline))
				{
					return;
				}

				Apply(*WorkingSpline);
			}

			virtual void Revert(UObject* Object) override
			{
				if (!ensure(Object && Object->IsA<UPCGDrawSplineToolBase>() && WorkingSpline))
				{
					return;
				}

				Revert(*WorkingSpline);
			}

		protected:
			virtual void Apply(USplineComponent& Spline) = 0;
			virtual void Revert(USplineComponent& Spline) = 0;

		private:
			USplineComponent* WorkingSpline = nullptr;
		};

		// Undoes a point addition with an auto tangent
		class FSimplePointInsertionChange final : public FSplineCommandChange
		{
		public:
			FSimplePointInsertionChange(
				USplineComponent* InWorkingSpline,
				const FVector& HitLocationIn,
				const FVector& UpVectorIn)
				: FSplineCommandChange(InWorkingSpline)
				, HitLocation(HitLocationIn)
				, UpVector(UpVectorIn) {}

			virtual void Apply(USplineComponent& Spline) override
			{
				Spline.AddSplinePoint(HitLocation, ESplineCoordinateSpace::World, false);
				const int32 PointIndex = Spline.GetNumberOfSplinePoints() - 1;
				Spline.SetUpVectorAtSplinePoint(PointIndex, UpVector, ESplineCoordinateSpace::World, true);
			}

			virtual void Revert(USplineComponent& Spline) override
			{
				if (Spline.GetNumberOfSplinePoints() > 0)
				{
					Spline.RemoveSplinePoint(Spline.GetNumberOfSplinePoints() - 1, true);
				}
			}

			virtual FString ToString() const override
			{
				return TEXT("FSimplePointInsertionChange");
			}

		protected:
			FVector HitLocation;
			FVector UpVector;
		};

		// Undoes a point addition with an explicit tangent
		class FTangentPointInsertionChange final : public FSplineCommandChange
		{
		public:
			FTangentPointInsertionChange(USplineComponent* InWorkingSpline,
										 const FVector& HitLocationIn,
										 const FVector& UpVectorIn,
										 const FVector& TangentIn)
				: FSplineCommandChange(InWorkingSpline)
				, HitLocation(HitLocationIn)
				, UpVector(UpVectorIn)
				, Tangent(TangentIn) {}

			virtual void Apply(USplineComponent& Spline) override
			{
				Spline.AddSplinePoint(HitLocation, ESplineCoordinateSpace::World, false);
				const int32 PointIndex = Spline.GetNumberOfSplinePoints() - 1;
				Spline.SetUpVectorAtSplinePoint(PointIndex, UpVector, ESplineCoordinateSpace::World, false);
				Spline.SetTangentAtSplinePoint(PointIndex, Tangent, ESplineCoordinateSpace::World, true);
			}

			virtual void Revert(USplineComponent& Spline) override
			{
				if (ensure(Spline.GetNumberOfSplinePoints() > 0))
				{
					Spline.RemoveSplinePoint(Spline.GetNumberOfSplinePoints() - 1, true);
				}
			}

			virtual FString ToString() const override
			{
				return TEXT("FTangentPointInsertionChange");
			}

		protected:
			FVector HitLocation;
			FVector UpVector;
			FVector Tangent;
		};

		// Undoes a free draw stroke (multiple points at once)
		class FFreeDrawStrokeChange final : public FSplineCommandChange
		{
		public:
			FFreeDrawStrokeChange(USplineComponent* InWorkingSpline,
								   const TArray<FVector>& HitLocationsIn,
								   const TArray<FVector>& UpVectorsIn)
				: FSplineCommandChange(InWorkingSpline)
				, HitLocations(HitLocationsIn)
				, UpVectors(UpVectorsIn)
			{
				if (!ensure(HitLocations.Num() == UpVectors.Num()))
				{
					const int32 Num = FMath::Min(HitLocations.Num(), UpVectors.Num());
					HitLocations.SetNum(Num);
					UpVectors.SetNum(Num);
				}
			}

			virtual void Apply(USplineComponent& Spline) override
			{
				for (int32 i = 0; i < HitLocations.Num(); ++i)
				{
					Spline.AddSplinePoint(HitLocations[i], ESplineCoordinateSpace::World, false);
					const int32 PointIndex = Spline.GetNumberOfSplinePoints() - 1;
					Spline.SetUpVectorAtSplinePoint(PointIndex, UpVectors[i], ESplineCoordinateSpace::World, false);
				}
				Spline.UpdateSpline();
			}

			virtual void Revert(USplineComponent& Spline) override
			{
				// A spline is invalid if it has just a single point. If after removing all the hit locations we have less than or exactly 1 point, just clear the spline.
				if (HitLocations.Num() - Spline.GetNumberOfSplinePoints() <= 1)
				{
					// Since Reset is not exposed, will set an empty array of points
					Spline.SetSplinePoints({}, ESplineCoordinateSpace::World, /*bUpdateSpline=*/true);
				}
				else
				{
					for (int32 i = 0; i < HitLocations.Num(); ++i)
					{
						Spline.RemoveSplinePoint(Spline.GetNumberOfSplinePoints() - 1, /*bUpdateSpline=*/false);
					}

					Spline.UpdateSpline();
				}
			}

			virtual FString ToString() const override
			{
				return TEXT("FFreeDrawStrokeChange");
			}

		protected:
			TArray<FVector> HitLocations;
			TArray<FVector> UpVectors;
		};
	} // namespace CommandChange

	namespace Constants
	{
		const FText ToolName = LOCTEXT("ToolName", "DrawSpline");
		const FText ToolDescription = LOCTEXT("DrawSplineDisplayMessage", "Draw a spline and distribute spline data directly to a PCG Graph Instance.");
		const FName SplineToolTag = "SplineTool";
		const FName SplineSurfaceToolTag = "SplineSurfaceTool";		
		static const FText AddPointTransactionName = LOCTEXT("AddPointTransactionName", "Add Point");
	}
} // namespace PCGDrawSplineTool

bool UPCGInteractiveToolSettings_SplineBase::IsWorkingActorCompatibleInternal(const AActor* InActor) const
{
	// Working actor needs to have a root component to attach a spline to it.
	return InActor && InActor->GetRootComponent();
}

UPCGInteractiveToolSettings_SplineBase::UPCGInteractiveToolSettings_SplineBase()
	: SplineName(GetDefault<UPCGEditorModeSettings>()->DefaultNewSplineComponentName)
{
	TInstancedStruct<FPCGRaycastFilterRule_Landscape> LandscapeRule = TInstancedStruct<FPCGRaycastFilterRule_Landscape>::Make();
	TInstancedStruct<FPCGRaycastFilterRule_Meshes> MeshesRule = TInstancedStruct<FPCGRaycastFilterRule_Meshes>::Make();
	TInstancedStruct<FPCGRaycastFilterRule_IgnorePCGGeneratedComponents> IgnorePCGGeneratedComponentsRule = TInstancedStruct<FPCGRaycastFilterRule_IgnorePCGGeneratedComponents>::Make();
	RaycastRuleCollection.RaycastRules.Add(LandscapeRule);
	RaycastRuleCollection.RaycastRules.Add(MeshesRule);
	RaycastRuleCollection.RaycastRules.Add(IgnorePCGGeneratedComponentsRule);
}

USplineComponent* UPCGInteractiveToolSettings_SplineBase::GetWorkingSplineComponent(FName DataInstanceIdentifier) const
{
	if(const FPCGInteractiveToolWorkingData_Spline* ToolData = GetTypedWorkingData<FPCGInteractiveToolWorkingData_Spline>(DataInstanceIdentifier))
	{
		return ToolData->GetSplineComponent();
	}
	
	return nullptr;
}

TArray<USplineComponent*> UPCGInteractiveToolSettings_SplineBase::GetWorkingSplineComponents() const
{
	TArray<USplineComponent*> Result;

	TArray<FName> AllWorkingDataInstanceNames = GetDataInstanceNamesForGraph();
	for(const FName& DataInstanceIdentifier : AllWorkingDataInstanceNames)
	{
		if(USplineComponent* WorkingSplineComponent = GetWorkingSplineComponent(DataInstanceIdentifier))
		{
			Result.Add(WorkingSplineComponent);
		}
	}

	return Result;
}

FName UPCGInteractiveToolSettings_Spline::StaticGetToolTag()
{
	return PCGDrawSplineTool::Constants::SplineToolTag;
}

const UScriptStruct* UPCGInteractiveToolSettings_Spline::GetWorkingDataType() const
{
	return FPCGInteractiveToolWorkingData_Spline::StaticStruct();
}

FName UPCGInteractiveToolSettings_Spline::GetToolTag() const
{
	return StaticGetToolTag();
}

void UPCGInteractiveToolSettings_Spline::PostWorkingDataInitialized(FPCGInteractiveToolWorkingData* WorkingData) const
{
	if(USplineComponent* SplineComponent = GetWorkingSplineComponent(DataInstance))
	{
		SplineComponent->SetClosedLoop(bClosedSpline);
	}
}

FName UPCGInteractiveToolSettings_SplineSurface::StaticGetToolTag()
{
	return PCGDrawSplineTool::Constants::SplineSurfaceToolTag;
}

const UScriptStruct* UPCGInteractiveToolSettings_SplineSurface::GetWorkingDataType() const
{
	return FPCGInteractiveToolWorkingData_SplineSurface::StaticStruct();
}

FName UPCGInteractiveToolSettings_SplineSurface::GetToolTag() const
{
	return StaticGetToolTag();
}

void UPCGInteractiveToolSettings_Spline::OnPropertyModified(UObject* Object, FProperty* Property)
{
	Super::OnPropertyModified(Object, Property);
	
	if(Object == this && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGInteractiveToolSettings_Spline, bClosedSpline))
	{
		if(USplineComponent* SplineComponent = GetWorkingSplineComponent(DataInstance))
		{
			SplineComponent->SetClosedLoop(bClosedSpline, true);

			FPropertyChangedEvent Event(nullptr);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(SplineComponent, Event);
		}
	}
}

void UPCGDrawSplineToolBase::Setup()
{
	using namespace PCGDrawSplineTool;

	Super::Setup();

	UPCGInteractiveToolSettings_SplineBase* ToolSettings = GetSplineSettings();
	
	ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Initialize(this, this);
	AddInputBehavior(ClickOrDragBehavior);
	
	CachedRayParams =
	{
		.World = GetWorld(),
		.FilterRuleCollection = &ToolSettings->RaycastRuleCollection,
		.CameraState = &CachedCameraState,
		.bHitGroundPlanes = &ToolSettings->bHitGroundPlanes
	};
}

void UPCGDrawSplineToolBase::Shutdown(const EToolShutdownType ShutdownType)
{
	// We have to call this before the super function as it would remove the property sets before we can route the shutdown function to the pcg settings
	UE::PCG::EditorMode::Tool::Shutdown(this, ShutdownType);
	
	Super::Shutdown(ShutdownType);
}

void UPCGDrawSplineToolBase::AddSplinePoint(const FVector& HitLocation, const FVector& HitNormal) const
{
	using namespace PCGDrawSplineTool;

	UPCGInteractiveToolSettings_SplineBase* ToolSettings = GetSplineSettings();
	
	USplineComponent* WorkingSpline = ToolSettings->GetWorkingSplineComponent(ToolSettings->DataInstance);
	if (!WorkingSpline)
	{
		return;
	}

	const int32 NumSplinePoints = WorkingSpline->GetNumberOfSplinePoints();
	const FVector NewNormal = GetNewPointNormal(HitNormal);

	FVector FinalLocation = HitLocation;
	switch (ToolSettings->Settings.OffsetMode)
	{
		case EPCGToolDrawTargetOffset::DistanceOffset:
			check(HitNormal.IsNormalized());
			FinalLocation += ToolSettings->Settings.OffsetDistance * NewNormal;
			break;
		case EPCGToolDrawTargetOffset::ExplicitOffset:
			FinalLocation += ToolSettings->Settings.OffsetVector;
			break;
		default: // fall-through
		case EPCGToolDrawTargetOffset::HitLocation:
			break;
	}

	WorkingSpline->AddSplinePoint(FinalLocation, ESplineCoordinateSpace::World, /*bUpdate=*/false);
	WorkingSpline->SetUpVectorAtSplinePoint(NumSplinePoints, NewNormal, ESplineCoordinateSpace::World, /*bUpdate=*/true);

	FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
	if(UE::PCG::EditorMode::Tool::CVarCommitPropertyChangedEventOnTimer.GetValueOnAnyThread())
	{
		ToolSettings->PropertyChangedEventQueue.Add(WorkingSpline, EmptyPropertyChangedEvent);
	}
	else
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(WorkingSpline, EmptyPropertyChangedEvent);
	}
}

FVector UPCGDrawSplineToolBase::GetNewPointNormal(const FVector& HitNormal) const
{
	UPCGInteractiveToolSettings_SplineBase* ToolSettings = GetSplineSettings();
	
	USplineComponent* WorkingSpline = ToolSettings->GetWorkingSplineComponent(ToolSettings->DataInstance);
	const int32 NumSplinePoints = WorkingSpline->GetNumberOfSplinePoints();
	switch (ToolSettings->Settings.NormalMode)
	{
		case EPCGToolDrawTargetNormal::AlignToPrevious:
		{
			if (NumSplinePoints == 1)
			{
				/*
				 * If there's only one point, GetUpVectorAtSplinePoint is unreliable because it seeks to build a quaternion
				 * from the tangent and the set-up vector, and the tangent is zero. We want to use the "stored" up
				 * vector directly.
				 */
				const FVector LocalUpVector = WorkingSpline->GetRotationAtSplinePoint(0, ESplineCoordinateSpace::Local).RotateVector(WorkingSpline->DefaultUpVector);
				return WorkingSpline->GetComponentTransform().TransformVectorNoScale(LocalUpVector);
			}
			else if (NumSplinePoints > 1)
			{
				return WorkingSpline->GetUpVectorAtSplinePoint(NumSplinePoints - 1, ESplineCoordinateSpace::World);
			}
			else // NumSplinePoints is zero or negative, which can happen, if there are no previous points.
			{
				return HitNormal;
			}
		}
		case EPCGToolDrawTargetNormal::WorldUp:
			return FVector::UnitZ();
		case EPCGToolDrawTargetNormal::Explicit:
			return ToolSettings->Settings.Normal;
		default: // fall-through
		case EPCGToolDrawTargetNormal::HitNormal:
			return HitNormal;
	}
}

FInputRayHit UPCGDrawSplineToolBase::IsHitByClick(const FInputDeviceRay& ClickPosition)
{
	using namespace UE::PCG::EditorMode::Scene;
	
	if (TOptional<FViewHitResult> Result = ViewportRay(ClickPosition, CachedRayParams))
	{
		return Result->ToInputRayHit();
	}

	return {};
}

void UPCGDrawSplineToolBase::ClearSplineOnFirstInteraction(UPCGInteractiveToolSettings_SplineBase* ToolSettings, const FVector& NewStartingLocation)
{
	if(ToolSettings && ToolSettings->DrawMode != PreviousDrawMode)
	{
		if (USplineComponent* SplineComponent = ToolSettings->GetWorkingSplineComponent(ToolSettings->DataInstance))
		{
			SplineComponent->bSplineHasBeenEdited = true;
			SplineComponent->ClearSplinePoints(true);

			// Update the actor pivot if it's a new actor
			if (ToolSettings->HasSpawnedActor() && SplineComponent->GetOwner() && SplineComponent->GetOwner()->GetRootComponent() != SplineComponent)
			{
				SplineComponent->GetOwner()->GetRootComponent()->SetWorldTransform(FTransform(NewStartingLocation));

				// Propagate change so the rest of the engine is aware of the change (esp. rendering/proxies).
				FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
				FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(SplineComponent->GetOwner()->GetRootComponent(), EmptyPropertyChangedEvent);
			}

			SplineComponent->SetWorldTransform(FTransform(NewStartingLocation));

			FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(SplineComponent, EmptyPropertyChangedEvent);
		}

		// For "continuous" draw modes like FreeDraw, we reset the spline data so so the user can re-drag as needed.
		if (ToolSettings->DrawMode != EPCGDrawSplineDrawMode::FreeDraw)
		{
			PreviousDrawMode = ToolSettings->DrawMode;
		}
	}
}

void UPCGDrawSplineToolBase::OnClicked(const FInputDeviceRay& ClickPosition)
{
	using namespace PCGDrawSplineTool;

	if (TOptional<Scene::FViewHitResult> Result = Scene::ViewportRay(ClickPosition, CachedRayParams))
	{
		UPCGInteractiveToolSettings_SplineBase* ToolSettings = GetSplineSettings();
		ClearSplineOnFirstInteraction(ToolSettings, Result->ImpactPosition);
		
		USplineComponent* WorkingSpline = ToolSettings->GetWorkingSplineComponent(ToolSettings->DataInstance);
		switch (ToolSettings->DrawMode)
		{
			case EPCGDrawSplineDrawMode::ClickAutoTangent:
			case EPCGDrawSplineDrawMode::FreeDraw:
			{
				AddSplinePoint(Result->ImpactPosition, Result->ImpactNormal);

				const int32 PointIndex = WorkingSpline->GetNumberOfSplinePoints() - 1;
				GetToolManager()->EmitObjectChange(this,
												   MakeUnique<CommandChange::FSimplePointInsertionChange>(
													   WorkingSpline,
													   Result->ImpactPosition,
													   WorkingSpline->GetUpVectorAtSplinePoint(PointIndex, ESplineCoordinateSpace::World)),
												   Constants::AddPointTransactionName);
				break;
			}
			case EPCGDrawSplineDrawMode::TangentDrag:
			{
				AddSplinePoint(Result->ImpactPosition, Result->ImpactNormal);

				const int32 PointIndex = WorkingSpline->GetNumberOfSplinePoints() - 1;
				WorkingSpline->SetTangentAtSplinePoint(PointIndex,
													   FVector::Zero(),
													   ESplineCoordinateSpace::World, true);

				GetToolManager()->EmitObjectChange(
					this,
					MakeUnique<CommandChange::FTangentPointInsertionChange>(
						WorkingSpline,
						Result->ImpactPosition,
						WorkingSpline->GetUpVectorAtSplinePoint(PointIndex, ESplineCoordinateSpace::World),
						FVector::Zero()),
					Constants::AddPointTransactionName);
				break;
			}
		}
	}
}

FInputRayHit UPCGDrawSplineToolBase::CanBeginClickDragSequence(const FInputDeviceRay& DragPosition)
{
	using namespace UE::PCG::EditorMode::Scene;
	if (TOptional<FViewHitResult> Result = ViewportRay(DragPosition, CachedRayParams))
	{
		return Result->ToInputRayHit();
	}

	return {};
}

void UPCGDrawSplineToolBase::OnClickPress(const FInputDeviceRay& ClickPosition)
{
	using namespace UE::PCG::EditorMode::Scene;
	if (TOptional<FViewHitResult> Result = ViewportRay(ClickPosition, CachedRayParams))
	{
		// @todo_pcg
		// When the Spline Tool exposes more controls, such as selecting spline points to move them, resetting the spline etc., we will remove this.
		UPCGInteractiveToolSettings_SplineBase* ToolSettings = GetSplineSettings();
		ClearSplineOnFirstInteraction(ToolSettings, Result->ImpactPosition);
		
		AddSplinePoint(Result->ImpactPosition, Result->ImpactNormal);
		
		if (ToolSettings->DrawMode == EPCGDrawSplineDrawMode::FreeDraw)
		{
			// Cache the index for where this stroke began
			FreeDrawStrokeStartIndex = ToolSettings->GetWorkingSplineComponent(ToolSettings->DataInstance)->GetNumberOfSplinePoints() - 1;
		}
	}
}

void UPCGDrawSplineToolBase::OnClickDrag(const FInputDeviceRay& DragPosition)
{
	using namespace UE::PCG::EditorMode::Scene;

	UPCGInteractiveToolSettings_SplineBase* ToolSettings = GetSplineSettings();
	
	USplineComponent* WorkingSpline = ToolSettings->GetWorkingSplineComponent(ToolSettings->DataInstance);
	const int32 NumSplinePoints = WorkingSpline->GetNumberOfSplinePoints();
	if (!ensure(NumSplinePoints > 0))
	{
		return;
	}

	if (TOptional<FViewHitResult> Result = ViewportRay(DragPosition, CachedRayParams))
	{
		const int32 FinalSplinePoint = NumSplinePoints - 1;
		switch (ToolSettings->DrawMode)
		{
			case EPCGDrawSplineDrawMode::ClickAutoTangent:
			{
				// Drag the last placed point
				const FVector Normal = GetNewPointNormal(Result->ImpactNormal);
				WorkingSpline->SetLocationAtSplinePoint(FinalSplinePoint, Result->ImpactPosition, ESplineCoordinateSpace::World, false);
				WorkingSpline->SetUpVectorAtSplinePoint(FinalSplinePoint, Normal, ESplineCoordinateSpace::World, true);

				break;
			}
			case EPCGDrawSplineDrawMode::TangentDrag:
			{
				// Set the tangent
				const FVector LastPoint = WorkingSpline->GetLocationAtSplinePoint(FinalSplinePoint, ESplineCoordinateSpace::World);
				const FVector Tangent = (Result->ImpactPosition - LastPoint) / FMath::Min(0.5f, GetDefault<ULevelEditorViewportSettings>()->SplineTangentScale);
				WorkingSpline->SetTangentAtSplinePoint(FinalSplinePoint, Tangent, ESplineCoordinateSpace::World, true);
				bDrawTangentForPreviousPoint = true;
				break;
			}

			case EPCGDrawSplineDrawMode::FreeDraw:
			{
				/*
				 * Instead of dragging the first placed point (which gets placed in OnClickPress), drag a second "preview" one
				 * until far enough from the previous to place a new control point.
				 */
				if (!bFreeDrawPlacedPreviewPoint)
				{
					AddSplinePoint(Result->ImpactPosition, Result->ImpactNormal);
					bFreeDrawPlacedPreviewPoint = true;
				}
				else
				{
					FVector FinalLocation = Result->ImpactPosition;
					const FVector Normal = GetNewPointNormal(Result->ImpactNormal);

					switch (ToolSettings->Settings.OffsetMode)
					{
						case EPCGToolDrawTargetOffset::DistanceOffset:
							check(Result->ImpactNormal.IsNormalized());
							FinalLocation += ToolSettings->Settings.OffsetDistance * Normal;
							break;
						case EPCGToolDrawTargetOffset::ExplicitOffset:
							FinalLocation += ToolSettings->Settings.OffsetVector;
							break;
						default: // fall-through
						case EPCGToolDrawTargetOffset::HitLocation:
							break;
					}

					WorkingSpline->SetLocationAtSplinePoint(FinalSplinePoint, FinalLocation, ESplineCoordinateSpace::World, /*bUpdateSpline=*/false);
					WorkingSpline->SetUpVectorAtSplinePoint(FinalSplinePoint, Normal, ESplineCoordinateSpace::World, /*bUpdateSpline=*/true);
					const FVector PreviousPoint = WorkingSpline->GetLocationAtSplinePoint(FinalSplinePoint - 1, ESplineCoordinateSpace::World);
					if (FVector::DistSquared(FinalLocation, PreviousPoint) >= (ToolSettings->MinPointSpacing * ToolSettings->MinPointSpacing))
					{
						AddSplinePoint(Result->ImpactPosition, Result->ImpactNormal);
					}
				}
				break;
			}
		}
	}
}

void UPCGDrawSplineToolBase::OnClickRelease(const FInputDeviceRay& ReleasePosition)
{
	// Activate the final position
	OnClickDrag(ReleasePosition);
	OnTerminateDragSequence();
}

void UPCGDrawSplineToolBase::OnTerminateDragSequence()
{
	using namespace PCGDrawSplineTool;

	UPCGInteractiveToolSettings_SplineBase* ToolSettings = GetSplineSettings();
	USplineComponent* WorkingSpline = ToolSettings->GetWorkingSplineComponent(ToolSettings->DataInstance);

	check(WorkingSpline);

	bDrawTangentForPreviousPoint = false;
	bFreeDrawPlacedPreviewPoint = false;

	const int32 NumSplinePoints = WorkingSpline->GetNumberOfSplinePoints();
	const int32 PreviousSplinePoint = NumSplinePoints - 1;

	check(PreviousSplinePoint >= 0);
	
	// Emit the appropriate undo transaction
	switch (ToolSettings->DrawMode)
	{
		case EPCGDrawSplineDrawMode::ClickAutoTangent:
		{
			GetToolManager()->EmitObjectChange(
				this,
				MakeUnique<CommandChange::FSimplePointInsertionChange>(
					WorkingSpline,
					WorkingSpline->GetLocationAtSplinePoint(PreviousSplinePoint, ESplineCoordinateSpace::World),
					WorkingSpline->GetUpVectorAtSplinePoint(PreviousSplinePoint, ESplineCoordinateSpace::World)),
				Constants::AddPointTransactionName);
			break;
		}
		case EPCGDrawSplineDrawMode::TangentDrag:
		{
			GetToolManager()->EmitObjectChange(
				this,
				MakeUnique<CommandChange::FTangentPointInsertionChange>(
					WorkingSpline,
					WorkingSpline->GetLocationAtSplinePoint(PreviousSplinePoint, ESplineCoordinateSpace::World),
					WorkingSpline->GetUpVectorAtSplinePoint(PreviousSplinePoint, ESplineCoordinateSpace::World),
					WorkingSpline->GetTangentAtSplinePoint(PreviousSplinePoint, ESplineCoordinateSpace::World)),
				Constants::AddPointTransactionName);
			break;
		}

		case EPCGDrawSplineDrawMode::FreeDraw:
		{
			TArray<FVector> HitLocations;
			TArray<FVector> UpVectors;
			for (int32 Index = FreeDrawStrokeStartIndex; Index < NumSplinePoints; ++Index)
			{
				HitLocations.Add(WorkingSpline->GetLocationAtSplinePoint(Index, ESplineCoordinateSpace::World));
				UpVectors.Add(WorkingSpline->GetUpVectorAtSplinePoint(Index, ESplineCoordinateSpace::World));
			}

			GetToolManager()->EmitObjectChange(
				this,
				MakeUnique<CommandChange::FFreeDrawStrokeChange>(WorkingSpline, HitLocations, UpVectors),
				Constants::AddPointTransactionName);
			break;
		}
	}
}

UPCGInteractiveToolSettings_SplineBase* UPCGDrawSplineToolBase::GetSplineSettings() const
{
	TArray<UObject*> AllToolProperties = GetToolProperties(false);

	UObject** FoundProperties = AllToolProperties.FindByPredicate([](const UObject* Candidate)
	{
		return Candidate->IsA<UPCGInteractiveToolSettings_SplineBase>();
	});

	if(FoundProperties)
	{
		return Cast<UPCGInteractiveToolSettings_SplineBase>(*FoundProperties);
	}

	return nullptr;
}

void UPCGDrawSplineToolBase::OnTick(const float DeltaTime)
{
	UPCGInteractiveToolSettings_SplineBase* ToolSettings = GetSplineSettings();
	
	USplineComponent* WorkingSpline = ToolSettings->GetWorkingSplineComponent(ToolSettings->DataInstance);
	if (!WorkingSpline)
	{
		GetToolManager()->PostActiveToolShutdownRequest(
			this,
			EToolShutdownType::Cancel,
			/*bShowUnexpectedShutdownMessage=*/true,
			LOCTEXT("LostWorkingSpline", "The Draw Spline tool must close because the in-progress spline has been unexpectedly deleted."));
	}

	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CachedCameraState);

	Super::OnTick(DeltaTime);
}

void UPCGDrawSplineToolBase::Render(IToolsContextRenderAPI* RenderAPI)
{
	using namespace PCG::EditorMode::Scene;

	UPCGInteractiveToolSettings_SplineBase* ToolSettings = GetSplineSettings();
	
	USplineComponent* WorkingSpline = ToolSettings->GetWorkingSplineComponent(ToolSettings->DataInstance);
	if (WorkingSpline)
	{
		const int32 LastPointIndex = WorkingSpline->GetNumberOfSplinePoints() - 1;

		if (bDrawTangentForPreviousPoint)
		{
			Visualize::SplineTangent(*WorkingSpline, *RenderAPI, LastPointIndex);
		}

		Visualize::Spline(*WorkingSpline, *RenderAPI, LastPointIndex, /*bShouldVisualizeScale=*/false, /*bShouldVisualizeTangents=*/true);
	}
}

bool UPCGDrawSplineToolBase::CanAccept() const
{
	UPCGInteractiveToolSettings_SplineBase* ToolSettings = GetSplineSettings();

	USplineComponent* WorkingSpline = ToolSettings->GetWorkingSplineComponent(ToolSettings->DataInstance);
	return WorkingSpline && WorkingSpline->GetNumberOfSplinePoints() > 0;
}

void UPCGDrawSplineTool::Setup()
{
	AddToolPropertySource(ToolSettings);
	
	Super::Setup();
}

void UPCGDrawSplineSurfaceTool::Setup()
{
	AddToolPropertySource(ToolSettings);
	
	Super::Setup();
}

bool UPCGDrawSplineToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	// @todo_pcg: To be factorized in a base class
	if (SplineToolClass == nullptr)
	{
		return false;
	}

	const UPCGInteractiveToolSettings_SplineBase* SettingsCDO = GetDefault<UPCGInteractiveToolSettings_SplineBase>();
	check(SettingsCDO)
	return SceneState.SelectedActors.IsEmpty()
		|| SceneState.SelectedActors.ContainsByPredicate([SettingsCDO](const AActor* InActor) { return SettingsCDO->IsWorkingActorCompatible(InActor); });
}

UInteractiveTool* UPCGDrawSplineToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UPCGDrawSplineToolBase* NewTool = NewObject<UPCGDrawSplineToolBase>(SceneState.ToolManager, SplineToolClass);

	NewTool->SetToolInfo(
	{
		.ToolDisplayName = PCGDrawSplineTool::Constants::ToolName,
		.ToolDisplayMessage = PCGDrawSplineTool::Constants::ToolDescription,
		.ToolIcon = FPCGEditorModeStyle::Get().GetBrush("PCGEditorMode.Tools.DrawSpline")
	});
	
	if(UE::PCG::EditorMode::Tool::BuildTool(NewTool) == false)
	{
		return nullptr;
	}

	return NewTool;
}

void UPCGDrawSplineToolBuilder::SetSplineToolClass(TSubclassOf<UPCGDrawSplineToolBase> InClass)
{
	SplineToolClass = InClass;
}

#undef LOCTEXT_NAMESPACE
