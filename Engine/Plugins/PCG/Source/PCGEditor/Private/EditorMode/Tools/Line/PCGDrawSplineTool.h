// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorMode/Tools/PCGInteractiveToolBuilder.h"
#include "EditorMode/Tools/PCGInteractiveToolCommon.h"
#include "EditorMode/Tools/PCGInteractiveToolSettings.h"
#include "EditorMode/Tools/Helpers/PCGEdModeSceneQueryHelpers.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "Components/SplineComponent.h"

#include "PCGDrawSplineTool.generated.h"

class USplineComponent;
class USingleClickOrDragInputBehavior;

UENUM()
enum class EPCGDrawSplineDrawMode : uint8
{
	TangentDrag UMETA(ToolTip = "Click to place a point and then drag to set its tangent. Clicking without dragging will create sharp corners."),
	ClickAutoTangent UMETA(ToolTip = "Click and drag new points, with the tangent set automatically."),
	FreeDraw UMETA(ToolTip = "Drag to place multiple points, with spacing controlled by Min Point Spacing."),
	None UMETA(Hidden)
};

UCLASS(Abstract)
class UPCGInteractiveToolSettings_SplineBase : public UPCGInteractiveToolSettings
{
	GENERATED_BODY()

public:
	UPCGInteractiveToolSettings_SplineBase();
	
	USplineComponent* GetWorkingSplineComponent(FName DataInstanceIdentifier) const;
	TArray<USplineComponent*> GetWorkingSplineComponents() const;
	
	UPROPERTY(EditAnywhere, Category = "Spline")
	EPCGDrawSplineDrawMode DrawMode = EPCGDrawSplineDrawMode::FreeDraw;
	
	/** Minimum world distance to cover while free drawing before a point will be generated. */
	UPROPERTY(EditAnywhere, Category = "Spline", meta = (
			ClampMin = 0,
			UIMax = 1000,
			EditCondition = "DrawMode == EPCGDrawSplineDrawMode::FreeDraw",
			EditConditionHides))
	double MinPointSpacing = 200.0;

	UPROPERTY(EditAnywhere, Category = "Spline", meta = (TransientToolProperty))
	FName SplineName;

	/** The raycast rules that determine whether a raycast is valid or not.
	 *  Used to determine different raycast targets such as landscapes, meshes, selected actor etc.	 */
	UPROPERTY(EditAnywhere, Category= "Raycast")
	FPCGRaycastFilterRuleCollection RaycastRuleCollection;

	/** Whether to place spline points on a plane through the origin aligned with the Z axis in perspective views, or facing the camera in othographic views */
	UPROPERTY(EditAnywhere, Category= "Raycast")
	bool bHitGroundPlanes = true;

	/** The settings to manipulate raycast results, such as normals or offsets of a spline point. */
	UPROPERTY(EditAnywhere, Category = "Raycast", meta = (ShowOnlyInnerProperties))
	FPCGToolRaycastSettings Settings{};

protected:
	/** This function is called indirectly by the builder using UPCGInteractiveToolSettings_SplineBase default class which prohibits to have override
	 * implementation of this function by child classes. Mark it final for that reason, to be revisited if we change the tool classes hierarchy. */
	virtual bool IsWorkingActorCompatibleInternal(const AActor* InActor) const override final;
};

UCLASS()
class UPCGInteractiveToolSettings_Spline : public UPCGInteractiveToolSettings_SplineBase
{
	GENERATED_BODY()

protected:
	virtual const UScriptStruct* GetWorkingDataType() const override;
	virtual FName GetToolTag() const override;
	virtual void PostWorkingDataInitialized(FPCGInteractiveToolWorkingData* WorkingData) const override;
	virtual void OnPropertyModified(UObject* Object, FProperty* Property) override;
public:
	UPROPERTY(EditAnywhere, Category = "Spline")
	bool bClosedSpline = false;

	static FName StaticGetToolTag();
};


UCLASS()
class UPCGInteractiveToolSettings_SplineSurface : public UPCGInteractiveToolSettings_SplineBase
{
	GENERATED_BODY()

protected:
	virtual const UScriptStruct* GetWorkingDataType() const override;
	virtual FName GetToolTag() const override;

public:
	static FName StaticGetToolTag();
};

UCLASS(Abstract, Experimental)
class UPCGDrawSplineToolBase : public UInteractiveTool, public IClickBehaviorTarget, public IClickDragBehaviorTarget
{
	GENERATED_BODY()

public:
	// ~Begin UInteractiveTool interface
	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	// ~End UInteractiveTool interface

	// ~Begin IClickBehaviorTarget interface
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPosition) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPosition) override;
	// ~End IClickBehaviorTarget interface

	// ~Begin IClickDragBehaviorTarget interface
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& DragPosition) override;
	virtual void OnClickPress(const FInputDeviceRay& ClickPosition) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPosition) override;
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePosition) override;
	virtual void OnTerminateDragSequence() override;
	// ~End IClickDragBehaviorTarget interface
	
	UPCGInteractiveToolSettings_SplineBase* GetSplineSettings() const;
private:
	void AddSplinePoint(const FVector& HitLocation, const FVector& HitNormal) const;
	FVector GetNewPointNormal(const FVector& HitNormal) const;

	void ClearSplineOnFirstInteraction(UPCGInteractiveToolSettings_SplineBase* InSettings, const FVector& NewStartingLocation);

	UPROPERTY(Instanced)
	TObjectPtr<USingleClickOrDragInputBehavior> ClickOrDragBehavior = nullptr;
	
	UE::PCG::EditorMode::Scene::FViewRayParams CachedRayParams{};
	FViewCameraState CachedCameraState;
	bool bDrawTangentForPreviousPoint = false;
	bool bFreeDrawPlacedPreviewPoint = false;
	int32 FreeDrawStrokeStartIndex = 0;

	EPCGDrawSplineDrawMode PreviousDrawMode = EPCGDrawSplineDrawMode::None;
};

UCLASS(Experimental)
class UPCGDrawSplineTool : public UPCGDrawSplineToolBase
{
	GENERATED_BODY()

public:
	virtual void Setup() override;
public:
	UPROPERTY(Transient)
	TObjectPtr<UPCGInteractiveToolSettings_Spline> ToolSettings;
};

UCLASS(Experimental)
class UPCGDrawSplineSurfaceTool : public UPCGDrawSplineToolBase
{
	GENERATED_BODY()

public:
	virtual void Setup() override;
public:
	UPROPERTY(Transient)
	TObjectPtr<UPCGInteractiveToolSettings_SplineSurface> ToolSettings;
};

UCLASS(Transient)
class UPCGDrawSplineToolBuilder : public UPCGInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	void SetSplineToolClass(TSubclassOf<UPCGDrawSplineToolBase> InClass);
private:
	UPROPERTY()
	TSubclassOf<UPCGDrawSplineToolBase> SplineToolClass;
};
