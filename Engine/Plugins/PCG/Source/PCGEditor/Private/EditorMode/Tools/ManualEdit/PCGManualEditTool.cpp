// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/ManualEdit/PCGManualEditTool.h"

#include "PCGComponent.h"
#include "PCGEditorModule.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGSettings.h"
#include "Elements/PCGStaticMeshSpawner.h"

#include "Editor.h"
#include "InteractiveToolManager.h"
#include "UnrealEdGlobals.h"
#include "Selection.h"
#include "Algo/AnyOf.h"
#include "Containers/Ticker.h"
#include "Editor/Transactor.h"
#include "Editor/UnrealEdEngine.h"
#include "Modules/ModuleManager.h"

#include "SPCGManualEditPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGManualEditTool)

#define LOCTEXT_NAMESPACE "UPCGManualEditTool"

namespace PCGManualEditTool::Constants
{
	const FName ManualEditToolTag = "ManualEditTool";
} // PCGManualEditTool::Constants

namespace PCGManualEditTool::Helpers
{
	TArray<UPCGNode*> CollectSpawnerNodes(const UPCGComponent* InComponent)
	{
		TArray<UPCGNode*> Result;
		if (const UPCGGraph* Graph = InComponent ? InComponent->GetGraph() : nullptr)
		{
			Graph->ForEachNodeRecursively([&Result](UPCGNode* Node)
			{
				const UPCGSettingsInterface* SettingsInterface = Node ? Node->GetSettingsInterface() : nullptr;
				const UPCGSettings* Settings = SettingsInterface ? SettingsInterface->GetSettings() : nullptr;
				if (Settings && Settings->IsA<UPCGStaticMeshSpawnerSettings>())
				{
					Result.Add(Node);
				}

				return true;
			});
		}

		return Result;
	}
} // PCGManualEditTool::Helpers

FName UPCGInteractiveToolSettings_ManualEdit::StaticGetToolTag()
{
	return PCGManualEditTool::Constants::ManualEditToolTag;
}

bool UPCGInteractiveToolSettings_ManualEdit::IsSelectionAllowed(AActor* InActor, bool bInSelection) const
{
	const AActor* LockedActor = SelectedActor.Get();
	if (!LockedActor)
	{
		return true;
	}

	return bInSelection == (InActor == LockedActor);
}

FName UPCGManualEditTool::StaticGetToolTag()
{
	return PCGManualEditTool::Constants::ManualEditToolTag;
}

bool UPCGManualEditTool::CanAccept() const
{
	if (TransactionHistoryMarker == INDEX_NONE || !GEditor || !GEditor->Trans)
	{
		return false;
	}

	const int32 TransactionPosition = (GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount());
	return TransactionPosition > TransactionHistoryMarker;
}

void UPCGManualEditTool::Setup()
{
	Super::Setup();

	if (!ToolSettings)
	{
		ToolSettings = NewObject<UPCGInteractiveToolSettings_ManualEdit>(this);
	}

	AddToolPropertySource(ToolSettings);

	// Transactions happen internally for manual edits. Mark where we are currently in the transaction history to track.
	// Activating the tool will increase the buffer by one, but this happens after the setup. So shift the marker by one.
	const UTransactor* Transactor = GEditor ? GEditor->Trans.Get() : nullptr;
	TransactionHistoryMarker = Transactor ? (Transactor->GetQueueLength() - Transactor->GetUndoCount() + 1) : INDEX_NONE;

	RefreshFromSelectedActor();

	if (const FPCGEditorModule* EditorModule = FModuleManager::GetModulePtr<FPCGEditorModule>("PCGEditor"))
	{
		if (TSharedPtr<SPCGManualEditPanel> Panel = EditorModule->GetManualEditPanel())
		{
			Panel->SetMode(EPCGManualEditPanelMode::ExternallyControlled);
		}
	}

	SelectionChangedDelegateHandle = USelection::SelectionChangedEvent.AddUObject(this, &UPCGManualEditTool::OnEditorSelectionChanged);
	PostUndoRedoHandle = FEditorDelegates::PostUndoRedo.AddUObject(this, &UPCGManualEditTool::OnPostUndoRedo);
}

void UPCGManualEditTool::Shutdown(const EToolShutdownType ShutdownType)
{
	// Clean up the visualizer and it's gizmo, etc.
	// @todo_pcg: Long term, the visualizer could be brought off the shared context
	if (GUnrealEd)
	{
		GUnrealEd->ComponentVisManager.ClearActiveComponentVis();
	}

	if (const FPCGEditorModule* EditorModule = FModuleManager::GetModulePtr<FPCGEditorModule>("PCGEditor"))
	{
		if (TSharedPtr<SPCGManualEditPanel> Panel = EditorModule->GetManualEditPanel())
		{
			Panel->SetMode(EPCGManualEditPanelMode::UserControlled);
		}
	}

	if (SelectionChangedDelegateHandle.IsValid())
	{
		USelection::SelectionChangedEvent.Remove(SelectionChangedDelegateHandle);
		SelectionChangedDelegateHandle.Reset();
	}

	if (PostUndoRedoHandle.IsValid())
	{
		FEditorDelegates::PostUndoRedo.Remove(PostUndoRedoHandle);
		PostUndoRedoHandle.Reset();
	}

	// Cancel reverts all session edits. Accept terminates the tool mode.
	// @todo_pcg: Deferred ticker might be unsafe across editor shutdown.
	if (ShutdownType != EToolShutdownType::Accept && TransactionHistoryMarker != INDEX_NONE)
	{
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([Mark = TransactionHistoryMarker](float)
		{
			if (UTransactor* Transactor = GEditor ? GEditor->Trans.Get() : nullptr; Transactor && Mark != INDEX_NONE)
			{
				while ((Transactor->GetQueueLength() - Transactor->GetUndoCount()) > Mark)
				{
					if (!Transactor->Undo())
					{
						break;
					}
				}
			}

			return false;
		}));
	}
	TransactionHistoryMarker = INDEX_NONE;

	RestoreEnabledSpawners();

	if (ToolSettings)
	{
		RemoveToolPropertySource(ToolSettings);
	}

	FModuleManager::GetModuleChecked<FPCGEditorModule>("PCGEditor").UpdateManualEditPanelVisibility();

	Super::Shutdown(ShutdownType);
}

void UPCGManualEditTool::OnEditorSelectionChanged(UObject* Object)
{
	if (bReSyncingSelection || !ToolSettings || !GEditor)
	{
		return;
	}

	if (Object != GEditor->GetSelectedActors())
	{
		return;
	}

	AActor* LockedActor = ToolSettings->SelectedActor.Get();
	if (!LockedActor)
	{
		return;
	}

	bool bLockedFound = false;
	TArray<AActor*> OtherActors;
	if (USelection* Selection = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator It(*Selection); It; ++It)
		{
			AActor* Actor = Cast<AActor>(*It);
			if (Actor == LockedActor)
			{
				bLockedFound = true;
			}
			else if (Actor)
			{
				OtherActors.Add(Actor);
			}
		}
	}

	if (bLockedFound && OtherActors.IsEmpty())
	{
		return;
	}

	TGuardValue<bool> Guard(bReSyncingSelection, true);

	USelection* SelectedActors = GEditor->GetSelectedActors();
	SelectedActors->Modify();
	SelectedActors->BeginBatchSelectOperation();

	for (AActor* OtherActor : OtherActors)
	{
		GEditor->SelectActor(OtherActor, /*bInSelected=*/false, /*bNotify=*/false);
	}

	if (!bLockedFound)
	{
		GEditor->SelectActor(LockedActor, /*bInSelected=*/true, /*bNotify=*/false);
	}

	SelectedActors->EndBatchSelectOperation();
}

void UPCGManualEditTool::OnPostUndoRedo()
{
	if (bClampingUndoRedo || TransactionHistoryMarker == INDEX_NONE || !GEditor || !GEditor->Trans)
	{
		return;
	}

	// @todo_pcg: Investigate if there's a better solution than using Transactions.
	// Clamp the user's Ctrl-Z to the tool session start. If they precede the history, roll it back.
	TGuardValue<bool> Guard(bClampingUndoRedo, true);

	while ((GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount()) < TransactionHistoryMarker)
	{
		if (!GEditor->Trans->Redo())
		{
			TransactionHistoryMarker = GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount();
			break;
		}
	}
}

void UPCGManualEditTool::RefreshFromSelectedActor()
{
	RestoreEnabledSpawners();

	if (!ToolSettings)
	{
		return;
	}

	AActor* SelectedActor = nullptr;
	if (USelection* Selection = GEditor ? GEditor->GetSelectedActors() : nullptr)
	{
		for (FSelectionIterator It(*Selection); It; ++It)
		{
			SelectedActor = Cast<AActor>(*It);
			if (SelectedActor)
			{
				break;
			}
		}
	}

	ToolSettings->SelectedActor = SelectedActor;
	ToolSettings->TrackedPCGComponents.Reset();
	ToolSettings->SpawnerNodeCount = 0;

	if (!SelectedActor)
	{
		FModuleManager::GetModuleChecked<FPCGEditorModule>("PCGEditor").UpdateManualEditPanelVisibility();
		return;
	}

	TArray<UPCGComponent*> PCGComponents;
	SelectedActor->GetComponents<UPCGComponent>(PCGComponents, /*bIncludeFromChildActors=*/true);

	for (UPCGComponent* Component : PCGComponents)
	{
		if (!Component || !Component->GetGraph())
		{
			continue;
		}

		ToolSettings->TrackedPCGComponents.Add(Component);
		EnableSpawnersOnComponent(Component);
	}

	ToolSettings->SpawnerNodeCount = EnabledSpawnerNodes.Num();

	FModuleManager::GetModuleChecked<FPCGEditorModule>("PCGEditor").UpdateManualEditPanelVisibility();
}

void UPCGManualEditTool::EnableSpawnersOnComponent(UPCGComponent* InComponent)
{
	TArray<UPCGNode*> Spawners = PCGManualEditTool::Helpers::CollectSpawnerNodes(InComponent);
	for (UPCGNode* Node : Spawners)
	{
		UPCGSettingsInterface* SettingsInterface = Node ? Node->GetSettingsInterface() : nullptr;
		if (!SettingsInterface || SettingsInterface->IsTemporaryManualEditingEnabled())
		{
			continue;
		}

		SettingsInterface->SetTemporaryManualEditingEnabled(true);
		Node->OnNodeChangedDelegate.Broadcast(Node, EPCGChangeType::Cosmetic);
		EnabledSpawnerNodes.Add(Node);
	}
}

void UPCGManualEditTool::RestoreEnabledSpawners()
{
	for (const TWeakObjectPtr<UPCGNode>& WeakNode : EnabledSpawnerNodes)
	{
		UPCGNode* Node = WeakNode.Get();
		UPCGSettingsInterface* SettingsInterface = Node ? Node->GetSettingsInterface() : nullptr;
		if (!SettingsInterface || !SettingsInterface->IsTemporaryManualEditingEnabled())
		{
			continue;
		}

		SettingsInterface->SetTemporaryManualEditingEnabled(false);
		Node->OnNodeChangedDelegate.Broadcast(Node, EPCGChangeType::Cosmetic);
	}

	EnabledSpawnerNodes.Reset();
}

bool UPCGManualEditToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (SceneState.SelectedActors.Num() != 1 || !SceneState.SelectedActors[0])
	{
		return false;
	}

	TArray<UPCGComponent*> PCGComponents;
	SceneState.SelectedActors[0]->GetComponents<UPCGComponent>(PCGComponents, /*bIncludeFromChildActors=*/true);

	return Algo::AnyOf(PCGComponents, [](const UPCGComponent* Component)
	{
		return Component && Component->GetGraph();
	});
}

UInteractiveTool* UPCGManualEditToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UPCGManualEditTool* NewTool = NewObject<UPCGManualEditTool>(SceneState.ToolManager);
	NewTool->ToolSettings = NewObject<UPCGInteractiveToolSettings_ManualEdit>(NewTool);
	return NewTool;
}

#undef LOCTEXT_NAMESPACE
