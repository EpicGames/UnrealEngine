// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorMode/Tools/PCGInteractiveToolBuilder.h"
#include "EditorMode/Tools/PCGInteractiveToolSettings.h"

#include "PCGManualEditTool.generated.h"

class AActor;
class UPCGComponent;
class UPCGNode;

UCLASS(BlueprintType)
class UPCGInteractiveToolSettings_ManualEdit : public UPCGInteractiveToolBaseSettings
{
	GENERATED_BODY()

public:
	UPCGInteractiveToolSettings_ManualEdit() = default;

	static FName StaticGetToolTag();

	virtual FName GetToolTag() const override { return StaticGetToolTag(); }

	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const override;

	UPROPERTY(VisibleAnywhere, Category = "Manual Edit")
	TWeakObjectPtr<AActor> SelectedActor;

	UPROPERTY(VisibleAnywhere, Category = "Manual Edit")
	TArray<TWeakObjectPtr<UPCGComponent>> TrackedPCGComponents;

	// The number of nodes with Static Mesh Spawners in the graph. Useful data to see in the tool properties panel.
	UPROPERTY(VisibleAnywhere, Category = "Manual Edit", DisplayName = "Static Mesh Spawner Nodes")
	int32 SpawnerNodeCount = 0;
};

UCLASS(Experimental)
class UPCGManualEditTool : public UPCGInteractiveTool
{
	GENERATED_BODY()

public:
	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override;

	virtual bool RequiresSettings() const override { return true; }

	/** Manual Edit selects a PCG-spawned element and uses the standard transform gizmo to create an override. */
	virtual bool UsesTransformGizmo() const override { return true; }

	static FName StaticGetToolTag();

private:
	friend class UPCGManualEditToolBuilder;

	void RefreshFromSelectedActor();
	void EnableSpawnersOnComponent(UPCGComponent* InComponent);
	void RestoreEnabledSpawners();
	void OnEditorSelectionChanged(UObject* Object);
	void OnPostUndoRedo();

	UPROPERTY(Transient)
	TObjectPtr<UPCGInteractiveToolSettings_ManualEdit> ToolSettings;

	TArray<TWeakObjectPtr<UPCGNode>> EnabledSpawnerNodes;
	FDelegateHandle SelectionChangedDelegateHandle;
	FDelegateHandle PostUndoRedoHandle;
	bool bReSyncingSelection = false;
	bool bClampingUndoRedo = false;

	/** Transactor queue length captured to revert all manual edit transactions on Cancel. */
	int32 TransactionHistoryMarker = INDEX_NONE;
};

UCLASS(Transient)
class UPCGManualEditToolBuilder : public UPCGInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};
