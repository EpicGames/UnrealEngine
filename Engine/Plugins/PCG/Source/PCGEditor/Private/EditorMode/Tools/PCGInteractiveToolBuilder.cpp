// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/PCGInteractiveToolBuilder.h"

#include "InteractiveToolManager.h"
#include "EditorMode/Tools/PCGInteractiveToolSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGInteractiveToolBuilder)

#define LOCTEXT_NAMESPACE "PCGInteractiveToolBuilder"

bool UE::PCG::EditorMode::Tool::BuildTool(UInteractiveTool* InTool)
{
	// This function uses reflection to instantiate and initialize the tool settings object.
	// This allows BP to only create a settings property for it to work, in theory
	
	using namespace PCG::EditorMode;

	if (!ensure(InTool))
	{
		return false;
	}

	if ((Cast<UPCGInteractiveTool>(InTool) && !Cast<UPCGInteractiveTool>(InTool)->RequiresSettings()))
	{
		return true;
	}

	const FObjectProperty* SettingsProperty = nullptr;
	const void* PropertyAddress = nullptr;
	UClass* SettingsClass = nullptr;
	for(TPropertyValueIterator<FObjectProperty> PropertyIt(InTool->GetClass(), InTool); PropertyIt; ++PropertyIt)
	{
		if(PropertyIt.Key()->IsA(FObjectProperty::StaticClass()))
		{
			const FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(PropertyIt.Key());
			if(ObjectProperty->PropertyClass->IsChildOf(UPCGInteractiveToolBaseSettings::StaticClass()) && ObjectProperty->PropertyClass->HasAnyClassFlags(EClassFlags::CLASS_Abstract) == false)
			{
				SettingsProperty = ObjectProperty;
				PropertyAddress = PropertyIt.Value();
				SettingsClass = ObjectProperty->PropertyClass;
				break;
			}
		}
	}
	
	if(SettingsProperty == nullptr || PropertyAddress == nullptr || SettingsClass == nullptr)
	{
		FText Message = FText::FormatOrdered(LOCTEXT("PCGSettingsPropertyMissingMessage", "Settings {0} of tool {1} does not have a valid PCGInteractiveToolBaseSettings property. Did you forget to mark it as UPROPERTY?"), SettingsClass ? SettingsClass->GetClass()->GetDisplayNameText() : FText::GetEmpty(), InTool->GetClass()->GetDisplayNameText());
		InTool->GetToolManager()->DisplayMessage(Message, EToolMessageLevel::UserError);
		return false;
	}
	
	UPCGInteractiveToolBaseSettings* Settings = NewObject<UPCGInteractiveToolBaseSettings>(InTool, SettingsClass);
	SettingsProperty->SetObjectPropertyValue(const_cast<void*>(PropertyAddress), Settings);
	if(TValueOrError<bool, FText> Result = Settings->Initialize(InTool); Result.HasError() || (Result.HasValue() && Result.GetValue() == false))
	{
		Settings->Cancel(InTool);
		Settings->Shutdown();

		FText Message = FText::FormatOrdered(LOCTEXT("PCGSettingsInitializationFailedMessage", "Tool {0} failed initialization."), InTool->GetClass()->GetDisplayNameText());
		
		if(Result.HasError())
		{
			Message = Result.GetError();
		}

		if(Message.IsEmpty() == false)
		{
			InTool->GetToolManager()->DisplayMessage(Message, EToolMessageLevel::UserError);
		}
		
		return false;
	}
	
	Settings->RestoreProperties(InTool);
	return true;
}

UInteractiveTool* UPCGInteractiveToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	return Super::BuildTool(SceneState);
}

void UPCGInteractiveToolBuilder::PostSetupTool(UInteractiveTool* InTool, const FToolBuilderState& SceneState) const
{
	if (!ensure(InTool))
	{
		return;
	}

	Super::PostSetupTool(InTool, SceneState);
	SceneState.ToolManager->DisplayMessage(InTool->GetToolInfo().ToolDisplayMessage, EToolMessageLevel::UserNotification);

	TArray<UObject*> ToolProperties = InTool->GetToolProperties();

	if(ToolGraph)
	{
		for(UObject* ToolPropertySet : ToolProperties)
		{
			if(UPCGInteractiveToolSettings* Settings = Cast<UPCGInteractiveToolSettings>(ToolPropertySet))
			{
				Settings->SetToolGraph(ToolGraph);
			}
		}
	}
}

void UPCGInteractiveToolWithToolTargetsBuilder::PostSetupTool(UInteractiveTool* InTool, const FToolBuilderState& SceneState) const
{
	if (!ensure(InTool))
	{
		return;
	}

	Super::PostSetupTool(InTool, SceneState);
	SceneState.ToolManager->DisplayMessage(InTool->GetToolInfo().ToolDisplayMessage, EToolMessageLevel::UserNotification);

	TArray<UObject*> ToolProperties = InTool->GetToolProperties();

	if(ToolGraph)
	{
		for(UObject* ToolPropertySet : ToolProperties)
		{
			if(UPCGInteractiveToolSettings* Settings = Cast<UPCGInteractiveToolSettings>(ToolPropertySet))
			{
				Settings->SetToolGraph(ToolGraph);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
