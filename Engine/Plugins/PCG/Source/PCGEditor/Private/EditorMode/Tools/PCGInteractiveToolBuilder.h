// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveToolBuilder.h"
#include "PCGGraph.h"

#include "PCGInteractiveToolBuilder.generated.h"

namespace UE::PCG::EditorMode::Tool
{
	bool BuildTool(UInteractiveTool* InTool);
}

UCLASS()
class UPCGInteractiveToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	// ~Begin UInteractiveToolBuilder interface
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	virtual void PostSetupTool(UInteractiveTool* InTool, const FToolBuilderState& SceneState) const override;
	// ~End UInteractiveToolBuilder interface

	void SetToolGraph(UPCGGraph* InGraph) { ToolGraph = InGraph; }
	UPCGGraph* GetToolGraph() const { return ToolGraph; }
private:
	UPROPERTY()
	TObjectPtr<UPCGGraph> ToolGraph;
};

UCLASS()
class UPCGInteractiveToolWithToolTargetsBuilder : public UInteractiveToolWithToolTargetsBuilder
{
	GENERATED_BODY()

public:
	// ~Begin UInteractiveToolBuilder interface
	virtual void PostSetupTool(UInteractiveTool* InTool, const FToolBuilderState& SceneState) const override;
	// ~End UInteractiveToolBuilder interface

	void SetToolGraph(UPCGGraph* InGraph) { ToolGraph = InGraph; }
	UPCGGraph* GetToolGraph() const { return ToolGraph; }
private:
	UPROPERTY()
	TObjectPtr<UPCGGraph> ToolGraph;
};
