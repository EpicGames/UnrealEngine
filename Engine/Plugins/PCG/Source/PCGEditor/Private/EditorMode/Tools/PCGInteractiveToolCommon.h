// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers/PCGEdModeSceneQueryHelpers.h"

#include "PCGInteractiveToolCommon.generated.h"

UENUM()
enum class EPCGToolDrawTargetOffset : uint8
{
	// Drawn points will be at the hit location.
	HitLocation,
	// Drawn points will be at the hit location with a distance offset.
	DistanceOffset,
	// Drawn points will be at the hit location with a custom vector offset.
	ExplicitOffset
};

UENUM()
enum class EPCGToolDrawTargetNormal : uint8
{
	// Drawn points will adopt the impact normal.
	HitNormal,
	// Drawn points will adopt the world up normal.
	WorldUp,
	// Align to the previous spline point's up vector.
	AlignToPrevious,
	// Drawn points will adopt an explicit normal.
	Explicit
};

USTRUCT()
struct FPCGToolRaycastSettings
{
	GENERATED_BODY()

	/** How to choose the direction to offset points from the drawn surface. */
	UPROPERTY(EditAnywhere, Category = "Draw Target")
	EPCGToolDrawTargetOffset OffsetMode = EPCGToolDrawTargetOffset::HitLocation;

	/** How far to offset drawn points from the clicked surface, along the normal. */
	UPROPERTY(EditAnywhere, Category = "Draw Target", DisplayName = "Offset", meta = (UIMin = 0, UIMax = 100, EditCondition = "OffsetMode == EPCGToolDrawTargetOffset::DistanceOffset", EditConditionHides))
	double OffsetDistance = 100.0;

	/** Manual vector to offset drawn points from the clicked surface. */
	UPROPERTY(EditAnywhere, Category = "Draw Target", DisplayName = "Offset", meta = (UIMin = 0, UIMax = 100, EditCondition = "OffsetMode == EPCGToolDrawTargetOffset::ExplicitOffset", EditConditionHides))
	FVector OffsetVector = FVector(0, 0, 100);

	/** How to choose the direction to offset points from the drawn surface. */
	UPROPERTY(EditAnywhere, Category = "Draw Target")
	EPCGToolDrawTargetNormal NormalMode = EPCGToolDrawTargetNormal::HitNormal;

	/** Manual vector to offset drawn points from the clicked surface. */
	UPROPERTY(EditAnywhere, Category = "Draw Target", meta = (UIMin = 0, UIMax = 100, EditCondition = "NormalMode == EPCGToolDrawTargetNormal::Explicit", EditConditionHides))
	FVector Normal = FVector::ZAxisVector;
};
