// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/PCGInteractiveToolSettings.h"
#include "PCGComponent.h"
#include "PCGEditorModule.h"
#include "PCGEditorSettings.h"
#include "Data/Tool/PCGToolBaseData.h"
#include "EditorMode/PCGEdModeHelpers.h"
#include "EditorMode/PCGEdModeSettings.h"
#include "EditorMode/Tools/Helpers/PCGEdModeEditorUtilities.h"
#include "Elements/PCGDataFromTool.h"
#include "Subsystems/PCGSubsystem.h"

#include "InteractiveToolManager.h"
#include "LevelEditorSubsystem.h"
#include "BaseBehaviors/KeyInputBehavior.h"
#include "Logging/StructuredLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGInteractiveToolSettings)

#define LOCTEXT_NAMESPACE "PCGInteractiveToolSettings"

namespace UE::PCG::EditorMode::Tool
{
	TAutoConsoleVariable<bool> CVarCommitPropertyChangedEventOnTimer(TEXT("pcg.Editor.Tools.CommitPropertyChangedEventOnTimer"), true,
		TEXT("If true, will queue up and automatically commit property changed events on a timer instead of committing directly."));

	void Shutdown(UInteractiveTool* InTool, EToolShutdownType ShutdownType)
	{
		for (UObject* ToolPropertyObject : InTool->GetToolProperties(false))
		{
			if (UPCGInteractiveToolBaseSettings* PCGToolSettings = Cast<UPCGInteractiveToolBaseSettings>(ToolPropertyObject))
			{
				PCGToolSettings->SaveProperties(InTool);

				if (ShutdownType == EToolShutdownType::Cancel)
				{
					PCGToolSettings->Cancel(InTool);
				}
				else if (ShutdownType == EToolShutdownType::Accept)
				{
					PCGToolSettings->Apply(InTool);
				}

				PCGToolSettings->Shutdown();
			}
		}
	}
}

TValueOrError<bool, FText> UPCGInteractiveToolBaseSettings::Initialize(UInteractiveTool* InTool)
{
	return MakeValue(true);
}

UPCGInteractiveToolSettings::UPCGInteractiveToolSettings()
	: NewPCGComponentName(GetDefault<UPCGEditorModeSettings>()->DefaultNewPCGComponentName)
{
	
}

TValueOrError<bool, FText> UPCGInteractiveToolSettings::Initialize(UInteractiveTool* InTool)
{
	using namespace UE::PCG::EditorMode;
	check(InTool);
	Tool = InTool;

	// In order to constrain component selection and actor creation, we need to make sure to setup the default settings first.
	if (TValueOrError<bool, FText> Result = TryLoadDefaults(); Result.HasError() || (Result.HasValue() && Result.GetValue() == false))
	{
		return Result;
	}

	OnModified.AddUObject(this, &UPCGInteractiveToolSettings::OnPropertyModified);

	// Keyboard setup for switching between data instance identifiers
	UKeyInputBehavior* KeyInputBehavior = NewObject<UKeyInputBehavior>();
	KeyInputBehavior->Initialize(this,
	{
		EKeys::One, EKeys::Two, EKeys::Three, EKeys::Four, EKeys::Five,
		EKeys::Six, EKeys::Seven, EKeys::Eight, EKeys::Nine, EKeys::Zero
	});
	KeyInputBehavior->bRequireAllKeys = false;
	Tool->AddInputBehavior(KeyInputBehavior);

	RegisterPropertyWatchers();

	TValueOrError<bool, FText> InitState = InitActorAndComponent();
	if (InitState.HasError() || InitState.GetValue() == false)
	{
		return InitState;
	}

	if (UPCGComponent* PCGComponent = GetWorkingPCGComponent())
	{
		UPCGGraphInstance* GraphInstance = PCGComponent->GetGraphInstance();
		CacheGraphInstance(GraphInstance); // Guaranteed to be an instance.

		// Finally, update the tool using the component graph. This will also update all tool data.
		// Note: ToolGraph just tracks the graph and the graph parameters will be populated directly from the component's instance
		SetToolGraph(GraphInstance->GetGraph());
	}

	return MakeValue(true);
}

TValueOrError<bool, FText> UPCGInteractiveToolSettings::InitActorAndComponent()
{
	check(Tool.IsValid());
	FToolBuilderState State;
	Tool->GetToolManager()->GetContextQueriesAPI()->GetCurrentSelectionState(State);
	TArray<AActor*> SelectedActors = State.SelectedActors;

	// If we have selected one actor, that's the actor we want to edit
	// This actor might:
	// - Not have a PCG Component
	// - Have a PCG Component but no valid working data
	// - Have a PCG Component with valid working data we want to edit
	check(!EditActor.IsValid());

	const bool HasRightNumberOfSelectedActors = CanHandleMultipleActorSelection() ? (SelectedActors.Num() >= 1) : (SelectedActors.Num() == 1);
	if (HasRightNumberOfSelectedActors && SelectedActors[0] && IsWorkingActorCompatible(SelectedActors[0]))
	{
		//Edit Actor is always the first one.
		EditActor = SelectedActors[0];
	}
	// If we have no selected actor, we spawn a working actor that needs to fulfill the same actor requirements. This actor is fully instantiated when Applied.
	else
	{
		check(GeneratedResources.SpawnedActor == nullptr);

		if (UWorld* World = Tool->GetToolManager()->GetContextQueriesAPI()->GetCurrentEditingWorld())
		{
			using namespace UE::PCG::EditorMode;
			AActor* SpawnedActor = Actor::SpawnWorking(World, FTransform::Identity, GetWorkingActorClass());
			// Our spawned actor is also the actor we are editing
			EditActor = SpawnedActor;

			if (SpawnedActor)
			{
				SpawnedActor->SetActorLabel(NewActorName.ToString(), false);

				FSelectedObjectsChangeList ChangeList;
				ChangeList.ModificationType = ESelectedObjectsModificationType::Replace;
				ChangeList.Actors = { SpawnedActor };

				Tool->GetToolManager()->RequestSelectionChange(ChangeList);

				GeneratedResources.SpawnedActor = SpawnedActor;
			}
		}
	}

	if (!ensure(EditActor.IsValid()))
	{
		return MakeError(LOCTEXT("InvalidEditActorDuringInit", "Invalid selected actor - either as a previously existing actor or a newly spawned actor."));
	}

	TWeakObjectPtr<UPCGComponent> PCGComponent = FindOrGeneratePCGComponent(*EditActor.Get());
	if (!ensure(PCGComponent.IsValid()))
	{
		return MakeError(LOCTEXT("InvalidPCGComponentDuringInit", "Invalid PCG Component - either as a previously existing or transient generated component."));
	}

	// If we don't have a valid graph already, we try to assign the default graph to the newly created pcg component.
	if (PCGComponent->GetGraph() == nullptr)
	{
		if (!ensure(ToolGraph))
		{
			return MakeError(LOCTEXT("ToolHasNoDefaultGraph", "The PCG component has no graph and no default graph for this tool was found in the editor preferences. Shutting down the tool."));
		}

		PCGComponent->SetGraph(ToolGraph);
		// Mark as we should try to generate - note that it will fail until the actor has bounds.
		bTryToGenerate = true;
	}

	return MakeValue(true);
}

bool UPCGInteractiveToolSettings::InitializeWorkingDataForGraph()
{
	TArray<FName> ValidDataInstanceNames = GetDataInstanceNamesForGraph();

	bool bAnyDataCreated = false;
	for (const FName& DataInstanceIdentifier : ValidDataInstanceNames)
	{
		bAnyDataCreated |= InitializeWorkingData(DataInstanceIdentifier);
	}

	return bAnyDataCreated;
}

void UPCGInteractiveToolSettings::Apply(UInteractiveTool* OwningTool)
{
	using namespace UE::PCG::EditorMode;

	OwningTool->GetToolManager()->BeginUndoTransaction(LOCTEXT("ApplyToolTransactionName", "Applied PCG Tool"));
	
	if (GeneratedResources.SpawnedActor.IsValid())
	{
		GeneratedResources.SpawnedActor->ClearFlags(RF_Transient);

		// @todo_pcg: Revisit in 5.8
		// When we create a tool working data, the data might create some generated resources (like a component), and it is
		// likely it is created as transient. Clearing the flag is done in the `OnToolApply` just below.
		// The problem is when we change graphs that don't have the same id for the data, we create "new" working data that re-use the newly spawned root component.
		// But the "old" working data of the first graph is still there, with its generated component still marked as transient. Since its id might be different, it is not
		// gathered by the `GetMutableTypedWorkingDataMap` and thus the transient flag is never cleared.
		// We definitely have a flow issue there.
		// Probably that the tool data should indicate to the tool that it needs a given component to work, and if it doesn't exist, tell the tool
		// to create it, and it is the tool that keeps the reference and is responsible to clear the transient flag.
		// In the mean time for 5.7.4, we will just look all outered objects to the actor and clear their flags. The Root component is assumed to be
		// outered to the actor.
		const UPCGComponent* WorkingPCGComponent = GetWorkingPCGComponent();
		ForEachObjectWithOuter(GeneratedResources.SpawnedActor.Get(),
			[WorkingPCGComponent](UObject* Obj) 
			{
				// To be sure we do not override what PCG is doing, we will ignore any object that was generated by PCG
				const UObject* ConstObj = Obj;
				if (WorkingPCGComponent && WorkingPCGComponent->IsAnyObjectManagedByResource(MakeArrayView(&ConstObj, 1)))
				{
					return;
				}

				Obj->ClearFlags(RF_Transient);
			}, EGetObjectsFlags::IncludeNestedObjects);
	}
	
	if (GeneratedResources.GeneratedPCGComponent.IsValid())
	{
		// On Apply, if we have a PCG Component that was generated by the tool, we remove the transient flag
		GeneratedResources.GeneratedPCGComponent->ClearFlags(RF_Transient);
	}

	/** Now each working data struct gets to Apply its changes. */
	for (const auto& WorkingDataMap : GetMutableTypedWorkingDataMap<FPCGInteractiveToolWorkingData>())
	{
		FPCGInteractiveToolWorkingDataContext Context;
		Context.OwningPCGComponent = GetWorkingPCGComponent();
		Context.DataInstanceIdentifier = WorkingDataMap.Key;
		Context.OwningActor = (Context.OwningPCGComponent.IsValid() ? Context.OwningPCGComponent->GetOwner() : nullptr);
		Context.InteractiveTool = OwningTool;
		Context.PCGSettings = this;
		
		WorkingDataMap.Value->OnToolApply(Context);
	}

	OwningTool->GetToolManager()->EndUndoTransaction();

	// Since we might have been moving things around (component wise) we need to force a selection set change, 
	// Which in turn will trigger a component visualizer update.
	if (GEditor && GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
	{
		if (UTypedElementSelectionSet* SelectionSet = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>()->GetSelectionSet())
		{
			SelectionSet->OnChanged().Broadcast(SelectionSet);
		}
	}
}

void UPCGInteractiveToolSettings::Cancel(UInteractiveTool* OwningTool)
{
	// Note: Okay for graph instance to be null here. Might need to restore a previous instance.
	RestoreGraphInstanceInitialState(GetGraphInstance());

	/** Each working data struct is responsible for wiping or restoring data to the previous state. */
	for(const auto& WorkingDataMap : GetMutableTypedWorkingDataMap<FPCGInteractiveToolWorkingData>())
	{
		FPCGInteractiveToolWorkingDataContext Context;
		Context.OwningPCGComponent = GetWorkingPCGComponent();
		Context.DataInstanceIdentifier = WorkingDataMap.Key;
		Context.OwningActor = (Context.OwningPCGComponent.IsValid() ? Context.OwningPCGComponent->GetOwner() : nullptr);
		Context.InteractiveTool = OwningTool;
		Context.PCGSettings = this;

		// OnToolCancel will delete resources generated by the tool data itself
		WorkingDataMap.Value->OnToolCancel(Context);
	}

	DeleteGeneratedResources();
}

void UPCGInteractiveToolSettings::Shutdown()
{
	using namespace UE::PCG::EditorMode;
	
	OnModified.RemoveAll(this);
	
	// We mark it as garbage so it stops ticking
	MarkAsGarbage();
	
	// On Shutdown, we flush the cache and regenerate to make sure our results are up to date
	if(UPCGComponent* Component = GetWorkingPCGComponent())
	{
		if (UPCGSubsystem* PCGSubsystem = Component->GetSubsystem())
		{
			PCGSubsystem->FlushCache();
		}

		// @todo_pcg: Revisit in 5.8
		// If the working actor was spawned for the tool, but this actor has a PCG component by default (like PCGVolume)
		// the default subobject component will also be flagged transient. In our case, the PCG Component will not be spawned by the tool
		// but reused from the working actor, and thus will never have its transient flag cleared. We need to make sure that we keep enough information
		// when spawning actors to clear all the transient flag.
		// For 5.7.4, just clear the flag here before the generation.
		Component->ClearFlags(RF_Transient);

		Component->Generate(true);
	}
}

void UPCGInteractiveToolSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// We flip the order compared to the Super function so that OnModified can trigger downstream changes before regular calls are made.
	// This makes it so that when global PostEditChange delegates are called, we have already made the modifications.
	OnModified.Broadcast(this, PropertyChangedEvent.Property);

	UInteractiveToolPropertySet::PostEditChangeProperty(PropertyChangedEvent);
}

void UPCGInteractiveToolSettings::RestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier)
{
	Super::RestoreProperties(RestoreToTool, CacheIdentifier);

	// Reset the data instance if it is not valid anymore.
	TArray<FName> DataInstanceNames = GetDataInstanceNamesForGraph();
	if (!DataInstanceNames.Contains(DataInstance) && !DataInstanceNames.IsEmpty())
	{
		SetDataInstance(GetDefaultDataInstanceName());
	}
}

bool UPCGInteractiveToolSettings::IsSelectionAllowed(AActor* InActor, bool bInSelection) const
{
	return InActor == EditActor;
}

bool UPCGInteractiveToolSettings::CanResetToolData(FName DataInstanceIdentifier)
{
	return true;
}

void UPCGInteractiveToolSettings::RegisterPropertyWatchers()
{
	WatchProperty(NewActorName, [this](FName InName)
	{
		if(InName.IsNone() == false && GeneratedResources.SpawnedActor.IsValid() && GeneratedResources.SpawnedActor->GetActorLabelView().Equals(InName.ToString()) == false)
		{
			FActorLabelUtilities::RenameExistingActor(GeneratedResources.SpawnedActor.Get(), InName.ToString());
		}
	});

	WatchProperty(NewPCGComponentName, [this](FName InName)
	{
		if(InName.IsNone() == false && GeneratedResources.GeneratedPCGComponent.IsValid())
		{
			// Only rename if the desired name is unique
			UActorComponent* Component = GeneratedResources.GeneratedPCGComponent.Get();
			UObject* Outer = Component->GetOuter();
			UObject* ExistingObjectWithName = StaticFindObject(UObject::StaticClass(), Outer, *InName.ToString());
			if (ExistingObjectWithName == nullptr)
			{
				const ERenameFlags Flags = REN_DontCreateRedirectors;
				Component->Rename(*InName.ToString(), nullptr, Flags);
			}
			else if (ExistingObjectWithName != GeneratedResources.GeneratedPCGComponent.Get())
			{
				UE_LOGFMT(LogPCGEditor, Warning, "Could not rename PCG Component to name {InName}. Subobject with that name already exists.", InName.ToString());
			}
		}
	});
}

bool UPCGInteractiveToolSettings::InitializeWorkingData(FName DataInstanceIdentifier, bool bAllowRetryAfterInit)
{	
	UPCGComponent* PCGComponent = GetWorkingPCGComponent();
	const UScriptStruct* Struct = GetWorkingDataType();
	ensureMsgf(Struct->IsChildOf<FPCGInteractiveToolWorkingData>(), TEXT("Working Data Type must inherit from FPCGInteractiveToolWorkingData."));

	FPCGInteractiveToolWorkingDataContext Context;
	Context.OwningPCGComponent = PCGComponent;
	Context.DataInstanceIdentifier = DataInstanceIdentifier;
	Context.OwningActor = (Context.OwningPCGComponent.IsValid() ? Context.OwningPCGComponent->GetOwner() : nullptr);
	Context.InteractiveTool = GetTypedOuter<UInteractiveTool>();
	Context.PCGSettings = this;
	Context.WorkingDataIdentifier = GetWorkingDataIdentifier(DataInstanceIdentifier);

	// It is possible this is called multiple times for existing tool data; we return whether it has been created in this call.
	bool bWorkingDataCreated = false;
	
	if (EditActor.IsValid() && PCGComponent)
	{
		// If we don't find the requested working data, we create and initialize it
		if (GetMutableTypedWorkingData(DataInstanceIdentifier) == nullptr)
		{
			TInstancedStruct<FPCGInteractiveToolWorkingData> NewToolWorkingData;
			NewToolWorkingData.InitializeAsScriptStruct(Struct);
			
			NewToolWorkingData.GetMutablePtr<>()->Initialize(Context);
			PostWorkingDataInitialized(NewToolWorkingData.GetMutablePtr<>());
			
			PCGComponent->ToolDataContainer.ToolData.Add(MoveTemp(NewToolWorkingData));
			GeneratedResources.AddedWorkingDataIdentifiers.Add(Context.WorkingDataIdentifier);

			bWorkingDataCreated = true;
		}
	}

	FPCGInteractiveToolWorkingData* WorkingData = GetMutableTypedWorkingData(DataInstanceIdentifier);
	
	if (!ensureMsgf(WorkingData, TEXT("Working Data has to be allocated at this point!.")))
	{
		return false;
	}
	
	// The working data needs to be initialized at this point
	ensureMsgf(WorkingData->IsInitialized(), TEXT("Working Data isn't initialized!. Have you forgotten to call the parent Initialize function?"));
	
	if (WorkingData->IsValid())
	{
		// If the data is valid and hasn't gotten OnToolStart called yet, we do so
		// It's also possible working data got removed (the actor respawns, component changes etc.) during tool usage.
		// In that case, since the working data just got created, we also call OnToolStart
		if (WorkingDataToolStartSet.Contains(Context.WorkingDataIdentifier) == false || bWorkingDataCreated)
		{
			// Notify the working data that we have begun accessing it. This allows to make a 'last state' that the tool can revert to on Cancel.
			WorkingData->OnToolStart(Context);
			WorkingDataToolStartSet.Add(Context.WorkingDataIdentifier);
		}
	}
	else if(bAllowRetryAfterInit)
	{
		// If the working data is invalid despite being initialized, we delete and request it again.
		// This can happen when data dependencies have been deleted outside of PCG, such as deleting a spline component on the actor
		WorkingData->OnToolCancel(Context);
		if (PCGComponent)
		{
			PCGComponent->ToolDataContainer.RemoveToolData(Context.WorkingDataIdentifier);
		}

		// In which case, requesting new working data could recreate new components.
		return InitializeWorkingData(DataInstanceIdentifier, /*bAllowRetryAfterInit=*/false);
	}

	return bWorkingDataCreated;
}

void UPCGInteractiveToolSettings::SetToolGraph(UPCGGraphInterface* InGraph)
{
	ToolGraph = InGraph;

	// This will call OnPropertyModified, forwarding the change
	FPropertyChangedEvent Event(FindFProperty<FProperty>(UPCGInteractiveToolSettings::StaticClass(), GET_MEMBER_NAME_CHECKED(UPCGInteractiveToolSettings, ToolGraph)));
	PostEditChangeProperty(Event);
}

void UPCGInteractiveToolSettings::SetDataInstance(FName DataInstanceIdentifier)
{
	DataInstance = DataInstanceIdentifier;
	
	// This will call OnPropertyModified, forwarding the change
	FPropertyChangedEvent Event(FindFProperty<FProperty>(UPCGInteractiveToolSettings::StaticClass(), GET_MEMBER_NAME_CHECKED(UPCGInteractiveToolSettings, DataInstance)));
	PostEditChangeProperty(Event);
}

void UPCGInteractiveToolSettings::SetActorClassToSpawn(TSubclassOf<AActor> Class)
{
	// If this is called, we assume the class is compatible already
	ActorClassToSpawn = Class;

	// This will call OnPropertyModified, forwarding the change
	FPropertyChangedEvent Event(FindFProperty<FProperty>(UPCGInteractiveToolSettings::StaticClass(), GET_MEMBER_NAME_CHECKED(UPCGInteractiveToolSettings, ActorClassToSpawn)));
	PostEditChangeProperty(Event);
}

void UPCGInteractiveToolSettings::SetActorLabel(FName InLabel)
{
	NewActorName = InLabel == NAME_None ? GetDefault<UPCGEditorModeSettings>()->DefaultNewActorName : InLabel;
	// This will call OnPropertyModified, forwarding the change
	FPropertyChangedEvent Event(FindFProperty<FProperty>(UPCGInteractiveToolSettings::StaticClass(), GET_MEMBER_NAME_CHECKED(UPCGInteractiveToolSettings, NewActorName)));
	PostEditChangeProperty(Event);
}

UPCGComponent* UPCGInteractiveToolSettings::FindOrGeneratePCGComponent(AActor& OwningActor) const
{
	using namespace UE::PCG::EditorMode;
	
	UPCGComponent* PCGComponent = PCGComponent::Find(&OwningActor, GetToolTag());

	if(PCGComponent == nullptr)
	{
		PCGComponent = PCGComponent::Create(&OwningActor, NewPCGComponentName);
		PCGComponent->SetFlags(RF_Transient);
		GeneratedResources.GeneratedPCGComponent = PCGComponent;
	}

	return PCGComponent;
}

void UPCGInteractiveToolSettings::DeleteGeneratedResources() const
{
	if(GeneratedResources.SpawnedActor.IsValid())
	{
		// Unselect the actor if it was previously selected
		FSelectedObjectsChangeList ChangeList;
		ChangeList.ModificationType = ESelectedObjectsModificationType::Remove;
		ChangeList.Actors.Add(GeneratedResources.SpawnedActor.Get());
		Tool->GetToolManager()->RequestSelectionChange(ChangeList);

		GeneratedResources.SpawnedActor->Destroy();
		GeneratedResources.SpawnedActor.Reset();
	}
	
	if(GeneratedResources.GeneratedPCGComponent.IsValid())
	{
		UPCGSubsystem* Subsystem = GeneratedResources.GeneratedPCGComponent->GetSubsystem();

		if (Subsystem)
		{
			// We schedule a cleanup and deletion
			FPCGTaskId CleanupTaskId = GeneratedResources.GeneratedPCGComponent->GetSubsystem()->ScheduleCleanup(GeneratedResources.GeneratedPCGComponent.Get(), true, {});
			TArray<FPCGTaskId> DestroyTaskDependencies;
			if (CleanupTaskId != InvalidPCGTaskId)
			{
				DestroyTaskDependencies.Add(CleanupTaskId);
			}

			GeneratedResources.GeneratedPCGComponent->GetSubsystem()->ScheduleGeneric([PCGComponent = GeneratedResources.GeneratedPCGComponent]() -> bool
			{
				if (PCGComponent.IsValid())
				{
					PCGComponent->DestroyComponent();
				}
			
				return true;
			},
			GeneratedResources.GeneratedPCGComponent.Get(),
			DestroyTaskDependencies);
		}
		
		GeneratedResources.GeneratedPCGComponent.Reset();
	}

	// If we still have a PCG Component, this means it was not a generated one. We have to remove the working data.
	WorkingComponentCache.Reset();
	if(UPCGComponent* PCGComponent = GetWorkingPCGComponent())
	{
		for(const FName& AddedWorkingDataIdentifier : GeneratedResources.AddedWorkingDataIdentifiers)
		{
			PCGComponent->ToolDataContainer.RemoveToolData(AddedWorkingDataIdentifier);
		}
	}
}

TValueOrError<bool, FText> UPCGInteractiveToolSettings::TryLoadDefaults()
{
	check(!HasAnyFlags(RF_ClassDefaultObject));
	if (UPCGGraphInterface* DefaultGraphInterface = CastChecked<UPCGGraphInterface>(GetDefaultGraph().TryLoad(), ECastCheckedType::NullAllowed))
	{
		TOptional<FPCGGraphToolData> GraphToolData = DefaultGraphInterface->GetGraphToolData();
		if (GraphToolData.IsSet() == false || GraphToolData.GetValue().CompatibleToolTags.Contains(GetToolTag()) == false)
		{
			return MakeError(FText::FormatOrdered(LOCTEXT("DefaultGraph_IncompatibleToolTag", "Graph {0} does not have compatible tool tag {1}"), FText::FromString(GetPathNameSafe(DefaultGraphInterface)), FText::FromName(GetToolTag())));
		}

		TSubclassOf<AActor> DefaultActorClassToSpawn = GraphToolData.GetValue().InitialActorClassToSpawn;

		if (!IsActorClassCompatible(DefaultActorClassToSpawn))
		{
			if (TOptional<TSubclassOf<AActor>> FallbackClass = GetFallbackActorClassToSpawn())
			{
				DefaultActorClassToSpawn = FallbackClass.GetValue();
			}
		}

		if (!IsActorClassCompatible(DefaultActorClassToSpawn))
		{
			return MakeError(LOCTEXT("DefaultActorClassToSpawn_Invalid", "No valid spawn class found."));
		}
		
		// At this point, the delegates (OnPropertyModified) aren't setup, so we need to set things up manually.
		SetToolGraph(DefaultGraphInterface);
		SetActorClassToSpawn(DefaultActorClassToSpawn);
		SetDataInstance(GetDefaultDataInstanceName());
		SetActorLabel(GraphToolData.GetValue().NewActorLabel);

		return MakeValue(true);
	}

	FText Message = LOCTEXT("DefaultGraph_NotFound", "No default graph found for tool settings {0}. Please specify a graph with tool tag {1} under Editor Preferences -> PCG Editor Mode Settings");
	return MakeError(FText::FormatOrdered(Message, GetClass()->GetDisplayNameText(), FText::FromName(GetToolTag())));
}

void UPCGInteractiveToolSettings::Tick(float DeltaTime)
{
	GraphRefreshTimer -= DeltaTime;
	if (GraphRefreshTimer < 0.f)
	{
		OnRefresh();
	}
}

bool UPCGInteractiveToolSettings::IsTickable() const
{
	return HasAnyFlags(RF_ClassDefaultObject | RF_MirroredGarbage) == false;
}

TStatId UPCGInteractiveToolSettings::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FPCGInteractiveToolSettings, STATGROUP_Tickables);
}

void UPCGInteractiveToolSettings::OnRefresh()
{
	if(HasAnyFlags(RF_MirroredGarbage) == false)
	{
		// We commit the property changed events now to let the graph regenerate, if the object is tracked
		CommitPropertyChangedEvents();

		if (bTryToGenerate)
		{
			if (UPCGComponent* PCGComponent = GetWorkingPCGComponent())
			{
				PCGComponent->GenerateLocal(/*bForce=*/true);
				bTryToGenerate = !PCGComponent->IsGenerating();
			}
		}
		
		// Reset the refresh timer
		GraphRefreshTimer = GetDefault<UPCGEditorModeSettings>()->GraphRefreshRate;

		OnRefreshInternal();
	}
}

bool UPCGInteractiveToolSettings::HasSpawnedActor() const
{
	return GeneratedResources.SpawnedActor.IsValid();
}

bool UPCGInteractiveToolSettings::HasGeneratedPCGComponent() const
{
	return GeneratedResources.GeneratedPCGComponent.IsValid();
}

void UPCGInteractiveToolSettings::CacheGraphInstance(UPCGGraphInstance* InGraphInstance)
{
	if (InGraphInstance)
	{
		GraphInstanceCache = DuplicateObject(InGraphInstance, this);
	}
}

void UPCGInteractiveToolSettings::RestoreGraphInstanceInitialState(UPCGGraphInstance* OutGraphInstance)
{
	// If the instance itself exists and was not changed, ONLY clobber the graph parameters...
	if (OutGraphInstance && GraphInstanceCache && OutGraphInstance->GetGraph() == GraphInstanceCache->GetGraph())
	{
		// Double-check the layout is correct, before overwriting the params.
		if (ensure(OutGraphInstance->ParametersOverrides.Parameters.HasSameLayout(GraphInstanceCache->ParametersOverrides.Parameters)))
		{
			OutGraphInstance->ParametersOverrides = GraphInstanceCache->ParametersOverrides;
			OutGraphInstance->OnGraphParametersChanged(EPCGGraphParameterEvent::UndoRedo, NAME_None);
		}
	}
	else // Otherwise, initial instance state was null, or the final state became null, or the graph was changed. In those cases, revert it all.
	{
		if (UPCGComponent* PCGComponent = GetWorkingPCGComponent())
		{
			if (GraphInstanceCache)
			{
				PCGComponent->SetGraph(GraphInstanceCache->GetGraph());
				if (UPCGGraphInstance* Instance = PCGComponent->GetGraphInstance())
				{
					Instance->ParametersOverrides = GraphInstanceCache->ParametersOverrides;
					Instance->OnGraphParametersChanged(EPCGGraphParameterEvent::UndoRedo, NAME_None);
				}

				SetToolGraph(PCGComponent->GetGraph());
			}
			else // If the original graph was null, reset back to null.
			{
				PCGComponent->SetGraph(nullptr);
				SetToolGraph(nullptr);
			}
		}
	}
}

TSubclassOf<AActor> UPCGInteractiveToolSettings::GetCompatibleActorClassFromGraphOrFallback() const
{
	TSubclassOf<AActor> Result = nullptr;
	
	if (ToolGraph)
	{
		TOptional<FPCGGraphToolData> GraphToolData = ToolGraph->GetGraphToolData();

		if (GraphToolData.IsSet())
		{
			TSubclassOf<AActor> GraphToolDataActorClass = GraphToolData->InitialActorClassToSpawn;

			// The graph might have specified an incompatible class too, so we make sure to check
			if (GraphToolDataActorClass && IsActorClassCompatible(GraphToolDataActorClass))
			{
				Result = GraphToolDataActorClass;
			}
		}
	}

	// If the graph didn't have a valid class, attempt to use a tool-specified fallback
	if (Result == nullptr)
	{
		TOptional<TSubclassOf<AActor>> FallbackClass = GetFallbackActorClassToSpawn();
		if (FallbackClass.IsSet() && FallbackClass.GetValue() != nullptr && IsActorClassCompatible(FallbackClass.GetValue()))
		{
			Result = FallbackClass.GetValue();
		}
	}

	return Result;
}

void UPCGInteractiveToolSettings::CommitPropertyChangedEvents()
{
	for(auto& Pair : PropertyChangedEventQueue)
	{
		if(Pair.Key.IsValid())
		{
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Pair.Key.Get(), Pair.Value);
		}
	}

	PropertyChangedEventQueue.Empty();
}

void UPCGInteractiveToolSettings::OnKeyPressed(const FKey& InKeyID)
{
	TArray<FName> ValidDataInstanceNames = GetDataInstanceNamesForGraph();

	int32 Index = INDEX_NONE;
	
	if(InKeyID == EKeys::One)
	{
		Index = 1;
	}
	else if(InKeyID == EKeys::Two)
	{
		Index = 2;
	}
	else if(InKeyID == EKeys::Three)
	{
		Index = 3;
	}
	else if(InKeyID == EKeys::Four)
	{
		Index = 4;
	}
	else if(InKeyID == EKeys::Five)
	{
		Index = 5;
	}
	else if(InKeyID == EKeys::Six)
	{
		Index = 6;
	}
	else if(InKeyID == EKeys::Seven)
	{
		Index = 7;
	}
	else if(InKeyID == EKeys::Eight)
	{
		Index = 8;
	}
	else if(InKeyID == EKeys::Nine)
	{
		Index = 9;
	}
	else if(InKeyID == EKeys::Zero)
	{
		Index = 10;
	}

	Index--;
	
	if (ValidDataInstanceNames.IsValidIndex(Index))
	{
		SetDataInstance(ValidDataInstanceNames[Index]);
	}
}

void UPCGInteractiveToolSettings::PostWorkingDataInitialized(FPCGInteractiveToolWorkingData* WorkingData) const
{
}

void UPCGInteractiveToolSettings::OnPropertyModified(UObject* Object, FProperty* Property)
{
	if (Object != this || Property == nullptr)
	{
		return;
	}

	if(Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGInteractiveToolSettings, ToolGraph))
	{
		if (UPCGComponent* WorkingComponent = GetWorkingPCGComponent())
		{
			if (ToolGraph != WorkingComponent->GetGraph())
			{
				WorkingComponent->SetGraph(ToolGraph);
			}
		}

		if(ToolGraph)
		{
			TOptional<FPCGGraphToolData> GraphToolData = ToolGraph->GetGraphToolData();

			if(GraphToolData.IsSet())
			{
				// If the graph changes, we also update the actor class to spawn
				SetActorClassToSpawn(GetCompatibleActorClassFromGraphOrFallback());
				SetActorLabel(GraphToolData.GetValue().NewActorLabel);
			}
		}

		// We set the data instance to the first one provided by the graph. Uses the default as fallback.
		FName DefaultDataInstanceName = GetDefaultDataInstanceName();
		SetDataInstance(DefaultDataInstanceName);
		
		// Update the working data to the available data instances ones for the current tool from the graph
		if(InitializeWorkingDataForGraph())
		{
			// We generate again to make sure the initialized working data has a chance to register its tracking keys.
			if (UPCGComponent* PCGComponent = GetWorkingPCGComponent())
			{
				PCGComponent->GenerateLocal(/*bForce=*/true);
			}
		}
	}
	else if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGInteractiveToolSettings, DataInstance))
	{
		if (InitializeWorkingData(DataInstance))
		{
			// We generate again to make sure the initialized working data has a chance to register its tracking keys.
			if (UPCGComponent* PCGComponent = GetWorkingPCGComponent())
			{
				PCGComponent->GenerateLocal(/*bForce=*/true);
			}
		}
	}
	else if(Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGInteractiveToolSettings, ActorClassToSpawn))
	{
		// Delete previously spawned actor and set up the new one. Ideally recycling the existing PCG Component.
		if (HasSpawnedActor() && EditActor->GetClass() != ActorClassToSpawn.Get())
		{
			// The newly selected actor class might not be compatible, so we give it a chance to fix itself up
			if (!IsActorClassCompatible(ActorClassToSpawn))
			{
				FNotificationInfo Info(LOCTEXT("InvalidActorClassSelected_Notification", "Class is invalid for tool. Resetting to base class."));
				FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);

				ActorClassToSpawn = GetCompatibleActorClassFromGraphOrFallback();
			}
			
			// If the class is still incompatible after attempted fixup, we cancel the tool
			if (!IsActorClassCompatible(ActorClassToSpawn))
			{
				FNotificationInfo Info(LOCTEXT("InvalidActorClassSelected_ToolShutdown_Notification", "No base class found. Shutting down tool."));
				Info.ExpireDuration = 3.f;
				FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
				Tool->GetToolManager()->DeactivateTool(EToolSide::Left, EToolShutdownType::Cancel);
				return;
			}
			
			AActor* PreviouslySpawnedActor = EditActor.Get();
			UPCGComponent* PreviousComponent = GetWorkingPCGComponent();
			EditActor = nullptr;
			GeneratedResources.SpawnedActor = nullptr;
			GeneratedResources.GeneratedPCGComponent = nullptr;
			WorkingComponentCache.Reset();
			WorkingDataToolStartSet.Reset();

			// Unselect the previously edited actor
			FSelectedObjectsChangeList ChangeList;
			ChangeList.ModificationType = ESelectedObjectsModificationType::Clear;
			Tool->GetToolManager()->RequestSelectionChange(ChangeList);

			InitActorAndComponent();
			if (UPCGComponent* NewComponent = GetWorkingPCGComponent(); NewComponent && PreviousComponent && NewComponent->GetGraph() == PreviousComponent->GetGraph())
			{
				// Copy parameters from previous component to new component
				NewComponent->GetGraphInstance()->ParametersOverrides = PreviousComponent->GetGraphInstance()->ParametersOverrides;
				NewComponent->GetGraphInstance()->OnGraphParametersChanged(EPCGGraphParameterEvent::UndoRedo, NAME_None);
			}

			// Update working data otherwise we'll have dangling pointers to components in the data, and the tool might close.
			InitializeWorkingDataForGraph();

			// @todo_pcg - copy over working data from previous to new on a per-name/matching class basis

			// Finally, get rid of the previous generated actor
			PreviouslySpawnedActor->Destroy();
		}
	}
}

void UPCGInteractiveToolSettings::OnRefreshInternal()
{
}

FSoftObjectPath UPCGInteractiveToolSettings::GetDefaultGraph() const
{
	const FPCGPerInteractiveToolSettingSettings* Settings = GetDefault<UPCGEditorModeSettings>()->InteractiveToolSettings.FindByPredicate([this](const FPCGPerInteractiveToolSettingSettings& SettingsCandidate)
	{
		return SettingsCandidate.SettingsClass == this->GetClass();
	});

	if(Settings != nullptr)
	{
		if(Settings->DefaultGraph.ToSoftObjectPath().IsAsset())
		{
			return Settings->DefaultGraph.ToSoftObjectPath();
		}
	}
	
	return FSoftObjectPath();
}

TSubclassOf<AActor> UPCGInteractiveToolSettings::GetWorkingActorClass() const
{
	return ActorClassToSpawn;
}

bool UPCGInteractiveToolSettings::GraphAssetFilter(const FAssetData& AssetData) const
{
	using namespace UE::PCG::EditorMode;
	
	// Asset is of the proper type
	if(AssetData.GetClass()->IsChildOf<UPCGGraphInterface>() == false)
	{
		return true;
	}

	// Asset has the compatible tool tag.
	if (!Utility::DoesPCGGraphInterfaceHaveToolTag(GetToolTag(), AssetData))
	{
		return true;
	}
	
	// There is no current selected/created actor, or it is a transient actor.
	if (!EditActor.IsValid() || GeneratedResources.SpawnedActor == EditActor.Get())
	{
		return false;
	}
	
	// Otherwise, the edit actor class must be compatible with the actor class prescribed by the asset.
	return !Utility::IsPCGGraphInterfaceCompatibleWithActor(EditActor.Get(), AssetData);
}

bool UPCGInteractiveToolSettings::UsesExistingActor() const
{
	return !GeneratedResources.SpawnedActor.IsValid() && EditActor.IsValid();
}

UPCGComponent* UPCGInteractiveToolSettings::GetWorkingPCGComponent() const
{
	if(WorkingComponentCache.IsValid() == false)
	{
		WorkingComponentCache = UE::PCG::EditorMode::PCGComponent::Find(EditActor.Get(), GetToolTag());
	}
	
	return WorkingComponentCache.Get();
}

UPCGGraphInstance* UPCGInteractiveToolSettings::GetGraphInstance() const
{
	if(UPCGComponent* PCGComponent = GetWorkingPCGComponent())
	{
		return PCGComponent->GetGraphInstance();
	}
	
	return nullptr;
}

TArray<FName> UPCGInteractiveToolSettings::GetDataInstanceNamesForGraph() const
{
	TArray<FName> Result = GetValidDataInstanceNamesForGraph();

	if (Result.IsEmpty())
	{
		Result.Add(GetDefaultDataInstanceName());
	}

	return Result;
}

TArray<FName> UPCGInteractiveToolSettings::GetValidDataInstanceNamesForGraph() const
{
	TArray<FName> Result;

	if (ToolGraph == nullptr)
	{
		return Result;
	}

	TArray<UPCGNode*> Nodes = ToolGraph->GetGraph()->FindNodesWithSettings(UPCGDataFromTool::StaticClass(), /*bRecursive=*/true);

	FName ToolTag = GetToolTag();
	for (UPCGNode* Node : Nodes)
	{
		if (UPCGDataFromTool* DataFromTool = Cast<UPCGDataFromTool>(Node->GetSettings()))
		{
			if (DataFromTool->ToolTag != ToolTag)
			{
				continue;
			}

			Result.AddUnique(DataFromTool->DataInstance);
		}
	}

	return Result;
}

FName UPCGInteractiveToolSettings::GetDefaultDataInstanceName() const
{
	TArray<FName> ValidDataInstanceNames = GetValidDataInstanceNamesForGraph();
	
	// By default, we use the regular tool tag as the working data identifier, if available
	// This means a NONE instance name.
	if (ValidDataInstanceNames.Contains(NAME_None))
	{
		return NAME_None;
	}

	// If we have multiple available and they don't include, we choose the first available one
	if (ValidDataInstanceNames.Num() > 0)
	{
		return ValidDataInstanceNames[0];
	}
	
	return NAME_None;
}

FName UPCGInteractiveToolSettings::GetWorkingDataIdentifier(FName DataInstanceIdentifier) const
{
	FNameBuilder DataIdentifierBuilder(GetToolTag());

	if (DataInstanceIdentifier.IsNone() == false)
	{
		DataIdentifierBuilder.AppendChar('.');
		DataIdentifierBuilder.Append(DataInstanceIdentifier.ToString());
	}

	return DataIdentifierBuilder.ToString();
}

#undef LOCTEXT_NAMESPACE
