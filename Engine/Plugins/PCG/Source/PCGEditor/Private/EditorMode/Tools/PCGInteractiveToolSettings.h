// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGraph.h"
#include "PCGInteractiveToolCommon.h"
#include "Data/Tool/PCGToolBaseData.h"
#include "Helpers/PCGEdModeActorHelpers.h"

#include "InteractiveTool.h"
#include "NativeGameplayTags.h"
#include "PCGComponent.h"
#include "TickableEditorObject.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "PCGInteractiveToolSettings.generated.h"

namespace UE::PCG::EditorMode::Tool
{
	void Shutdown(UInteractiveTool* InTool, EToolShutdownType ShutdownType);
	
	extern TAutoConsoleVariable<bool> CVarCommitPropertyChangedEventOnTimer;
}

/**
* Base class for interactive tool settings in PCG.
* It is useful to derive from this class instead of UPCGInteractiveToolSettings when the tool does not require to create an actor,
* use a graph or a PCG component.
*/
UCLASS(Abstract)
class UPCGInteractiveToolBaseSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	virtual TValueOrError<bool, FText> Initialize(UInteractiveTool* InTool);
	virtual void Apply(UInteractiveTool* OwningTool) {}
	virtual void Cancel(UInteractiveTool* OwningTool) {}
	virtual void Shutdown() {}
	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const { return true; }

	/** Tool tag for use with filtering compatible graphs. */
	virtual FName GetToolTag() const PURE_VIRTUAL(UPCGInteractiveToolBaseSettings::GetToolTag, return NAME_None;);
};

/**
* Base class for tools that need to manage creating an actor, a pcg component and graph tool settings in PCG.
*/
UCLASS(Abstract)
class UPCGInteractiveToolSettings : public UPCGInteractiveToolBaseSettings, public FTickableEditorObject, public IKeyInputBehaviorTarget
{
	GENERATED_BODY()

public:
	UPCGInteractiveToolSettings();
	virtual ~UPCGInteractiveToolSettings() override = default;

	virtual TValueOrError<bool, FText> Initialize(UInteractiveTool* InTool) override;
	virtual void Apply(UInteractiveTool* OwningTool) override;
	virtual void Cancel(UInteractiveTool* OwningTool) override;
	virtual void Shutdown() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	//~Begin interface UInteractiveToolPropertySet
	virtual void RestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier = TEXT("")) override;
	//~End interface UInteractiveToolPropertySet

	/** By default, when a tool is active, we can only select the Edit Actor in the Outliner. */
	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const override;

	/** Whether tool data can be reset. Controls visibility of the Reset button in the UI. */
	virtual bool CanResetToolData(FName DataInstanceIdentifier);
	
	/** Register property watchers and callbacks in here. */
	virtual void RegisterPropertyWatchers();

	virtual bool CanHandleMultipleActorSelection() const { return false; }

	/** Initializes or finds the working data associated with the data instance identifier.
	 * If NAME_None, the ToolTag is used as identifier.
	 * If bAllowRetryAfterInit is true, if the data was properly initialized, it will retry (once) after. Otherwise, it will not.
	 * @returns Whether new working data was created and initialized.
	 * False does not imply working data is invalid; just that it didn't get created. */
	bool InitializeWorkingData(FName DataInstanceIdentifier = NAME_None, bool bAllowRetryAfterInit = true);
	
	UPCGGraphInterface* GetToolGraph() const { return ToolGraph; }
	void SetToolGraph(UPCGGraphInterface* InGraph);

	void SetDataInstance(FName DataInstanceIdentifier);

	/** A soft object path to the graph asset to load by default for this tool. */
	virtual FSoftObjectPath GetDefaultGraph() const;
	
	/** The working data type for this settings class. */
	virtual const UScriptStruct* GetWorkingDataType() const PURE_VIRTUAL(UPCGInteractiveToolSettings::GetWorkingDataType, return FPCGInteractiveToolWorkingData::StaticStruct(););
	
	/** The actor to instantiate for edit-purposes when we aren't editing an existing actor. 
	 * If your target actor has any special requirements, create an actor for that purpose and return it in your settings. */
	virtual TSubclassOf<AActor> GetWorkingActorClass() const;

	/** Whether we are operating on a spawned or a previously-existing actor. */
	bool UsesExistingActor() const;
	
	void SetActorClassToSpawn(TSubclassOf<AActor> Class);

	void SetActorLabel(FName InLabel);

	bool IsWorkingActorCompatible(const AActor* InActor) const { return InActor != nullptr && IsWorkingActorCompatibleInternal(InActor); }
	bool IsActorClassCompatible(TSubclassOf<AActor> InClass) const { return InClass != nullptr && !InClass->HasAnyClassFlags(CLASS_Abstract) && IsActorClassCompatibleInternal(InClass); }
	
	/** Filter details view graph selection. */
	UFUNCTION()
	virtual bool GraphAssetFilter(const FAssetData& AssetData) const;

	/** Returns true if the actor being edited has been spawned by this tool. */
	UFUNCTION()
	bool HasSpawnedActor() const;

	/** Returns true if the pcg component being edited has been spawned by this tool. */
	UFUNCTION()
	bool HasGeneratedPCGComponent() const;
	
	UPCGComponent* GetWorkingPCGComponent() const;
	UPCGGraphInstance* GetGraphInstance() const;

	UFUNCTION()
	TArray<FName> GetDataInstanceNamesForGraph() const;

	UFUNCTION()
	TArray<FName> GetValidDataInstanceNamesForGraph() const;
	FName GetDefaultDataInstanceName() const;
	
	template<typename T = AActor>
	T* GetTypedWorkingActor() const
	{
		static_assert(TIsDerivedFrom<T, AActor>::IsDerived, "T needs to be derived from AActor.");
		
		return Cast<T>(EditActor.Get());
	}

	FName GetWorkingDataIdentifier(FName DataInstanceIdentifier = NAME_None) const;
	
	template<typename T = FPCGInteractiveToolWorkingData>
	T* GetMutableTypedWorkingData(FName DataInstanceIdentifier = NAME_None)
	{
		static_assert(TIsDerivedFrom<T, FPCGInteractiveToolWorkingData>::IsDerived, "T needs to be derived from FPCGInteractiveToolWorkingData.");

		FName WorkingDataIdentifier = GetWorkingDataIdentifier(DataInstanceIdentifier);
		
		if (UPCGComponent* PCGComponent = GetWorkingPCGComponent())
		{
			return PCGComponent->ToolDataContainer.GetMutableTypedWorkingData<T>(WorkingDataIdentifier);
		}

		return nullptr;
	}
	
	template<typename T = FPCGInteractiveToolWorkingData>
	const T* GetTypedWorkingData(FName DataInstanceIdentifier = NAME_None) const
	{
		static_assert(TIsDerivedFrom<T, FPCGInteractiveToolWorkingData>::IsDerived, "T needs to be derived from FPCGInteractiveToolWorkingData.");

		FName WorkingDataIdentifier = GetWorkingDataIdentifier(DataInstanceIdentifier);

		if (UPCGComponent* PCGComponent = GetWorkingPCGComponent())
		{
			return PCGComponent->ToolDataContainer.GetTypedWorkingData<T>(WorkingDataIdentifier);
		}
		
		return nullptr;
	}

	template<typename T = FPCGInteractiveToolWorkingData>
	TMap<FName, T*> GetMutableTypedWorkingDataMap() const
	{
		static_assert(TIsDerivedFrom<T, FPCGInteractiveToolWorkingData>::IsDerived, "T needs to be derived from FPCGInteractiveToolWorkingData.");

		TMap<FName, T*> Result;

		if (UPCGComponent* PCGComponent = GetWorkingPCGComponent())
		{
			for(const FName& DataInstanceName : GetDataInstanceNamesForGraph())
			{
				FName WorkingDataIdentifier = GetWorkingDataIdentifier(DataInstanceName);
			
				if (T* ToolData = PCGComponent->ToolDataContainer.GetMutableTypedWorkingData<T>(WorkingDataIdentifier))
				{
					Result.Add(WorkingDataIdentifier, ToolData);
				}
			}
		}

		return Result;
	}

	template<typename T = FPCGInteractiveToolWorkingData>
	TMap<FName, const T*> GetTypedWorkingDataMap() const
	{
		static_assert(TIsDerivedFrom<T, FPCGInteractiveToolWorkingData>::IsDerived, "T needs to be derived from FPCGInteractiveToolWorkingData.");

		TMap<FName, const T*> Result;

		if (UPCGComponent* PCGComponent = GetWorkingPCGComponent())
		{
			for(const FName& DataInstanceName : GetDataInstanceNamesForGraph())
			{
				FName WorkingDataIdentifier = GetWorkingDataIdentifier(DataInstanceName);
			
				if (const T* ToolData = PCGComponent->ToolDataContainer.GetTypedWorkingData<T>(WorkingDataIdentifier))
				{
					Result.Add(WorkingDataIdentifier, ToolData);
				}
			}
		}

		return Result;
	}
protected:
	
	/** Tools can have a requirement on the working actor */
	virtual bool IsWorkingActorCompatibleInternal(const AActor* InActor) const { return true; }
	virtual bool IsActorClassCompatibleInternal(TSubclassOf<AActor> InClass) const { return true; }
	virtual TOptional<TSubclassOf<AActor>> GetFallbackActorClassToSpawn() const { return {}; }
	
	/** Try to load the defaults (graph, actor class, ...) into the tool settings (ToolGraph, etc.). */
	virtual TValueOrError<bool, FText> TryLoadDefaults();
	virtual TValueOrError<bool, FText> InitActorAndComponent();
	virtual bool InitializeWorkingDataForGraph();
	
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;

	void CommitPropertyChangedEvents();

	/** IKeyInputBehaviorTarget */
	virtual void OnKeyPressed(const FKey& KeyID) override;

	/** Allows to modify the just initialized working data from the editor. This can be used if the initialization of working data requires editor data access. */
	virtual void PostWorkingDataInitialized(FPCGInteractiveToolWorkingData* WorkingData) const;
	virtual void OnPropertyModified(UObject* Object, FProperty* Property);
	virtual void OnRefreshInternal();

private:
	UPCGComponent* FindOrGeneratePCGComponent(AActor& OwningActor) const;
	
	void DeleteGeneratedResources() const;
	
	void OnRefresh();

	void CacheGraphInstance(UPCGGraphInstance* InGraphInstance);
	void RestoreGraphInstanceInitialState(UPCGGraphInstance* OutGraphInstance);
	TSubclassOf<AActor> GetCompatibleActorClassFromGraphOrFallback() const;

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "PCG", meta = (TransientToolProperty, NoResetToDefault, GetAssetFilter = "GraphAssetFilter"))
	TObjectPtr<UPCGGraphInterface> ToolGraph = nullptr; // Marked transient because the tool system will either fill this with a default or take the graph from the selected actor.

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "PCG", meta = (NoResetToDefault, EditCondition="ToolGraph != nullptr", EditConditionHides, GetOptions="GetValidDataInstanceNamesForGraph"))
	FName DataInstance = NAME_None;
	
	// @todo_pcg: Convert to a target object, rather than actor, to ease transition into SceneGraph
	/** The actor to modify with the tool. Either a selected actor, or a newly spawned one. */
	UPROPERTY(meta = (TransientToolProperty))
	TWeakObjectPtr<AActor> EditActor = nullptr;

	/** The name to be issued to a newly created actor when the tool is applied. */
	UPROPERTY(EditAnywhere, DisplayName="Actor Label", Category = "PCG", meta = (TransientToolProperty, EditCondition="HasSpawnedActor()", EditConditionHides))
	FName NewActorName;

	/** The name to be issued to a newly created PCG component when the tool is applied. */
	UPROPERTY(EditAnywhere, DisplayName="Component Name", Category = "PCG", meta = (TransientToolProperty, EditCondition="HasGeneratedPCGComponent()", EditConditionHides))
	FName NewPCGComponentName;

	/** The actor class to use when the tool works on a spawned actor opposed to a pre-existing one. */
	UPROPERTY(EditAnywhere, Category = "PCG", meta=(TransientToolProperty, EditCondition="HasSpawnedActor()", EditConditionHides, NoResetToDefault))
	TSubclassOf<AActor> ActorClassToSpawn;

	/** The current time left before a graph refresh (in seconds). */
	UPROPERTY(meta = (TransientToolProperty))
	float GraphRefreshTimer = 0.f;
	
	UPROPERTY(meta = (TransientToolProperty))
	mutable TWeakObjectPtr<UPCGComponent> WorkingComponentCache;
	
	TMap<TWeakObjectPtr<UObject>, FPropertyChangedEvent> PropertyChangedEventQueue;

	struct FGeneratedResources
	{
		TWeakObjectPtr<AActor> SpawnedActor;
		TWeakObjectPtr<UPCGComponent> GeneratedPCGComponent;
		/** A list of working data identifiers added this tool instantiation. Used to delete data on Cancel. */
		TArray<FName> AddedWorkingDataIdentifiers;
	};
	
	mutable FGeneratedResources GeneratedResources;

	bool bTryToGenerate = false;

private:
	UPROPERTY(Transient, meta = (TransientToolProperty))
	TObjectPtr<UPCGGraphInstance> GraphInstanceCache;

	UPROPERTY(Transient, meta = (TransientToolProperty))
	TSet<FName> WorkingDataToolStartSet;

	TWeakObjectPtr<UInteractiveTool> Tool = nullptr;
};

UCLASS(Abstract)
class UPCGInteractiveTool : public UInteractiveTool
{
	GENERATED_BODY()

public:
	virtual bool RequiresSettings() const { return true; }
	virtual bool UsesTransformGizmo() const { return false; }
};