// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/BaseBrushTool.h"

#include "PCGEdModeBrushBase.generated.h"

UCLASS()
class UPCGEdModeBrushBase : public UBrushBaseProperties
{
	GENERATED_BODY()

public:
	UPCGEdModeBrushBase() = default;
};
