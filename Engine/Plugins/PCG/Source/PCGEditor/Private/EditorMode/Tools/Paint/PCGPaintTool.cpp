// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/Paint/PCGPaintTool.h"
#include "Data/PCGPointArrayData.h"
#include "Data/Tool/PCGToolPointData.h"
#include "EditorMode/PCGEdModeStyle.h"
#include "EditorMode/Tools/Gizmos/PCGGizmos.h"
#include "Helpers/PCGHelpers.h"

#include "InteractiveGizmoManager.h"
#include "InteractiveToolManager.h"
#include "LandscapeComponent.h"
#include "ToolTargetManager.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"
#include "BaseBehaviors/TwoAxisPropertyEditBehavior.h"
#include "PreviewMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MaterialEditingLibrary.h"
#include "BaseBehaviors/MouseWheelBehavior.h"
#include "Generators/MinimalBoxMeshGenerator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPaintTool)

#define LOCTEXT_NAMESPACE "UPCGPaintTool"

namespace PCGPaintTool::Constants
{
	const FText ToolDisplayName = LOCTEXT("ToolName", "PaintTool");
	const FName ToolTag = "PaintTool";
	const FText ToolDescription = LOCTEXT("DisplayMessage", "Draw points onto a surface and let the PCG graph process them.");
}

UPCGInteractiveToolSettings_PaintTool::UPCGInteractiveToolSettings_PaintTool()
{
	TInstancedStruct<FPCGRaycastFilterRule_Landscape> LandscapeRule = TInstancedStruct<FPCGRaycastFilterRule_Landscape>::Make();
	TInstancedStruct<FPCGRaycastFilterRule_Meshes> MeshesRule = TInstancedStruct<FPCGRaycastFilterRule_Meshes>::Make();
	TInstancedStruct<FPCGRaycastFilterRule_IgnorePCGGeneratedComponents> IgnorePCGGeneratedComponentsRule = TInstancedStruct<FPCGRaycastFilterRule_IgnorePCGGeneratedComponents>::Make();
	RaycastRuleCollection.RaycastRules.Add(LandscapeRule);
	RaycastRuleCollection.RaycastRules.Add(MeshesRule);
	RaycastRuleCollection.RaycastRules.Add(IgnorePCGGeneratedComponentsRule);
} 

FName UPCGInteractiveToolSettings_PaintTool::StaticGetToolTag()
{
	return PCGPaintTool::Constants::ToolTag;
}

FName UPCGInteractiveToolSettings_PaintTool::GetToolTag() const
{
	return StaticGetToolTag();
}

bool UPCGInteractiveToolSettings_PaintTool::IsWorkingActorCompatibleInternal(const AActor* InActor) const
{
	// Working actor should have a root component, we'll create one only for new actors.
	return InActor && InActor->GetRootComponent();
}

const UScriptStruct* UPCGInteractiveToolSettings_PaintTool::GetWorkingDataType() const
{
	return FPCGInteractiveToolWorkingData_PointArrayData::StaticStruct();
}

void UPCGInteractiveToolSettings_PaintTool::RegisterPropertyWatchers()
{
	Super::RegisterPropertyWatchers();

	WatchProperty(MinPointSpacing, [this](double InValue)
	{
		CachedMinPointSpacingSquared = InValue * InValue;
	});
}

void UPCGPaintTool::Setup()
{
	PropertyClass = UPCGEdModeBrushBase::StaticClass();
	
	ToolSettings->WatchProperty(ToolSettings->BrushMode, [this](EPCGBrushMode InBrushMode)
	{
		if (UPCGVolumetricBrushStampIndicator* TypedBrushStampIndicator = Cast<UPCGVolumetricBrushStampIndicator>(BrushStampIndicator))
		{
			TypedBrushStampIndicator->SetBrushMode(InBrushMode);
		}
	});
	
	AddToolPropertySource(ToolSettings);

	UPrimitiveComponentToolTarget* PrimitiveTarget = Cast<UPrimitiveComponentToolTarget>(GetTarget());
	AActor* TargetActor = PrimitiveTarget ? PrimitiveTarget->GetOwnerActor() : nullptr;

	if(TargetActor && ToolSettings->GeneratedResources.SpawnedActor.IsValid())
	{
		ToolSettings->GeneratedResources.SpawnedActor->SetActorLocation(TargetActor->GetActorLocation());
	}

	CachedRayParams =
	{
		.World = GetWorld(),
		.FilterRuleCollection = &ToolSettings->RaycastRuleCollection
	};

	MouseWheelInputBehavior = NewObject<UMouseWheelInputBehavior>();
	MouseWheelInputBehavior->Initialize(this);
	AddInputBehavior(MouseWheelInputBehavior);
	
	Super::Setup();	

	// Finally, set our default values for brush properties
	BrushProperties->BrushStrength = 1.0;
	BrushProperties->BrushFalloffAmount = 0.0;
}

void UPCGPaintTool::Shutdown(EToolShutdownType ShutdownType)
{
	UE::PCG::EditorMode::Tool::Shutdown(this, ShutdownType);
	
	Super::Shutdown(ShutdownType);
}

bool UPCGPaintTool::HasAccept() const
{
	return true;
}

bool UPCGPaintTool::CanAccept() const
{
	return true;
}

bool UPCGPaintTool::HasCancel() const
{
	return true;
}

void UPCGPaintTool::OnTick(float DeltaTime)
{
	Super::OnTick(DeltaTime);

	bool bShiftPressedThisTick = GetShiftToggle();

	if(bShiftPressedThisTick != bShiftPressedLastTick)
	{
		if(UPCGVolumetricBrushStampIndicator* VolumetricStampIndicator = Cast<UPCGVolumetricBrushStampIndicator>(BrushStampIndicator))
		{
			if(UMaterialInstanceDynamic* Instance = Cast<UMaterialInstanceDynamic>(VolumetricStampIndicator->GetPreviewMesh()->GetActiveMaterial()))
			{
				if(bShiftPressedThisTick)
				{
					Instance->SetVectorParameterValue("IndicatorColor", FLinearColor::Red);
				}
				else
				{
					FLinearColor DefaultColor;
					Instance->GetVectorParameterDefaultValue(FName("IndicatorColor"), DefaultColor);
					Instance->SetVectorParameterValue("IndicatorColor", DefaultColor);
				}
			}
		}
	}

	bShiftPressedLastTick = bShiftPressedThisTick;
}

bool UPCGPaintTool::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	using namespace UE::PCG::EditorMode::Scene;
	if (TOptional<FHitResult> Result = TraceToNearestObject(Ray, CachedRayParams))
	{
		OutHit = Result.GetValue();
		return Result->bBlockingHit;
	}

	return false;
}

void UPCGPaintTool::OnClickPress(const FInputDeviceRay& PressPos)
{
	Super::OnClickPress(PressPos);

	// Reset the last point transform on a new click
	ToolSettings->LastPointTransform.Reset();

	if(GetShiftToggle())
	{
		TryDeletePointsFromBrush();
	}
	else
	{
		AddSinglePointFromBrush();
	}
}

void UPCGPaintTool::OnUpdateDrag(const FRay& Ray)
{
	Super::OnUpdateDrag(Ray);

	if (GetShiftToggle())
	{
		TryDeletePointsFromBrush();
	}
	else
	{
		// If we added a point in one continuous operation, respect the min distance for the next point 
		if (ToolSettings->LastPointTransform.IsSet())
		{
			if (FVector::DistSquared(LastBrushStamp.WorldPosition, ToolSettings->LastPointTransform.GetValue().GetLocation()) >= ToolSettings->CachedMinPointSpacingSquared)
			{
				AddSinglePointFromBrush();
			}
		}
		else
		{
			AddSinglePointFromBrush();
		}
	}
}

FInputRayHit UPCGPaintTool::ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos)
{
	FInputRayHit Result;
	Result.bHit = true;

	return Result;
}

void UPCGPaintTool::OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos)
{
	if (UPCGVolumetricBrushStampIndicator* TypedBrushStampIndicator = Cast<UPCGVolumetricBrushStampIndicator>(BrushStampIndicator))
	{
		TypedBrushStampIndicator->AddYawRotation(ToolSettings->RotateYawStepSize);
	}
}

void UPCGPaintTool::OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos)
{
	if (UPCGVolumetricBrushStampIndicator* TypedBrushStampIndicator = Cast<UPCGVolumetricBrushStampIndicator>(BrushStampIndicator))
	{
		TypedBrushStampIndicator->AddYawRotation(-ToolSettings->RotateYawStepSize);
	}
}

void UPCGPaintTool::SetupBrushStampIndicator()
{
	if (!BrushStampIndicator)
	{
		GetToolManager()->GetPairedGizmoManager()->RegisterGizmoType(UE::PCG::EditorMode::Gizmos::Constants::BrushIdentifier_VolumetricBox, NewObject<UVolumetricBoxBrushStampIndicatorBuilder>());
		BrushStampIndicator = GetToolManager()->GetPairedGizmoManager()->CreateGizmo<UPCGVolumetricBrushStampIndicator>(UE::PCG::EditorMode::Gizmos::Constants::BrushIdentifier_VolumetricBox, FString(), this);
		if (UPCGVolumetricBrushStampIndicator* VolumetricBrushIndicator = Cast<UPCGVolumetricBrushStampIndicator>(BrushStampIndicator))
		{
			VolumetricBrushIndicator->SetBrushMode(ToolSettings->BrushMode);
			VolumetricBrushIndicator->MakeBrushIndicatorMesh(this, GetTargetWorld());
		}
	}
}

void UPCGPaintTool::UpdateBrushStampIndicator()
{
	if (BrushStampIndicator)
	{
		if (BrushEditBehavior.IsValid())
		{
			FLinearColor NewColor;
			if(BrushEditBehavior->IsEditing())
			{
				NewColor = FLinearColor::White;
			}
			else
			{
				if(GetShiftToggle())
				{
					NewColor = FLinearColor::Red;
				}
				else
				{
					NewColor = FLinearColor::Green;
				}
			}
			
			BrushStampIndicator->LineColor = NewColor;	
		}
		
		BrushStampIndicator->Update(BrushProperties->BrushRadius, LastBrushStamp.WorldPosition, LastBrushStamp.WorldNormal, BrushProperties->BrushFalloffAmount, BrushProperties->BrushStrength);
	}
}

void UPCGPaintTool::ShutdownBrushStampIndicator()
{
	if (BrushStampIndicator)
	{
		GetToolManager()->GetPairedGizmoManager()->DestroyGizmo(BrushStampIndicator);
		BrushStampIndicator = nullptr;
		GetToolManager()->GetPairedGizmoManager()->DeregisterGizmoType(UE::PCG::EditorMode::Gizmos::Constants::BrushIdentifier_VolumetricBox);
	}
}

void UPCGPaintTool::RestartBrushStampIndicator()
{
	ShutdownBrushStampIndicator();
	SetupBrushStampIndicator();
}

double UPCGPaintTool::EstimateMaximumTargetDimension()
{
	// 400 will mean brush radius at default Brush Size 0.25 will be 50, meaning 1m in diameter 
	return 400.f;
}

void UPCGPaintTool::NotifyDataChanged(UPCGData* InData)
{
	if (!InData)
	{
		return;
	}

	FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
	if (UE::PCG::EditorMode::Tool::CVarCommitPropertyChangedEventOnTimer.GetValueOnAnyThread())
	{
		ToolSettings->PropertyChangedEventQueue.Add(InData, EmptyPropertyChangedEvent);
	}
	else
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(InData, EmptyPropertyChangedEvent);
	}
}

void UPCGPaintTool::AddSinglePointFromBrush()
{
	check(ToolSettings);
	const FPCGInteractiveToolWorkingData_PointArrayData* WorkingData = ToolSettings->GetTypedWorkingData<FPCGInteractiveToolWorkingData_PointArrayData>(ToolSettings->DataInstance);
	UPCGPointArrayData* PointData = WorkingData ? WorkingData->GetPointArrayData() : nullptr;

	// If the current layer doesn't exist anymore (from the graph), it's possible the working data has been changed/removed.
	// @todo_pcg: Preferably this should get caught when the graph changes.
	if (!PointData)
	{
		return;
	}

	const int32 PointIndex = PointData->GetNumPoints();
	PointData->SetNumPoints(PointIndex + 1);

	TOptional<FRotator> BrushRotator;
	if (UPCGVolumetricBrushStampIndicator* TypedBrushStampIndicator = Cast<UPCGVolumetricBrushStampIndicator>(BrushStampIndicator))
	{
		BrushRotator = TypedBrushStampIndicator->GetPreviewMesh()->GetActor()->GetActorRotation();
	}

	const FTransform PointTransform = FTransform(BrushRotator.Get(FRotator::ZeroRotator), LastBrushStamp.WorldPosition);
	FPCGPoint NewPoint(PointTransform, BrushProperties->BrushStrength, PCGHelpers::ComputeSeedFromPosition(PointTransform.GetLocation()));
	// Affect bounds based on brush size & falloff.
	const float PointSteepness = (BrushProperties->bShowFalloff ? 1.0f - BrushProperties->BrushFalloffAmount : 1.0f);
	NewPoint.Steepness = PointSteepness;

	const float BoundsAdjustment = 0.5f + 0.5f * PointSteepness;

	NewPoint.BoundsMin = FVector(-CurrentBrushRadius * BoundsAdjustment);
	NewPoint.BoundsMax = FVector(+CurrentBrushRadius * BoundsAdjustment);
	
	// We use everything but the metadata entry & color
	PointData->AllocateProperties(PointData->GetAllocatedProperties() | (EPCGPointNativeProperties::All & ~EPCGPointNativeProperties::Color & ~EPCGPointNativeProperties::MetadataEntry));
	FPCGPointValueRanges PointDataRanges(PointData, /*bAllocate=*/false);
	PointDataRanges.SetFromPoint(PointIndex, NewPoint);

	// @todo_pcg
	// Instead of using a queue on timer, can we put this on another thread? Direct broadcasting takes too much perf on tick, timed queue results in delays

	// GetToolData dynamically tracks the pointdata object and will rerun the graph when it changes
	// To avoid doing this on tick, we queue up the changes
	NotifyDataChanged(PointData);

	ToolSettings->LastPointTransform = PointTransform;
}

void UPCGPaintTool::TryDeletePointsFromBrush()
{
	auto DeletePointsFromDataInstance = [this](FName DataInstanceName)
	{
		check(ToolSettings);
		const FPCGInteractiveToolWorkingData_PointArrayData* WorkingData = ToolSettings->GetTypedWorkingData<FPCGInteractiveToolWorkingData_PointArrayData>(DataInstanceName);
		UPCGPointArrayData* PointData = WorkingData ? WorkingData->GetPointArrayData() : nullptr;

		if (!PointData)
		{
			return;
		}

		// @todo_pcg - replace this by an octree search when we have proper iterative change for this kind of thing
		FSphere Sphere(LastBrushStamp.WorldPosition, CurrentBrushRadius);

		TArray<FTransform> Transforms = PointData->GetTransformsCopy();
		for(int32 Index = PointData->GetNumPoints() - 1; Index >= 0 ; --Index)
		{
			if(Sphere.IsInside(Transforms[Index].GetLocation()))
			{
				PointData->RemoveAt(Index);
			}
		}

		NotifyDataChanged(PointData);
	};
	
	if(ToolSettings->bErasePointsOfSelectedDataInstance)
	{
		DeletePointsFromDataInstance(ToolSettings->DataInstance);
	}
	else
	{
		for(const FName& DataInstanceName : ToolSettings->GetDataInstanceNamesForGraph())
		{
			DeletePointsFromDataInstance(DataInstanceName);
		}
	}
}

bool UPCGPaintToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	const UPCGInteractiveToolSettings_PaintTool* SettingsCDO = GetDefault<UPCGInteractiveToolSettings_PaintTool>();
	check(SettingsCDO);

	return SceneState.SelectedActors.IsEmpty()
		|| SceneState.SelectedActors.ContainsByPredicate([SettingsCDO](const AActor* InActor) { return SettingsCDO->IsWorkingActorCompatible(InActor); });
}

UInteractiveTool* UPCGPaintToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UPCGPaintTool* NewTool = NewObject<UPCGPaintTool>(SceneState.ToolManager);
	
	NewTool->SetToolInfo(
	{
		.ToolDisplayName = PCGPaintTool::Constants::ToolDisplayName,
		.ToolDisplayMessage = PCGPaintTool::Constants::ToolDescription,
		.ToolIcon = FPCGEditorModeStyle::Get().GetBrush("PCGEditorMode.Tools.Paint")
	});

	UToolTarget* const Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, FToolTargetTypeRequirements{});
	NewTool->SetTarget(Target);
	NewTool->SetWorld(SceneState.World);

	if(UE::PCG::EditorMode::Tool::BuildTool(NewTool) == false)
	{
		return nullptr;
	}
	
	return NewTool;
}

#undef LOCTEXT_NAMESPACE
