// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGEdModeBrushBase.h"
#include "BaseBehaviors/KeyAsModifierInputBehavior.h"
#include "EditorMode/Tools/PCGInteractiveToolBuilder.h"
#include "EditorMode/Tools/PCGInteractiveToolSettings.h"
#include "EditorMode/Tools/Gizmos/PCGGizmos.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"
#include "PCGPaintTool.generated.h"

class UMouseWheelInputBehavior;

UCLASS()
class UPCGInteractiveToolSettings_PaintTool : public UPCGInteractiveToolSettings
{
	GENERATED_BODY()

public:
	UPCGInteractiveToolSettings_PaintTool();

	static FName StaticGetToolTag();

	// ~Begin UPCGInteractiveToolSettings interface
	virtual FName GetToolTag() const override;
	virtual const UScriptStruct* GetWorkingDataType() const override;
	virtual void RegisterPropertyWatchers() override;
	// ~End UPCGInteractiveToolSettings interface

	/** The brush shape to use. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Paint")
	EPCGBrushMode BrushMode = EPCGBrushMode::Sphere;
	
	/** If false, will erase points on all paint data instances. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Paint")
	bool bErasePointsOfSelectedDataInstance = true;

	/** The step size for rotation of the brush. Moving the mouse wheel up or down will add or subtract from the brush rotation. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Paint")
	float RotateYawStepSize = 10.f;

	/** The minimum amount of distance to pass before drawing another point while dragging the mouse. */
	UPROPERTY(EditAnywhere, Category = "Paint", meta = (ClampMin = 0, UIMax = 1000))
	double MinPointSpacing = 50.0;

	UPROPERTY(EditAnywhere, Category= "Raycast")
	FPCGRaycastFilterRuleCollection RaycastRuleCollection;

	/** Helper property so we can avoid sqrt. */
	double CachedMinPointSpacingSquared = 0;

	/** Tracks the last point that was added. Used for min-spacing during drag. */
	TOptional<FTransform> LastPointTransform;

protected:
	virtual bool IsWorkingActorCompatibleInternal(const AActor* InActor) const override final;
};

UCLASS(Transient)
class UPCGPaintTool : public UBaseBrushTool, public IMouseWheelBehaviorTarget
{
	GENERATED_BODY()

public:
	// ~Begin UPCGInteractiveToolSettings interface
	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual bool HasAccept() const override;
	virtual bool CanAccept() const override;
	virtual bool HasCancel() const override;
	virtual void OnTick(float DeltaTime) override;
	// ~End UPCGInteractiveToolSettings interface

	virtual bool HitTest(const FRay& Ray, FHitResult& OutHit) override;
	
protected:
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual void OnUpdateDrag(const FRay& Ray) override;

	/** IMouseWheelBehaviorTarget */
	virtual FInputRayHit ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos) override;
	virtual void OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos) override;
	virtual void OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos) override;
	
	virtual void SetupBrushStampIndicator() override;
	virtual void UpdateBrushStampIndicator() override;
	virtual void ShutdownBrushStampIndicator() override;

	void RestartBrushStampIndicator();
	
	/** This will be used as the brush base, if not specifying a radius directly */
	virtual double EstimateMaximumTargetDimension() override;
	
private:
	void AddSinglePointFromBrush();
	void TryDeletePointsFromBrush();

	void NotifyDataChanged(UPCGData* InData);
	
private:
	UPROPERTY(Transient)
	TObjectPtr<UPCGInteractiveToolSettings_PaintTool> ToolSettings;

	UPROPERTY(Transient)
	TObjectPtr<UMouseWheelInputBehavior> MouseWheelInputBehavior;

	UPROPERTY(Transient)
	bool bShiftPressedLastTick = false;

	UE::PCG::EditorMode::Scene::FViewRayParams CachedRayParams{};
};

UCLASS(Transient)
class UPCGPaintToolBuilder : public UPCGInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};
