// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/Query/PCGQueryTool.h"

#include "PCGEditorModule.h"
#include "EditorMode/PCGEdModeStyle.h"

#include "PCGIsolatedActor.h"
#include "PCGManagedResource.h"
#include "Helpers/PCGHelpers.h"

#include "EditorViewportClient.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "InteractiveToolManager.h"
#include "BaseBehaviors/SingleClickBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGQueryTool)

#define LOCTEXT_NAMESPACE "UPCGQueryTool"

namespace PCGQueryTool
{
	namespace Constants
	{
		const FName ToolTag = TEXT("QueryTool");
		const FText ToolDisplayName = LOCTEXT("QueryToolName", "Query Tool");
		const FText ToolDescription = LOCTEXT("QueryToolDisplayMessage", "Queries the scene from a user input (e.g. clicking) to retrieve data (tags, etc.) and optionally affects the selected content.");
	}
}

namespace PCGIsolateTool
{
	namespace Constants
	{
		const FName ToolTag = TEXT("IsolateTool");
	}
}

void UPCGBaseQueryTool::Setup()
{
	Super::Setup();

	ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Initialize(this);
	AddInputBehavior(ClickBehavior);
}

void UPCGBaseQueryTool::Shutdown(const EToolShutdownType ShutdownType)
{
	// We have to call this before the super function as it would remove the property sets before we can route the shutdown function to the pcg settings
	UE::PCG::EditorMode::Tool::Shutdown(this, ShutdownType);

	Super::Shutdown(ShutdownType);
}

HHitProxy* UPCGBaseQueryTool::GetHitProxy(const FInputDeviceRay& ClickPosition) const
{
	FViewport* FocusedViewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	if (FocusedViewport)
	{
		if (HHitProxy* HitProxy = FocusedViewport->GetHitProxy(ClickPosition.ScreenPosition.X, ClickPosition.ScreenPosition.Y))
		{
			return HitProxy;
		}
	}
	
	return nullptr;
}

UActorComponent* UPCGBaseQueryTool::GetHitComponent(HHitProxy* HitProxy) const
{
	if (!HitProxy)
	{
		return nullptr;
	}
	else if (HInstancedStaticMeshInstance* ISMProxy = HitProxyCast<HInstancedStaticMeshInstance>(HitProxy))
	{
		return ISMProxy->Component.Get();
	}
	// implementation note: also catches HActor.
	else if (TTypedElement<ITypedElementObjectInterface> ObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(HitProxy->GetElementHandle()))
	{
		return ObjectInterface.GetObjectAs<UActorComponent>();
	}

	// Unknown/incompatible proxy type
	return nullptr;
}

UActorComponent* UPCGBaseQueryTool::GetHitComponent(const FInputDeviceRay& ClickPosition) const
{
	return GetHitComponent(GetHitProxy(ClickPosition));
}

FInputRayHit UPCGBaseQueryTool::IsHitByClick(const FInputDeviceRay& ClickPosition)
{
	using namespace UE::PCG::EditorMode;

	if (UActorComponent* HitComponent = GetHitComponent(ClickPosition))
	{
		FInputRayHit Result;
		Result.bHit = true;
		Result.HitObject = HitComponent;

		return Result;
	}

	return {};
}

void UPCGBaseQueryTool::OnClicked(const FInputDeviceRay& ClickPosition)
{
	using namespace UE::PCG::EditorMode;

	if (HHitProxy* HitProxy = GetHitProxy(ClickPosition))
	{
		UActorComponent* HitComponent = GetHitComponent(HitProxy);

		int32 InstanceIndex = INDEX_NONE;
		if (HInstancedStaticMeshInstance* ISMProxy = HitProxyCast<HInstancedStaticMeshInstance>(HitProxy))
		{
			InstanceIndex = ISMProxy->InstanceIndex;
		}

		OnHitProxy(HitProxy, HitComponent, InstanceIndex);
	}
}

void UPCGBaseQueryTool::VisitHierarchy(UActorComponent* HitComponent, int32 InstanceIndex, TFunctionRef<void(UActorComponent*, AActor*, int32)> Func)
{
	if (!HitComponent)
	{
		return;
	}

	// Start from the hit component, get its name + tags
	UActorComponent* CurrentComponent = HitComponent;

	while (CurrentComponent)
	{
		Func(CurrentComponent, nullptr, InstanceIndex);
		// The instance index is valid just for the first component we're visiting
		InstanceIndex = INDEX_NONE;
		
		UActorComponent* ParentComponent = nullptr;
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(CurrentComponent))
		{
			ParentComponent = SceneComponent->GetAttachParent();
		}

		CurrentComponent = (CurrentComponent == ParentComponent ? nullptr : ParentComponent);
	}

	// go up the root hierarchy
	AActor* CurrentActor = HitComponent->GetOwner();

	while (CurrentActor)
	{
		Func(nullptr, CurrentActor, INDEX_NONE);

		AActor* NewActor = CurrentActor->GetSceneOutlinerParent();
		CurrentActor = (CurrentActor == NewActor ? nullptr : NewActor);
	}
}

FName UPCGQueryTool::StaticGetToolTag()
{
	return PCGQueryTool::Constants::ToolTag;
}

void UPCGQueryTool::OnHitProxy(HHitProxy* HitProxy, UActorComponent* HitComponent, int32 InstanceIndex)
{
	auto OutputToLog = [&FoundActorLabel = FoundActorLabel, &SelectedTags = SelectedTags](UActorComponent* CurrentComponent, AActor* CurrentActor, int32 InstanceIndex)
	{
		if(CurrentComponent)
		{
			UE_LOGF(LogPCGEditor, Warning, "Component: '%ls'", *CurrentComponent->GetName());
			
			if (InstanceIndex != INDEX_NONE)
			{
				UE_LOGF(LogPCGEditor, Warning, "Instance: '%d'", InstanceIndex);
			}

			if (!CurrentComponent->ComponentTags.IsEmpty())
			{
				UE_LOGF(LogPCGEditor, Warning, "Tags: ");
				for (FName Tag : CurrentComponent->ComponentTags)
				{
					SelectedTags.Add(Tag.ToString());
					UE_LOGF(LogPCGEditor, Warning, "      %ls", *Tag.ToString());
				}
			}
		}
		else if (CurrentActor)
		{
			if (FoundActorLabel.IsEmpty())
			{
				FoundActorLabel = CurrentActor->GetActorLabel();
			}
			UE_LOGF(LogPCGEditor, Warning, "Actor: '%ls'", *CurrentActor->GetActorLabel());
			if (!CurrentActor->Tags.IsEmpty())
			{
				UE_LOGF(LogPCGEditor, Warning, "Tags: ");
				for (FName Tag : CurrentActor->Tags)
				{
					SelectedTags.Add(Tag.ToString());
					UE_LOGF(LogPCGEditor, Warning, "      %ls", *Tag.ToString());
				}
			}
		}
	};

	SelectedTags.Empty();
	FoundActorLabel = FString();
	VisitHierarchy(HitComponent, InstanceIndex, OutputToLog);
}

UInteractiveTool* UPCGQueryToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UPCGBaseQueryTool* NewTool = NewObject<UPCGBaseQueryTool>(SceneState.ToolManager, ToolClass);

	NewTool->SetToolInfo(
	{
		.ToolDisplayName = PCGQueryTool::Constants::ToolDisplayName,
		.ToolDisplayMessage = PCGQueryTool::Constants::ToolDescription,
		.ToolIcon = FPCGEditorModeStyle::Get().GetBrush("PCGEditorMode.Tools.Query")
	});

	if (!UE::PCG::EditorMode::Tool::BuildTool(NewTool))
	{
		return nullptr;
	}
	else
	{
		return NewTool;
	}
}

bool UPCGQueryToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return !!ToolClass;
}

void UPCGQueryToolBuilder::SetToolClass(TSubclassOf<UPCGBaseQueryTool> InClass)
{
	ToolClass = InClass;
}

void UPCGInteractiveToolSettings_Isolate::Apply(UInteractiveTool* OwningTool)
{
	if (AActor* Actor = SelectedActor.Get())
	{
		FPCGMoveResourceParams Params
		{
			.TemplateTargetClass = APCGIsolatedActor::StaticClass(),
			.Target = (bExtractToSelf ? Actor : nullptr),
			.RequiredTags = SelectedTags,
			.ExcludedTags = ExcludedTags,
			.bAttachToParent = bAttachToActor,
			.DefaultName = IsolatedActorLabel
		};

		UPCGActorHelpers::IsolateFromActor(Actor, Params);
	}
}

FName UPCGInteractiveToolSettings_Isolate::GetToolTag() const
{
	return UPCGIsolateTool::StaticGetToolTag();
}

bool UPCGInteractiveToolSettings_Isolate::IsSelectionAllowed(AActor* InActor, bool bInSelection) const
{
	return InActor && InActor->GetComponentByClass<UPCGComponent>() != nullptr;
}

FName UPCGIsolateTool::StaticGetToolTag()
{
	return PCGIsolateTool::Constants::ToolTag;
}

void UPCGIsolateTool::Setup()
{
	AddToolPropertySource(ToolSettings);
	Super::Setup();
}

bool UPCGIsolateTool::CanAccept() const
{
	// Can return true if we picked an actor that has a pcg component.
	if (UPCGInteractiveToolSettings_Isolate* Settings = GetSettings())
	{
		if (AActor* SelectedActor = Settings->SelectedActor.Get())
		{
			return SelectedActor->GetComponentByClass<UPCGComponent>() != nullptr;
		}
	}

	return false;
}

UPCGInteractiveToolSettings_Isolate* UPCGIsolateTool::GetSettings() const
{
	TArray<UObject*> AllToolProperties = GetToolProperties(false);

	UObject** FoundProperties = AllToolProperties.FindByPredicate([](const UObject* Candidate)
		{
			return Candidate->IsA<UPCGInteractiveToolSettings_Isolate>();
		});

	if (FoundProperties)
	{
		return Cast<UPCGInteractiveToolSettings_Isolate>(*FoundProperties);
	}

	return nullptr;
}

void UPCGIsolateTool::OnHitProxy(HHitProxy* HitProxy, UActorComponent* HitComponent, int InstanceIndex)
{
	AActor* FoundActor = nullptr;
	TSet<FName> FoundTags;
	int32 FoundInstanceIndex = INDEX_NONE;

	auto GatherTags = [&FoundTags, &FoundActor, &FoundInstanceIndex](UActorComponent* CurrentComponent, AActor* CurrentActor, int32 InstanceIndex)
	{
		if(CurrentComponent)
		{
			FoundTags.Append(CurrentComponent->ComponentTags);
		}

		if (CurrentActor)
		{
			if (!FoundActor)
			{
				FoundActor = CurrentActor;
			}

			FoundTags.Append(CurrentActor->Tags);
		}

		if (InstanceIndex != INDEX_NONE)
		{
			FoundInstanceIndex = InstanceIndex;
		}
	};

	VisitHierarchy(HitComponent, InstanceIndex, GatherTags);

	if (UPCGInteractiveToolSettings_Isolate* Settings = GetSettings())
	{
		Settings->SelectedActor = FoundActor;
		Settings->Tags = FoundTags;
		Settings->InstanceIndex = FoundInstanceIndex;
	}
}

#undef LOCTEXT_NAMESPACE