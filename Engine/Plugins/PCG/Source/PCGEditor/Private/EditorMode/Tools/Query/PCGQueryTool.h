// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorMode/Tools/PCGInteractiveToolBuilder.h"
#include "EditorMode/Tools/PCGInteractiveToolCommon.h"
#include "EditorMode/Tools/PCGInteractiveToolSettings.h"
#include "EditorMode/Tools/Helpers/PCGEdModeSceneQueryHelpers.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "PCGQueryTool.generated.h"

class AActor;
class HHitProxy;
class UActorComponent;
class USingleClickInputBehavior;


/**
* Simple editor tool to gather information from a picked object in the world. 
* By default logs visited hierarchy tags (from touched component, to actor, ...) but can be reimplemented in a derived class.
*/
UCLASS()
class UPCGBaseQueryTool : public UPCGInteractiveTool, public IClickBehaviorTarget
{
	GENERATED_BODY()

public:
	//~Begin UInteractiveTool interface
	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	//~End UInteractiveTool interface

	//~Begin UPCGInteractiveTool interface
	virtual bool RequiresSettings() const override { return false; }
	//~End UPCGInteractiveTool interface

	//~Begin IClickBehaviorTarget interface
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPosition) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPosition) override;
	//~End IClickBehaviorTarget interface

protected:
	HHitProxy* GetHitProxy(const FInputDeviceRay& ClickPosition) const;
	UActorComponent* GetHitComponent(HHitProxy* HitProxy) const;
	UActorComponent* GetHitComponent(const FInputDeviceRay& ClickPosition) const;
	virtual void OnHitProxy(HHitProxy* HitProxy, UActorComponent* HitComponent, int InstanceIndex) {}
	void VisitHierarchy(UActorComponent* HitComponent, int32 InstanceIndex, TFunctionRef<void(UActorComponent*, AActor*, int32)> Func);

private:
	UPROPERTY(Instanced)
	TObjectPtr<USingleClickInputBehavior> ClickBehavior = nullptr;
};
/**
* Simple editor tool to gather information from a picked object in the world. 
* By default logs visited hierarchy tags (from touched component, to actor, ...) but can be reimplemented in a derived class.
*/
UCLASS()
class UPCGQueryTool : public UPCGBaseQueryTool
{
	GENERATED_BODY()

public:
	//~Begin UInteractiveTool interface
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override { return !SelectedTags.IsEmpty(); }
	//~End UInteractiveTool interface

	//~Begin UPCGInteractiveTool interface
	virtual bool RequiresSettings() const override { return false; }
	//~End UPCGInteractiveTool interface

	static FName StaticGetToolTag();
	
	UPROPERTY()
	TArray<FString> SelectedTags;

	UPROPERTY()
	FString FoundActorLabel;

protected:
	virtual void OnHitProxy(HHitProxy* HitProxy, UActorComponent* HitComponent, int InstanceIndex);
};



UCLASS(Transient)
class UPCGQueryToolBuilder : public UPCGInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	void SetToolClass(TSubclassOf<UPCGBaseQueryTool> InClass);

private:
	UPROPERTY()
	TSubclassOf<UPCGBaseQueryTool> ToolClass = nullptr;
};

UCLASS()
class UPCGInteractiveToolSettings_Isolate : public UPCGInteractiveToolBaseSettings
{
	GENERATED_BODY()

public:
	virtual void Apply(UInteractiveTool* OwningTool) override;
	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const override;
	virtual FName GetToolTag() const override;
	//virtual const UScriptStruct* GetWorkingDataType() const { return nullptr; }

	UPROPERTY(VisibleAnywhere, Category = PCG)
	TWeakObjectPtr<AActor> SelectedActor;

	UPROPERTY(VisibleAnywhere, Category = PCG)
	TSet<FName> Tags;

	UPROPERTY(VisibleAnywhere, Category = PCG)
	int32 InstanceIndex = INDEX_NONE;

	UPROPERTY(EditAnywhere, Category = PCG)
	TArray<FName> SelectedTags;

	UPROPERTY(EditAnywhere, Category = PCG)
	TArray<FName> ExcludedTags;

	UPROPERTY(EditAnywhere, Category = PCG)
	bool bExtractToSelf = false;

	UPROPERTY(EditAnywhere, Category = PCG, meta = (EditCondition = "!bExtractToSelf"))
	bool bAttachToActor = false;

	UPROPERTY(EditAnywhere, Category = PCG, meta = (EditCondition = "!bExtractToSelf"))
	FName IsolatedActorLabel = NAME_None;
};

/**
* Tool that extracts the artifacts (through Clear PCG Link) from an actor with a PCG component according to selected tags.
* Tags are gathered through picking (from the Query tool), and listed so the user can choose a combination.
*/
UCLASS()
class UPCGIsolateTool : public UPCGBaseQueryTool
{
	GENERATED_BODY()

public:
	//~Begin UInteractiveTool interface
	virtual void Setup() override;
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override;
	//~End UInteractiveTool interface

	//~Begin UPCGInteractiveTool interface
	virtual bool RequiresSettings() const override { return true; }
	//~End UPCGInteractiveTool interface

	static FName StaticGetToolTag();

protected:
	//~Begin UPCGQueryTool interface
	virtual void OnHitProxy(HHitProxy* HitProxy, UActorComponent* HitComponent, int InstanceIndex) override;
	//~End UPCGQueryTool interface

	UPCGInteractiveToolSettings_Isolate* GetSettings() const;

	UPROPERTY(Transient)
	TObjectPtr<UPCGInteractiveToolSettings_Isolate> ToolSettings;
};