// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Tools/Volume/PCGVolumeTool.h"

#include "PCGEditorModule.h"
#include "PCGVolume.h"
#include "Data/Tool/PCGToolVolumeData.h"
#include "EditorMode/PCGEdModeStyle.h"

#include "InteractiveToolManager.h"
#include "SnappingUtils.h"
#include "SubobjectDataSubsystem.h"
#include "ToolTargetManager.h"
#include "ActorFactories/ActorFactory.h"
#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "Components/BoxComponent.h"
#include "Builders/CubeBuilder.h"
#include "GameFramework/Volume.h"
#include "ToolTargets/ToolTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGVolumeTool)

#define LOCTEXT_NAMESPACE "UPCGVolumeTool"

namespace PCGVolumeTool::Constants
{
	const FText ToolDisplayName = LOCTEXT("ToolDisplayName", "Volume");
	const FName ToolTag = "VolumeTool";
}

UPCGInteractiveToolSettings_Volume::UPCGInteractiveToolSettings_Volume()
{
	TInstancedStruct<FPCGRaycastFilterRule_Landscape> LandscapeRule = TInstancedStruct<FPCGRaycastFilterRule_Landscape>::Make();
	TInstancedStruct<FPCGRaycastFilterRule_Meshes> MeshesRule = TInstancedStruct<FPCGRaycastFilterRule_Meshes>::Make();
	TInstancedStruct<FPCGRaycastFilterRule_IgnorePCGGeneratedComponents> IgnorePCGGeneratedComponentsRule = TInstancedStruct<FPCGRaycastFilterRule_IgnorePCGGeneratedComponents>::Make();
	RaycastRuleCollection.RaycastRules.Add(LandscapeRule);
	RaycastRuleCollection.RaycastRules.Add(MeshesRule);
	RaycastRuleCollection.RaycastRules.Add(IgnorePCGGeneratedComponentsRule);
}

FName UPCGInteractiveToolSettings_Volume::StaticGetToolTag()
{
	return PCGVolumeTool::Constants::ToolTag;
}

FName UPCGInteractiveToolSettings_Volume::GetToolTag() const
{
	return StaticGetToolTag();
}

const UScriptStruct* UPCGInteractiveToolSettings_Volume::GetWorkingDataType() const
{
	return FPCGInteractiveToolWorkingData_Volume::StaticStruct();
}

bool UPCGInteractiveToolSettings_Volume::IsWorkingActorCompatibleInternal(const AActor* InActor) const
{
	// Working actor needs to be a volume or have a box component
	return InActor && (InActor->IsA<AVolume>() || InActor->GetComponentByClass<UBoxComponent>() != nullptr);
}

bool UPCGInteractiveToolSettings_Volume::IsActorClassCompatibleInternal(TSubclassOf<AActor> InClass) const
{	
	// Accept volumes or actors that have a box component
	if (InClass != nullptr)
	{
		if (InClass->IsChildOf<AVolume>())
		{
			return true;
		}		
		
		bool bActorClassHasBoxComponent = false;

		if (AActor* ActorCDO = InClass->GetDefaultObject<AActor>())
		{
			bActorClassHasBoxComponent |= ActorCDO->GetComponentByClass<UBoxComponent>() != nullptr;

			// Check subobjects to cover for BP classes.
			if (!bActorClassHasBoxComponent && Cast<UBlueprintGeneratedClass>(InClass) != nullptr)
			{
				TArray<FSubobjectDataHandle> SubobjectHandles;
				USubobjectDataSubsystem::Get()->GatherSubobjectData(ActorCDO, SubobjectHandles);

				const FSubobjectDataHandle* FoundHandle = SubobjectHandles.FindByPredicate(
					[](const FSubobjectDataHandle& Handle)
					{
						if (FSubobjectData* SubobjectData = Handle.GetData())
						{
							if (const UActorComponent* ComponentTemplate = SubobjectData->GetComponentTemplate())
							{
								return ComponentTemplate->IsA<UBoxComponent>();
							}
						}

						return false;
					}
				);

				if (FoundHandle && FoundHandle->IsValid())
				{
					bActorClassHasBoxComponent |= (FoundHandle->GetData()->GetObject<UBoxComponent>() != nullptr);
				}
			}
		}

		return bActorClassHasBoxComponent;
	}
	
	return false;
}

TOptional<TSubclassOf<AActor>> UPCGInteractiveToolSettings_Volume::GetFallbackActorClassToSpawn() const
{
	TSubclassOf<AActor> Result = APCGVolume::StaticClass();
	return Result;
}

AVolume* UPCGInteractiveToolSettings_Volume::GetWorkingVolume(FName DataInstanceIdentifier) const
{
	if (const FPCGInteractiveToolWorkingData_Volume* ToolData = GetTypedWorkingData<FPCGInteractiveToolWorkingData_Volume>(DataInstanceIdentifier))
	{
		return ToolData->Volume.Get();
	}

	return nullptr;
}

UBoxComponent* UPCGInteractiveToolSettings_Volume::GetWorkingBoxComponent(FName DataInstanceIdentifier) const
{
	if (const FPCGInteractiveToolWorkingData_Volume* ToolData = GetTypedWorkingData<FPCGInteractiveToolWorkingData_Volume>(DataInstanceIdentifier))
	{
		return ToolData->BoxComponent.Get();
	}

	return nullptr;
}

void UPCGVolumeTool::Setup()
{
	Super::Setup();

	DefaultHalfHeight = 100; // @todo_pcg: get from existing volume if any

	ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Initialize(this, this);
	AddInputBehavior(ClickOrDragBehavior);

	HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	AddInputBehavior(HoverBehavior);
	
	AddToolPropertySource(ToolSettings);

	CachedRayParams =
	{
		.World = GetWorld(),
		.FilterRuleCollection = &ToolSettings->RaycastRuleCollection,
		.bHitGroundPlanes = &ToolSettings->bHitGroundPlanes
	};
}

void UPCGVolumeTool::Shutdown(EToolShutdownType ShutdownType)
{
	UE::PCG::EditorMode::Tool::Shutdown(this, ShutdownType);

	Super::Shutdown(ShutdownType);
}

void UPCGVolumeTool::OnTick(const float DeltaTime)
{
	// if we are currently dragging, we should update the temporary contents with a fixed Z size
	// or update w/ live Z coordinates
	if (bIsDraggingBase || bIsDraggingHeight)
	{
		FBox Box = GetCurrentBox();

		if (LastBox.IsValid &&
			(Box.GetCenter() - LastBox.GetCenter()).SquaredLength() < UE_KINDA_SMALL_NUMBER &&
			(Box.GetExtent() - LastBox.GetExtent()).SquaredLength() < UE_KINDA_SMALL_NUMBER)
		{
			// Nothing to do
		}
		else
		{
			LastBox = Box;
			UpdateVolumeFromBox();
		}
	}

	Super::OnTick(DeltaTime);
}

void UPCGVolumeTool::UpdateVolumeFromBox()
{
	const FVector BoxCenter = LastBox.GetCenter();
	FVector Extent = 2.0 * LastBox.GetExtent();

	// if the user is using snapping, we still need to make sure that the final location would cover the full extent.
	// This is because the brush builder will enforce snapping.
	if (FSnappingUtils::IsSnapToGridEnabled())
	{
		FVector SnappedCenter = BoxCenter;
		FRotator DummyRotator = FRotator::ZeroRotator;
		FSnappingUtils::SnapToBSPVertex(SnappedCenter, FVector::ZeroVector, DummyRotator);

		// We will not change the X/Y coordinates, but will adjust the Z coordinate so that we're always covering the bottom of the original box
		Extent.Z += 2.0 * FMath::Abs(SnappedCenter.Z - BoxCenter.Z);
	}

	if (AVolume* WorkingVolume = ToolSettings ? ToolSettings->GetWorkingVolume(ToolSettings->DataInstance) : nullptr; WorkingVolume && WorkingVolume->GetBrushComponent())
	{
		// Move actor to bounds center
		WorkingVolume->SetActorTransform(FTransform(BoxCenter));

		// setup cube builder
		UCubeBuilder* Builder = NewObject<UCubeBuilder>();
		Builder->X = Extent.X;
		Builder->Y = Extent.Y;
		Builder->Z = Extent.Z;
		UActorFactory::CreateBrushForVolumeActor(WorkingVolume, Builder);
	}
	else if (UBoxComponent* WorkingBoxComponent = ToolSettings ? ToolSettings->GetWorkingBoxComponent(ToolSettings->DataInstance) : nullptr)
	{
		// If we've spawned this actor, we'll assume we can take over the root transform otherwise, everything will behave weirdly.
		if (ToolSettings->HasSpawnedActor() && 
			WorkingBoxComponent->GetOwner() && 
			WorkingBoxComponent->GetOwner()->GetRootComponent() &&
			WorkingBoxComponent->GetOwner()->GetRootComponent() != WorkingBoxComponent)
		{
			WorkingBoxComponent->GetOwner()->GetRootComponent()->SetWorldTransform(FTransform(BoxCenter));

			FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(WorkingBoxComponent->GetOwner()->GetRootComponent(), EmptyPropertyChangedEvent);
		}

		WorkingBoxComponent->SetWorldTransform(FTransform(BoxCenter));
		WorkingBoxComponent->SetBoxExtent(0.5 * Extent);

		FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(WorkingBoxComponent, EmptyPropertyChangedEvent);
	}
}

FBox UPCGVolumeTool::GetCurrentBox() const
{
	FVector BoxCenter;
	FVector HalfExtents;

	if (bIsDraggingBase)
	{
		BoxCenter.X = 0.5 * (FirstPoint.X + InteractiveSecondPoint.X);
		BoxCenter.Y = 0.5 * (FirstPoint.Y + InteractiveSecondPoint.Y);

		HalfExtents.X = 0.5 * FMath::Abs(FirstPoint.X - InteractiveSecondPoint.X);
		HalfExtents.Y = 0.5 * FMath::Abs(FirstPoint.Y - InteractiveSecondPoint.Y);
	}
	else
	{
		BoxCenter.X = 0.5 * (FirstPoint.X + SecondPoint.X);
		BoxCenter.Y = 0.5 * (FirstPoint.Y + SecondPoint.Y);

		HalfExtents.X = 0.5 * FMath::Abs(FirstPoint.X - SecondPoint.X);
		HalfExtents.Y = 0.5 * FMath::Abs(FirstPoint.Y - SecondPoint.Y);
	}

	// Bias needed so the volume ends up covering the touching face.
	const double Bias = 1.0;

	if (bIsDraggingHeight)
	{
		const double MaxRayLength = 1000000.0;
		FVector PointOnRay, PointOnVertical;
		FMath::SegmentDistToSegment(LastRay.Origin, LastRay.Origin + LastRay.Direction * MaxRayLength, SecondPoint - 0.5 * MaxRayLength * FVector::ZAxisVector, SecondPoint + 0.5 * MaxRayLength * FVector::ZAxisVector, PointOnRay, PointOnVertical);

		double MinZ, MaxZ;
		if (PointOnVertical.Z < FirstPoint.Z)
		{
			MinZ = PointOnVertical.Z;
			MaxZ = FirstPoint.Z + Bias;
		}
		else
		{
			MinZ = FirstPoint.Z - Bias;
			MaxZ = PointOnVertical.Z;
		}

		BoxCenter.Z = 0.5 * (MaxZ + MinZ);
		HalfExtents.Z = 0.5 * (MaxZ - MinZ);
	}
	else
	{
		BoxCenter.Z = FirstPoint.Z - Bias;
		HalfExtents.Z = DefaultHalfHeight;
	}

	return FBox(BoxCenter - HalfExtents, BoxCenter + HalfExtents);
}

bool UPCGVolumeTool::CanAccept() const
{
	// @todo_pcg also make sure we're in a correct state (e.g. not being interacted with at this time).
	return ToolSettings &&
		(ToolSettings->GetWorkingVolume(ToolSettings->DataInstance) != nullptr || ToolSettings->GetWorkingBoxComponent(ToolSettings->DataInstance) != nullptr);
}

FInputRayHit UPCGVolumeTool::IsHitByClick(const FInputDeviceRay& ClickPosition)
{
	// Always accept clicks; invalid clicks when in the base-dragging phase will be discarded by the CanBeginClickDragSequence call.
	FInputRayHit DummyHit;
	DummyHit.bHit = true;
	return DummyHit;
}

void UPCGVolumeTool::OnClicked(const FInputDeviceRay& ClickPosition)
{
	bIsDraggingBase = false;

	if (bBaseSelectionDone)
	{
		LastBox = GetCurrentBox();
		bBaseSelectionDone = false;
		bIsDraggingHeight = false;
		
		UpdateVolumeFromBox();
	}

	// @todo_pcg: Different chords to translate, rotate, scale at any step of the process?
}

FInputRayHit UPCGVolumeTool::CanBeginClickDragSequence(const FInputDeviceRay& DragPosition)
{
	// Dragging the base requires a ray to hit the physical scene.
	using namespace UE::PCG::EditorMode::Scene;
	if (TOptional<FViewHitResult> Result = ViewportRay(DragPosition, CachedRayParams))
	{
		return Result->ToInputRayHit();
	}

	return {};
}

void UPCGVolumeTool::OnClickPress(const FInputDeviceRay& ClickPosition)
{
	// Restart xy selection
	bBaseSelectionDone = false;
	bIsDraggingBase = false;

	using namespace UE::PCG::EditorMode::Scene;
	if (TOptional<FViewHitResult> Result = ViewportRay(ClickPosition, CachedRayParams))
	{
		FirstPoint = Result->ImpactPosition;
		InteractiveSecondPoint = FirstPoint;
		bIsDraggingBase = true;
	}
}

void UPCGVolumeTool::OnClickDrag(const FInputDeviceRay& DragPosition)
{
	// Update interactive second point only.
	InteractiveSecondPoint = GetBaseDragPosition(DragPosition);
}

void UPCGVolumeTool::OnClickRelease(const FInputDeviceRay& ReleasePosition)
{
	OnClickDrag(ReleasePosition);
	SecondPoint = GetBaseDragPosition(ReleasePosition);
	// Hover isn't updated during drag, so we need to update it here, otherwise there will be a visible jerk
	LastRay = ReleasePosition.WorldRay;

	bIsDraggingBase = false;
	bBaseSelectionDone = true;
	bIsDraggingHeight = true;
	OnTerminateDragSequence();
}

FVector UPCGVolumeTool::GetBaseDragPosition(const FInputDeviceRay& DragPosition) const
{
	// Converts world ray into a projection on the first point plane, with normal z-up.
	FPlane BasePlane(FirstPoint, FVector::ZAxisVector);
	return FMath::RayPlaneIntersection(DragPosition.WorldRay.Origin, DragPosition.WorldRay.Direction, BasePlane);
}

FInputRayHit UPCGVolumeTool::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	// This hit will not be consumed, but only used to control when we are in the hover state.
	FInputRayHit Hit;

	// Hover sequence can only happen when we're done with the base selection
	if (bBaseSelectionDone)
	{
		Hit.bHit = true;
	}
	
	return Hit;
}

bool UPCGVolumeTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	LastRay = DevicePos.WorldRay;
	return true;
}

bool UPCGVolumeToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	// @todo_pcg: To be factorized in a base class
	const UPCGInteractiveToolSettings_Volume* SettingsCDO = GetDefault<UPCGInteractiveToolSettings_Volume>();
	check(SettingsCDO)
	return SceneState.SelectedActors.IsEmpty()
		|| SceneState.SelectedActors.ContainsByPredicate([SettingsCDO](const AActor* InActor) { return SettingsCDO->IsWorkingActorCompatible(InActor); });
}

UInteractiveTool* UPCGVolumeToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UPCGVolumeTool* NewTool = NewObject<UPCGVolumeTool>(SceneState.ToolManager);

	NewTool->SetToolInfo(
	{
		.ToolDisplayName = PCGVolumeTool::Constants::ToolDisplayName,
		.ToolDisplayMessage = LOCTEXT("VolumeToolDisplayMessage", "Creates or changes a volume."),
		.ToolIcon = FPCGEditorModeStyle::Get().GetBrush("PCGEditorMode.Tools.Volume")
	});
	
	if(UE::PCG::EditorMode::Tool::BuildTool(NewTool) == false)
	{
		return nullptr;
	}	

	return NewTool;
}

#undef LOCTEXT_NAMESPACE