// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorMode/Tools/PCGInteractiveToolBuilder.h"
#include "EditorMode/Tools/PCGInteractiveToolSettings.h"
#include "EditorMode/Tools/Helpers/PCGEdModeSceneQueryHelpers.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "PCGVolumeTool.generated.h"

class AVolume;
class UBoxComponent;
class USingleClickOrDragInputBehavior;
class UMouseHoverBehavior;

UCLASS()
class UPCGInteractiveToolSettings_Volume : public UPCGInteractiveToolSettings
{
	GENERATED_BODY()

public:
	UPCGInteractiveToolSettings_Volume();

	static FName StaticGetToolTag();

	// ~Begin UPCGInteractiveToolSettings interface
	virtual FName GetToolTag() const override;
	virtual const UScriptStruct* GetWorkingDataType() const override;
	virtual bool CanResetToolData(FName DataInstanceIdentifier) override { return false; }
	// ~End UPCGInteractiveToolSettings interface

	AVolume* GetWorkingVolume(FName DataInstanceIdentifier) const;
	UBoxComponent* GetWorkingBoxComponent(FName DataInstanceIdentifier) const;
	// @todo_pcg: Origin modes: Center, Corner
	// @todo_pcg: Snap to grid

	UPROPERTY(EditAnywhere, Category = Raycast)
	FPCGRaycastFilterRuleCollection RaycastRuleCollection;

	/** Whether to place the volume base on a plane through the origin aligned with the Z axis in perspective views. */
	UPROPERTY(EditAnywhere, Category = "Raycast")
	bool bHitGroundPlanes = true;

protected:
	/** This function is called by the builder using UPCGInteractiveToolSettings_Volume default class which prohibits to have override
	 * implementation of this function by child classes. Mark it final for that reason, to be revisited if we change the tool classes hierarchy. */
	virtual bool IsWorkingActorCompatibleInternal(const AActor* InActor) const override final;
	virtual bool IsActorClassCompatibleInternal(TSubclassOf<AActor> InClass) const override final;
	virtual TOptional<TSubclassOf<AActor>> GetFallbackActorClassToSpawn() const override final;
};

/** 
 * Tool class to create 3d volumes. Current implementation only creates cubes.
 *	Click + drag -> creates the base.
 *	Second click -> creates the height.
 */
UCLASS(Transient, MinimalAPI)
class UPCGVolumeTool : public UInteractiveTool, public IClickBehaviorTarget, public IClickDragBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	// ~Begin UInteractiveTool interface
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual void OnTick(float DeltaTime) override;
	virtual bool CanAccept() const override;
	// ~End UInteractiveTool interface

	// ~Begin IClickBehaviorTarget interface
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPosition) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPosition) override;
	// ~End IClickBehaviorTarget interface

	// ~Begin IClickDragBehaviorTarget interface
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& DragPosition) override;
	virtual void OnClickPress(const FInputDeviceRay& ClickPosition) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPosition) override;
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePosition) override;
	virtual void OnTerminateDragSequence() override {}
	// ~End IClickDragBehaviorTarget interface

	// ~Begin IHoverBehaviorTarget interface
	virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override {}
	// ~End IHoverBehaviorTarget interface

private:
	FBox GetCurrentBox() const;
	void UpdateVolumeFromBox();
	FVector GetBaseDragPosition(const FInputDeviceRay& DragPosition) const;

	UE::PCG::EditorMode::Scene::FViewRayParams CachedRayParams;

	UPROPERTY(Instanced)
	TObjectPtr<USingleClickOrDragInputBehavior> ClickOrDragBehavior = nullptr;

	UPROPERTY(Instanced)
	TObjectPtr<UMouseHoverBehavior> HoverBehavior = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<UPCGInteractiveToolSettings_Volume> ToolSettings;

	bool bIsDraggingBase = false;
	bool bIsDraggingHeight = false;
	bool bBaseSelectionDone = false;
	FVector FirstPoint = FVector::ZeroVector;
	FVector SecondPoint = FVector::ZeroVector;
	FVector InteractiveSecondPoint = FVector::ZeroVector;
	double DefaultHalfHeight = 0;

	FRay LastRay;
	FBox LastBox = FBox(EForceInit::ForceInit);
};

UCLASS(Transient, MinimalAPI)
class UPCGVolumeToolBuilder : public UPCGInteractiveToolWithToolTargetsBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};
