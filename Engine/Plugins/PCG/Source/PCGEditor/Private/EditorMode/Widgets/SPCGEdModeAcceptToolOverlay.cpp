// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMode/Widgets/SPCGEdModeAcceptToolOverlay.h"

#include "SPrimaryButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

void SPCGEdModeAcceptToolOverlay::Construct(const FArguments& InArgs)
{
	Args = InArgs;

	// Non-volatile. Should remain static until tool changes.
	SetCanTick(false);

	SetVisibility(EVisibility::SelfHitTestInvisible);

	this->ChildSlot
	[
		SNew(SHorizontalBox)
		.Visibility(EVisibility::SelfHitTestInvisible)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Bottom)
		.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
			.Padding(8.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
				[
					SNew(SImage)
					.Image(Args._ActiveToolIcon)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
				[
					SNew(STextBlock)
					.Text(Args._ActiveToolDisplayName)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
				[
					SNew(SPrimaryButton)
					.Text(Args._AcceptButtonLabel)
					.ToolTipText(Args._AcceptButtonTooltip)
					.OnClicked(Args._OnAcceptButtonClicked)
					.IsEnabled(Args._IsAcceptButtonEnabled)
					.Visibility(Args._GetAcceptButtonVisibility)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.TextStyle(FAppStyle::Get(), "DialogButtonText")
					.Text(Args._CancelButtonLabel)
					.ToolTipText(Args._CancelButtonTooltip)
					.OnClicked(Args._OnCancelButtonClicked)
					.IsEnabled(Args._IsCancelButtonEnabled)
					.Visibility(Args._GetCancelButtonVisibility)
				]
			]
		]
	];

	SlatePrepass();
}
