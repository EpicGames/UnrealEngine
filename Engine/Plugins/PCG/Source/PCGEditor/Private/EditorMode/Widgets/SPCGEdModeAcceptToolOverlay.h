// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"

#include "Widgets/SCompoundWidget.h"

/** A simple Accept/Cancel widget for the PCG Editor Mode Toolkit. */
class SPCGEdModeAcceptToolOverlay : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPCGEdModeAcceptToolOverlay)
		: _ActiveToolDisplayName(FText::GetEmpty())
		, _ActiveToolIcon(nullptr)
		, _IsAcceptButtonEnabled(false)
		, _IsCancelButtonEnabled(false)
		, _GetAcceptButtonVisibility(EVisibility::Collapsed)
		, _GetCancelButtonVisibility(EVisibility::Collapsed) {}

		SLATE_ATTRIBUTE(FText, ActiveToolDisplayName)
		SLATE_ATTRIBUTE(const FSlateBrush*, ActiveToolIcon)

		SLATE_ATTRIBUTE(FText, AcceptButtonLabel)
		SLATE_ATTRIBUTE(FText, CancelButtonLabel)
		SLATE_ATTRIBUTE(FText, AcceptButtonTooltip)
		SLATE_ATTRIBUTE(FText, CancelButtonTooltip)
		SLATE_ATTRIBUTE(bool, IsAcceptButtonEnabled)
		SLATE_ATTRIBUTE(bool, IsCancelButtonEnabled)
		SLATE_ATTRIBUTE(EVisibility, GetAcceptButtonVisibility)
		SLATE_ATTRIBUTE(EVisibility, GetCancelButtonVisibility)

		SLATE_EVENT(FOnClicked, OnAcceptButtonClicked)
		SLATE_EVENT(FOnClicked, OnCancelButtonClicked)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	FArguments Args;
};
