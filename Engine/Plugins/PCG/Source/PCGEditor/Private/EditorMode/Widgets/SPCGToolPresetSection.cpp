// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGToolPresetSection.h"
#include "EditorMode/PCGEdMode.h"
#include "EditorMode/PCGEdModeToolkit.h"
#include "EditorMode/Tools/PCGInteractiveToolSettings.h"
#include "EditorMode/Tools/Helpers/PCGEdModeEditorUtilities.h"

#include "ToolMenus.h"
#include "Styling/SlateIconFinder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"

void SPCGToolPresetSection::Construct(const FArguments& InArgs, UEdMode& InOwningEditorMode)
{
	OwningEditorMode = &InOwningEditorMode;

	ThisContext.Reset(NewObject<UPCGToolPresetMenuContext>());
	ThisContext->ThisSection = SharedThis(this).ToWeakPtr();
	
	SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SPCGToolPresetSection::OnGetVisibility));
}

void SPCGToolPresetSection::UpdatePresets()
{
	Presets.Reset();

	if(OwningEditorMode.IsValid() == false)
	{
		return;
	}

	UPCGEditorMode* EdMode = CastChecked<UPCGEditorMode>(OwningEditorMode.Get());

	if(EdMode->GetToolkit().IsValid() == false)
	{
		return;
	}

	UPCGInteractiveToolSettings* ToolSettings = Cast<UPCGInteractiveToolSettings>(EdMode->GetCurrentToolSettings());

	if(ToolSettings == nullptr)
	{
		return;
	}
	TSharedPtr<FPCGEditorModeToolkit> Toolkit = StaticCastSharedPtr<FPCGEditorModeToolkit>(EdMode->GetToolkit().Pin());

	TSharedPtr<FPCGToolPalette> PCGToolPalette = Toolkit->GetActivePCGToolPalette();
	if(PCGToolPalette.IsValid() == false)
	{
		return;
	}

	FName ToolTag = ToolSettings->GetToolTag();

	if(ToolTag.IsNone())
	{
		return;
	}

	using namespace UE::PCG::EditorMode::Utility;

	TArray<FPCGGraphToolEditorData> GraphToolEditorData = GetGraphToolsWithToolTag(ToolTag);

	bool bUsesExistingActor = ToolSettings->UsesExistingActor();
	AActor* EditActor = ToolSettings->GetTypedWorkingActor<AActor>();

	Presets.Reserve(GraphToolEditorData.Num());
	for (const FPCGGraphToolEditorData& Candidate : GraphToolEditorData)
	{
		if (!Candidate.GraphToolData.bIsPreset)
		{
			continue;
		}
		
		// If our existing edit actor isn't compatible with the graph candidate, we skip it
		if (bUsesExistingActor && EditActor)
		{
			if (!IsPCGGraphInterfaceCompatibleWithActor(EditActor, Candidate.AssetData))
			{
				continue;
			}
		}
		
		FPresetItemPtr NewPreset = MakeShared<FPresetItem>();
		NewPreset->Item = TSoftObjectPtr<UPCGGraphInterface>(Candidate.AssetData.ToSoftObjectPath());
		NewPreset->Name = Candidate.GraphToolData.DisplayName.IsEmpty()
			? FText::AsCultureInvariant(Candidate.AssetData.AssetName.ToString())
			: Candidate.GraphToolData.DisplayName;
		NewPreset->Tooltip = Candidate.GraphToolData.Tooltip;
		Presets.Add(NewPreset);
	}

	ChildSlot
	[
		MakePresetToolbar().ToSharedRef()
	];
}

bool SPCGToolPresetSection::HasPresets() const
{
	return Presets.Num() > 0;
}

TSharedPtr<SWidget> SPCGToolPresetSection::MakePresetToolbar() const
{
	TSharedRef<FSlimHorizontalUniformToolBarBuilder> ToolBarBuilder = MakeShared<FSlimHorizontalUniformToolBarBuilder>(nullptr,
		FMultiBoxCustomization::None, nullptr, false);
	ToolBarBuilder->SetStyle(&FAppStyle::Get(), "SlimPaletteToolBar");

	static FName MenuName = "PCGToolPresetMenu";
	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

		UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(MenuName, NAME_None);
		ToolMenu->MenuType = EMultiBoxType::SlimHorizontalUniformToolBar;
		ToolMenu->StyleName = "SlimPaletteToolBar";
		ToolMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateStatic(&SPCGToolPresetSection::OnGeneratePresetMenu));
	}

	FToolMenuContext Context;
	Context.AddObject(ThisContext.Get());
	
	return UToolMenus::Get()->GenerateWidget(MenuName, Context);
}

void SPCGToolPresetSection::OnGeneratePresetMenu(UToolMenu* ToolMenu)
{
	const UPCGToolPresetMenuContext* MenuContext = ToolMenu->FindContext<UPCGToolPresetMenuContext>();

	if(!ensure(MenuContext && MenuContext->ThisSection.IsValid()))
	{
		return;
	}

	TSharedPtr<SPCGToolPresetSection> ThisSection = MenuContext->ThisSection.Pin();
	
	for(const FPresetItemPtr& PresetItemPtr : ThisSection->GetPresets())
	{
		FUIAction Action;
		Action.CanExecuteAction = FCanExecuteAction::CreateStatic(&SPCGToolPresetSection::CanActivatePreset, PresetItemPtr.ToWeakPtr(), MenuContext);
		Action.ExecuteAction = FExecuteAction::CreateStatic(&SPCGToolPresetSection::ActivatePreset, PresetItemPtr.ToWeakPtr(), MenuContext);
		Action.GetActionCheckState = FGetActionCheckState::CreateStatic(&SPCGToolPresetSection::IsPresetActive, PresetItemPtr.ToWeakPtr(), MenuContext);

		TAttribute<FText> TooltipAttribute = TAttribute<FText>::CreateStatic(&SPCGToolPresetSection::GetPresetTooltip, PresetItemPtr.ToWeakPtr(), MenuContext);

		ToolMenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitToolBarButton(FName(PresetItemPtr->Name.ToString()), Action, PresetItemPtr->Name, TooltipAttribute, FSlateIconFinder::FindIconForClass(UPCGGraph::StaticClass()), EUserInterfaceActionType::ToggleButton));
	}
}

bool SPCGToolPresetSection::CanActivatePreset(TWeakPtr<FPresetItem> PresetItemWeakPtr, const UPCGToolPresetMenuContext* Context)
{
	return PresetItemWeakPtr.IsValid();
}

void SPCGToolPresetSection::ActivatePreset(TWeakPtr<FPresetItem> PresetItemWeakPtr, const UPCGToolPresetMenuContext* Context)
{
	if (PresetItemWeakPtr.IsValid() == false || Context->ThisSection.IsValid() == false)
	{
		return;
	}

	UPCGEditorMode* EdMode = CastChecked<UPCGEditorMode>(Context->ThisSection.Pin()->OwningEditorMode.Get());

	if (UPCGInteractiveToolSettings* ToolSettings = Cast<UPCGInteractiveToolSettings>(EdMode->GetCurrentToolSettings()))
	{
		if(UPCGGraphInterface* Graph = PresetItemWeakPtr.Pin()->Item.LoadSynchronous())
		{
			ToolSettings->SetToolGraph(Graph);
		}
	}
}

FText SPCGToolPresetSection::GetPresetTooltip(TWeakPtr<FPresetItem> PresetItemWeakPtr, const UPCGToolPresetMenuContext* Context)
{
	if (PresetItemWeakPtr.IsValid() == false || Context->ThisSection.IsValid() == false)
	{
		return FText::GetEmpty();
	}

	return PresetItemWeakPtr.Pin()->Tooltip;
}

ECheckBoxState SPCGToolPresetSection::IsPresetActive(TWeakPtr<FPresetItem> PresetItemWeakPtr, const UPCGToolPresetMenuContext* Context)
{
	if (PresetItemWeakPtr.IsValid() == false || Context->ThisSection.IsValid() == false)
	{
		return ECheckBoxState::Undetermined;
	}
	
	TSharedPtr<FPresetItem> PresetItem = PresetItemWeakPtr.Pin();
	
	UPCGEditorMode* EdMode = CastChecked<UPCGEditorMode>(Context->ThisSection.Pin()->OwningEditorMode.Get());
	
	if (UPCGInteractiveToolSettings* ToolSettings = Cast<UPCGInteractiveToolSettings>(EdMode->GetCurrentToolSettings()))
	{
		if (UPCGGraphInterface* Graph = PresetItem->Item.LoadSynchronous())
		{
			return ToolSettings->GetToolGraph() == Graph ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	}

	return ECheckBoxState::Unchecked;
}

EVisibility SPCGToolPresetSection::OnGetVisibility() const
{
	if (bHidePresets)
	{
		return EVisibility::Collapsed;
	}

	return HasPresets() ? EVisibility::Visible : EVisibility::Collapsed;
}
