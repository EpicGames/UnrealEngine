// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"
#include "UObject/StrongObjectPtrTemplates.h"
#include "Misc/Optional.h"
#include "SPCGToolPresetSection.generated.h"

class UPCGGraphInterface;
class UEdMode;
class UToolMenu;

UCLASS()
class UPCGToolPresetMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<class SPCGToolPresetSection> ThisSection;	
};

class SPCGToolPresetSection : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGToolPresetSection)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdMode& InOwningEditorMode);

	void UpdatePresets();

	void HidePresets(bool bInHide) { bHidePresets = bInHide; }
	
	bool HasPresets() const;

	struct FPresetItem
	{
		TSoftObjectPtr<UPCGGraphInterface> Item;
		FText Name;
		FText Tooltip;
	};

	using FPresetItemPtr = TSharedPtr<FPresetItem, ESPMode::ThreadSafe>;
	
	const TArray<FPresetItemPtr>& GetPresets() const { return Presets; }
	
private:
	TSharedPtr<SWidget> MakePresetToolbar() const;
	static void OnGeneratePresetMenu(UToolMenu* ToolMenu);

	static bool CanActivatePreset(TWeakPtr<FPresetItem> PresetItem, const UPCGToolPresetMenuContext* Context);
	static void ActivatePreset(TWeakPtr<FPresetItem> PresetItem, const UPCGToolPresetMenuContext* Context);
	static FText GetPresetTooltip(TWeakPtr<FPresetItem> PresetItem, const UPCGToolPresetMenuContext* Context);
	static ECheckBoxState IsPresetActive(TWeakPtr<FPresetItem> PresetItem, const UPCGToolPresetMenuContext* Context);

	EVisibility OnGetVisibility() const;
private:
	TStrongObjectPtr<UPCGToolPresetMenuContext> ThisContext;
	
	TWeakObjectPtr<UEdMode> OwningEditorMode;
	TArray<FPresetItemPtr> Presets;
	
	bool bHidePresets = false;
};
