// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGBuilderSettingsFactory.h"
#include "WorldPartitionBuilder/PCGBuilderSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBuilderSettingsFactory)

UPCGBuilderSettingsFactory::UPCGBuilderSettingsFactory()
{
	SupportedClass = UPCGBuilderSettings::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UPCGBuilderSettingsFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UPCGBuilderSettings>(InParent, InClass, InName, Flags);
}

bool UPCGBuilderSettingsFactory::ShouldShowInNewMenu() const
{
	return true;
}
