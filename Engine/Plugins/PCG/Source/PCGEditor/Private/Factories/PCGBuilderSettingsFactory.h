// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "PCGBuilderSettingsFactory.generated.h"

UCLASS(hidecategories = Object)
class UPCGBuilderSettingsFactory : public UFactory
{
	GENERATED_BODY()

public:
	UPCGBuilderSettingsFactory();

	//~UFactory interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
};