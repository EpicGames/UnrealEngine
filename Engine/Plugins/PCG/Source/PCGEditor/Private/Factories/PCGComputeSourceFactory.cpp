// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGComputeSourceFactory.h"

#include "Compute/PCGComputeSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGComputeSourceFactory)

UPCGComputeSourceFactory::UPCGComputeSourceFactory()
{
	SupportedClass = UPCGComputeSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UPCGComputeSourceFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UPCGComputeSource>(InParent, InClass, InName, Flags);
}
