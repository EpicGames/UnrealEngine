// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "PCGComputeSourceFactory.generated.h"

UCLASS()
class UPCGComputeSourceFactory : public UFactory
{
	GENERATED_BODY()

public:
	UPCGComputeSourceFactory();

	//~Begin UFactory interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override { return true; }
	//~End UFactory interface
};
