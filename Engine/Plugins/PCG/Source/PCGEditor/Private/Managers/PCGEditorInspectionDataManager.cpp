// Copyright Epic Games, Inc. All Rights Reserved.

#include "Managers/PCGEditorInspectionDataManager.h"

#include "PCGEditorGraph.h"
#include "PCGGraphExecutionInspection.h"
#include "PCGGraphExecutionStateInterface.h"
#include "Nodes/PCGEditorGraphNodeBase.h"

bool FPCGEditorInspectionDataEntry::IsValid() const
{
	return !InspectionData.TaggedData.IsEmpty();
}

void FPCGEditorInspectionDataEntry::Clear()
{
	InspectionData = FPCGDataCollection{};
	Node.Reset();
	PinName = NAME_None;
	bIsOutputPin = true;
	Stack = FPCGStack{};
}

void FPCGEditorInspectionDataManager::Cleanup()
{
	if (PCGSourceBeingInspected.IsValid())
	{
		if (PCGSourceBeingInspected->GetExecutionState().GetInspection().IsInspecting())
		{
			PCGSourceBeingInspected->GetExecutionState().GetInspection().DisableInspection();
		}

		PCGSourceBeingInspected.Reset();
	}

	if (LastValidPCGSourceBeingInspected.IsValid())
	{
		if (LastValidPCGSourceBeingInspected->GetExecutionState().GetInspection().IsInspecting())
		{
			LastValidPCGSourceBeingInspected->GetExecutionState().GetInspection().DisableInspection();
		}

		LastValidPCGSourceBeingInspected.Reset();
	}

	for (FPCGEditorInspectionDataEntry& Entry : InspectionEntries)
	{
		Entry.Clear();
	}
}

void FPCGEditorInspectionDataManager::SetStackBeingInspected(const FPCGStack& FullStack)
{
	if (FullStack == StackBeingInspected)
	{
		// No-op if we're already inspecting this stack.
		return;
	}

	IPCGGraphExecutionSource* LastSource = LastValidPCGSourceBeingInspected.Get();
	IPCGGraphExecutionSource* NewSource = const_cast<IPCGGraphExecutionSource*>(FullStack.GetRootSource());

	if (NewSource && NewSource != LastSource)
	{
		if (LastSource && LastSource->GetExecutionState().GetInspection().IsInspecting())
		{
			LastSource->GetExecutionState().GetInspection().DisableInspection();
		}

		LastValidPCGSourceBeingInspected = NewSource;
	}

	PCGSourceBeingInspected = NewSource;

	StackBeingInspected = FullStack;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGEditor::SetStackBeingInspected::BroadcastStackBeingInspected);
		OnInspectedStackChangedDelegate.Broadcast(StackBeingInspected);
	}	
}

void FPCGEditorInspectionDataManager::OnSourceGenerated(IPCGGraphExecutionSource* InSource) const
{
	if (InSource == GetPCGSourceBeingInspected())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGEditor::OnSourceGenerated::BroadcastStackBeingInspected);
		OnInspectedStackChangedDelegate.Broadcast(StackBeingInspected);
	}
}

const TSharedPtr<FPCGInspectionData> FPCGEditorInspectionDataManager::GetInspectionDataPtr(const FPCGEditorInspectionDataEntrySetupParams& InParams) const
{
	FPCGStack Stack{};
	const UPCGPin* Pin = nullptr;

	const IPCGGraphExecutionSource* ExecutionSource = PCGSourceBeingInspected.Get();
	if (!ExecutionSource)
	{
		return nullptr;
	}

	if (!GetInspectionDataParams(InParams, Stack, Pin))
	{
		return nullptr;
	}

	return ExecutionSource->GetExecutionState().GetInspection().GetInspectionDataPtr(Stack);
}

bool FPCGEditorInspectionDataManager::InspectData(const FPCGEditorInspectionDataEntrySetupParams& InParams, TFunctionRef<void(const FPCGDataCollection&)> InspectFunc) const
{
	FPCGStack Stack{};
	const UPCGPin* Pin = nullptr;

	const IPCGGraphExecutionSource* ExecutionSource = PCGSourceBeingInspected.Get();
	if (!ExecutionSource)
	{
		return false;
	}

	if (!GetInspectionDataParams(InParams, Stack, Pin))
	{
		return false;
	}

	return ExecutionSource->GetExecutionState().GetInspection().InspectData(Stack, InspectFunc);
}

void FPCGEditorInspectionDataManager::SetupInspectionEntry(const FPCGEditorInspectionDataEntrySetupParams& InParams)
{
	ConstructEntry(InParams, InspectionEntries[InParams.EntryIndex]);
	OnInspectionEntryChangedDelegate.Broadcast(InspectionEntries[InParams.EntryIndex], InParams.EntryIndex);
}

const FPCGEditorInspectionDataEntry& FPCGEditorInspectionDataManager::GetInspectionEntry(int32 EntryIndex)
{
	check(EntryIndex >= 0 && EntryIndex < NumberOfEntries);
	return InspectionEntries[EntryIndex].IsValid() ? InspectionEntries[EntryIndex] : InspectionEntries[0];
}

void FPCGEditorInspectionDataManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (FPCGEditorInspectionDataEntry& Entry : InspectionEntries)
	{
		Entry.InspectionData.AddReferences(Collector);
	}
}

void FPCGEditorInspectionDataManager::ConstructEntry(const FPCGEditorInspectionDataEntrySetupParams& InParams, FPCGEditorInspectionDataEntry& OutEntry) const
{
	OutEntry = {};

	const IPCGGraphExecutionSource* ExecutionSource = PCGSourceBeingInspected.Get();
	if(!ExecutionSource)
	{
		return;
	}

	if (InParams.EntryIndex < 0
		|| InParams.EntryIndex >= InspectionEntries.Num()
		|| !InParams.Node
		|| StackBeingInspected.GetStackFrames().IsEmpty())
	{
		return;
	}

	FPCGStack Stack{};
	const UPCGPin* Pin = nullptr;
	
	if (!GetInspectionDataParams(InParams, Stack, Pin))
	{
		return;
	}

	check(Pin);

	const bool bFoundData = ExecutionSource->GetExecutionState().GetInspection().InspectData(Stack, [&OutEntry](const FPCGDataCollection& InspectionData)
	{
		OutEntry.InspectionData = InspectionData;
	});

	if (bFoundData)
	{
		OutEntry.Node = InParams.Node;
		OutEntry.PinName = Pin->Properties.Label;
		OutEntry.bIsOutputPin = InParams.bIsOutputPin;
		OutEntry.Stack = MoveTemp(Stack);
	}
}

bool FPCGEditorInspectionDataManager::GetInspectionDataParams(const FPCGEditorInspectionDataEntrySetupParams& InParams, FPCGStack& OutStack, UPCGPin const*& OutPin) const
{
	const UPCGNode* PCGNode = InParams.Node ? InParams.Node->GetPCGNode() : nullptr;
	if (!PCGNode)
	{
		return false;
	}

	const TArray<TObjectPtr<UPCGPin>>& Pins = InParams.bIsOutputPin ? PCGNode->GetOutputPins() : PCGNode->GetInputPins();
	int32 PinIndex = InParams.PinIndex;
	if (PinIndex == INDEX_NONE)
	{
		PinIndex = InParams.PinName == NAME_None ? 0 : Pins.IndexOfByPredicate([PinName = InParams.PinName](const UPCGPin* InPin) { return InPin && InPin->Properties.Label == PinName; });
	}
	
	if (!Pins.IsValidIndex(PinIndex))
	{
		return false;
	}

	const UPCGPin* Pin = Pins[PinIndex];

	PCGEditorGraphUtils::GetInspectablePin(PCGNode, Pin, PCGNode, OutPin);

	if (!PCGNode || !OutPin)
	{
		return false;
	}

	// Create a temporary stack with Node+Pin to query the exact DataCollection we are inspecting
	OutStack = StackBeingInspected;
	TArray<FPCGStackFrame>& StackFrames = OutStack.GetStackFramesMutable();
	StackFrames.Reserve(StackFrames.Num() + 2);
	StackFrames.Emplace(PCGNode);
	StackFrames.Emplace(OutPin);

	return true;
}
