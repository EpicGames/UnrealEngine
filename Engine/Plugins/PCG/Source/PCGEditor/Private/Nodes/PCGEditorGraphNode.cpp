// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/PCGEditorGraphNode.h"

#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "PCGSubsystem.h"

#include "PCGEditorModule.h"
#include "PCGEditorSettings.h"

#include "GraphEditorActions.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "Framework/Commands/GenericCommands.h"

#include "SPCGEditorGraphNode.h"
#include "SPCGEditorGraphNodeCompact.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorGraphNode)

#define LOCTEXT_NAMESPACE "PCGEditorGraphNode"

void UPCGEditorGraphNode::Construct(UPCGNode* InPCGNode)
{
	Super::Construct(InPCGNode);

	const UPCGSettings* Settings = InPCGNode->GetSettings();
	bCanRenameNode = Settings && Settings->CanUserEditTitle();
}

FText UPCGEditorGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (!PCGNode)
	{
		return NSLOCTEXT("PCGEditorGraphNode", "UnnamedNodeTitle", "Unnamed Node");
	}

	if (TitleType == ENodeTitleType::FullTitle)
	{
		return PCGNode->GetNodeTitle(EPCGNodeTitleType::FullTitle);
	}
	else if (TitleType == ENodeTitleType::MenuTitle)
	{
		return FText::FromName(PCGNode->NodeTitle);
	}
	else
	{
		return PCGNode->GetNodeTitle(EPCGNodeTitleType::ListView);
	}
}

void UPCGEditorGraphNode::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->Node)
	{
		return;
	}

	Super::GetNodeContextMenuActions(Menu, Context);

	// General node actions
	if (GetDefault<UPCGEditorSettings>()->bShowNodeGeneralActionsRightClickContextMenu)
	{
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaGeneral", LOCTEXT("GeneralHeader", "General"));
		Section.AddMenuEntry(FGenericCommands::Get().Delete);
		Section.AddMenuEntry(FGenericCommands::Get().Cut);
		Section.AddMenuEntry(FGenericCommands::Get().Copy);
		Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
	}
}

void UPCGEditorGraphNode::AllocateDefaultPins()
{
	if (PCGNode)
	{
		CreatePins(PCGNode->GetInputPins(), PCGNode->GetOutputPins());
	}
}

TSharedPtr<SGraphNode> UPCGEditorGraphNode::CreateVisualWidget()
{
	return ShouldDrawCompact() ? SNew(SPCGEditorGraphNodeCompact, this) : SNew(SPCGEditorGraphNode, this);
}

bool UPCGEditorGraphNode::ShouldDrawCompact() const
{
	const UPCGSettings* Settings = GetSettings();
	return Settings && Settings->ShouldDrawNodeCompact();
}

bool UPCGEditorGraphNode::GetCompactNodeIcon(FName& OutCompactNodeIcon) const
{
	const UPCGSettings* Settings = GetSettings();
	return Settings && Settings->GetCompactNodeIcon(OutCompactNodeIcon);
}

void UPCGEditorGraphNode::OnRenameNode(const FString& NewName)
{
	check(PCGNode);

	const FName Name = FName(NewName);
	if (PCGNode->NodeTitle != Name)
	{
		if (PCGNode->SetNodeTitle(Name))
		{
			Modify();
		}
	}
}

bool UPCGEditorGraphNode::OnValidateNodeTitle(const FText& NewName, FText& OutErrorMessage)
{
	if (NewName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("InvalidNodeTitleEmptyName", "Empty name");
		return false;
	}

	if (NewName.ToString().Len() > MaxNodeNameCharacterCount)
	{
		OutErrorMessage = LOCTEXT("InvalidNodeTitleTooLong", "Name too long");
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
