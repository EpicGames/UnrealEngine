// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/PCGEditorGraphNodeBase.h"

#include "PCGDynamicPins.h"
#include "PCGEdge.h"
#include "PCGEngineSettings.h"
#include "PCGGraph.h"
#include "PCGGraphExecutionInspection.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGPin.h"
#include "PCGSettingsWithDynamicInputs.h"
#include "PCGSubsystem.h"
#include "PCGWorldActor.h"
#include "Editor/IPCGEditorModule.h"
#include "Elements/PCGHiGenGridSize.h"
#include "Elements/PCGReroute.h"
#include "Graph/DataOverride/PCGDataOverrideHelpers.h"
#include "Helpers/PCGMetadataHelpers.h"
#include "Metadata/PCGDefaultValueInterface.h"

#include "PCGEditor.h"
#include "PCGEditorCommands.h"
#include "PCGEditorCommon.h"
#include "PCGEditorGraph.h"
#include "PCGEditorSettings.h"
#include "PCGEditorStyle.h"
#include "AssetEditorMode/PCGAssetEditorToolRegistry.h"
#include "Schema/PCGEditorGraphSchema.h"

#include "CoreGlobals.h"
#include "DiffResults.h"
#include "GraphEditorActions.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/TransactionObjectEvent.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include "Widgets/Colors/SColorPicker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorGraphNodeBase)

#define LOCTEXT_NAMESPACE "PCGEditorGraphNodeBase"

namespace PCGEditorGraphSwitches
{
	TAutoConsoleVariable<bool> CVarCheckConnectionCycles{
		TEXT("pcg.Editor.CheckConnectionCycles"),
		true,
		TEXT("Prevents user from creating cycles in graph")
	};
}

namespace PCGEditorGraphNodeBase
{
	IPCGSettingsDefaultValueProvider* GetDefaultValueInterface(UPCGSettings* InSettings)
	{
		return (InSettings && InSettings->Implements<UPCGSettingsDefaultValueProvider>()) ? CastChecked<IPCGSettingsDefaultValueProvider>(InSettings) : nullptr;
	}

	IPCGDynamicPinsProvider* GetDynamicPinsInterface(UPCGSettings* InSettings)
	{
		return (InSettings && InSettings->Implements<UPCGDynamicPinsProvider>()) ? CastChecked<IPCGDynamicPinsProvider>(InSettings) : nullptr;
	}

	/** Whether this node was culled during graph compilation or during graph execution. */
	bool ShouldDisplayAsActive(const UPCGEditorGraphNodeBase* InNode, const IPCGGraphExecutionSource* InSourceBeingDebugged, const FPCGStack* InStackBeingInspected)
	{
		if (!InNode)
		{
			return true;
		}

		const UPCGNode* PCGNode = InNode->GetPCGNode();
		if (!PCGNode)
		{
			return true;
		}

		// Don't display as culled while component is executing or about to refresh as nodes will flash to culled state and back
		// which looks disturbing.
		if (!InSourceBeingDebugged || InSourceBeingDebugged->GetExecutionState().IsGenerating() || InSourceBeingDebugged->GetExecutionState().IsRefreshInProgress())
		{
			return true;
		}

		const UPCGEngineSettings* EngineSettings = GetDefault<UPCGEngineSettings>();
		const bool bActiveVisualizationEnabled = !ensure(EngineSettings) || EngineSettings->bDisplayCullingStateWhenDebugging;
		const UPCGSettings* Settings = PCGNode ? PCGNode->GetSettings() : nullptr;

		// Display whether node was culled dynamically or statically.
		if (InStackBeingInspected && bActiveVisualizationEnabled)
		{
			if (!Settings || !Settings->IsA<UPCGRerouteSettings>())
			{
				// Task will be displayed as active if it was executed.
				return InSourceBeingDebugged->GetExecutionState().GetInspection().WasNodeExecuted(PCGNode, *InStackBeingInspected);
			}
			else
			{
				// Named reroute usages mirror the enabled state of the upstream declaration.
				if (const UPCGNamedRerouteUsageSettings* RerouteUsageSettings = Cast<UPCGNamedRerouteUsageSettings>(Settings))
				{
					const UPCGNode* DeclarationPCGNode = RerouteUsageSettings->Declaration ? Cast<UPCGNode>(RerouteUsageSettings->Declaration->GetOuter()) : nullptr;
					const UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(InNode->GetOuter());
					const UPCGEditorGraphNodeBase* DeclarationNode = EditorGraph && DeclarationPCGNode ? EditorGraph->GetEditorNodeFromPCGNode(DeclarationPCGNode) : nullptr;
					return !DeclarationNode || ShouldDisplayAsActive(DeclarationNode, InSourceBeingDebugged, InStackBeingInspected);
				}

				// Special case - reroute culled state is evaluated here based on upstream connections. Reroutes are always culled/never executed, but still need
				// to reflect the active/inactive state to not look wrong/confusing.
				for (const UEdGraphPin* Pin : InNode->Pins)
				{
					if (Pin && Pin->Direction == EEdGraphPinDirection::EGPD_Input)
					{
						for (const UEdGraphPin* LinkedPin : Pin->LinkedTo)
						{
							if (const UPCGEditorGraphNodeBase* UpstreamNode = LinkedPin ? Cast<UPCGEditorGraphNodeBase>(LinkedPin->GetOwningNode()) : nullptr)
							{
								const bool bUpstreamNodeActive = ShouldDisplayAsActive(UpstreamNode, InSourceBeingDebugged, InStackBeingInspected);
								const bool bUpstreamPinActive = UpstreamNode->IsOutputPinActive(LinkedPin);

								if (bUpstreamNodeActive && bUpstreamPinActive)
								{
									// Active if any input is active.
									return true;
								}
							}
						}
					}
				}

				return false;
			}
		}

		return true;
	}

	// @todo_pcg: Factorize the storage collection iteration into a templated ForEachStorageCollection helper.
	bool NodeHasDataOverrides(const UPCGNode* InNode, const IPCGGraphExecutionSource* InExecutionSource)
	{
		const FPCGSourceDataContainer* DataContainer = PCG::DataOverride::Helpers::GetSourceDataContainer(InExecutionSource);
		if (!InNode || !DataContainer)
		{
			return false;
		}

		const TArray<FPCGSourceDataStorageKey> StorageKeys = PCG::DataOverride::Helpers::CollectNodeStorageKeys(InNode, InExecutionSource);
		for (const FPCGSourceDataStorageKey& StorageKey : StorageKeys)
		{
			const FConstSharedStruct SharedStruct = DataContainer->Get<FPCGDeltaCollection>(StorageKey);
			const FPCGDeltaCollection* DeltaCollection = SharedStruct.GetPtr<const FPCGDeltaCollection>();
			if (DeltaCollection && !DeltaCollection->IsEmpty())
			{
				return true;
			}
		}
		return false;
	}

	bool NodeHasOrphanedDeltas(const UPCGNode* InNode, const IPCGGraphExecutionSource* InExecutionSource)
	{
		const FPCGSourceDataContainer* DataContainer = PCG::DataOverride::Helpers::GetSourceDataContainer(InExecutionSource);
		if (!InNode || !DataContainer)
		{
			return false;
		}

		const FPCGGraphExecutionInspection& Inspection = InExecutionSource->GetExecutionState().GetInspection();

		const TArray<FPCGSourceDataStorageKey> StorageKeys = PCG::DataOverride::Helpers::CollectNodeStorageKeys(InNode, InExecutionSource);
		for (const FPCGSourceDataStorageKey& StorageKey : StorageKeys)
		{
			const FConstSharedStruct SharedStruct = DataContainer->Get<FPCGDeltaCollection>(StorageKey);
			const FPCGDeltaCollection* DeltaCollection = SharedStruct.GetPtr<const FPCGDeltaCollection>();
			if (!DeltaCollection || DeltaCollection->IsEmpty())
			{
				continue;
			}

			bool bFoundOrphan = false;
			DeltaCollection->ForEachDelta(
				[&Inspection, &StorageKey, &bFoundOrphan](const FPCGDeltaKey& DeltaKey, const TInstancedStruct<FPCGDeltaBase>& /*Delta*/) -> bool
				{
					if (!Inspection.IsDeltaResolved(StorageKey, DeltaKey))
					{
						bFoundOrphan = true;
						return false;
					}
					return true;
				});

			if (bFoundOrphan)
			{
				return true;
			}
		}
		return false;
	}

	namespace Constants
	{
		FName NodeConversionActionName = TEXT("NodeConversion");

		FText NodeConversionActionLabel = LOCTEXT("NodeConversion", "Convert Node");
		FText NodeConversionActionTooltip = LOCTEXT("NodeConversionTooltip", "Convert a single node into a different node, or otherwise compatible output.");
		FTextFormat ConvertToFormat = LOCTEXT("ConvertTo", "Convert to {0}");

		FText ConversionHeaderLabel = LOCTEXT("ConvertHeader", "Convert To");
		FText OrganizationHeaderLabel = LOCTEXT("OrganizationHeader", "Organization");
		FText DeterminismHeaderLabel = LOCTEXT("DeterminismHeader", "Determinism");
	}
}

UPCGEditorGraphNodeBase::UPCGEditorGraphNodeBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRenameNode = false;
}

void UPCGEditorGraphNodeBase::InitNodeGuid()
{
	if (PCGNode)
	{
		// Assign a deterministic NodeGuid based on the underlying UPCGNode's name.
		// UPCGNode names are serialized and stable across revisions, which allows
		// FGraphDiffControl to reliably match editor nodes when diffing two revisions.
		NodeGuid = FGuid::NewDeterministicGuid(PCGNode->GetName());
	}
}

void UPCGEditorGraphNodeBase::FindDiffs(UEdGraphNode* OtherNode, FDiffResults& Results)
{
	Super::FindDiffs(OtherNode, Results);

	// Compare the underlying UPCGSettings between matched nodes
	const UPCGEditorGraphNodeBase* OtherPCGNode = Cast<UPCGEditorGraphNodeBase>(OtherNode);
	if (!OtherPCGNode)
	{
		return;
	}

	const UPCGSettings* SettingsA = GetSettings();
	const UPCGSettings* SettingsB = OtherPCGNode->GetSettings();

	if (SettingsA && SettingsB)
	{
		FDiffSingleResult Diff;
		Diff.Diff = EDiffType::NODE_PROPERTY;
		Diff.Node1 = this;
		Diff.Node2 = OtherNode;
		Diff.ToolTip = LOCTEXT("DIF_PCGSettingsPropertyToolTip", "A PCG Settings property has changed");
		Diff.Category = EDiffType::MODIFICATION;

		DiffProperties(SettingsA->GetClass(), SettingsB->GetClass(), const_cast<UPCGSettings*>(SettingsA), const_cast<UPCGSettings*>(SettingsB), Results, Diff);
	}
}

void UPCGEditorGraphNodeBase::Construct(UPCGNode* InPCGNode)
{
	check(InPCGNode);
	PCGNode = InPCGNode;
	InPCGNode->OnNodeChangedDelegate.AddUObject(this, &UPCGEditorGraphNodeBase::OnNodeChanged);

	NodePosX = InPCGNode->PositionX;
	NodePosY = InPCGNode->PositionY;
	NodeComment = InPCGNode->NodeComment;
	bCommentBubblePinned = InPCGNode->bCommentBubblePinned;
	bCommentBubbleVisible = InPCGNode->bCommentBubbleVisible;

	if (const UPCGSettingsInterface* PCGSettingsInterface = InPCGNode->GetSettingsInterface())
	{
		const ENodeEnabledState NewEnabledState = !PCGSettingsInterface->bEnabled ? ENodeEnabledState::Disabled : ENodeEnabledState::Enabled;
		SetEnabledState(NewEnabledState);
	}
	
	InitNodeGuid();

	// Update to current graph/inspection state.
	const UPCGEditorGraph* Graph = Cast<UPCGEditorGraph>(GetOuter());
	const FPCGEditor* Editor = Graph ? Graph->GetEditor().Pin().Get() : nullptr;
	UpdateStructuralVisualization(Editor ? Editor->GetPCGSourceBeingInspected() : nullptr, Editor ? Editor->GetStackBeingInspected() : nullptr, /*bNewlyPlaced=*/true);
	UpdateGPUVisualization(Editor ? Editor->GetPCGSourceBeingInspected() : nullptr, Editor ? Editor->GetStackBeingInspected() : nullptr);
}

void UPCGEditorGraphNodeBase::BeginDestroy()
{
	if (PCGNode)
	{
		PCGNode->OnNodeChangedDelegate.RemoveAll(this);
	}

	Super::BeginDestroy();
}

FLinearColor UPCGEditorGraphNodeBase::GetCachedPinColor(const UPCGPin* PCGPin) const
{
	if (!PCGPin)
	{
		return GetDefault<UPCGEditorSettings>()->DefaultPinColor;
	}

	if (const FLinearColor* Cached = CachedPinColors.Find(PCGPin))
	{
		return *Cached;
	}

	const FLinearColor Color = FPCGModule::GetConstDataTypeRegistry().GetPinColor(PCGPin->GetCurrentTypesID());
	CachedPinColors.Add(PCGPin, Color);

	return Color;
}

void UPCGEditorGraphNodeBase::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	TArray<FName> PropertiesChanged = TransactionEvent.GetChangedProperties();

	if (PropertiesChanged.Contains(TEXT("bCommentBubblePinned")))
	{
		UpdateCommentBubblePinned();
	}

	if (PropertiesChanged.Contains(TEXT("NodePosX")) || PropertiesChanged.Contains(TEXT("NodePosY")))
	{
		UpdatePosition();
	}
}

void UPCGEditorGraphNodeBase::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGInlineConstantDefaultValues)
	{
		bHasEverBeenConnected = true;
	}
}

void UPCGEditorGraphNodeBase::GetNodeContextMenuActions(UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	using namespace PCGEditorGraphNodeBase::Constants;

	if (!Context->Node)
	{
		return;
	}

	// Local pin special actions should come first.
	if (Context->Pin)
	{
		const bool bIsInputPin = Context->Pin->Direction == EGPD_Input;
		const EPCGPinDirection PinDirection = bIsInputPin ? EPCGPinDirection::Input : EPCGPinDirection::Output;
		const bool bDynamicPinSubMenu = CanUserAddRemoveDynamicPins(PinDirection);
		const bool bCanRenameDynamicPin = CanUserRenameDynamicPin(const_cast<UEdGraphPin*>(Context->Pin));
		const bool bDefaultValueSubMenu = !Context->Pin->HasAnyConnections() && IsSettingsDefaultValuesEnabled();

		if (bDynamicPinSubMenu || bCanRenameDynamicPin || bDefaultValueSubMenu)
		{
			FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaPinActions", LOCTEXT("PinActionsMenuHeader", "Pin Actions"));

			if (bDynamicPinSubMenu)
			{
				Section.AddMenuEntry(bIsInputPin ? FPCGEditorCommands::Get().AddInputPin : FPCGEditorCommands::Get().AddOutputPin);

				const FText RemoveLabel = bIsInputPin ? LOCTEXT("RemoveInputPin", "Remove Input Pin") : LOCTEXT("RemoveOutputPin", "Remove Output Pin");
				const FText RemoveTooltip = bIsInputPin ? LOCTEXT("RemoveInputPinTooltip", "Remove this input pin from the current node") : LOCTEXT("RemoveOutputPinTooltip", "Remove this output pin from the current node");

				Section.AddMenuEntry("RemovePin",
					RemoveLabel,
					RemoveTooltip,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([Pin = Context->Pin, PinDirection, this]
						{
							const_cast<UPCGEditorGraphNodeBase*>(this)->OnUserRemoveDynamicPin(const_cast<UEdGraphPin*>(Pin), PinDirection);
						}),
						FCanExecuteAction::CreateLambda([Pin = Context->Pin, PinDirection, this]
						{
							return this->CanUserRemoveDynamicPin(const_cast<UEdGraphPin*>(Pin), PinDirection);
						})));
			}

			if (bCanRenameDynamicPin)
			{
				Section.AddMenuEntry("RenamePin",
					LOCTEXT("RenamePin", "Rename Pin"),
					LOCTEXT("RenamePinTooltip", "Rename this pin"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([Pin = Context->Pin, this]
					{
						OnRequestPinRenameDelegate.ExecuteIfBound(const_cast<UEdGraphPin*>(Pin));
					})));
			}

			// Default value inline constants
			if (bDefaultValueSubMenu)
			{
				const FName PinLabel = Context->Pin->PinName;
				const bool bIsActive = IsPinDefaultValueActivated(PinLabel);

				Section.AddMenuEntry(TEXT("ActivateInlineConstantPin"),
					bIsActive
						? LOCTEXT("DeactivateInlineConstantPin", "Deactivate Inline Constant")
						: LOCTEXT("ActivateInlineConstantPin", "Activate Inline Constant"),
					FText(),
					FSlateIcon(),
					FUIAction(
							FExecuteAction::CreateLambda([this, PinLabel, bIsActive] { OnUserSetPinDefaultValueActivated(PinLabel, !bIsActive); }),
							FCanExecuteAction::CreateLambda([this, Pin = Context->Pin] { return IsPinDefaultValueEnabled(Pin->PinName) && !Pin->HasAnyConnections(); })));

				const UEnum* EnumPtr = StaticEnum<EPCGMetadataTypes>();
				for (int32 I = 0; I < EnumPtr->NumEnums() - 1; ++I)
				{
					if (!EnumPtr->GetMetaData(TEXT("Hidden"), I).IsEmpty())
					{
						continue;
					}

					EPCGMetadataTypes DataType = static_cast<EPCGMetadataTypes>(EnumPtr->GetValueByIndex(I));
					if (bIsActive && CanConvertToDefaultValueMetadataType(PinLabel, DataType))
					{
						const FName EntryName = FName(FString(TEXT("ConvertPinType")) + EnumPtr->GetAuthoredNameStringByIndex(I));
						Section.AddMenuEntry(EntryName,
							FText::Format(ConvertToFormat, EnumPtr->GetDisplayNameTextByIndex(I)),
							LOCTEXT("ConvertPinTypeTooltip", "Convert this pin's inline constant to a different type."),
							FSlateIcon(),
							FUIAction(
									FExecuteAction::CreateLambda([this, PinLabel, DataType] { ConvertPinDefaultValueMetadataType(PinLabel, DataType); }),
									FCanExecuteAction::CreateLambda([this, Pin = Context->Pin] { return IsPinDefaultValueEnabled(Pin->PinName) && !Pin->HasAnyConnections(); })));
					}
				}

				Section.AddMenuEntry(
					TEXT("ResetInlineConstantPin"),
					LOCTEXT("ResetInlineConstantPinLabel", "Reset Inline Constant Value"),
					LOCTEXT("ResetInlineConstantPinTooltip", "Reset the inline constant to its default value."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this, Pin = Context->Pin]
						{
							OnUserResetPinDefaultValue(Pin->PinName, const_cast<UEdGraphPin*>(Pin));
						}),
						FCanExecuteAction::CreateLambda([this, Pin = Context->Pin] { return CanResetPinDefaultValue(Pin->PinName) && !Pin->HasAnyConnections(); })));
			}
		}
	}

	// Node special actions should come after pin actions
	{
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaNodeActions", LOCTEXT("NodeActionsHeader", "Node Actions"));

		const UPCGSettings* Settings = PCGNode->GetSettings();

		// Special nodes operations
		if (PCGNode && Settings)
		{
			if (Settings->IsA<UPCGNamedRerouteDeclarationSettings>())
			{
				Section.AddMenuEntry(FPCGEditorCommands::Get().SelectNamedRerouteUsages);
			}
			else if (Settings->IsA<UPCGNamedRerouteUsageSettings>())
			{
				Section.AddMenuEntry(FPCGEditorCommands::Get().SelectNamedRerouteDeclaration);
			}

			// Operation to convert a node in place into something else, such as a different node.
			TArray<FPCGPreconfiguredInfo> ConversionInfo = Settings->GetConversionInfo();
			if (!ConversionInfo.IsEmpty())
			{
				const int32 NumConversions = ConversionInfo.Num();

				auto AddEntries = [this, &Section, ConversionInfo = MoveTemp(ConversionInfo)](UToolMenu* AlignmentMenu)
				{
					FToolMenuSection& SubSection = (ConversionInfo.Num() > 1) ? AlignmentMenu->AddSection("EdGraphSchemaConversion", ConversionHeaderLabel) : Section;

					for (const FPCGPreconfiguredInfo& Conversion : ConversionInfo)
					{
						const FText Label = (ConversionInfo.Num() > 1) ? Conversion.Label : FText::Format(ConvertToFormat, Conversion.Label);
						const FText Tooltip = Conversion.Tooltip;
						FUIAction Action(FExecuteAction::CreateLambda([this, Conversion = std::move(Conversion)] { const_cast<UPCGEditorGraphNodeBase*>(this)->OnConvertNode(Conversion); }));
						// TODO: UX feedback (read-only or greyed out font, etc) if the conversion can't happen.
						SubSection.AddMenuEntry(FName(Label.ToString()), Label, std::move(Tooltip), FSlateIcon(), std::move(Action));
					}
				};

				if (NumConversions == 1) // Single conversion, just add it to the action list
				{
					AddEntries(Menu);
				}
				else // Add the entries to a submenu
				{
					Section.AddSubMenu(NodeConversionActionName, NodeConversionActionLabel, NodeConversionActionTooltip, FNewToolMenuDelegate::CreateLambda(AddEntries));
				}
			}
		}

		// General PCG node actions
		Section.AddMenuEntry(FPCGEditorCommands::Get().ToggleEnabled, LOCTEXT("ToggleEnabledLabel", "Enable"));
		Section.AddMenuEntry(FPCGEditorCommands::Get().RenameNode, LOCTEXT("RenameNode", "Rename"));
		Section.AddMenuEntry(FPCGEditorCommands::Get().ToggleDebug, LOCTEXT("ToggleDebugLabel", "Debug"));
		Section.AddMenuEntry(FPCGEditorCommands::Get().DebugOnlySelected);
		Section.AddMenuEntry(FPCGEditorCommands::Get().DisableDebugOnAllNodes);
		Section.AddMenuEntry(FPCGEditorCommands::Get().ToggleInspect, LOCTEXT("ToggleinspectionLabel", "Inspect"));
		Section.AddMenuEntry(FPCGEditorCommands::Get().EditInViewport);
		Section.AddMenuEntry(FPCGEditorCommands::Get().MarkForViewportEditing, LOCTEXT("MarkForViewportEditingLabel", "Mark for Edit"));
		Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
		Section.AddMenuEntry(FPCGEditorCommands::Get().ExportNodes, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "ClassIcon.PCGSettings"));
		
		auto CollapseEntries = [this](UToolMenu* CollapseMenu)
		{
			FToolMenuSection& CollapseSubSection = CollapseMenu->AddSection(TEXT("CollapseSection"));
			CollapseSubSection.AddMenuEntry(FPCGEditorCommands::Get().CollapseNodes);
			CollapseSubSection.AddMenuEntry(FPCGEditorCommands::Get().CollapseNodesToEmbeddedSubgraph);
		};

		Section.AddSubMenu(TEXT("CollapseSubMenu"), LOCTEXT("CollapseSubMenuLabel", "Collapse Nodes"), FText(), FNewToolMenuDelegate::CreateLambda(CollapseEntries));

		Section.AddMenuEntry(FPCGEditorCommands::Get().ConvertToStandaloneNodes);
		Section.AddMenuEntry(FPCGEditorCommands::Get().JumpToSource);
	}

	// Organizational actions
	if (GetDefault<UPCGEditorSettings>()->bShowNodeOrganizationalActionsRightClickContextMenu)
	{
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaOrganization", OrganizationHeaderLabel);
		Section.AddMenuEntry(
			"PCGNode_SetColor",
			LOCTEXT("PCGNode_SetColor", "Set Node Color"),
			LOCTEXT("PCGNode_SetColorTooltip", "Sets a specific color on the given node. Note that white maps to the default value"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ColorPicker.Mode"),
			FUIAction(FExecuteAction::CreateUObject(const_cast<UPCGEditorGraphNodeBase*>(this), &UPCGEditorGraphNodeBase::OnPickColor),
				FCanExecuteAction::CreateUObject(const_cast<UPCGEditorGraphNodeBase*>(this), &UPCGEditorGraphNodeBase::CanPickColor)));

		Section.AddSubMenu("Alignment", LOCTEXT("AlignmentHeader", "Alignment"), FText(), FNewToolMenuDelegate::CreateLambda([](UToolMenu* AlignmentMenu)
		{
			{
				FToolMenuSection& SubSection = AlignmentMenu->AddSection("EdGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
				SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
				SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
				SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
				SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
				SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
				SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
				SubSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
			}

			{
				FToolMenuSection& SubSection = AlignmentMenu->AddSection("EdGraphSchemaDistribution", LOCTEXT("DistributionHeader", "Distribution"));
				SubSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
				SubSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
			}
		}));
	}

	// PCG Determinism actions
	if (GetDefault<UPCGEditorSettings>()->bShowNodeDeterminismActionsRightClickContext)
	{
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaDeterminism", DeterminismHeaderLabel);
		Section.AddMenuEntry(FPCGEditorCommands::Get().RunDeterminismNodeTest,
			LOCTEXT("Determinism_RunTest", "Validate Determinism on Selection"),
			LOCTEXT("Determinism_RunTestToolTip", "Run a test to validate the selected nodes for determinism."));
	}

	// Comment Group is the final section
	{
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaCommentGroup", LOCTEXT("CommentGroupHeader", "Comment Group"));
		Section.AddMenuEntry(FGraphEditorCommands::Get().CreateComment,
			LOCTEXT("MultiCommentDesc", "Create Comment from Selection"),
			LOCTEXT("CommentToolTip", "Create a resizable comment box around selection."));
	}
}

void UPCGEditorGraphNodeBase::AutowireNewNode(UEdGraphPin* FromPin)
{
	if (PCGNode == nullptr || FromPin == nullptr)
	{
		return;
	}

	const bool bFromPinIsInput = FromPin->Direction == EEdGraphPinDirection::EGPD_Input;
	const TArray<TObjectPtr<UPCGPin>>& OtherPinsList = bFromPinIsInput ? PCGNode->GetOutputPins() : PCGNode->GetInputPins();

	// Try to connect to the first compatible pin
	for (const TObjectPtr<UPCGPin>& OtherPin : OtherPinsList)
	{
		check(OtherPin);

		// TODO: Allow autoconnecting output dependency pins to input dependency pins.
		if (OtherPin->Properties.IsAdvancedPin() || OtherPin->Properties.IsDatalessPin())
		{
			continue;
		}

		const FName& OtherPinName = OtherPin->Properties.Label;
		UEdGraphPin* ToPin = FindPinChecked(OtherPinName, bFromPinIsInput ? EEdGraphPinDirection::EGPD_Output : EEdGraphPinDirection::EGPD_Input);
		if (ToPin && GetSchema()->TryCreateConnection(FromPin, ToPin))
		{
			// Connection succeeded
			break;
		}
	}

	NodeConnectionListChanged();
}

void UPCGEditorGraphNodeBase::PrepareForCopying()
{
	if (PCGNode)
	{
		// Temporarily take ownership of the MaterialExpression, so that it is not deleted when cutting
		PCGNode->Rename(nullptr, this, REN_DontCreateRedirectors | REN_DoNotDirty);
	}
}

bool UPCGEditorGraphNodeBase::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Schema->IsA(UPCGEditorGraphSchema::StaticClass());
}

void UPCGEditorGraphNodeBase::PostCopy()
{
	if (PCGNode)
	{
		UPCGEditorGraph* PCGEditorGraph = CastChecked<UPCGEditorGraph>(GetGraph());
		UPCGGraph* PCGGraph = PCGEditorGraph->GetPCGGraph();
		check(PCGGraph);
		PCGNode->Rename(nullptr, PCGGraph, REN_DontCreateRedirectors | REN_DoNotDirty);
	}
}

void UPCGEditorGraphNodeBase::PostPasteNode()
{
	bDisableReconstructFromNode = true;
}

void UPCGEditorGraphNodeBase::RebuildAfterPaste()
{
	if (PCGNode)
	{
		PCGNode->RebuildAfterPaste();

		RebuildEdgesFromPins();

		PCGNode->OnNodeChangedDelegate.AddUObject(this, &UPCGEditorGraphNodeBase::OnNodeChanged);
		PCGNode->PositionX = NodePosX;
		PCGNode->PositionY = NodePosY;

		// Refresh the node if it has dynamic pins
		if (const UPCGSettings* Settings = PCGNode->GetSettings())
		{
			if (Settings->HasDynamicPins())
			{
				PCGNode->OnNodeChangedDelegate.Broadcast(PCGNode, EPCGChangeType::Node);
			}
		}
	}
}

void UPCGEditorGraphNodeBase::PostPaste()
{
	bDisableReconstructFromNode = false;
}

void UPCGEditorGraphNodeBase::SetInspected(bool bInIsInspecting)
{
	if (UPCGSettings* Settings = PCGNode ? PCGNode->GetSettings() : nullptr; ensure(Settings))
	{
		const bool bWasInspectingBefore = Settings->bIsInspecting;

		Settings->bIsInspecting = bInIsInspecting;

		if (!Settings->bEnabled)
		{
			return;
		}

		const bool bStartInspection = !bWasInspectingBefore && bInIsInspecting;
		const bool bStopInspection = bWasInspectingBefore && !bInIsInspecting;

		if (IsInteractiveNode())
		{
			if (bStartInspection && !Settings->IsNodeToolActive() && Settings->GetNodeToolStartBehaviour() == EPCGNodeToolStartBehavior::OnInspect)
			{
				if (UPCGEditorGraph* const PCGEditorGraph = Cast<UPCGEditorGraph>(GetGraph()))
				{
					if (const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorGraph->GetEditor().Pin())
					{
						PCGEditor->OnNodeToolStarted(this);
					}
				}
			}
			else if (bStopInspection && Settings->IsNodeToolActive())
			{
				if (UPCGEditorGraph* const PCGEditorGraph = Cast<UPCGEditorGraph>(GetGraph()))
				{
					if (const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorGraph->GetEditor().Pin())
					{
						PCGEditor->OnNodeToolEnded(this);
					}
				}
			}
		}

		// If we start inspecting a GPU node, we need to generate to populate inspection data. The
		// normal workflow optimization that avoids re-executing when moving inspection flag around
		// graph relies on all nodes storing inspection data which is efficient for CPU but not for GPU,
		// where we only do expensive GPU->CPU readbacks for the currently inspected GPU node.
		if (bStartInspection && Settings->ShouldExecuteOnGPU())
		{
			const UPCGEditorGraph* EditorGraph = CastChecked<UPCGEditorGraph>(GetGraph());
			const FPCGEditor* Editor = EditorGraph ? EditorGraph->GetEditor().Pin().Get() : nullptr;

			// Force refresh/regenerate.
			if (IPCGGraphExecutionSource* Source = Editor ? Editor->GetPCGSourceBeingInspected() : nullptr)
			{
				if (Source->GetExecutionState().IsManagedByRuntimeGenSystem())
				{
					if (UPCGSubsystem* Subsystem = Cast<UPCGSubsystem>(Source->GetExecutionState().GetSubsystem()))
					{
						Subsystem->RefreshRuntimeGenExecutionSource(Source, EPCGChangeType::Node);
					}
				}
				else
				{
					IPCGGraphExecutionState::FGenerateParams GenerateParams;
					GenerateParams.bEvenIfAlreadyGenerated = true;

					Source->GetExecutionState().Generate(GenerateParams);
				}
			}
		}
	}
}

bool UPCGEditorGraphNodeBase::GetInspected() const
{
	const UPCGSettings* Settings = PCGNode ? PCGNode->GetSettings() : nullptr;
	return ensure(Settings) && Settings->bIsInspecting;
}

void UPCGEditorGraphNodeBase::EnableDeferredReconstruct()
{
	ensure(DeferredReconstructCounter >= 0);
	++DeferredReconstructCounter;
}

void UPCGEditorGraphNodeBase::DisableDeferredReconstruct()
{
	ensure(DeferredReconstructCounter > 0);
	--DeferredReconstructCounter;

	if (DeferredReconstructCounter == 0 && bDeferredReconstruct)
	{
		ReconstructNode();
		bDeferredReconstruct = false;
	}
}

void UPCGEditorGraphNodeBase::RebuildEdgesFromPins()
{
	check(PCGNode);
	check(bDisableReconstructFromNode);

	if (PCGNode->GetGraph())
	{
		PCGNode->GetGraph()->DisableNotificationsForEditor();
	}

	RebuildEdgesFromPins_Internal();
	
	if (PCGNode->GetGraph())
	{
		PCGNode->GetGraph()->EnableNotificationsForEditor();
	}
}

void UPCGEditorGraphNodeBase::RebuildEdgesFromPins_Internal()
{
	check(PCGNode);
	check(bDisableReconstructFromNode);

	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
		{
			for (UEdGraphPin* ConnectedPin : Pin->LinkedTo)
			{
				UEdGraphNode* ConnectedGraphNode = ConnectedPin->GetOwningNode();
				UPCGEditorGraphNodeBase* ConnectedPCGGraphNode = CastChecked<UPCGEditorGraphNodeBase>(ConnectedGraphNode);

				if (UPCGNode* ConnectedPCGNode = ConnectedPCGGraphNode->GetPCGNode())
				{
					PCGNode->AddEdgeTo(Pin->PinName, ConnectedPCGNode, ConnectedPin->PinName);
				}
			}
		}
	}
}

void UPCGEditorGraphNodeBase::OnNodeChanged(UPCGNode* InNode, EPCGChangeType ChangeType)
{
	if (InNode == PCGNode)
	{
		// Invalidate pin color cache before reconstruction so that Slate widget construction (which bakes
		// colors at build time) always uses fresh values. Without this, the cache can contain pre-change
		// colors that get baked into reconstructed widgets before the graph-change delegate fires.
		ClearPinColorCache();

		bool bRequiresReconstructNode = false;

		if (!!(ChangeType & EPCGChangeType::Settings))
		{
			if (const UPCGSettingsInterface* PCGSettingsInterface = InNode->GetSettingsInterface())
			{
				const ENodeEnabledState NewEnabledState = PCGSettingsInterface->bEnabled ? ENodeEnabledState::Enabled : ENodeEnabledState::Disabled;
				if (NewEnabledState != GetDesiredEnabledState())
				{
					SetEnabledState(NewEnabledState);
					bRequiresReconstructNode = true;
				}
			}
		}

		ChangeType |= UpdateErrorsAndWarnings();

		if (!!(ChangeType & (EPCGChangeType::Structural | EPCGChangeType::Node | EPCGChangeType::Edge | EPCGChangeType::Cosmetic)) ||
			bRequiresReconstructNode)
		{
			ReconstructNodeOnChange();
		}
	}
}

EPCGChangeType UPCGEditorGraphNodeBase::UpdateErrorsAndWarnings()
{
	IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get();
	if (!PCGNode || !PCGEditorModule)
	{
		return EPCGChangeType::None;
	}
	
	const FPCGStack* InspectedStack = nullptr;
	{
		const UPCGEditorGraph* EditorGraph = CastChecked<UPCGEditorGraph>(GetGraph());
		const FPCGEditor* Editor = (EditorGraph && EditorGraph->GetEditor().IsValid()) ? EditorGraph->GetEditor().Pin().Get() : nullptr;
		InspectedStack = Editor ? Editor->GetStackBeingInspected() : nullptr;
	}

	const bool bOldHasCompilerMessage = bHasCompilerMessage;
	const int32 OldErrorType = ErrorType;
	const FString OldErrorMsg = ErrorMsg;
		
	if (InspectedStack)
	{
		// Get errors/warnings for the inspected stack.
		FPCGStack StackWithNode = *InspectedStack;
		StackWithNode.PushFrame(PCGNode);
		bHasCompilerMessage = PCGEditorModule->GetNodeVisualLogs().HasLogs(StackWithNode);

		if (bHasCompilerMessage)
		{
			ErrorMsg = PCGEditorModule->GetNodeVisualLogs().GetLogsSummaryText(StackWithNode).ToString();

			const bool bHasErrors = PCGEditorModule->GetNodeVisualLogs().HasLogsOfVerbosity(StackWithNode, ELogVerbosity::Error);
			ErrorType = bHasErrors ? EMessageSeverity::Error : EMessageSeverity::Warning;
		}
		else
		{
			ErrorMsg.Empty();
			ErrorType = 0;
		}
	}
	else
	{
		// Collect all errors/warnings for this node.
		ELogVerbosity::Type MinimumVerbosity;
		ErrorMsg = PCGEditorModule->GetNodeVisualLogs().GetLogsSummaryText(PCGNode.Get(), MinimumVerbosity).ToString();

		bHasCompilerMessage = !ErrorMsg.IsEmpty();

		if (bHasCompilerMessage)
		{
			ErrorType = MinimumVerbosity < ELogVerbosity::Warning ? EMessageSeverity::Error : EMessageSeverity::Warning;
		}
		else
		{
			ErrorType = 0;
		}
	}

	const bool bStateChanged = (bHasCompilerMessage != bOldHasCompilerMessage) || (ErrorType != OldErrorType) || (ErrorMsg != OldErrorMsg);
	return bStateChanged ? EPCGChangeType::Cosmetic : EPCGChangeType::None;
}

EPCGChangeType UPCGEditorGraphNodeBase::UpdateStructuralVisualization(const IPCGGraphExecutionSource* InSourceBeingDebugged, const FPCGStack* InStackBeingInspected, bool bNewlyPlaced)
{
	const UPCGGraph* Graph = PCGNode ? PCGNode->GetGraph() : nullptr;
	if (!Graph)
	{
		return EPCGChangeType::None;
	}

	const bool bInspecting = InSourceBeingDebugged && InStackBeingInspected && !InStackBeingInspected->GetStackFrames().IsEmpty();

	EPCGChangeType ChangeType = EPCGChangeType::None;

	const uint64 NewInactiveMask = bInspecting ? InSourceBeingDebugged->GetExecutionState().GetInspection().GetNodeInactivePinMask(PCGNode, *InStackBeingInspected) : 0;
	if (NewInactiveMask != InactiveOutputPinMask)
	{
		InactiveOutputPinMask = NewInactiveMask;
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	// Check top graph for higen enable - subgraphs always inherit higen state from the top graph.
	const UPCGGraph* TopGraph = bInspecting ? InStackBeingInspected->GetRootGraph() : Graph;
	const bool HiGenEnabled = TopGraph && TopGraph->IsHierarchicalGenerationEnabled();

	// Set the inspected grid size - this is used for grid size visualization.
	uint32 InspectingGridSize = PCGHiGenGrid::UninitializedGridSize();
	if (TopGraph && TopGraph->IsHierarchicalGenerationEnabled() && InSourceBeingDebugged && (InSourceBeingDebugged->GetExecutionState().IsPartitioned() || InSourceBeingDebugged->GetExecutionState().IsLocalSource()))
	{
		InspectingGridSize = InSourceBeingDebugged->GetExecutionState().GetGenerationGridSize();
	}

	if (InspectedGridSize != InspectingGridSize)
	{
		InspectedGridSize = InspectingGridSize;
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	bool bShouldDisplayAsDisabled = false;

	// Special treatment for higen grid sizes nodes which do nothing if higen is disabled.
	// TODO: Drive this from an API on settings as we add more higen-specific functionality.
	if (PCGNode && Cast<UPCGHiGenGridSizeSettings>(PCGNode->GetSettings()))
	{
		// Higen must be enabled on graph, and we must be editing top graph.
		bShouldDisplayAsDisabled = (Graph != TopGraph) || !TopGraph->IsHierarchicalGenerationEnabled();

		// If we're inspecting a component, it must either be a partitioned OC or an LC (because higen requires partitioning).
		if (!bShouldDisplayAsDisabled && InSourceBeingDebugged)
		{
			bShouldDisplayAsDisabled = !InSourceBeingDebugged->GetExecutionState().IsPartitioned() && !InSourceBeingDebugged->GetExecutionState().IsLocalSource();
		}
	}

	// Don't do culling visualization on newly placed nodes. Let the execution complete notification update that.
	bool bIsCulled = !bNewlyPlaced && !PCGEditorGraphNodeBase::ShouldDisplayAsActive(this, InSourceBeingDebugged, InStackBeingInspected);

	uint32 ThisGridSize = PCGHiGenGrid::UninitializedGridSize();

	// Show grid size visualization if higen is enabled and if we're inspecting a specific grid, and we're inspecting a subgraph since subgraphs
	// execute at the invoked grid level.
	if (HiGenEnabled && InspectingGridSize != PCGHiGenGrid::UninitializedGridSize())
	{
		if (InStackBeingInspected && InStackBeingInspected->IsCurrentFrameInRootGraph())
		{
			const uint32 DefaultGridSize = TopGraph->GetDefaultGridSize();
			const uint32 NodeGridSize = Graph->GetNodeGenerationGridSize(PCGNode, DefaultGridSize);

			if (NodeGridSize < InspectingGridSize)
			{
				// Disable nodes that are on a smaller grid
				bShouldDisplayAsDisabled |= NodeGridSize < InspectingGridSize;

				// We don't know if the node was culled or not on that grid, disable visualization.
				bIsCulled = false;
			}
			else if (NodeGridSize > InspectingGridSize)
			{
				// We don't know if the node was culled or not on that grid, disable visualization.
				bIsCulled = false;
			}

			ThisGridSize = NodeGridSize;
		}
		else
		{
			// If higen is enabled then we are inspecting an invoked subgraph. Display the inspected grid size so that the user still
			// gets the execution grid information.
			ThisGridSize = InspectingGridSize;
		}
	}

	if (GenerationGridSize != ThisGridSize)
	{
		// Update node if the grid size has changed.
		ChangeType |= EPCGChangeType::Cosmetic;
		GenerationGridSize = ThisGridSize;
	}

	SetIsCulledFromExecution(bIsCulled);

	if (bIsCulled)
	{
		bShouldDisplayAsDisabled = true;
	}

	if (IsDisplayAsDisabledForced() != bShouldDisplayAsDisabled)
	{
		SetForceDisplayAsDisabled(bShouldDisplayAsDisabled);
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	return ChangeType;
}

EPCGChangeType UPCGEditorGraphNodeBase::UpdateGPUVisualization(const IPCGGraphExecutionSource* InSourceBeingDebugged, const FPCGStack* InStackBeingInspected)
{
	EPCGChangeType ChangeType = EPCGChangeType::None;

	const UPCGNode* Node = GetPCGNode();

	const bool bTriggeredUpload = InSourceBeingDebugged && InStackBeingInspected && Node
		&& InSourceBeingDebugged->GetExecutionState().GetInspection().DidNodeTriggerCPUToGPUUpload(Node, *InStackBeingInspected);

	if (bTriggeredUpload != GetTriggeredGPUUpload())
	{
		SetTriggeredGPUUpload(bTriggeredUpload);
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	const bool bTriggeredReadback = InSourceBeingDebugged && InStackBeingInspected && Node
		&& InSourceBeingDebugged->GetExecutionState().GetInspection().DidNodeTriggerGPUToCPUReadback(Node, *InStackBeingInspected);

	if (bTriggeredReadback != GetTriggeredGPUReadback())
	{
		SetTriggeredGPUReadback(bTriggeredReadback);
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	return ChangeType;
}

EPCGChangeType UPCGEditorGraphNodeBase::UpdateOverrideVisualization(const IPCGGraphExecutionSource* InSourceBeingDebugged, const FPCGStack* InStackBeingInspected)
{
	EPCGChangeType ChangeType = EPCGChangeType::None;

	const UPCGNode* Node = GetPCGNode();

	const bool bOverridesApplied = Node
		&& InStackBeingInspected
		&& InSourceBeingDebugged
		&& InSourceBeingDebugged->GetExecutionState().GetInspection().NodeAppliedDataOverrides(Node, *InStackBeingInspected);

	if (bOverridesApplied != GetAppliedDataOverrides())
	{
		SetAppliedDataOverrides(bOverridesApplied);
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	const bool bHasOverrides = PCGEditorGraphNodeBase::NodeHasDataOverrides(Node, InSourceBeingDebugged);
	const bool bHasOrphaned = bHasOverrides && PCGEditorGraphNodeBase::NodeHasOrphanedDeltas(Node, InSourceBeingDebugged);

	if (bHasOverrides != GetHasDefinedOverrides())
	{
		SetHasDefinedOverrides(bHasOverrides);
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	if (bHasOrphaned != GetHasOrphanedOverrides())
	{
		SetHasOrphanedOverrides(bHasOrphaned);
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	return ChangeType;
}

bool UPCGEditorGraphNodeBase::CanUserAddRemoveDynamicPins(EPCGPinDirection Direction) const
{
	const UPCGSettings* Settings = PCGNode ? PCGNode->GetSettings() : nullptr;
	if (!Settings)
	{
		return false;
	}

	// @todo_pcg: To be removed with UPCGSettingsWithDynamicInputs when deprecated.
	if (Direction == EPCGPinDirection::Input && Settings->IsA<UPCGSettingsWithDynamicInputs>())
	{
		return true;
	}

	if (const IPCGDynamicPinsProvider* Interface = GetDynamicPinsInterface())
	{
		return Interface->CanUserAddDynamicPins(Direction);
	}

	return false;
}

bool UPCGEditorGraphNodeBase::IsInteractiveNode() const
{
	const UPCGSettings* PCGSettings = GetSettings();
	return PCGSettings && FPCGAssetEditorToolRegistry::Get().FindToolForSettings(PCGSettings->GetClass()) != nullptr;
}

bool UPCGEditorGraphNodeBase::IsSettingsDefaultValuesEnabled() const
{
	const IPCGSettingsDefaultValueProvider* DefaultValueInterface = GetDefaultValueInterface();
	return DefaultValueInterface && DefaultValueInterface->DefaultValuesAreEnabled();
}

bool UPCGEditorGraphNodeBase::IsPinDefaultValueEnabled(const FName PinLabel) const
{
	const IPCGSettingsDefaultValueProvider* DefaultValueInterface = GetDefaultValueInterface();
	return DefaultValueInterface && DefaultValueInterface->IsPinDefaultValueEnabled(PinLabel);
}

bool UPCGEditorGraphNodeBase::IsPinDefaultValueActivated(const FName PinLabel) const
{
	const IPCGSettingsDefaultValueProvider* DefaultValueInterface = GetDefaultValueInterface();
	return DefaultValueInterface && DefaultValueInterface->IsPinDefaultValueActivated(PinLabel);
}

void UPCGEditorGraphNodeBase::OnUserSetPinDefaultValueActivated(const FName PinLabel, const bool bIsActivated) const
{
	UPCGSettings* Settings = PCGNode ? PCGNode->GetSettings() : nullptr;
	if (IPCGSettingsDefaultValueProvider* DefaultValueInterface = GetDefaultValueInterface())
	{
		FText TransactionDescription = bIsActivated
			? LOCTEXT("PCGEditorSetPinInlineConstantDeactivated", "Deactivate Pin Inline Constant")
			: LOCTEXT("PCGEditorSetPinInlineConstantActivated", "Activate Pin Inline Constant");
		FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, std::move(TransactionDescription), Settings);
		DefaultValueInterface->SetPinDefaultValueIsActivated(PinLabel, bIsActivated);
	}
}

void UPCGEditorGraphNodeBase::ConvertPinDefaultValueMetadataType(const FName PinLabel, const EPCGMetadataTypes DataType) const
{
	UPCGSettings* Settings = PCGNode ? PCGNode->GetSettings() : nullptr;
	if (IPCGSettingsDefaultValueProvider* Interface = GetDefaultValueInterface())
	{
		FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorConvertPinInlineConstantMetadataType", "Convert Pin Inline Constant Type"), Settings);
		Interface->ConvertPinDefaultValueMetadataType(PinLabel, DataType);
	}
}

bool UPCGEditorGraphNodeBase::CanConvertToDefaultValueMetadataType(const FName PinLabel, const EPCGMetadataTypes DataType) const
{
	// Early out for invalid types and if the pin is already this type.
	if (!PCGMetadataHelpers::MetadataTypeSupportsDefaultValues(DataType) || DataType == GetPinDefaultValueType(PinLabel))
	{
		return false;
	}

	const IPCGSettingsDefaultValueProvider* Interface = GetDefaultValueInterface();
	return Interface && Interface->IsPinDefaultValueMetadataTypeValid(PinLabel, DataType);
}

void UPCGEditorGraphNodeBase::OnUserResetPinDefaultValue(const FName PinLabel, UEdGraphPin* OutPin) const
{
	if (IPCGSettingsDefaultValueProvider* Interface = GetDefaultValueInterface())
	{
		FScopedTransaction Transaction(
			*FPCGEditorCommon::ContextIdentifier,
			LOCTEXT("PCGEditorResetInlineConstantDefaultValue", "Reset Pin Inline Constant to Default Value"),
			OutPin ? OutPin->GetOwningNode() : nullptr);
		Interface->ResetDefaultValue(PinLabel);
		if (OutPin)
		{
			OutPin->Modify();
			OutPin->DefaultValue = Interface->GetPinDefaultValueAsString(PinLabel);
		}
	}
}

bool UPCGEditorGraphNodeBase::CanResetPinDefaultValue(const FName PinLabel) const
{
	const IPCGSettingsDefaultValueProvider* Interface = GetDefaultValueInterface();
	return Interface && Interface->DefaultValuesAreEnabled() && Interface->IsPinDefaultValueActivated(PinLabel);
}

EPCGMetadataTypes UPCGEditorGraphNodeBase::GetPinDefaultValueType(const FName PinLabel) const
{
	const IPCGSettingsDefaultValueProvider* Interface = GetDefaultValueInterface();
	return Interface ? Interface->GetPinDefaultValueType(PinLabel) : EPCGMetadataTypes::Unknown;
}

bool UPCGEditorGraphNodeBase::HasFlippedTitleLines() const
{
	return PCGNode ? PCGNode->HasFlippedTitleLines() : false;
}

FText UPCGEditorGraphNodeBase::GetAuthoredTitleLine() const
{
	return PCGNode ? PCGNode->GetAuthoredTitleLine() : FText();
}

FText UPCGEditorGraphNodeBase::GetGeneratedTitleLine() const
{
	return PCGNode ? PCGNode->GetGeneratedTitleLine() : FText();
}

bool UPCGEditorGraphNodeBase::IsOutputPinActive(const UEdGraphPin* InOutputPin) const
{
	bool bPinActive = true;

	if (InactiveOutputPinMask != 0)
	{
		bool bFoundPin = false;
		int OutputPinIndex = 0;

		for (const UEdGraphPin* NodePin : Pins)
		{
			if (NodePin == InOutputPin)
			{
				bFoundPin = true;
				break;
			}

			if (NodePin->Direction == EEdGraphPinDirection::EGPD_Output)
			{
				++OutputPinIndex;
			}
		}

		if (bFoundPin)
		{
			bPinActive = !((1ULL << OutputPinIndex) & InactiveOutputPinMask);
		}
	}

	return bPinActive;
}

bool UPCGEditorGraphNodeBase::IsCompatible(const UPCGPin* InputPin, const UPCGPin* OutputPin, FText& OutReason) const
{
	if (PCGEditorGraphSwitches::CVarCheckConnectionCycles.GetValueOnAnyThread() && InputPin && OutputPin && InputPin->Node == PCGNode)
	{
		// Upstream Visitor
		TSet<const UPCGNode*> VisitedNodes;
		auto Visitor = [&VisitedNodes, ThisPCGNode = PCGNode](const UPCGNode* InNode, auto VisitorLambda) -> bool
		{
			if (InNode)
			{
				if (InNode == ThisPCGNode)
				{
					return false;
				}
				else if (VisitedNodes.Contains(InNode))
				{
					return true;
				}

				VisitedNodes.Add(InNode);

				for (const TObjectPtr<UPCGPin>& InputPin : InNode->GetInputPins())
				{
					if (InputPin)
					{
						for (const TObjectPtr<UPCGEdge>& Edge : InputPin->Edges)
						{
							if (Edge)
							{
								if (const UPCGPin* OtherPin = Edge->GetOtherPin(InputPin.Get()))
								{
									if (!VisitorLambda(OtherPin->Node, VisitorLambda))
									{
										return false;
									}
								}
							}
						}
					}
				}
			}

			return true;
		};

		// OutputPin is trying to connect to this nodes InputPin so visit the OutputPin upstream and try to find
		// a existing connection to this UPCGEditorGraphNodeNamedRerouteDeclaration's PCGNode. If we do deny connection which would create cycle.
		if (!Visitor(OutputPin->Node, Visitor))
		{
			OutReason = LOCTEXT("ConnectionFailedCyclic", "Connection would create cycle");
			return false;
		}
	}

	return true;
}

UPCGSettings* UPCGEditorGraphNodeBase::GetSettings() const
{
	return PCGNode ? PCGNode->GetSettings() : nullptr;
}

IPCGSettingsDefaultValueProvider* UPCGEditorGraphNodeBase::GetDefaultValueInterface() const
{
	return PCGEditorGraphNodeBase::GetDefaultValueInterface(GetSettings());
}

IPCGDynamicPinsProvider* UPCGEditorGraphNodeBase::GetDynamicPinsInterface() const
{
	return PCGEditorGraphNodeBase::GetDynamicPinsInterface(GetSettings());
}

void UPCGEditorGraphNodeBase::OnPickColor()
{
	FColorPickerArgs PickerArgs;
	PickerArgs.bIsModal = true;
	PickerArgs.bUseAlpha = false;
	PickerArgs.InitialColor = GetNodeTitleColor();
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateUObject(this, &UPCGEditorGraphNodeBase::OnColorPicked);

	OpenColorPicker(PickerArgs);
}

void UPCGEditorGraphNodeBase::OnColorPicked(FLinearColor NewColor)
{
	if (PCGNode && GetNodeTitleColor() != NewColor)
	{
		PCGNode->Modify();
		PCGNode->NodeTitleColor = NewColor;
	}
}

void UPCGEditorGraphNodeBase::ReconstructNode()
{
	// In copy-paste cases, we don't want to remove the pins
	if (bDisableReconstructFromNode)
	{
		return;
	}

	if (DeferredReconstructCounter > 0)
	{
		bDeferredReconstruct = true;
		return;
	}

	// While in an Undo/Redo a call to ReconstructNode should not be needed as the transaction object records
	// should be enough to serialize the nodes back into their proper state 
	if (GIsTransacting)
	{
		return;
	}
	
	Modify();

	// Store copy of old pins
	TArray<UEdGraphPin*> OldPins = MoveTemp(Pins);
	Pins.Reset();

	// Generate new pins
	AllocateDefaultPins();

	// Transfer persistent data from old to new pins
	for (UEdGraphPin* OldPin : OldPins)
	{
		const FName& OldPinName = OldPin->PinName;
		if (UEdGraphPin** NewPin = Pins.FindByPredicate([&OldPinName](UEdGraphPin* InPin) { return InPin->PinName == OldPinName; }))
		{
			(*NewPin)->MovePersistentDataFromOldPin(*OldPin);
		}
	}

	// Remove old pins
	for (UEdGraphPin* OldPin : OldPins)
	{
		RemovePin(OldPin);
	}

	// Generate new links
	// TODO: we should either keep a map in the PCGEditorGraph or do this elsewhere
	// TODO: this will not work if we have non-PCG nodes in the graph
	if (PCGNode)
	{
		for (UEdGraphPin* Pin : Pins)
		{
			Pin->BreakAllPinLinks();
		}
		
		UPCGEditorGraph* PCGEditorGraph = CastChecked<UPCGEditorGraph>(GetGraph());
		PCGEditorGraph->CreateLinks(this, /*bCreateInbound=*/true, /*bCreateOutbound=*/true);
	}

	// Notify editor
	OnNodeChangedDelegate.ExecuteIfBound();
}

FLinearColor UPCGEditorGraphNodeBase::GetNodeTitleColor() const
{
	if (PCGNode)
	{
		const UPCGSettingsInterface* PCGSettingsInterface = PCGNode->GetSettingsInterface();
		const UPCGSettings* PCGSettings = PCGSettingsInterface ? PCGSettingsInterface->GetSettings() : nullptr;

		if (PCGNode->NodeTitleColor != FLinearColor::White)
		{
			return PCGNode->NodeTitleColor;
		}
		else if (PCGSettings)
		{
			FLinearColor SettingsColor = PCGNode->GetSettings()->GetNodeTitleColor();
			if (SettingsColor == FLinearColor::White)
			{
				SettingsColor = GetDefault<UPCGEditorSettings>()->GetColor(PCGNode->GetSettings());
			}

			if (SettingsColor != FLinearColor::White)
			{
				return SettingsColor;
			}
		}
	}

	return GetDefault<UPCGEditorSettings>()->DefaultNodeColor;
}

FLinearColor UPCGEditorGraphNodeBase::GetNodeBodyTintColor() const
{
	if (PCGNode)
	{
		const UPCGSettingsInterface* PCGSettingsInterface = PCGNode->GetSettingsInterface();
		if (PCGSettingsInterface)
		{
			if (PCGSettingsInterface->IsInstance())
			{
				return GetDefault<UPCGEditorSettings>()->InstancedNodeBodyTintColor;
			}
		}
	}

	return Super::GetNodeBodyTintColor();
}

FEdGraphPinType UPCGEditorGraphNodeBase::GetPinType(const UPCGPin* InPin)
{
	FEdGraphPinType EdPinType;
	EdPinType.ResetToDefaults();

	check(InPin);
	const FPCGDataTypeIdentifier PinType = InPin->GetCurrentTypesID();

	auto CheckType = [PinType](FPCGDataTypeIdentifier AllowedType)
	{
		return !!(PinType & AllowedType) && !PinType.IsWider(AllowedType);
	};

	if (CheckType(EPCGDataType::Concrete))
	{
		EdPinType.PinCategory = FPCGEditorCommon::ConcreteDataType;

		// Assign subcategory if we have precise information
		if (CheckType(EPCGDataType::Point))
		{
			EdPinType.PinSubCategory = FPCGEditorCommon::PointDataType;
		}
		else if (CheckType(EPCGDataType::PolyLine))
		{
			EdPinType.PinSubCategory = FPCGEditorCommon::PolyLineDataType;
		}
		else if (CheckType(EPCGDataType::Landscape))
		{
			EdPinType.PinSubCategory = FPCGEditorCommon::LandscapeDataType;
		}
		else if (CheckType(EPCGDataType::VirtualTexture))
		{
			EdPinType.PinSubCategory = FPCGEditorCommon::VirtualTextureDataType;
		}
		else if (CheckType(EPCGDataType::BaseTexture))
		{
			EdPinType.PinSubCategory = FPCGEditorCommon::BaseTextureDataType;
		}
		else if (CheckType(EPCGDataType::Texture))
		{
			EdPinType.PinSubCategory = FPCGEditorCommon::TextureDataType;
		}
		else if (CheckType(EPCGDataType::RenderTarget))
		{
			EdPinType.PinSubCategory = FPCGEditorCommon::RenderTargetDataType;
		}
		else if (CheckType(EPCGDataType::Surface))
		{
			EdPinType.PinSubCategory = FPCGEditorCommon::SurfaceDataType;
		}
		else if (CheckType(EPCGDataType::Volume))
		{
			EdPinType.PinSubCategory = FPCGEditorCommon::VolumeDataType;
		}
		else if (CheckType(EPCGDataType::DynamicMesh))
		{
			EdPinType.PinSubCategory = FPCGEditorCommon::DynamicMeshDataType;
		}
		else if (CheckType(EPCGDataType::Primitive))
		{
			EdPinType.PinSubCategory = FPCGEditorCommon::PrimitiveDataType;
		}
	}
	else if (CheckType(EPCGDataType::Spatial))
	{
		EdPinType.PinCategory = FPCGEditorCommon::SpatialDataType;
	}
	else if (CheckType(EPCGDataType::StaticMeshResource))
	{
		EdPinType.PinSubCategory = FPCGEditorCommon::StaticMeshResourceDataType;
	}
	else if (CheckType(EPCGDataType::Param))
	{
		EdPinType.PinCategory = FPCGEditorCommon::ParamDataType;
	}
	else if (CheckType(EPCGDataType::Settings))
	{
		EdPinType.PinCategory = FPCGEditorCommon::SettingsDataType;
	}
	else if (CheckType(EPCGDataType::Other))
	{
		EdPinType.PinCategory = FPCGEditorCommon::OtherDataType;
	}

	return EdPinType;
}

FText UPCGEditorGraphNodeBase::GetTooltipText() const
{
	// Either use specified tooltip for description, or fall back to node name if none given.
	const FText Description = (PCGNode && !PCGNode->GetNodeTooltipText().IsEmpty()) ? PCGNode->GetNodeTooltipText() : GetNodeTitle(ENodeTitleType::FullTitle);

	return FText::Format(LOCTEXT("NodeTooltip", "{0}\n\n{1} - Node index {2}"),
		Description,
		PCGNode ? FText::FromName(PCGNode->GetFName()) : LOCTEXT("InvalidNodeName", "Unbound node"),
		PCGNode && PCGNode->GetGraph() ? FText::AsNumber(PCGNode->GetGraph()->GetNodes().IndexOfByKey(PCGNode)) : LOCTEXT("InvalidNodeIndex", "Invalid index"));
}

void UPCGEditorGraphNodeBase::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	const bool bIsInputPin = (Pin.Direction == EGPD_Input);
	UPCGPin* MatchingPin = (PCGNode ? (bIsInputPin ? PCGNode->GetInputPin(Pin.PinName) : PCGNode->GetOutputPin(Pin.PinName)) : nullptr);
	const FPCGDataTypeIdentifier Type = MatchingPin ? MatchingPin->GetCurrentTypesID() : FPCGDataTypeIdentifier{};

	auto PinTypeToText = [&Type](const FName& Category)
	{
		if (Category != NAME_None)
		{
			return FText::FromName(Category);
		}

		if (Type == EPCGDataType::Any)
		{
			return FText::FromName(FName("Any"));
		}
		else if (Type.IsValid())
		{
			return Type.ToDisplayText();
		}
		else
		{
			return FText(LOCTEXT("Unknown data type", "Unknown data type"));
		}
	};

	const FText DataTypeText = PinTypeToText(Pin.PinType.PinCategory);
	const FText DataSubtypeText = Type.GetSubtypeTooltip();

	FText Description;
	if (MatchingPin)
	{
		Description = MatchingPin->Properties.Tooltip.IsEmpty() ? FText::FromName(MatchingPin->Properties.Label) : MatchingPin->Properties.Tooltip;
	}

	FText Required;
	FText MultiDataSupport;
	FText MultiConnectionSupport;	

	if (MatchingPin)
	{
		if (bIsInputPin)
		{
			if (PCGNode && PCGNode->IsInputPinRequiredByExecution(MatchingPin))
			{
				Required = LOCTEXT("InputIsRequired", "Required input. ");
			}

			MultiDataSupport = MatchingPin->Properties.bAllowMultipleData ? LOCTEXT("InputSupportsMultiData", "Supports multiple data in input(s). ") : LOCTEXT("InputSingleDataOnly", "Supports only single data in input(s). ");

			MultiConnectionSupport = MatchingPin->Properties.AllowsMultipleConnections() ? LOCTEXT("SupportsMultiInput", "Supports multiple inputs.") : LOCTEXT("SingleInputOnly", "Supports only one input.");
		}
		else
		{
			MultiDataSupport = MatchingPin->Properties.bAllowMultipleData ? LOCTEXT("OutputSupportsMultiData", "Can generate multiple data.") : LOCTEXT("OutputSingleDataOnly", "Generates only single data.");
		}
	}

	HoverTextOut = FText::Format(LOCTEXT("PinHoverToolTipFull", "{0}\n\nType: {1}\nSubtype: {2}\nAdditional information: {3}{4}{5}"),
		Description,
		DataTypeText,
		DataSubtypeText,
		Required,
		MultiDataSupport,
		MultiConnectionSupport).ToString();

	const FText ExtraTooltip = Type.GetExtraTooltip();
	if (!ExtraTooltip.IsEmpty())
	{
		HoverTextOut = FText::Format(INVTEXT("{0}\n{1}"), FText::FromString(HoverTextOut), ExtraTooltip).ToString();
	}
}

UObject* UPCGEditorGraphNodeBase::GetJumpTargetForDoubleClick() const
{
	if (PCGNode)
	{
		if (UPCGSettings* Settings = PCGNode->GetSettings())
		{
			return Settings->GetJumpTargetForDoubleClick();
		}
	}

	return nullptr;
}

void UPCGEditorGraphNodeBase::OnUpdateCommentText(const FString& NewComment)
{
	Super::OnUpdateCommentText(NewComment);

	if (PCGNode && PCGNode->NodeComment != NewComment)
	{
		PCGNode->Modify();
		PCGNode->NodeComment = NewComment;
	}
}

void UPCGEditorGraphNodeBase::OnCommentBubbleToggled(bool bInCommentBubbleVisible)
{
	Super::OnCommentBubbleToggled(bInCommentBubbleVisible);

	if (PCGNode && PCGNode->bCommentBubbleVisible != bInCommentBubbleVisible)
	{
		PCGNode->Modify();
		PCGNode->bCommentBubbleVisible = bInCommentBubbleVisible;
	}
}

void UPCGEditorGraphNodeBase::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	check(PCGNode);

	if (Pin && !Pin->IsPendingKill())
	{
		Super::PinDefaultValueChanged(Pin);

		if (IPCGSettingsDefaultValueProvider* DefaultValueInterface = GetDefaultValueInterface())
		{
			if (ensure(DefaultValueInterface->IsPinDefaultValueActivated(Pin->PinName)))
			{
				DefaultValueInterface->SetPinDefaultValue(Pin->PinName, Pin->DefaultValue, /*bCreateIfNeeded=*/true);
			}
		}
	}
}

void UPCGEditorGraphNodeBase::PinConnectionListChanged(UEdGraphPin* Pin)
{
	check(PCGNode);

	Super::PinConnectionListChanged(Pin);

	// One-time UX functionality, where when a user connects a node with default values, it will activate them for all other pins
	if (!bHasEverBeenConnected && Pin && !Pin->IsPendingKill())
	{
		if (IPCGSettingsDefaultValueProvider* Interface = GetDefaultValueInterface())
		{
			if (Pin->Direction == EGPD_Input && Interface->DefaultValuesAreEnabled())
			{
				for (const UPCGPin* InputPin : PCGNode->GetInputPins())
				{
					if (InputPin && InputPin->Properties.Label != Pin->PinName && Interface->IsPinDefaultValueEnabled(InputPin->Properties.Label))
					{
						// Connecting a pin will dirty anyway, so no need to dirty again
						Interface->SetPinDefaultValueIsActivated(InputPin->Properties.Label, /*bIsActive=*/true, /*bDirtySettings=*/false);
					}
				}

				bHasEverBeenConnected = true;
			}
		}
	}
}

void UPCGEditorGraphNodeBase::OnUserAddDynamicInputPin()
{
	OnUserAddDynamicPin(EPCGPinDirection::Input);
}

bool UPCGEditorGraphNodeBase::CanUserRemoveDynamicInputPin(UEdGraphPin* InPinToRemove) const
{
	return CanUserRemoveDynamicPin(InPinToRemove, EPCGPinDirection::Input);
}

void UPCGEditorGraphNodeBase::OnUserRemoveDynamicInputPin(UEdGraphPin* InRemovedPin)
{
	OnUserRemoveDynamicPin(InRemovedPin, EPCGPinDirection::Input);
}

void UPCGEditorGraphNodeBase::OnUserAddDynamicPin(EPCGPinDirection Direction)
{
	check(PCGNode);
	UPCGSettings* Settings = PCGNode->GetSettings();

	if (Direction == EPCGPinDirection::Input)
	{
		// @todo_pcg: To be removed with UPCGSettingsWithDynamicInputs when deprecated.
		if (UPCGSettingsWithDynamicInputs* DynamicNodeSettings = Cast<UPCGSettingsWithDynamicInputs>(Settings))
		{
			const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorUserAddDynamicInputPin", "Add Source Pin"), DynamicNodeSettings);
			DynamicNodeSettings->Modify();
			DynamicNodeSettings->OnUserAddDynamicInputPin();
			return;
		}
	}

	if (IPCGDynamicPinsProvider* Interface = GetDynamicPinsInterface())
	{
		if (FPCGDynamicPinContainer* Container = Interface->GetMutableDynamicPinContainer(Direction))
		{
			const FText TransactionDesc = (Direction == EPCGPinDirection::Input)
				? LOCTEXT("PCGEditorUserAddDynamicInputPinInterface", "Add Input Pin")
				: LOCTEXT("PCGEditorUserAddDynamicOutputPinInterface", "Add Output Pin");
			const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, TransactionDesc, Settings);
			Settings->Modify();
			Container->AddPin(Interface->CreateDefaultDynamicPin(Direction));
			Settings->OnSettingsChangedDelegate.Broadcast(Settings, EPCGChangeType::Node | EPCGChangeType::Settings);
		}
	}
}

bool UPCGEditorGraphNodeBase::CanUserRemoveDynamicPin(UEdGraphPin* InPinToRemove, EPCGPinDirection Direction) const
{
	check(PCGNode && InPinToRemove);
	UPCGSettings* Settings = PCGNode->GetSettings();

	if (Direction == EPCGPinDirection::Input)
	{
		// @todo_pcg: To be removed with UPCGSettingsWithDynamicInputs when deprecated.
		if (UPCGSettingsWithDynamicInputs* DynamicNodeSettings = Cast<UPCGSettingsWithDynamicInputs>(Settings))
		{
			const UPCGEditorGraphNodeBase* PCGGraphNode = CastChecked<const UPCGEditorGraphNodeBase>(InPinToRemove->GetOwningNode());
			return DynamicNodeSettings->CanUserRemoveDynamicInputPin(PCGGraphNode->GetPinIndex(InPinToRemove));
		}
	}

	// New interface path
	if (const IPCGDynamicPinsProvider* Interface = GetDynamicPinsInterface())
	{
		if (const FPCGDynamicPinContainer* Container = Interface->GetDynamicPinContainer(Direction))
		{
			return Container->CanRemovePin(Container->FindPinIndex(InPinToRemove->PinName));
		}
	}

	return false;
}

void UPCGEditorGraphNodeBase::OnUserRemoveDynamicPin(UEdGraphPin* InRemovedPin, EPCGPinDirection Direction)
{
	check(PCGNode && InRemovedPin);
	UPCGSettings* Settings = PCGNode->GetSettings();

	if (Direction == EPCGPinDirection::Input)
	{
		// @todo_pcg: To be removed with UPCGSettingsWithDynamicInputs when deprecated.
		if (UPCGSettingsWithDynamicInputs* DynamicNodeSettings = Cast<UPCGSettingsWithDynamicInputs>(Settings))
		{
			if (UPCGEditorGraphNodeBase* PCGGraphNode = Cast<UPCGEditorGraphNodeBase>(InRemovedPin->GetOwningNode()))
			{
				const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorUserRemoveDynamicInputPin", "Remove Source Pin"), DynamicNodeSettings);
				DynamicNodeSettings->Modify();
				DynamicNodeSettings->OnUserRemoveDynamicInputPin(PCGGraphNode->GetPCGNode(), PCGGraphNode->GetPinIndex(InRemovedPin));
			}
			return;
		}
	}

	if (IPCGDynamicPinsProvider* Interface = GetDynamicPinsInterface())
	{
		if (FPCGDynamicPinContainer* Container = Interface->GetMutableDynamicPinContainer(Direction))
		{
			const int32 Index = Container->FindPinIndex(InRemovedPin->PinName);
			if (Index != INDEX_NONE)
			{
				const FText TransactionDesc = (Direction == EPCGPinDirection::Input)
					? LOCTEXT("PCGEditorUserRemoveDynamicInputPinInterface", "Remove Input Pin")
					: LOCTEXT("PCGEditorUserRemoveDynamicOutputPinInterface", "Remove Output Pin");
				const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, TransactionDesc, Settings);
				Settings->Modify();
				const EPCGChangeType ChangeType = Container->RemovePin(Index, PCGNode, Direction);
				Settings->OnSettingsChangedDelegate.Broadcast(Settings, ChangeType);
			}
		}
	}
}

bool UPCGEditorGraphNodeBase::CanUserRenameDynamicPin(UEdGraphPin* InPin) const
{
	if (!PCGNode || !InPin)
	{
		return false;
	}

	const IPCGDynamicPinsProvider* Interface = GetDynamicPinsInterface();
	if (!Interface)
	{
		return false;
	}

	const EPCGPinDirection Direction = (InPin->Direction == EGPD_Input) ? EPCGPinDirection::Input : EPCGPinDirection::Output;

	if (!Interface->CanUserRenameDynamicPin(Direction))
	{
		return false;
	}

	if (const FPCGDynamicPinContainer* Container = Interface->GetDynamicPinContainer(Direction))
	{
		return Container->ContainsPin(InPin->PinName);
	}

	return false;
}

void UPCGEditorGraphNodeBase::OnUserRenameDynamicPin(UEdGraphPin* InPin, FName NewLabel)
{
	check(PCGNode && InPin);

	if (IPCGDynamicPinsProvider* Interface = GetDynamicPinsInterface())
	{
		const EPCGPinDirection Direction = (InPin->Direction == EGPD_Input) ? EPCGPinDirection::Input : EPCGPinDirection::Output;

		if (FPCGDynamicPinContainer* Container = Interface->GetMutableDynamicPinContainer(Direction))
		{
			const FName PreviousName = InPin->PinName;
			const int32 Index = Container->FindPinIndex(PreviousName);
			if (Index != INDEX_NONE)
			{
				UPCGSettings* Settings = PCGNode->GetSettings();
				const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorUserRenameDynamicPin", "Rename Pin"), Settings);
				Settings->Modify();
				const EPCGChangeType ChangeType = Container->RenamePin(Index, NewLabel, PCGNode, Direction);
				if (ChangeType != EPCGChangeType::None)
				{
					Interface->OnPinRenamed(PreviousName, NewLabel, Direction);
					Settings->OnSettingsChangedDelegate.Broadcast(Settings, ChangeType);
				}
			}
		}
	}
}

void UPCGEditorGraphNodeBase::OnConvertNode(const FPCGPreconfiguredInfo& ConversionInfo)
{
	check(PCGNode && PCGNode->GetSettings());

	if (UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(GetGraph()))
	{
		if (UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph())
		{
			FText ConversionTransactionName = FText::Format(INVTEXT("{0}: {1}"), LOCTEXT("PCGEditorConvertNode", "Convert Node"), PCGNode->GetDefaultTitle());
			FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, std::move(ConversionTransactionName), PCGGraph);
			PCGNode->Modify();
			PCGNode->GetSettings()->Modify();

			if (!PCGNode->GetSettings()->ConvertNode(ConversionInfo))
			{
				// TODO: It would be useful to have an error/feedback process for it can not be converted.
				Transaction.Cancel();
				return;
			}

			EditorGraph->ReconstructGraph();
		}
	}
}

void UPCGEditorGraphNodeBase::UpdateCommentBubblePinned()
{
	if (PCGNode)
	{
		PCGNode->Modify();
		PCGNode->bCommentBubblePinned = bCommentBubblePinned;
	}
}

void UPCGEditorGraphNodeBase::UpdatePosition()
{
	if (PCGNode)
	{
		PCGNode->Modify();
		PCGNode->PositionX = NodePosX;
		PCGNode->PositionY = NodePosY;
	}
}

bool UPCGEditorGraphNodeBase::ShouldCreatePin(const UPCGPin* InPin) const
{
	return InPin && !InPin->Properties.bInvisiblePin;
}

void UPCGEditorGraphNodeBase::CreatePins(const TArray<UPCGPin*>& InInputPins, const TArray<UPCGPin*>& InOutputPins)
{
	bool bHasAdvancedPin = false;
	
	const FString NodeName = PCGNode ? PCGNode->GetName() : TEXT("UNKNOWN");
	auto InitPinGuid = [&NodeName](UEdGraphPin* Pin, bool bIn)
	{
		// Assign deterministic PinIds so FGraphDiffControl can match pins across
		// reconstructed editor graphs. PinIds are normally random GUIDs, but the editor
		// graph is transient and recreated on load, so we derive stable IDs from the
		// owning PCG node name + pin name + direction.
		const FString PinIdentity = FString::Printf(TEXT("%s_%s_%s"), *NodeName, *Pin->PinName.ToString(), bIn ? TEXT("In") : TEXT("Out"));
		Pin->PinId = FGuid::NewDeterministicGuid(PinIdentity);
	};

	for (const UPCGPin* InputPin : InInputPins)
	{
		if (!ShouldCreatePin(InputPin))
		{
			continue;
		}

		UEdGraphPin* Pin = CreatePin(EGPD_Input, GetPinType(InputPin), InputPin->Properties.Label);
		Pin->PinFriendlyName = GetPinFriendlyName(InputPin);
		Pin->bAdvancedView = InputPin->Properties.IsAdvancedPin() && !IsPinDefaultValueActivated(InputPin->Properties.Label);
		bHasAdvancedPin |= Pin->bAdvancedView;
		InitPinGuid(Pin, /*bIn=*/true);
	}

	for (const UPCGPin* OutputPin : InOutputPins)
	{
		if (!ShouldCreatePin(OutputPin))
		{
			continue;
		}

		UEdGraphPin* Pin = CreatePin(EGPD_Output, GetPinType(OutputPin), OutputPin->Properties.Label);
		Pin->PinFriendlyName = GetPinFriendlyName(OutputPin);
		Pin->bAdvancedView = OutputPin->Properties.IsAdvancedPin() && !IsPinDefaultValueActivated(OutputPin->Properties.Label);
		bHasAdvancedPin |= Pin->bAdvancedView;
		InitPinGuid(Pin, /*bIn=*/false);
	}

	if (bHasAdvancedPin && AdvancedPinDisplay == ENodeAdvancedPins::NoPins)
	{
		AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
	}
	else if (!bHasAdvancedPin)
	{
		AdvancedPinDisplay = ENodeAdvancedPins::NoPins;
	}
}

FText UPCGEditorGraphNodeBase::GetPinFriendlyName(const UPCGPin* InPin) const
{
	check(InPin);

	// For overridable params, use the display name of properties (for localized version or overridden display name in metadata).
	if (InPin->Properties.IsAdvancedPin() && InPin->Properties.AllowedTypes == EPCGDataType::Param)
	{
		const UPCGSettings* Settings = InPin->Node ? InPin->Node->GetSettings() : nullptr;
		if (Settings)
		{
			const FPCGSettingsOverridableParam* Param = Settings->OverridableParams().FindByPredicate([Label = InPin->Properties.Label](const FPCGSettingsOverridableParam& ParamToCheck)
			{
				return ParamToCheck.Label == Label;
			});

			if (Param)
			{
				return Param->GetDisplayPropertyPathText();
			}
		}
	}

	if (const IPCGDynamicPinsProvider* Interface = GetDynamicPinsInterface())
	{
		const EPCGPinDirection Direction = InPin->IsOutputPin() ? EPCGPinDirection::Output : EPCGPinDirection::Input;
		if (Interface->CanUserRenameDynamicPin(Direction))
		{
			if (const FPCGDynamicPinContainer* Container = Interface->GetDynamicPinContainer(Direction))
			{
				if (Container->PinProperties.ContainsByPredicate([&InPin](const FPCGPinProperties& Props) { return Props.Label == InPin->Properties.Label; }))
				{
					return FText::FromName(InPin->Properties.Label);
				}
			}
		}
	}

	return FText::FromString(FName::NameToDisplayString(InPin->Properties.Label.ToString(), /*bIsBool=*/false));
}

#undef LOCTEXT_NAMESPACE
