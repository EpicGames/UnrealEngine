// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/PCGEditorGraphNodeComment.h"

#include "PCGEditorGraph.h"
#include "Misc/TransactionObjectEvent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorGraphNodeComment)

void UPCGEditorGraphNodeComment::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	// If anything happens, we need to dirty the pcg graph, as it will need to serialize the changes
	if (TransactionEvent.HasPropertyChanges())
	{
		UPCGEditorGraph* PCGEditorGraph = CastChecked<UPCGEditorGraph>(GetGraph());
		if (UPCGGraph* PCGGraph = PCGEditorGraph->GetPCGGraph(); ensure(PCGGraph))
		{
			PCGGraph->Modify();	
		}
	}
}

void UPCGEditorGraphNodeComment::InitializeFromNodeData(const FPCGGraphCommentNodeData& NodeData)
{
	NodePosX = NodeData.NodePosX;
	NodePosY = NodeData.NodePosY;
	NodeWidth = NodeData.NodeWidth;
	NodeHeight = NodeData.NodeHeight;
	NodeComment = NodeData.NodeComment;
	CommentColor = NodeData.CommentColor;
	FontSize = NodeData.FontSize;
	bCommentBubbleVisible_InDetailsPanel = NodeData.bCommentBubbleVisible_InDetailsPanel;
	bColorCommentBubble = NodeData.bColorCommentBubble;
	MoveMode = static_cast<ECommentBoxMode::Type>(NodeData.MoveMode);
	NodeDetails = NodeData.NodeDetails;
	CommentDepth = NodeData.CommentDepth;
	NodeGuid = NodeData.GUID;
	bCommentBubblePinned = NodeData.bCommentBubblePinned;
	bCommentBubbleVisible = NodeData.bCommentBubbleVisible;
}
