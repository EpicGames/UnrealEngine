// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphNode_Comment.h"

#include "Editor/PCGGraphComment.h"

#include "PCGEditorGraphNodeComment.generated.h"


UCLASS()
class UPCGEditorGraphNodeComment : public UEdGraphNode_Comment
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	// ~End UObject interface
	
	void InitializeFromNodeData(const FPCGGraphCommentNodeData& NodeData);
	// Export version is the import version in FPCGGraphCommentNodeData
};