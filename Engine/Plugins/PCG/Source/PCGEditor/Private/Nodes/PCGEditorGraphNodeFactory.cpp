// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorGraphNodeFactory.h"

#include "SPCGEditorGraphNode.h"
#include "Nodes/PCGEditorGraphNodeBase.h"

TSharedPtr<SGraphNode> FPCGEditorGraphNodeFactory::CreateNode(UEdGraphNode* InNode) const
{
	if (TSharedPtr<SGraphNode> VisualNode = InNode->CreateVisualWidget())
	{
		VisualNode->SlatePrepass();
		return VisualNode;
	}

	return nullptr;
}
