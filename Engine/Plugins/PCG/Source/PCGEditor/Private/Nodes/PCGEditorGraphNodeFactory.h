// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"

class SGraphNode;
class UEdGraphNode;

class FPCGEditorGraphNodeFactory : public FGraphPanelNodeFactory
{
	virtual TSharedPtr<SGraphNode> CreateNode(UEdGraphNode* InNode) const override;
};
