// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorGraphNodeGetUserParameter.h"

#include "PCGEditorSettings.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "Elements/PCGReroute.h"
#include "Elements/PCGUserParameterGet.h"
#include "PCGEditorUtils.h"

#include "Misc/MessageDialog.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "StructUtils/PropertyBag.h"

#include "SPCGEditorGraphNodeCompact.h"

#include "Widgets/Input/SCheckBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorGraphNodeGetUserParameter)

enum class EPropertyBagAlterationResult : uint8;

#define LOCTEXT_NAMESPACE "PCGEditorGraphGetUserParameter"

void UPCGEditorGraphGetUserParameter::OnRenameNode(const FString& NewName)
{
	UPCGGraph* Graph = PCGNode->GetGraph();
	const UPCGUserParameterGetSettings* Settings = Cast<UPCGUserParameterGetSettings>(GetSettings());
	if (ensure(Graph && Settings) && NewName != Settings->PropertyName.ToString())
	{
		FText DialogMessage = FText::Format(LOCTEXT("RenameGraphParameterConfirmationMessage", "Confirm renaming node from '{0}' to '{1}'?"), FText::FromName(Settings->PropertyName), FText::FromString(NewName));
		FText CheckBoxMessage = LOCTEXT("RenameGraphParameterConfirmationCheckBoxMessage", "Disable this notification. Re-enable via user preferences.");

		if (GetDefault<UPCGEditorSettings>()->bConfirmLocalGraphParameterNameChanges)
		{
			TSharedPtr<SCheckBox> CheckBox = SNew(SCheckBox);
			TSharedRef<SWidget> DisableNotificationWidget =
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				.Padding(4, 4, 0, 4)
				[
					CheckBox.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				.Padding(4, 6, 0, 4)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
					.Text(MoveTemp(CheckBoxMessage))
				];

			bool bConfirmClicked = false;
			auto OnConfirm = [&bConfirmClicked, WeakCheckBox = CheckBox.ToWeakPtr()]
			{
				if (TSharedPtr<SCheckBox> CheckBoxPtr = WeakCheckBox.Pin())
				{
					if (CheckBoxPtr->IsChecked())
					{
						GetMutableDefault<UPCGEditorSettings>()->bConfirmLocalGraphParameterNameChanges = false;
					}
				}

				bConfirmClicked = true;
			};

			PCGEditorUtils::OpenConfirmationDialog(MoveTemp(DialogMessage), OnConfirm, /*OnCancel=*/[] {}, DisableNotificationWidget);

			if (!bConfirmClicked)
			{
				return;
			}
		}

		const EPropertyBagAlterationResult RenameResult = Graph->RenameUserParameter(Settings->PropertyName, FName(NewName));
		if (RenameResult != EPropertyBagAlterationResult::Success)
		{
			FText ReasonText;
			switch (RenameResult)
			{
				case EPropertyBagAlterationResult::PropertyNameEmpty:
					ReasonText = LOCTEXT("RenameFailureEmptyName", "Empty property name.");
					break;
				case EPropertyBagAlterationResult::PropertyNameInvalidCharacters:
					ReasonText = LOCTEXT("RenameFailureInvalidCharacters", "Invalid characters in property name.");
					break;
				case EPropertyBagAlterationResult::SourcePropertyNotFound:
					ReasonText = LOCTEXT("RenameFailureSourcePropertyNotFound", "Source property wasn't found.");
					break;
				case EPropertyBagAlterationResult::TargetPropertyAlreadyExists:
					ReasonText = LOCTEXT("RenameFailureTargetPropertyAlreadyExists", "Target property already exists.");
					break;
				default:
					break;
			}
			const FText ErrorMessage = FText::Format(LOCTEXT("RenameGraphParameterFailure", "Failed to rename graph parameter: '{0}' to '{1}'. {2}"), FText::FromName(Settings->PropertyName), FText::FromString(NewName), std::move(ReasonText));

			FMessageDialog::Open(EAppMsgType::Ok, std::move(ErrorMessage));
		}

		const FString NodeTitle = GetNodeTitle(ENodeTitleType::FullTitle).ToString();
		PCGNode->SetNodeTitle(FName(NodeTitle));
		Super::OnRenameNode(NodeTitle);
	}
}

bool UPCGEditorGraphGetUserParameter::OnValidateNodeTitle(const FText& NewName, FText& OutErrorMessage)
{
	if (!Super::OnValidateNodeTitle(NewName, OutErrorMessage))
	{
		return false;
	}

	const FName Name(NewName.ToString());

	if (!FInstancedPropertyBag::IsPropertyNameValid(Name))
	{
		OutErrorMessage = LOCTEXT("InvalidPropertyNameInvalidCharacters", "Invalid character(s)");
		return false;
	}

	// Prevent name clashing with any existant Named Reroute or Graph Parameter node, to avoid confusion in the graph and graph context action search menu.
	if (const UPCGGraph* PCGGraph = PCGNode ? PCGNode->GetGraph() : nullptr)
	{
		if (PCGGraph->FindNodeByTitleName(Name, /*bRecursive=*/false, UPCGUserParameterGetSettings::StaticClass()))
		{
			OutErrorMessage = LOCTEXT("NameAlreadyInUseUserParameterErrorMessage", "Name already in use (Graph Parameter)");
			return false;
		}

		if (PCGGraph->FindNodeByTitleName(Name, /*bRecursive=*/false, UPCGNamedRerouteDeclarationSettings::StaticClass()))
		{
			OutErrorMessage = LOCTEXT("NameAlreadyInUseNamedRerouteErrorMessage", "Name already in use: (Named Reroute)");
			return false;
		}
	}

	return true;
}

TSharedPtr<SGraphNode> UPCGEditorGraphGetUserParameter::CreateVisualWidget()
{
	return SNew(SPCGEditorGraphNodeCompact, this);
}

FText UPCGEditorGraphGetUserParameter::GetNodeTitle(const ENodeTitleType::Type TitleType) const
{
	const UPCGUserParameterGetSettings* Settings = CastChecked<UPCGUserParameterGetSettings>(GetSettings());
	if (ensure(Settings))
	{
		if (TitleType == ENodeTitleType::EditableTitle)
		{
			return FText::FromString(FName::NameToDisplayString(Settings->PropertyName.ToString(), /*bIsBool=*/false));
		}
		else if (TitleType == ENodeTitleType::MenuTitle)
		{
			return FText::FromName(Settings->PropertyName);
		}
		else if (TitleType == ENodeTitleType::ListView)
		{
			return FText::Format(LOCTEXT("GetUserParamListViewTitleFormat", "Graph Param: {0}"), FText::FromName(Settings->PropertyName));
		}
	}

	return Super::GetNodeTitle(TitleType);
}

#undef LOCTEXT_NAMESPACE
