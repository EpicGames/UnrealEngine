// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/PCGEditorGraphNode.h"

#include "PCGEditorGraphNodeGetUserParameter.generated.h"

// @todo_pcg: A 'graph parameter' icon or other symbology on the node would be more UX friendly

UCLASS()
class UPCGEditorGraphGetUserParameter : public UPCGEditorGraphNode
{
	GENERATED_BODY()

public:
	//~ Begin UPCGEditorGraphNode Interface
	virtual void OnRenameNode(const FString& NewName) override;
	virtual bool OnValidateNodeTitle(const FText& NewName, FText& OutErrorMessage) override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	//~ End UPCGEditorGraphNode Interface

	// ~Begin UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// ~End UEdGraphNode interface
};
