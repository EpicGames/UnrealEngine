// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorGraphNodeInput.h"

#include "PCGNode.h"
#include "PCGPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorGraphNodeInput)

#define LOCTEXT_NAMESPACE "PCGEditorGraphNodeInput"

FText UPCGEditorGraphNodeInput::GetNodeTitle(ENodeTitleType::Type TitleType) const
{	
	return LOCTEXT("NodeTitle", "Input");
}

void UPCGEditorGraphNodeInput::AllocateDefaultPins()
{
	if (PCGNode)
	{
		CreatePins(/*InInputPins=*/{}, PCGNode->GetOutputPins());
	}
}

void UPCGEditorGraphNodeInput::ReconstructNode()
{
	Super::ReconstructNode();
	//TODO: Implement special version for input to avoid the enum type
}

#undef LOCTEXT_NAMESPACE
