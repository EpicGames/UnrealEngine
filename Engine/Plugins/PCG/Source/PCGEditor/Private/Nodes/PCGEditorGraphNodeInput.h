// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/PCGEditorGraphNode.h"

#include "PCGEditorGraphNodeInput.generated.h"

namespace ENodeTitleType { enum Type : int; }

UCLASS()
class UPCGEditorGraphNodeInput : public UPCGEditorGraphNode
{
	GENERATED_BODY()

public:
	UPCGEditorGraphNodeInput(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
		bCanRenameNode = false;
	}

	// ~Begin UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AllocateDefaultPins() override;
	virtual bool CanUserDeleteNode() const override { return false; }
	virtual bool CanDuplicateNode() const override { return false; }
	virtual void ReconstructNode() override;
	// ~End UEdGraphNode interface
};
