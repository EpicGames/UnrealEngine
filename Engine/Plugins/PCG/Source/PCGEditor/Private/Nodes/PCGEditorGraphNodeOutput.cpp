// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorGraphNodeOutput.h"

#include "PCGNode.h"
#include "PCGPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorGraphNodeOutput)

#define LOCTEXT_NAMESPACE "PCGEditorGraphNodeOutput"

FText UPCGEditorGraphNodeOutput::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Output");
}

void UPCGEditorGraphNodeOutput::AllocateDefaultPins()
{
	if (PCGNode)
	{
		CreatePins(PCGNode->GetInputPins(), /*InOutputPins=*/{});
	}
}

void UPCGEditorGraphNodeOutput::ReconstructNode()
{
	Super::ReconstructNode();
	//TODO: Implement special version for output to avoid the enum type
}

#undef LOCTEXT_NAMESPACE
