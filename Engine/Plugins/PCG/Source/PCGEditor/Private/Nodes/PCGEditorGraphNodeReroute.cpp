// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorGraphNodeReroute.h"

#include "PCGEditorGraph.h"

#include "PCGNode.h"
#include "Elements/PCGReroute.h"
#include "Elements/PCGUserParameterGet.h"

#include "EdGraph/EdGraphPin.h"

#include "SPCGEditorGraphNodeCompact.h"

#include "SGraphNodeKnot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorGraphNodeReroute)

#define LOCTEXT_NAMESPACE "PCGEditorGraphNodeReroute"

class SPCGEditorGraphNodeKnot : public SGraphNodeKnot
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphNodeKnot) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UPCGEditorGraphNodeBase* InPCGGraphNode)
	{
		SGraphNodeKnot::Construct(SGraphNodeKnot::FArguments(), InPCGGraphNode);
		InPCGGraphNode->OnNodeChangedDelegate.BindSP(this, &SPCGEditorGraphNodeKnot::OnNodeChanged);
	}

private:
	void OnNodeChanged()
	{
		UpdateGraphNode();
	}
};

UPCGEditorGraphNodeReroute::UPCGEditorGraphNodeReroute(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRenameNode = false;
}

FText UPCGEditorGraphNodeReroute::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("PCGEditorGraphNodeReroute", "NodeTitle", "Reroute");
}

bool UPCGEditorGraphNodeReroute::ShouldOverridePinNames() const
{
	return true;
}

FText UPCGEditorGraphNodeReroute::GetPinNameOverride(const UEdGraphPin& Pin) const
{
	return FText::GetEmpty();
}

bool UPCGEditorGraphNodeReroute::CanSplitPin(const UEdGraphPin* Pin) const
{
	return false;
}

bool UPCGEditorGraphNodeReroute::ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const
{
	OutInputPinIndex = 0;
	OutOutputPinIndex = 1;
	return true;
}

FText UPCGEditorGraphNodeReroute::GetTooltipText() const
{
	return FText::GetEmpty();
}

UEdGraphPin* UPCGEditorGraphNodeReroute::GetPassThroughPin(const UEdGraphPin* FromPin) const
{
	if (FromPin == GetInputPin())
	{
		return GetOutputPin();
	}
	else
	{
		return GetInputPin();
	}
}

TSharedPtr<SGraphNode> UPCGEditorGraphNodeReroute::CreateVisualWidget()
{
	return SNew(SPCGEditorGraphNodeKnot, this);
}

UEdGraphPin* UPCGEditorGraphNodeReroute::GetInputPin() const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EGPD_Input)
		{
			return Pin;
		}
	}

	return nullptr;
}

UEdGraphPin* UPCGEditorGraphNodeReroute::GetOutputPin() const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EGPD_Output)
		{
			return Pin;
		}
	}

	return nullptr;
}

FText UPCGEditorGraphNodeNamedRerouteBase::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	// Never reformat the node name, as we don't want to have nodes that seems the same but ar not (example, "As_A" and "As A" would appear the same)
	return PCGNode ? FText::FromName(PCGNode->NodeTitle) : FText{};
}

TSharedPtr<SGraphNode> UPCGEditorGraphNodeNamedRerouteBase::CreateVisualWidget()
{
	return SNew(SPCGEditorGraphNodeCompact, this);
}

bool UPCGEditorGraphNodeNamedRerouteBase::OnValidateNodeTitle(const FText& NewName, FText& OutErrorMessage)
{
	if (!Super::OnValidateNodeTitle(NewName, OutErrorMessage))
	{
		return false;
	}

	const FName Name(NewName.ToString());

	// Prevent name clashing with any existant Named Reroute or Graph Parameter node, to avoid confusion in the graph and graph context action search menu.
	if (const UPCGGraph* PCGGraph = PCGNode ? PCGNode->GetGraph() : nullptr)
	{
		if (PCGGraph->FindNodeByTitleName(Name, /*bRecursive=*/false, UPCGNamedRerouteDeclarationSettings::StaticClass()))
		{
			OutErrorMessage = LOCTEXT("NameAlreadyInUseNamedRerouteErrorMessage", "Name already in use: (Named Reroute)");
			return false;
		}

		if (PCGGraph->FindNodeByTitleName(Name, /*bRecursive=*/false, UPCGUserParameterGetSettings::StaticClass()))
		{
			OutErrorMessage = LOCTEXT("NameAlreadyInUseUserParameterErrorMessage", "Name already in use (Graph Parameter)");
			return false;
		}
	}

	return true;
}

void UPCGEditorGraphNodeNamedRerouteUsage::OnRenameNode(const FString& NewName)
{
	ApplyToDeclarationNode([&NewName](UPCGEditorGraphNodeNamedRerouteDeclaration* Declaration)
	{
		Declaration->OnRenameNode(NewName);
	});
}

void UPCGEditorGraphNodeNamedRerouteUsage::InheritRename(const FString& NewName)
{
	UPCGEditorGraphNodeNamedRerouteBase::OnRenameNode(NewName);
}

FText UPCGEditorGraphNodeNamedRerouteUsage::GetPinFriendlyName(const UPCGPin* InPin) const
{
	return FText::FromString(" ");
}

void UPCGEditorGraphNodeNamedRerouteUsage::ApplyToDeclarationNode(TFunctionRef<void(UPCGEditorGraphNodeNamedRerouteDeclaration*)> Action) const
{
	if (const UPCGNamedRerouteUsageSettings* Settings = Cast<UPCGNamedRerouteUsageSettings>(GetSettings()))
	{
		if (const UPCGNamedRerouteDeclarationSettings* Declaration = Cast<UPCGNamedRerouteDeclarationSettings>(Settings->Declaration))
		{
			if (UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(GetGraph()))
			{
				for (const TObjectPtr<UEdGraphNode>& EdGraphNode : EditorGraph->Nodes)
				{
					if (UPCGEditorGraphNodeNamedRerouteDeclaration* RerouteDeclaration = Cast<UPCGEditorGraphNodeNamedRerouteDeclaration>(EdGraphNode))
					{
						if (RerouteDeclaration->GetSettings() == Declaration)
						{
							Action(RerouteDeclaration);
							break;
						}
					}
				}
			}
		}
	}
}

void UPCGEditorGraphNodeNamedRerouteDeclaration::OnRenameNode(const FString& NewName)
{
	check(PCGNode);
	if (TOptional<FName> ModifiedName = UPCGRerouteSettings::GetCollisionFreeNodeName(PCGNode->GetGraph(), FName(NewName)))
	{
		// Propagate the name change to downstream usage nodes
		ApplyToUsageNodes([ModifiedName = *ModifiedName](UPCGEditorGraphNodeNamedRerouteUsage* RerouteNode)
		{
			RerouteNode->InheritRename(ModifiedName.ToString());
		});

		Super::OnRenameNode(ModifiedName->ToString());
		ReconstructNodeOnChange();
	}
	else
	{
		Super::OnRenameNode(NewName);
	}
}

void UPCGEditorGraphNodeNamedRerouteUsage::RebuildEdgesFromPins_Internal()
{
	Super::RebuildEdgesFromPins_Internal();

	check(PCGNode);

	if (PCGNode->HasInboundEdges())
	{
		return;
	}

	UPCGGraph* Graph = PCGNode->GetGraph();

	if (!Graph)
	{
		return;
	}

	UPCGNamedRerouteUsageSettings* Usage = Cast<UPCGNamedRerouteUsageSettings>(PCGNode->GetSettings());

	if (!Usage)
	{
		return;
	}

	// Make sure we're hooked to the declaration if it's not already the case
	if (UPCGNode* DeclarationNode = Graph->FindNodeWithSettings(Usage->Declaration))
	{
		DeclarationNode->AddEdgeTo(PCGNamedRerouteConstants::InvisiblePinLabel, PCGNode, PCGPinConstants::DefaultInputLabel);
	}
}

FText UPCGEditorGraphNodeNamedRerouteDeclaration::GetPinFriendlyName(const UPCGPin* InPin) const
{
	return FText::FromString(" ");
}

void UPCGEditorGraphNodeNamedRerouteDeclaration::ApplyToUsageNodes(TFunctionRef<void(UPCGEditorGraphNodeNamedRerouteUsage*)> Action)
{
	if (!GetPCGNode())
	{
		return;
	}

	const UPCGNamedRerouteDeclarationSettings* Declaration = Cast<UPCGNamedRerouteDeclarationSettings>(GetPCGNode()->GetSettings());

	if (!Declaration)
	{
		return;
	}

	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(GetGraph());

	if (!EditorGraph)
	{
		return;
	}

	for (const TObjectPtr<UEdGraphNode>& EdGraphNode : EditorGraph->Nodes)
	{
		if (UPCGEditorGraphNodeNamedRerouteUsage* RerouteNode = Cast<UPCGEditorGraphNodeNamedRerouteUsage>(EdGraphNode))
		{
			if (RerouteNode->GetPCGNode() &&
				Cast<UPCGNamedRerouteUsageSettings>(RerouteNode->GetPCGNode()->GetSettings()) &&
				Cast<UPCGNamedRerouteUsageSettings>(RerouteNode->GetPCGNode()->GetSettings())->Declaration == Declaration)
			{
				Action(RerouteNode);
			}
		}
	}
}

void UPCGEditorGraphNodeNamedRerouteDeclaration::ReconstructNodeOnChange()
{
	Super::ReconstructNodeOnChange();

	// We must make sure to trigger a notify node changed on all editor nodes that are usages of that declaration
	ApplyToUsageNodes([](UPCGEditorGraphNodeNamedRerouteUsage* RerouteNode)
	{
		RerouteNode->ReconstructNodeOnChange();
	});
}

FString UPCGEditorGraphNodeNamedRerouteDeclaration::GenerateNodeName(const UPCGNode* FromNode, FName FromPinName)
{
	if (FromNode)
	{
		return FromNode->GetNodeTitle(EPCGNodeTitleType::ListView).ToString() + " " + FromPinName.ToString();
	}
	else if (FromPinName != NAME_None)
	{
		return FromPinName.ToString();
	}
	else
	{
		return TEXT("Reroute");
	}
}

void UPCGEditorGraphNodeNamedRerouteDeclaration::OnColorPicked(FLinearColor NewColor)
{
	Super::OnColorPicked(NewColor);

	// Propagate change to downstream usage nodes
	ApplyToUsageNodes([&NewColor](UPCGEditorGraphNodeNamedRerouteUsage* RerouteNode)
	{
		RerouteNode->OnColorPicked(NewColor);
	});
}

void UPCGEditorGraphNodeNamedRerouteDeclaration::PostPaste()
{
	Super::PostPaste();
	FixNodeNameCollision();
}

void UPCGEditorGraphNodeNamedRerouteDeclaration::FixNodeNameCollision()
{
	const FString BaseName = GetNodeTitle(ENodeTitleType::ListView).ToString();

	OnRenameNode(BaseName);
}

#undef LOCTEXT_NAMESPACE
