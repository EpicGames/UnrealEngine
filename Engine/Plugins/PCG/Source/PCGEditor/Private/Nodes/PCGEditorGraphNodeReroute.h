// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/PCGEditorGraphNode.h"

class UPCGNode;

#include "PCGEditorGraphNodeReroute.generated.h"

UCLASS()
class UPCGEditorGraphNodeReroute : public UPCGEditorGraphNode
{
	GENERATED_BODY()

public:
	UPCGEditorGraphNodeReroute(const FObjectInitializer& ObjectInitializer);

	// ~Begin UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldOverridePinNames() const override;
	virtual FText GetPinNameOverride(const UEdGraphPin& Pin) const override;
	virtual bool CanSplitPin(const UEdGraphPin* Pin) const override;
	virtual bool ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const override;
	virtual FText GetTooltipText() const override;
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* FromPin) const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	// ~End UEdGraphNode interface

	UEdGraphPin* GetInputPin() const;
	UEdGraphPin* GetOutputPin() const;
};

UCLASS()
class UPCGEditorGraphNodeNamedRerouteBase : public UPCGEditorGraphNode
{
	GENERATED_BODY()

public:
	// ~Begin UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	// ~End UEdGraphNode interface

	// ~Begin UPCGEditorGraphNode interface
	virtual bool OnValidateNodeTitle(const FText& NewName, FText& OutErrorMessage) override;
	// ~End UPCGEditorGraphNode interface
};

UCLASS()
class UPCGEditorGraphNodeNamedRerouteUsage : public UPCGEditorGraphNodeNamedRerouteBase
{
	GENERATED_BODY()
	friend class UPCGEditorGraphNodeNamedRerouteDeclaration;

public:
	virtual void OnRenameNode(const FString& NewName) override;

	void InheritRename(const FString& NewName);

protected:
	virtual void RebuildEdgesFromPins_Internal() override;
	virtual bool CanPickColor() const override { return false; }
	virtual FText GetPinFriendlyName(const UPCGPin* InPin) const override;

	void ApplyToDeclarationNode(TFunctionRef<void(UPCGEditorGraphNodeNamedRerouteDeclaration*)> Action) const;
};

UCLASS()
class UPCGEditorGraphNodeNamedRerouteDeclaration : public UPCGEditorGraphNodeNamedRerouteBase
{
	GENERATED_BODY()

public:
	// ~Begin UPCGEditorGraphNodeBase interface
	virtual void PostPaste() override;
	// ~End UPCGEditorGraphNodeBase interface

	// ~Begin UPCGEditorGraphNode interface
	virtual void OnRenameNode(const FString& NewName) override;
	// ~End UPCGEditorGraphNode interface

	FString GenerateNodeName(const UPCGNode* FromNode, FName FromPinName);
	void FixNodeNameCollision();

protected:
	virtual FText GetPinFriendlyName(const UPCGPin* InPin) const override;
	virtual void OnColorPicked(FLinearColor NewColor) override;
	virtual void ReconstructNodeOnChange() override;

	void ApplyToUsageNodes(TFunctionRef<void(UPCGEditorGraphNodeNamedRerouteUsage*)> Action);
};