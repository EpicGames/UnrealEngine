// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphNode.h"

#include "PCGComponent.h"
#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "PCGSettingsWithDynamicInputs.h"
#include "AssetEditorMode/PCGAssetEdMode.h"
#include "Metadata/PCGDefaultValueInterface.h"

#include "Nodes/PCGEditorGraphNodeBase.h"
#include "PCGEditorStyle.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Pins/SPCGEditorGraphNodePin.h"
#include "Pins/SPCGEditorGraphPinBool.h"
#include "Pins/SPCGEditorGraphPinNumSlider.h"
#include "Pins/SPCGEditorGraphPinString.h"
#include "Pins/SPCGEditorGraphPinQuaternion.h"
#include "Pins/SPCGEditorGraphPinSoftClassPath.h"
#include "Pins/SPCGEditorGraphPinSoftObjectPath.h"
#include "Pins/SPCGEditorGraphPinTransform.h"
#include "Pins/SPCGEditorGraphPinVectorSlider.h"
#include "Schema/PCGEditorGraphSchema.h"

#include "AssetThumbnail.h"
#include "GraphEditorSettings.h"
#include "IDocumentation.h"
#include "Nodes/PCGEditorGraphNode.h"
#include "SCommentBubble.h"
#include "SGraphPin.h"
#include "SPinTypeSelector.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "ThumbnailRendering/ThumbnailManager.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SPCGEditorGraphNode"

namespace PCGEditorGraphNode
{
	constexpr float OverlayIconSize = 16.0f;
	constexpr float OverlayHalfIconSize = OverlayIconSize / 2.0f;
	constexpr float TitleButtonIconSize = 16.0f;

	constexpr FLinearColor OrphanedOverrideButtonColor(1.0f, 0.5f, 0.0f, 1.0f);
	constexpr FLinearColor TemporaryManualEditButtonColor(1.0f, 0.9f, 0.0f, 1.0f);
}

void SPCGEditorGraphNode::Construct(const FArguments& InArgs, UPCGEditorGraphNodeBase* InNode)
{
	// Change right Margin from 30.0f to 5.0f (For Tool Start button)
	// The left-over padding is added to the Title Widget to maintain the same width of the node
	// The final padding is 4.0f + 16.0f (Button size) + 10.0f (Title Padding) = 30.0f. See PCGEditorGraphNode::TitleButtonIconSize
	TitleBorderMargin.Right = 4.0f;
	
	GraphNode = InNode;
	PCGEditorGraphNode = InNode;

	if (InNode)
	{
		InNode->OnNodeChangedDelegate.BindSP(this, &SPCGEditorGraphNode::OnNodeChanged);
		InNode->OnRequestPinRenameDelegate.BindSP(this, &SPCGEditorGraphNode::OnRequestPinRename);
	}

	// Create GPU transfer indicator widgets once and reuse them across GetOverlayWidgets() calls. A stable widget pointer is required so that Slate can match the widget seen
	// during OnArrangeChildren (hit-testing) with the one seen during Paint, which is the prerequisite for hover state tracking and tooltip display.
	GPUUploadIndicatorWidget =
		SNew(SImage)
		.Image(FPCGEditorStyle::Get().GetBrush(TEXT("PCG.NodeOverlay.GPUUpload")))
		.ToolTipText(LOCTEXT("GPUUploadTooltip", "One or more outputs of this node were uploaded from the CPU to the GPU. Uploads are often expensive and should be minimized."));

	GPUReadbackIndicatorWidget =
		SNew(SImage)
		.Image(FPCGEditorStyle::Get().GetBrush(TEXT("PCG.NodeOverlay.GPUReadback")))
		.ToolTipText(LOCTEXT("GPUReadbackTooltip", "One or more inputs of this node were read back form the GPU to the CPU. Readbacks are often expensive and should be minimized."));

	UpdateGraphNode();
}

void SPCGEditorGraphNode::CreateDynamicPinButton(EPCGPinDirection Direction)
{
	const bool bIsInput = (Direction == EPCGPinDirection::Input);
	const FText TooltipText = bIsInput ? LOCTEXT("AddInputPinTooltip", "Add a dynamic input pin") : LOCTEXT("AddOutputPinTooltip", "Add a dynamic output pin");

	// Cannot use AddPinButtonContent() because it hardcodes OnAddPin (input-only).
	const TSharedRef<SWidget> Icon = SNew(SImage).Image(FAppStyle::GetBrush(TEXT("Icons.PlusCircle")));
	const TSharedRef<SWidget> Text = SNew(STextBlock).Text(LOCTEXT("AddPin", "Add Pin")).ColorAndOpacity(FLinearColor::White);

	// Icon on the pin side, text on the inner side.
	TSharedRef<SHorizontalBox> ButtonContent = SNew(SHorizontalBox);

	ButtonContent->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			bIsInput ? Icon : Text
		];

	ButtonContent->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(bIsInput ? FMargin(0, 0, 7, 0) : FMargin(7, 0, 0, 0))
		[
			bIsInput ? Text : Icon
		];

	TSharedRef<SButton> AddPinButton = SNew(SButton)
		.ContentPadding(0)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.OnClicked_Lambda([this, Direction]() { return OnAddDynamicPin(Direction); })
		.IsEnabled(this, &SGraphNode::IsNodeEditable)
		.ToolTipText(TooltipText)
		.Visibility_Lambda([this, Direction]() { return IsDynamicPinButtonVisible(Direction); })
		[
			ButtonContent
		];

	AddPinButton->SetCursor(EMouseCursor::Hand);

	FMargin AddPinPadding = bIsInput ? Settings->GetInputPinPadding() : Settings->GetOutputPinPadding();
	AddPinPadding.Top += 6.0f;

	TSharedPtr<SVerticalBox> TargetBox = bIsInput ? LeftNodeBox : RightNodeBox;
	TargetBox->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Bottom)
		.Padding(AddPinPadding)
		[
			AddPinButton
		];
}

FReply SPCGEditorGraphNode::OnAddDynamicPin(const EPCGPinDirection Direction)
{
	check(PCGEditorGraphNode);
	PCGEditorGraphNode->OnUserAddDynamicPin(Direction);
	return FReply::Handled();
}

EVisibility SPCGEditorGraphNode::IsDynamicPinButtonVisible(const EPCGPinDirection Direction) const
{
	if (!PCGEditorGraphNode || !PCGEditorGraphNode->IsNodeEnabled() || SGraphNode::IsAddPinButtonVisible() != EVisibility::Visible)
	{
		return EVisibility::Hidden;
	}

	const bool bCanAdd = PCGEditorGraphNode->CanUserAddRemoveDynamicPins(Direction);

	return bCanAdd ? EVisibility::Visible : EVisibility::Hidden;
}

void SPCGEditorGraphNode::OnRequestPinRename(UEdGraphPin* InPin)
{
	if (const TSharedPtr<SGraphPin> PinWidget = FindWidgetForPin(InPin))
	{
		if (SPCGEditorGraphNodePin* PCGPin = static_cast<SPCGEditorGraphNodePin*>(PinWidget.Get()))
		{
			PCGPin->EnterLabelEditMode();
		}
	}
}

TSharedRef<SWidget> SPCGEditorGraphNode::CreateNodeToolButtonWidget()
{
	const FSlateIcon InteractIcon = FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), PCGEditorStyleConstants::Node_ToolStart);
	return SNew(SBox)
		.Padding(0, 0, 0, 0)
		.HAlign(HAlign_Right)
		.Visibility_Lambda([this]()
		{
			if (!PCGEditorGraphNode || !PCGEditorGraphNode->IsInteractiveNode())
			{
				return EVisibility::Collapsed;
			}

			const UPCGSettings* NodeSettings = PCGEditorGraphNode->GetSettings();
			return (NodeSettings && NodeSettings->GetNodeToolStartBehaviour() == EPCGNodeToolStartBehavior::OnToolStartButton) ? EVisibility::Visible : EVisibility::Collapsed;
		})
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("ViewportInteractButtonTooltip", "Click for viewport interaction"))
			.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
			.IsFocusable(false)
			.Padding(0.0f)
			.IsChecked_Lambda([this]()
				{
					if (const UPCGSettings* PCGSettings = PCGEditorGraphNode->GetSettings())
					{
						return PCGSettings->IsNodeToolActive() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}
					return ECheckBoxState::Unchecked;
				})
			.OnCheckStateChanged_Lambda(
				[this](ECheckBoxState State)
					{
						UPCGEditorGraph* const PCGEditorGraph = Cast<UPCGEditorGraph>(PCGEditorGraphNode->GetGraph());
						if (!PCGEditorGraph)
						{
							return;
						}

						UPCGGraph* const PCGGraph = PCGEditorGraph->GetPCGGraph();
						if (!PCGGraph)
						{
							return;
						}

						if (const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorGraph->GetEditor().Pin())
						{
							if (State == ECheckBoxState::Checked)
							{
								PCGEditor->OnNodeToolStarted(PCGEditorGraphNode);
							}
							else if (State == ECheckBoxState::Unchecked)
							{
								PCGEditor->OnNodeToolEnded(PCGEditorGraphNode);
							}
						}
					})

			[
				SNew(SImage)
				.Image(InteractIcon.GetIcon())
				.DesiredSizeOverride(FVector2D(PCGEditorGraphNode::TitleButtonIconSize, PCGEditorGraphNode::TitleButtonIconSize))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
}

TSharedRef<SWidget> SPCGEditorGraphNode::CreateDataOverrideButton() const
{
	const FSlateIcon DataOverrideIcon = FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), PCGEditorStyleConstants::Node_DataOverride);

	const TAttribute<EVisibility> ButtonVisibility = TAttribute<EVisibility>::CreateLambda([this]()
	{
		return (PCGEditorGraphNode && PCGEditorGraphNode->GetHasDefinedOverrides()) ? EVisibility::Visible : EVisibility::Collapsed;
	});

	const TAttribute<FSlateColor> ColorAttribute = TAttribute<FSlateColor>::CreateLambda([this]() -> FSlateColor
	{
		return (PCGEditorGraphNode && PCGEditorGraphNode->GetHasOrphanedOverrides()) ? FSlateColor(PCGEditorGraphNode::OrphanedOverrideButtonColor) : FSlateColor(FLinearColor::White);
	});

	return SNew(SCheckBox)
		.Visibility(ButtonVisibility)
		.ToolTipText_Lambda([this]() -> FText
		{
			if (PCGEditorGraphNode && PCGEditorGraphNode->GetHasOrphanedOverrides())
			{
				return LOCTEXT("DataOverrideOrphanedTooltip", "There are orphaned overrides. Open Data Overrides for this node.");
			}
			else
			{
				return LOCTEXT("DataOverrideTooltip", "Open Data Overrides for this node.");
			}
		})
		.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
		.IsFocusable(false)
		.Padding(0.0f)
		.IsChecked_Lambda([]() { return ECheckBoxState::Unchecked; })
		.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
		{
			if (State != ECheckBoxState::Checked || !PCGEditorGraphNode)
			{
				return;
			}

			const UPCGEditorGraph* PCGEditorGraph = Cast<UPCGEditorGraph>(PCGEditorGraphNode->GetGraph());
			if (!PCGEditorGraph)
			{
				return;
			}

			if (const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorGraph->GetEditor().Pin())
			{
				PCGEditor->OpenDataOverridesAndInspect(PCGEditorGraphNode);
			}
		})
		[
			SNew(SImage)
			.Image(DataOverrideIcon.GetIcon())
			.DesiredSizeOverride(FVector2D(PCGEditorGraphNode::TitleButtonIconSize, PCGEditorGraphNode::TitleButtonIconSize))
			.ColorAndOpacity(ColorAttribute)
		];
}

TSharedRef<SWidget> SPCGEditorGraphNode::CreateManualEditButton() const
{
	const FSlateIcon ManualEditIcon = FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), PCGEditorStyleConstants::Node_ManualEdit);

	const TAttribute<EVisibility> ButtonVisibility = TAttribute<EVisibility>::CreateLambda([this]()
	{
		if (!PCGEditorGraphNode)
		{
			return EVisibility::Collapsed;
		}

		const UPCGNode* PCGNode = PCGEditorGraphNode->GetPCGNode();
		const UPCGSettingsInterface* SettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;
		if (!SettingsInterface || (!SettingsInterface->IsMarkedForManualEditing() && !SettingsInterface->IsTemporaryManualEditingEnabled()))
		{
			return EVisibility::Collapsed;
		}

		const UPCGEditorGraph* PCGEditorGraph = Cast<UPCGEditorGraph>(PCGEditorGraphNode->GetGraph());
		const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorGraph ? PCGEditorGraph->GetEditor().Pin() : nullptr;
		if (!PCGEditor.IsValid() || !Cast<UPCGComponent>(PCGEditor->GetPCGSourceBeingInspected()))
		{
			return EVisibility::Collapsed;
		}

		return EVisibility::Visible;
	});

	const TAttribute<FSlateColor> ColorAttribute = TAttribute<FSlateColor>::CreateLambda([this]() -> FSlateColor
	{
		const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
		const UPCGSettingsInterface* SettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;
		return (SettingsInterface && SettingsInterface->IsTemporaryManualEditingEnabled()) ? FSlateColor(PCGEditorGraphNode::TemporaryManualEditButtonColor) : FSlateColor(FLinearColor::White);
	});

	return SNew(SCheckBox)
		.Visibility(ButtonVisibility)
		.ToolTipText(LOCTEXT("ManualEditTooltip", "Select parent actor and focus viewport."))
		.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
		.IsFocusable(false)
		.Padding(0.0f)
		.IsChecked_Lambda([]() { return ECheckBoxState::Unchecked; })
		.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
		{
			if (State != ECheckBoxState::Checked || !PCGEditorGraphNode)
			{
				return;
			}

			const UPCGEditorGraph* PCGEditorGraph = Cast<UPCGEditorGraph>(PCGEditorGraphNode->GetGraph());
			const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorGraph ? PCGEditorGraph->GetEditor().Pin() : nullptr;
			if (PCGEditor.IsValid())
			{
				PCGEditor->FocusOwningActorInLevelViewport();
				PCGEditor->SelectManualEditNode(PCGEditorGraphNode);
			}
		})
		[
			SNew(SImage)
			.Image(ManualEditIcon.GetIcon())
			.DesiredSizeOverride(FVector2D(PCGEditorGraphNode::TitleButtonIconSize, PCGEditorGraphNode::TitleButtonIconSize))
			.ColorAndOpacity(ColorAttribute)
		];
}

void SPCGEditorGraphNode::UpdateGraphNode()
{
	SGraphNode::UpdateGraphNode();

	if (PCGEditorGraphNode->CanUserAddRemoveDynamicPins(EPCGPinDirection::Input))
	{
		CreateDynamicPinButton(EPCGPinDirection::Input);
	}

	if (PCGEditorGraphNode->CanUserAddRemoveDynamicPins(EPCGPinDirection::Output))
	{
		CreateDynamicPinButton(EPCGPinDirection::Output);
	}
}

const FSlateBrush* SPCGEditorGraphNode::GetNodeBodyBrush() const
{
	const bool bNeedsTint = PCGEditorGraphNode && PCGEditorGraphNode->GetPCGNode() && PCGEditorGraphNode->GetPCGNode()->IsInstance();
	if (bNeedsTint)
	{
		return FAppStyle::GetBrush("Graph.Node.TintedBody");
	}
	else
	{
		return FAppStyle::GetBrush("Graph.Node.Body");
	}
}

TSharedRef<SWidget> SPCGEditorGraphNode::CreateTitleWidget(TSharedPtr<SNodeTitle> InNodeTitle)
{
	// Reimplementation of the SGraphNode::CreateTitleWidget so we can control the style
	const bool bIsInstanceNode = (PCGEditorGraphNode && PCGEditorGraphNode->GetPCGNode() && PCGEditorGraphNode->GetPCGNode()->IsInstance());
	
	TSharedRef<SBox> ReturnWidget = SNew(SBox)
		.Padding(0, 0, 10.0f, 0)
		[
			SAssignNew(InlineEditableText, SInlineEditableTextBlock)
			.Style(FPCGEditorStyle::Get(), bIsInstanceNode ? "PCG.Node.InstancedNodeTitleInlineEditableText" : "PCG.Node.NodeTitleInlineEditableText")
			.Text(InNodeTitle.Get(), &SNodeTitle::GetHeadTitle)
			.OnVerifyTextChanged(this, &SPCGEditorGraphNode::OnVerifyNameTextChanged)
			.OnTextCommitted(this, &SPCGEditorGraphNode::OnNameTextCommited)
			.IsReadOnly(this, &SPCGEditorGraphNode::IsNameReadOnly)
			.IsSelected(this, &SPCGEditorGraphNode::IsSelectedExclusively)
			.MultiLine(false)
			.MaximumLength(UPCGEditorGraphNode::MaxNodeNameCharacterCount)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.DelayedLeftClickEntersEditMode(false)
		];

	InlineEditableText->SetColorAndOpacity(TAttribute<FLinearColor>::Create(TAttribute<FLinearColor>::FGetter::CreateSP(this, &SPCGEditorGraphNode::GetNodeTitleTextColor)));

	return ReturnWidget;
}

TSharedRef<SWidget> SPCGEditorGraphNode::CreateTitleRightWidget()
{
	if (!PCGEditorGraphNode)
	{
		return SGraphNode::CreateTitleRightWidget();
	}

	return SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f)
			[
				CreateDataOverrideButton()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f)
			[
				CreateManualEditButton()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f)
			[
				CreateNodeToolButtonWidget()
			]
		];
}

TSharedPtr<SGraphPin> SPCGEditorGraphNode::CreatePinWidget(UEdGraphPin* InPin) const
{
	const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
	UPCGSettings* NodeSettings = PCGNode ? PCGNode->GetSettings() : nullptr;

	const bool bIsRenameable = PCGEditorGraphNode && PCGEditorGraphNode->CanUserRenameDynamicPin(InPin);

	// Renameable dynamic pins always use the base pin widget so that the inline rename label is available.
	if (!bIsRenameable && InPin && NodeSettings && NodeSettings->Implements<UPCGSettingsDefaultValueProvider>())
	{
		const IPCGSettingsDefaultValueProvider* DefaultValueInterface = CastChecked<IPCGSettingsDefaultValueProvider>(NodeSettings);
		const bool bDefaultIsEnabled = DefaultValueInterface->DefaultValuesAreEnabled();
		const bool bDefaultIsActivated = DefaultValueInterface->IsPinDefaultValueActivated(InPin->PinName);
		const bool bInputHasAnyConnections = InPin->HasAnyConnections();
		const bool bIsInput = InPin->Direction == EGPD_Input;
		// Output pins (i.e. inline constants) always show the widget and input pins only when unconnected.
		if (bDefaultIsEnabled && bDefaultIsActivated && (!bIsInput || !bInputHasAnyConnections))
		{
			// Set the string default value to match the settings' source of truth.
			InPin->DefaultValue = DefaultValueInterface->GetPinDefaultValueAsString(InPin->PinName);

			// To link the transaction to the settings for Undo/Redo.
			TDelegate<void()> OnModify = FSimpleDelegate::CreateLambda([NodeSettingsPtr = TWeakObjectPtr(NodeSettings)]
			{
				if (NodeSettingsPtr.IsValid())
				{
					NodeSettingsPtr.Pin()->Modify();
				}
			});

			switch (DefaultValueInterface->GetPinDefaultValueType(InPin->PinName))
			{
				case EPCGMetadataTypes::Name: // fall-through
				case EPCGMetadataTypes::String:
				{
					const EPCGSettingDefaultValueExtraFlags Flags = DefaultValueInterface->GetDefaultValueExtraFlags(InPin->PinName);
					const bool bIsWide = EnumHasAnyFlags(Flags, EPCGSettingDefaultValueExtraFlags::WideText);
					const bool bIsMultiLine = EnumHasAnyFlags(Flags, EPCGSettingDefaultValueExtraFlags::MultiLineText);
					return SNew(SPCGEditorGraphPinString, InPin, MoveTemp(OnModify))
						.MinDesiredBoxWidth(bIsWide ? 150.f : 60.f)
						.MaxDesiredBoxWidth(bIsWide ? 600.f : 200.f)
						.IsMultiline(bIsMultiLine)
						.OverflowPolicy(bIsMultiLine ? ETextOverflowPolicy::MultilineEllipsis : ETextOverflowPolicy::Ellipsis);
				}
				// Float is converted to double by the property accessor under the hood
				case EPCGMetadataTypes::Float: // fall-through
				case EPCGMetadataTypes::Double:
					return SNew(SPCGEditorGraphPinNumSlider<double>, InPin, MoveTemp(OnModify));
				case EPCGMetadataTypes::Integer32:
					return SNew(SPCGEditorGraphPinNumSlider<int32>, InPin, MoveTemp(OnModify))
						.MinDesiredBoxWidth(40.f);
				case EPCGMetadataTypes::Integer64:
					return SNew(SPCGEditorGraphPinNumSlider<int64>, InPin, MoveTemp(OnModify))
						.MinDesiredBoxWidth(40.f);
				case EPCGMetadataTypes::Vector:
					return SNew(SPCGEditorGraphPinVectorSlider<FVector>, InPin, MoveTemp(OnModify));
				case EPCGMetadataTypes::Vector2:
					return SNew(SPCGEditorGraphPinVectorSlider<FVector2D>, InPin, MoveTemp(OnModify));
				case EPCGMetadataTypes::Vector4:
					return SNew(SPCGEditorGraphPinVectorSlider<FVector4>, InPin, MoveTemp(OnModify));
				case EPCGMetadataTypes::Rotator:
					return SNew(SPCGEditorGraphPinVectorSlider<FRotator>, InPin, MoveTemp(OnModify));
				case EPCGMetadataTypes::Boolean:
					return SNew(SPCGEditorGraphPinBool, InPin, MoveTemp(OnModify));
				case EPCGMetadataTypes::Quaternion:
					return SNew(SPCGEditorGraphPinQuaternion, InPin, MoveTemp(OnModify));
				case EPCGMetadataTypes::Transform:
					return SNew(SPCGEditorGraphPinTransform, InPin, MoveTemp(OnModify));
				case EPCGMetadataTypes::SoftObjectPath: // fall-through
				case EPCGMetadataTypes::SoftObject:
					return SNew(SPCGEditorGraphPinSoftObjectPath, InPin, MoveTemp(OnModify));
				case EPCGMetadataTypes::SoftClassPath: // fall-through
				case EPCGMetadataTypes::SoftClass:
					return SNew(SPCGEditorGraphPinSoftClassPath, InPin, MoveTemp(OnModify));
				default:
					break;
			}
		}
	}

	return SNew(SPCGEditorGraphNodePin, InPin)
		.IsRenameable(bIsRenameable);
}

EVisibility SPCGEditorGraphNode::IsAddPinButtonVisible() const
{
	return IsDynamicPinButtonVisible(EPCGPinDirection::Input);
}

FReply SPCGEditorGraphNode::OnAddPin()
{
	return OnAddDynamicPin(EPCGPinDirection::Input);
}

void SPCGEditorGraphNode::CreateBelowPinControls(const TSharedPtr<SVerticalBox> MainBox)
{
	const UPCGSettings* NodeSettings = PCGEditorGraphNode->GetSettings();
	TSharedPtr<FAssetThumbnailPool> ThumbnailPool = UThumbnailManager::Get().GetSharedThumbnailPool();
	if (!NodeSettings || !ThumbnailPool.IsValid())
	{
		return;
	}

	if (TOptional<FPCGNodeThumbnailProxy> ThumbnailProxy = NodeSettings->GetNodeThumbnail(); ThumbnailProxy.IsSet())
	{
		TSharedPtr<FAssetThumbnail> NodeThumbnail;

		if (ThumbnailProxy->TextureSource.IsType<FSoftObjectPath>())
		{
			const FSoftObjectPath& Path = ThumbnailProxy->TextureSource.Get<FSoftObjectPath>();
			if (const FAssetData Asset = IAssetRegistry::Get()->GetAssetByObjectPath(Path); Asset.IsValid())
			{
				NodeThumbnail = MakeShared<FAssetThumbnail>(Asset, 64, 64, ThumbnailPool);
			}
		}
		else if (ensure(ThumbnailProxy->TextureSource.IsType<const UTexture*>()))
		{
			// @todo_pcg: handle UTexture here in the future
		}

		if (NodeThumbnail.IsValid())
		{
			FAssetThumbnailConfig ThumbnailConfig;
			ThumbnailConfig.ThumbnailLabel = EThumbnailLabel::AssetName;

			MainBox->AddSlot()
			   .AutoHeight()
			   .HAlign(HAlign_Center)
			   .Padding(2)
				[
					SNew(SBox)
				   .MaxDesiredWidth(64)
				   .MaxDesiredHeight(64)
					[
						NodeThumbnail->MakeThumbnailWidget(ThumbnailConfig)
					]
				];
		}
	}
}

void SPCGEditorGraphNode::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	check(PCGEditorGraphNode);
	UPCGNode* PCGNode = PCGEditorGraphNode->GetPCGNode();

	if (PCGNode && PinToAdd->GetPinObj())
	{
		const bool bIsInPin = PinToAdd->GetPinObj()->Direction == EEdGraphPinDirection::EGPD_Input;
		const FName& PinName = PinToAdd->GetPinObj()->PinName;

		if(UPCGPin* Pin = (bIsInPin ? PCGNode->GetInputPin(PinName) : PCGNode->GetOutputPin(PinName)))
		{
			if (!ensure(Pin))
			{
				return;
			}
			
			const FPCGDataTypeIdentifier PinType = Pin->GetCurrentTypesID();
			TTuple<const FSlateBrush*, const FSlateBrush*> PinBrushes = FPCGModule::GetConstDataTypeRegistry().GetPinIcons(PinType, Pin->Properties, bIsInPin);

			// If any of the pin brushes are null, error and fallback on default ones
			if (PinBrushes.Get<0>() == nullptr || PinBrushes.Get<1>() == nullptr)
			{
				PinBrushes = FPCGModule::GetConstDataTypeRegistry().GetPinIcons(FPCGDataTypeInfo::AsId(), Pin->Properties, bIsInPin);
			}

			PinToAdd->SetCustomPinIcon(PinBrushes.Get<0>(), PinBrushes.Get<1>());
		}
	}

	SGraphNode::AddPin(PinToAdd);

	// The base class does not give an override to change the padding of the pin widgets, so do it here. Our input pins widgets include
	// a small marker to indicate the pin is required, which need to display at the left edge of the node, so remove left padding.
	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		const int LastIndex = LeftNodeBox->GetChildren()->Num() - 1;
		check(LastIndex >= 0);

		SVerticalBox::FSlot& PinSlot = LeftNodeBox->GetSlot(LastIndex);

		FMargin Margin = Settings->GetInputPinPadding();
		Margin.Left = 0;
		PinSlot.SetPadding(Margin);
	}
}

TArray<FOverlayWidgetInfo> SPCGEditorGraphNode::GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const
{
	TArray<FOverlayWidgetInfo> OverlayWidgets = SGraphNode::GetOverlayWidgets(bSelected, WidgetSize);

	if (UsesHiGenOverlay())
	{
		AddHiGenOverlayWidget(OverlayWidgets);
	}

	if (UsesGPUOverlay())
	{
		AddGPUOverlayWidget(OverlayWidgets);
	}

	if (PCGEditorGraphNode->GetTriggeredGPUUpload())
	{
		AddGPUUploadWidget(OverlayWidgets);
	}

	if (PCGEditorGraphNode->GetTriggeredGPUReadback())
	{
		AddGPUReadbackWidget(OverlayWidgets);
	}

	return OverlayWidgets;
}

void SPCGEditorGraphNode::GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const
{
	check(PCGEditorGraphNode);

	// Reserve space already occupied by overlay widgets on the same side so brushes stack below them.
	float YOffsetLeft = PCGEditorGraphNode->GetTriggeredGPUReadback() ? PCGEditorGraphNode::OverlayIconSize : 0.0f;
	// Start lower down to be clear of the HiGen grid label and the upload widget (both on the right edge).
	float YOffsetRight = (UsesHiGenOverlay() ? PCGEditorGraphNode::OverlayIconSize + 2.0f : 0.0f) + (PCGEditorGraphNode->GetTriggeredGPUUpload() ? PCGEditorGraphNode::OverlayIconSize : 0.0f);

	auto AddOverlayBrush = [&YOffsetLeft, &YOffsetRight, &Brushes, this](const FName& BrushName, bool bRightSide = false)
	{
		const FSlateBrush* Brush = FPCGEditorStyle::Get().GetBrush(BrushName);

		if (Brush)
		{
			float& YOffset = bRightSide ? YOffsetRight : YOffsetLeft;

			FOverlayBrushInfo BrushInfo;
			BrushInfo.Brush = Brush;
			BrushInfo.OverlayOffset = FVector2f(0.0f, YOffset) - Brush->GetImageSize() / 2.0f;

			if (bRightSide)
			{
				BrushInfo.OverlayOffset.X += GetDesiredSize().X;
			}

			Brushes.Add(BrushInfo);

			YOffset += Brush->GetImageSize().Y;
		}
	};

	if (PCGEditorGraphNode->IsCulledFromExecution())
	{
		AddOverlayBrush(PCGEditorStyleConstants::Node_Overlay_Inactive);
	}

	if (const UPCGNode* PCGNode = PCGEditorGraphNode->GetPCGNode())
	{
		if (PCGNode->GetSettingsInterface() && PCGNode->GetSettingsInterface()->bDebug && UsesDebugBrush())
		{
			AddOverlayBrush(TEXT("PCG.NodeOverlay.Debug"));
		}
	}

	if (PCGEditorGraphNode->GetInspected() && UsesInspectBrush())
	{
		AddOverlayBrush(TEXT("PCG.NodeOverlay.Inspect"));
	}
}

void SPCGEditorGraphNode::OnNodeChanged()
{
	// Avoid crashing inside slate if we got triggered from a non-game-thread via any
	//  experimental worker-thread executor
	// @todo_pcg: revisit
	if (IsInGameThread()) 
	{
		UpdateGraphNode(); 
	}
	else
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this]() { UpdateGraphNode(); });
	}
}

bool SPCGEditorGraphNode::UsesHiGenOverlay() const
{
	return PCGEditorGraphNode->GetInspectedGridSize() != PCGHiGenGrid::UninitializedGridSize() && PCGEditorGraphNode->IsNodeEnabled();
}

bool SPCGEditorGraphNode::UsesGPUOverlay() const
{
	return PCGEditorGraphNode->GetSettings() && PCGEditorGraphNode->GetSettings()->ShouldExecuteOnGPU();
}

bool SPCGEditorGraphNode::UsesInspectBrush() const
{
	const UEdGraph* Graph = PCGEditorGraphNode->GetGraph();
	const UPCGEditorGraphSchema* Schema = Graph ? Cast<UPCGEditorGraphSchema>(Graph->GetSchema()) : nullptr;
	return Schema ? Schema->ShouldAddInspectBrushOnNode() : true;
}

bool SPCGEditorGraphNode::UsesDebugBrush() const
{
	const UEdGraph* Graph = PCGEditorGraphNode->GetGraph();
	const UPCGEditorGraphSchema* Schema = Graph ? Cast<UPCGEditorGraphSchema>(Graph->GetSchema()) : nullptr;
	return Schema ? Schema->ShouldAddDebugBrushOnNode() : true;
}

FLinearColor SPCGEditorGraphNode::GetGridLabelColor(uint32 NodeGrid) const
{
	check(PCGEditorGraphNode);

	EPCGHiGenGrid UnscaledGrid = EPCGHiGenGrid::Uninitialized;

	if (UPCGGraph* Graph = PCGEditorGraphNode->GetPCGNode() ? PCGEditorGraphNode->GetPCGNode()->GetGraph() : nullptr)
	{
		// Account for grid scaling.
		UnscaledGrid = PCGHiGenGrid::ScaledGridSizeToGrid(NodeGrid, Graph->GetGridSizeMultiplier());
	}
	else
	{
		UnscaledGrid = PCGHiGenGrid::GridSizeToGrid(NodeGrid);
	}

	// All colors hand tweaked to give a kind of "temperature scale" for the hierarchy.
	switch (UnscaledGrid)
	{
	case EPCGHiGenGrid::Unbounded:
		return FColor(255, 255, 255, 255);
	case EPCGHiGenGrid::Grid4194304: // fall-through
	case EPCGHiGenGrid::Grid2097152: // fall-through
	case EPCGHiGenGrid::Grid1048576: // fall-through
	case EPCGHiGenGrid::Grid524288: // fall-through
	case EPCGHiGenGrid::Grid262144: // fall-through
	case EPCGHiGenGrid::Grid131072: // fall-through
	case EPCGHiGenGrid::Grid65536: // fall-through
	case EPCGHiGenGrid::Grid32768: // fall-through
	case EPCGHiGenGrid::Grid16384: // fall-through
	case EPCGHiGenGrid::Grid8192: // fall-through
	case EPCGHiGenGrid::Grid4096: // fall-through
	case EPCGHiGenGrid::Grid2048:
		return FColor(53, 60, 171, 255);
	case EPCGHiGenGrid::Grid1024:
		return FColor(31, 82, 210, 255);
	case EPCGHiGenGrid::Grid512:
		return FColor(16, 120, 217, 255);
	case EPCGHiGenGrid::Grid256:
		return FColor(8, 151, 208, 255);
	case EPCGHiGenGrid::Grid128:
		return FColor(9, 170, 188, 255);
	case EPCGHiGenGrid::Grid64:
		return FColor(64, 185, 150, 255);
	case EPCGHiGenGrid::Grid32:
		return FColor(144, 189, 114, 255);
	case EPCGHiGenGrid::Grid16:
		return FColor(207, 185, 89, 255);
	case EPCGHiGenGrid::Grid8:
		return FColor(252, 189, 61, 255);
	case EPCGHiGenGrid::Grid4:
		return FColor(243, 227, 28, 255);
	default:
		ensure(false);
		return FLinearColor::White;
	}
}

// @todo_pcg: Should return a FOverlayWidgetInfo, rather than updating a passed in argument array
void SPCGEditorGraphNode::AddHiGenOverlayWidget(TArray<FOverlayWidgetInfo>& OverlayWidgets) const
{
	check(PCGEditorGraphNode);
	check(UsesHiGenOverlay());

	// Higen grid size overlay widget. All magic numbers below hand tweaked to match UI mockup.
	const uint32 InspectedGrid = PCGEditorGraphNode->GetInspectedGridSize();
	const uint32 Grid = PCGEditorGraphNode->GetGenerationGridSize();

	FText GenerationGridText;

	if (Grid == PCGHiGenGrid::UnboundedGridSize())
	{
		GenerationGridText = FText::FromString(TEXT("UB"));
	}
	else
	{
		// Meters are easier on the eyes.
		GenerationGridText = FText::AsNumber(Grid / 100, &FNumberFormattingOptions::DefaultNoGrouping());
	}

	FLinearColor Tint = FLinearColor::White;
	if (Grid != PCGHiGenGrid::UninitializedGridSize())
	{
		Tint = GetGridLabelColor(Grid);
	}
	else if (PCGEditorGraphNode->IsDisplayAsDisabledForced())
	{
		Tint.A *= 0.35f;
	}

	// Create a border brush for each combination of grids, to workaround issue where the tint does not apply
	// to the border element.
	const FSlateBrush* BorderBrush = GetBorderBrush(InspectedGrid, Grid);

	FLinearColor TextColor = FColor::White;
	FLinearColor BackgroundColor = FColor::Black;
	if (InspectedGrid == Grid)
	{
		// Flip colors for active grid to highlight them.
		Swap(TextColor, BackgroundColor);
	}

	TSharedPtr<SWidget> GridSizeLabel =
		SNew(SHorizontalBox)
		.Visibility(EVisibility::Visible)
		+ SHorizontalBox::Slot()
		[
			SNew(SBorder)
			.BorderImage(BorderBrush)
			.Padding(FMargin(12, 3))
			.ColorAndOpacity(Tint)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "Graph.Node.NodeTitle")
				.Text(GenerationGridText)
				.Justification(ETextJustify::Center)
				.ColorAndOpacity(TextColor)
			]
		];

	FOverlayWidgetInfo GridSizeLabelInfo(GridSizeLabel);
	GridSizeLabelInfo.OverlayOffset = FVector2D(GetDesiredSize().X - 30.0f, -9.0f);

	OverlayWidgets.Add(GridSizeLabelInfo);
}

// @todo_pcg: Should return a FOverlayWidgetInfo, rather than updating a passed in argument array
void SPCGEditorGraphNode::AddGPUOverlayWidget(TArray<FOverlayWidgetInfo>& OverlayWidgets) const
{
	check(PCGEditorGraphNode);
	check(UsesGPUOverlay())

	constexpr float BorderRadius = 7.0f;
	constexpr float BorderStroke = 1.0f;
	constexpr FLinearColor BorderColor(0.5f, 0.5f, 0.5f, 0.5f);
	constexpr FLinearColor TextColor(0.5f, 0.5f, 0.5f, 0.8f);
	const FText GPUText = LOCTEXT("GPULabel", "GPU");

	const FSlateBrush* BorderBrush = new FSlateRoundedBoxBrush(FLinearColor::Transparent, BorderRadius, BorderColor, BorderStroke);

	TSharedPtr<SWidget> GPUUsageLabel =
		SNew(SHorizontalBox)
		.Visibility(EVisibility::Visible)
		+ SHorizontalBox::Slot()
		[
			SNew(SBorder)
			.BorderImage(BorderBrush)
			.Padding(FMargin(4, 3))
			[
				SNew(STextBlock)
				.TextStyle(FPCGEditorStyle::Get(), "PCG.Node.AdditionalOverlayWidgetText")
				.Text(std::move(GPUText))
				.Justification(ETextJustify::Center)
				.ColorAndOpacity(TextColor)
			]
		];

	FOverlayWidgetInfo GPUUsageLabelInfo(GPUUsageLabel);
	GPUUsageLabelInfo.OverlayOffset = FVector2D(GetDesiredSize().X - 34.0f, GetDesiredSize().Y + 5.0f);

	OverlayWidgets.Add(GPUUsageLabelInfo);
}

void SPCGEditorGraphNode::AddGPUUploadWidget(TArray<FOverlayWidgetInfo>& OverlayWidgets) const
{
	check(GPUUploadIndicatorWidget.IsValid());

	// Mirror the position used in GetOverlayBrushes: right edge of the node, offset downward when the HiGen label is also visible so the two don't overlap.
	const float YOffset = (UsesHiGenOverlay() ? PCGEditorGraphNode::OverlayIconSize + 2.0f : 0.0f) - PCGEditorGraphNode::OverlayHalfIconSize;

	FOverlayWidgetInfo Info(GPUUploadIndicatorWidget);
	Info.OverlayOffset = FVector2D(GetDesiredSize().X - PCGEditorGraphNode::OverlayHalfIconSize, YOffset);
	OverlayWidgets.Add(Info);
}

void SPCGEditorGraphNode::AddGPUReadbackWidget(TArray<FOverlayWidgetInfo>& OverlayWidgets) const
{
	check(GPUReadbackIndicatorWidget.IsValid());

	// Mirror the position used in GetOverlayBrushes: left edge of the node, top.
	FOverlayWidgetInfo Info(GPUReadbackIndicatorWidget);
	Info.OverlayOffset = FVector2D(-PCGEditorGraphNode::OverlayHalfIconSize, -PCGEditorGraphNode::OverlayHalfIconSize);
	OverlayWidgets.Add(Info);
}

const FSlateBrush* SPCGEditorGraphNode::GetBorderBrush(uint32 InspectedGrid, uint32 NodeGrid) const
{
	if (InspectedGrid == NodeGrid)
	{
		return FPCGEditorStyle::Get().GetBrush(PCGEditorStyleConstants::Node_Overlay_GridSizeLabel_Active_Border);
	}

	// Hand tweaked multiplier to fade child node grid size labels.
	const float Opacity = (InspectedGrid < NodeGrid) ? 1.0f : 0.5f;

	return new FSlateRoundedBoxBrush(
		FLinearColor::Black * Opacity,
		PCGEditorStyleConstants::Node_Overlay_GridSizeLabel_BorderRadius,
		GetGridLabelColor(NodeGrid) * Opacity,
		PCGEditorStyleConstants::Node_Overlay_GridSizeLabel_BorderStroke);
}

#undef LOCTEXT_NAMESPACE
