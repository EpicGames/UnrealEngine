// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"

#include "PCGCommon.h"

enum class EPCGPinDirection : uint8;
struct FOverlayBrushInfo;

class UPCGEditorGraphNodeBase;

class SPCGEditorGraphNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphNode){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UPCGEditorGraphNodeBase* InNode);

	//~ Begin SGraphNode Interface
	virtual void UpdateGraphNode() override;
	virtual const FSlateBrush* GetNodeBodyBrush() const override;
	virtual void RequestRenameOnSpawn() override { /* Empty to avoid the default behavior to rename on node spawn */ }
	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	virtual TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> InNodeTitle) override;
	virtual TSharedRef<SWidget> CreateTitleRightWidget() override;
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	virtual EVisibility IsAddPinButtonVisible() const override;
	virtual FReply OnAddPin() override;
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	//~ End SGraphNode Interface

	//~ Begin SNodePanel::SNode Interface
	virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const override;
	virtual void GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const override;
	//~ End SNodePanel::SNode Interface

protected:
	void OnNodeChanged();

	/** Will add the Hierarchical Generation overlay to the node. */
	virtual bool UsesHiGenOverlay() const;
	/** Will add the GPU icon overlay to the node. */
	virtual bool UsesGPUOverlay() const;
	/** Will add the inspect brush to the node. */
	virtual bool UsesInspectBrush() const;
	/** Will add the debug brush to the node. */
	virtual bool UsesDebugBrush() const;

	void CreateDynamicPinButton(EPCGPinDirection Direction);
	FReply OnAddDynamicPin(EPCGPinDirection Direction);
	EVisibility IsDynamicPinButtonVisible(EPCGPinDirection Direction) const;
	void OnRequestPinRename(UEdGraphPin* InPin);

private:
	FLinearColor GetGridLabelColor(uint32 NodeGrid) const;

	/** Adds the Hierarchical Generation overlay to the array, displaying the HiGen grid size on the node. */
	void AddHiGenOverlayWidget(TArray<FOverlayWidgetInfo>& OverlayWidgets) const;

	/** Adds the "GPU" tag to the node to indicate the node will execute on the GPU. */
	void AddGPUOverlayWidget(TArray<FOverlayWidgetInfo>& OverlayWidgets) const;

	/** Indicates an upload of data to the GPU occurred as a result of this node. */
	void AddGPUUploadWidget(TArray<FOverlayWidgetInfo>& OverlayWidgets) const;

	/** Indicates a readback of data from the GPU occurred as a result of this node. */
	void AddGPUReadbackWidget(TArray<FOverlayWidgetInfo>& OverlayWidgets) const;

	/**
	* Get the border brush for the given combination of grid sizes and enabled state. All a big workaround for FSlateRoundedBoxBrush not respecting
	* the tint colour.
	*/
	const FSlateBrush* GetBorderBrush(uint32 InspectedGrid, uint32 NodeGrid) const;

	/**
	 * Creates the interact button widget displayed in the node's title area, providing access to node-specific interaction actions.
	 */
	TSharedRef<SWidget> CreateNodeToolButtonWidget();

	/** Creates the pencil button that opens the Data Overrides panel and inspects this node. */
	TSharedRef<SWidget> CreateDataOverrideButton() const;

	/** Creates the wrench button that selects the parent actor and focuses the level viewport. */
	TSharedRef<SWidget> CreateManualEditButton() const;

	UPCGEditorGraphNodeBase* PCGEditorGraphNode = nullptr;

	/**
	* Stable widget instances for GPU transfer indicators. Must be the same pointer across GetOverlayWidgets() calls (Paint vs OnArrangeChildren) so that Slate
	* can track hover state and show tooltips.
	*/
	TSharedPtr<SWidget> GPUUploadIndicatorWidget;
	TSharedPtr<SWidget> GPUReadbackIndicatorWidget;
};
