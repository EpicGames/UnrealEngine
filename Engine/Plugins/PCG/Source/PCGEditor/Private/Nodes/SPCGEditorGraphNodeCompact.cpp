// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphNodeCompact.h"

#include "Nodes/PCGEditorGraphNode.h"
#include "Nodes/PCGEditorGraphNodeBase.h"
#include "PCGEditorStyle.h"
#include "Pins/SPCGEditorGraphNodePin.h"

#include "GraphEditorSettings.h"
#include "IDocumentation.h"
#include "TutorialMetaData.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"

#include "SCommentBubble.h"
#include "SGraphPin.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SPCGEditorGraphNodeCompact"

namespace PCGEditorGraphNodeCompact
{
	namespace Constants
	{
		static constexpr float CompactNodeSize = 36.f;
		static constexpr float CompactNodeIconSize = 20.f;
		static constexpr float IconTitleWidth = 45.f;
		static constexpr float TitleTextExtraWidth = 5.f;
		static constexpr float PinExtraPadding = 8.f;
		static constexpr float NoPinPadding = 16.f;

		static constexpr float SubduedSpillColorMultiplier = 0.6f;
	}

	namespace Helpers
	{
		static float GetTitleTextWidth(const FText& Text, const float ContentScale = 1.f)
		{
			static constexpr float FontScale = 0.07f;
			const FInlineEditableTextBlockStyle& Style = FPCGEditorStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("PCG.Node.CompactNodeTitle");
			if (const TSharedPtr<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService())
			{
				return FontMeasure->Measure(Text, Style.EditableTextBoxStyle.TextStyle.Font, ContentScale).X * FontScale;
			}
			else // Guess based on an average font size of about 0.1f per character
			{
				return Text.ToString().Len() * FontScale * ContentScale;
			}
		}
	}
}

void SPCGEditorGraphNodeCompact::Construct(const FArguments& InArgs, UPCGEditorGraphNodeBase* InNode)
{
	PCGEditorGraphNode = CastChecked<UPCGEditorGraphNode>(InNode);
	SPCGEditorGraphNode::Construct(SPCGEditorGraphNode::FArguments{}, InNode);
}

void SPCGEditorGraphNodeCompact::UpdateGraphNode()
{
	using namespace PCGEditorGraphNodeCompact;
	check(PCGEditorGraphNode);

	// Based on SGraphNodeK2Base::UpdateCompactNode. Changes:
	// * Removed creation of tooltip widget, it did not port across trivially and the usage is fairly obvious for the current
	//   compact nodes, but this could be re-added - TODO
	// * Changed title style - reduced font size substantially
	// * Layout differentiation for "pure" vs "impure" K2 nodes removed
	// * Title Widget created independently
	InputPins.Empty();
	OutputPins.Empty();

	// Error handling set-up
	SetupErrorReporting();

	// Reset variables that are going to be exposed, in case we are refreshing an already set up node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	if (!SWidget::GetToolTip().IsValid())
	{
		// @todo_pcg: Disabled temporarily to avoid new (Clang?) static analysis warning. Fix when upgrading to advanced tooltips.
		// TSharedPtr<SToolTip> NewToolTip = IDocumentation::Get()->CreateToolTip(TAttribute<FText>(this, &SGraphNode::GetNodeTooltip), nullptr, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName());
		SetToolTipText(TAttribute<FText>(this, &SGraphNode::GetNodeTooltip));
	}

	// Set up a meta tag for this node
	FGraphNodeMetaData TagMeta(TEXT("Graphnode"));
	PopulateMetaTag(&TagMeta);

	IconColor = FLinearColor::White;

	check(GraphNode)

	const TSharedPtr<SWidget> TitleWidget = CreateTitleWidget(SAssignNew(NodeTitle, SNodeTitle, GraphNode));
	FName NodeIcon = NAME_None;
	const bool bNeedsTitle = !PCGEditorGraphNode->GetCompactNodeIcon(NodeIcon);

	check(InlineEditableText || NodeIcon != NAME_None);

	const bool bHasInputPins = nullptr != GraphNode->FindPinByPredicate([](const UEdGraphPin* Pin) { return Pin->Direction == EGPD_Input; });
	const bool bHasOutputPins = nullptr != GraphNode->FindPinByPredicate([](const UEdGraphPin* Pin) { return Pin->Direction == EGPD_Output; });

	const float TitleWidth =
		bNeedsTitle
			? Constants::TitleTextExtraWidth + Helpers::GetTitleTextWidth(InlineEditableText->GetText(), InlineEditableText->GetContentScale().X)
			: Constants::IconTitleWidth; // Default width for icon based compact nodes

	const float PinPadding = Constants::PinExtraPadding + Settings->GetInputPinPadding().GetTotalSpaceAlong<Orient_Horizontal>();
	const float InputSidePadding = bHasInputPins ? PinPadding : Constants::NoPinPadding;
	const float OutputSidePadding = bHasOutputPins ? PinPadding : Constants::NoPinPadding;
	FMargin ContentMargin(TitleWidth * 0.5, 0);
	ContentMargin.Left += InputSidePadding;
	ContentMargin.Right += OutputSidePadding;

	const FVector2D NodeSize = ContentMargin.GetDesiredSize();

	//
	//             ______________________
	//            | (<) L |   +  | R (>) |
	//            |_______|______|_______|
	//
	const TSharedPtr<SOverlay> ContentOverlay =
	SNew(SOverlay)
	+ SOverlay::Slot()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	.Padding(ContentMargin)
	[
		TitleWidget.ToSharedRef()
	]
	+ SOverlay::Slot()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.Padding(0, 0, /*Right=*/InputSidePadding, 0)
	[
		// LEFT
		SAssignNew(LeftNodeBox, SVerticalBox)
	]
	+ SOverlay::Slot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(/*Left=*/OutputSidePadding, 0, 0, 0)
	[
		// RIGHT
		SAssignNew(RightNodeBox, SVerticalBox)
	];

	// Add optional node specific widget to the overlay:
	const TSharedPtr<SWidget> CustomOverlay = GraphNode->CreateNodeImage();
	if (CustomOverlay.IsValid())
	{
		ContentOverlay->AddSlot()
		   .HAlign(HAlign_Center)
		   .VAlign(VAlign_Center)
		   [
			   SNew(SBox)
			   [
				   CustomOverlay.ToSharedRef()
			   ]
		   ];
	}

	check(ContentOverlay);

	// First add the body overlay
	const TSharedRef<SOverlay> NodeContentOverlay =
		SNew(SOverlay)
		.AddMetaData<FGraphNodeMetaData>(TagMeta)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.ZOrder(0)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Graph.VarNode.Body"))
			.DesiredSizeOverride(NodeSize)
			.ColorAndOpacity(this, &SGraphNode::GetNodeBodyColor)
			.Visibility(EVisibility::SelfHitTestInvisible)
		]
		// Color spill is ZOrder 1 and will be inserted next
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.ZOrder(2)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Graph.VarNode.Gloss"))
			.DesiredSizeOverride(NodeSize)
			.ColorAndOpacity(FLinearColor::White)
			.Visibility(EVisibility::SelfHitTestInvisible)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.ZOrder(3)
		[
			ContentOverlay.ToSharedRef()
		];

	// Color spill not added to icon nodes, ex. filters, conversions
	if (bNeedsTitle)
	{
		NodeContentOverlay->AddSlot()
			.ZOrder(1)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Graph.VarNode.ColorSpill"))
				.DesiredSizeOverride(NodeSize)
				.ColorAndOpacity(this, &SPCGEditorGraphNodeCompact::GetSubduedSpillColor)
				.Visibility(EVisibility::SelfHitTestInvisible)
			];
	}

	const TSharedRef<SVerticalBox> InnerVerticalBox =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBox)
			.MinDesiredHeight(Constants::CompactNodeSize)
			[
				NodeContentOverlay
			]
		];

	// Enabled state bar
	const TSharedPtr<SWidget> EnabledStateWidget = GetEnabledStateWidget();
	if (EnabledStateWidget.IsValid())
	{
		InnerVerticalBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.Padding(FMargin(3, 0))
			[
				EnabledStateWidget.ToSharedRef()
			];
	}

	InnerVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(FMargin(5, 1))
		[
			ErrorReporting->AsWidget()
		];

	this->GetOrAddSlot(ENodeZone::Center)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			InnerVerticalBox
		];

	CreatePinWidgets();

	// @todo_pcg: Likely this is the same as non-compact. Could be factored out.
	// Create comment bubble
	TSharedPtr<SCommentBubble> CommentBubble;
	const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

	SAssignNew(CommentBubble, SCommentBubble)
	.GraphNode(GraphNode)
	.Text(this, &SGraphNode::GetNodeComment)
	.OnTextCommitted(this, &SGraphNode::OnCommentTextCommitted)
	.ColorAndOpacity(CommentColor)
	.AllowPinning(true)
	.EnableTitleBarBubble(true)
	.EnableBubbleCtrls(true)
	.GraphLOD(this, &SGraphNode::GetCurrentLOD)
	.IsGraphNodeHovered(this, &SGraphNode::IsHovered);

	GetOrAddSlot(ENodeZone::TopCenter)
		.SlotOffset2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetOffset2f))
		.SlotSize2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetSize2f))
		.AllowScaling(TAttribute<bool>(CommentBubble.Get(), &SCommentBubble::IsScalingAllowed))
		.VAlign(VAlign_Top)
		[
			CommentBubble.ToSharedRef()
		];

	// @todo_pcg: Look into using these to replace the current implementation of Dynamic Input Pins
	CreateInputSideAddButton(LeftNodeBox);
	CreateOutputSideAddButton(RightNodeBox);
}

TSharedPtr<SGraphPin> SPCGEditorGraphNodeCompact::CreatePinWidget(UEdGraphPin* Pin) const
{
	// Compact nodes never show pin labels — bake ShowLabel=false at construction so no dynamic attribute is needed.
	// Note: compact nodes do not use default value widgets (no UPCGSettingsDefaultValueProvider), so the parent
	// CreatePinWidget logic for specialised pin types is intentionally bypassed here.
	return SNew(SPCGEditorGraphNodePin, Pin).ShowLabel(false);
}

TSharedRef<SWidget> SPCGEditorGraphNodeCompact::CreateTitleWidget(TSharedPtr<SNodeTitle> InNodeTitle)
{
	check(PCGEditorGraphNode);

	// Could be text or icon as the center of the graph
	FName CompactBodyIcon = NAME_None;
	if (PCGEditorGraphNode->GetCompactNodeIcon(CompactBodyIcon)) // Use the icon for the title
	{
		const FSlateBrush* ImageBrush = FPCGEditorStyle::Get().GetBrush(CompactBodyIcon);
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.MaxDesiredWidth(PCGEditorGraphNodeCompact::Constants::CompactNodeIconSize)
			.MaxDesiredHeight(PCGEditorGraphNodeCompact::Constants::CompactNodeIconSize)
			[
				SNew(SImage)
				.Image(ImageBrush)
				.ColorAndOpacity(this, &SPCGEditorGraphNode::GetNodeTitleIconColor)
			];
	}
	else // Create a title widget with the node title
	{
		/**
		 * Reimplementation of the SGraphNode::CreateTitleWidget
		 * - Add a box to limit the width
		 * - Control the style
		 * - Control the inner text box for retrieving the property name
		 */
		SAssignNew(InlineEditableText, SInlineEditableTextBlock)
		.Style(FPCGEditorStyle::Get(), "PCG.Node.CompactNodeTitle")
		.Text(InNodeTitle.Get(), &SNodeTitle::GetHeadTitle)
		.ColorAndOpacity(GetNodeTitleTextColor())
		.OnEnterEditingMode_Lambda(
			[this]
			{
				InlineEditableText->SetText(PCGEditorGraphNode->GetNodeTitle(ENodeTitleType::MenuTitle));
			})
		.OnVerifyTextChanged(this, &SPCGEditorGraphNodeCompact::OnVerifyNameTextChanged)
		.OnTextCommitted(this, &SPCGEditorGraphNodeCompact::OnNameTextCommited)
		.IsReadOnly(this, &SPCGEditorGraphNodeCompact::IsNameReadOnly)
		.IsSelected(this, &SPCGEditorGraphNodeCompact::IsSelectedExclusively)
		.MultiLine(false)
		.MaximumLength(UPCGEditorGraphNode::MaxNodeNameCharacterCount)
		.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
		.DelayedLeftClickEntersEditMode(false);

		InlineEditableText->SlatePrepass(); // Prepass to calculated desired size

		return SNew(SBox)
			.MaxDesiredWidth(UPCGEditorGraphNode::MaxNodeTitleWidth)
			[
				InlineEditableText.ToSharedRef()
			];
	}
}

const FSlateBrush* SPCGEditorGraphNodeCompact::GetShadowBrush(bool bSelected) const
{
	check(GraphNode)

	if (GraphNode->GetDesiredEnabledState() == ENodeEnabledState::Enabled)
	{
		return bSelected ? FAppStyle::GetBrush(TEXT("Graph.VarNode.ShadowSelected")) : FAppStyle::GetBrush(TEXT("Graph.VarNode.Shadow"));
	}
	else // The disabled widget adds a disabled bar, forcing the widget into a rectangular box shape. Use default Shadow Brush.
	{
		return SGraphNode::GetShadowBrush(bSelected);
	}
}

FSlateColor SPCGEditorGraphNodeCompact::GetSubduedSpillColor() const
{
	FLinearColor SpillColor(GetNodeTitleColor().GetSpecifiedColor());
	SpillColor.A *= PCGEditorGraphNodeCompact::Constants::SubduedSpillColorMultiplier; // Subdue the color with alpha only to prevent darkening.
	return SpillColor;
}

#undef LOCTEXT_NAMESPACE
