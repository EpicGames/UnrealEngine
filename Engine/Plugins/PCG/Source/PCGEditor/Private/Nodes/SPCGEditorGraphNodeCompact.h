// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPCGEditorGraphNode.h"

class UPCGEditorGraphNode;

class SPCGEditorGraphNodeCompact : public SPCGEditorGraphNode
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphNodeCompact) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UPCGEditorGraphNodeBase* InNode);

	//~ Begin SGraphNode Interface
	virtual void UpdateGraphNode() override;
	virtual TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> InNodeTitle) override;
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	//~ End SGraphNode Interface

private:
	FSlateColor GetSubduedSpillColor() const;

	UPCGEditorGraphNode* PCGEditorGraphNode = nullptr;
	TSharedPtr<SNodeTitle> NodeTitle = nullptr;
};
