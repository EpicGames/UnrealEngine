// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGComponentVisualizer.h"

#include "PCGComponent.h"
#include "PCGDebug.h"
#include "PCGEditorModule.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "Data/PCGBasePointData.h"
#include "DeltaViewportExtensions/PCGDeltaViewportExtension.h"
#include "DeltaViewportExtensions/PCGDeltaViewportExtensionHelpers.h"
#include "Graph/DataOverride/PCGDataOverrideHelpers.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "EditorViewportClient.h"
#include "PrimitiveDrawInterface.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "StaticMeshResources.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/StaticMesh.h"
#include "Misc/ScopeExit.h"
#include "Modules/ModuleManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"

#include "SPCGManualEditPanel.h"

IMPLEMENT_HIT_PROXY(HPCGComponentVisualizerHitProxy, HComponentVisProxy);

#define LOCTEXT_NAMESPACE "PCGComponentVisualizer"

namespace PCG::ComponentVisualizer
{
	bool GEnableCulling = true;
	float GCullingDistance2D = 10000.0f;
	bool GFrustumCulling = true;

	FAutoConsoleVariableRef CVarEnableCulling(
		TEXT("pcg.ComponentVisualizer.EnableCulling"),
		GEnableCulling,
		TEXT("If we should cull the debug meshes based on camera position. Distance can be controlled with 'pcg.ComponentVisualizer.CullingDistance2D'"));

	FAutoConsoleVariableRef CVarCullingDistance2D(
		TEXT("pcg.ComponentVisualizer.CullingDistance2D"),
		GCullingDistance2D,
		TEXT("Culling distance for the debug meshes in 2D based on camera position. 100m by default."));

	FAutoConsoleVariableRef CVarCullingMethod(
		TEXT("pcg.ComponentVisualizer.CullingMethod"),
		GFrustumCulling,
		TEXT("If culling is enabled, can on top of distance culling do a frustum culling. Can cull too much on the edge of the screen."));

	/** Result of looking up an element's delta state in a collection. */
	struct FDeltaStateResult
	{
		FPCGDeltaKey Key = {};
		IPCGDeltaViewportExtension* Extension = nullptr;
		FConstStructView DeltaView;
	};

	/** Cached mesh geometry for wireframe drawing. Persists across frames on the visualizer. */
	struct FCachedMeshWireframe
	{
		const UStaticMesh* Mesh = nullptr;
		TArray<uint32> Indices;
		const FPositionVertexBuffer* PositionBuffer = nullptr;

		bool IsValid() const { return Mesh && PositionBuffer && !Indices.IsEmpty(); }

		/** Returns the triangle count of the mesh's LOD 0, or 0 if the mesh has no valid render data. */
		static int32 GetTriangleCount(const UStaticMesh* InMesh)
		{
			if (!InMesh)
			{
				return 0;
			}

			const FStaticMeshRenderData* RenderData = InMesh->GetRenderData();
			if (!RenderData || RenderData->LODResources.IsEmpty())
			{
				return 0;
			}

			return RenderData->LODResources[0].GetNumTriangles();
		}

		bool TryCache(const UStaticMesh* InMesh)
		{
			if (InMesh == Mesh)
			{
				return IsValid();
			}

			Reset();
			Mesh = InMesh;

			if (!Mesh)
			{
				return false;
			}

			const FStaticMeshRenderData* RenderData = Mesh->GetRenderData();
			if (!RenderData || RenderData->LODResources.IsEmpty())
			{
				return false;
			}

			const FStaticMeshLODResources& LOD = RenderData->LODResources[0];
			if (LOD.IndexBuffer.GetNumIndices() == 0)
			{
				return false;
			}

			LOD.IndexBuffer.GetCopy(Indices);
			PositionBuffer = &LOD.VertexBuffers.PositionVertexBuffer;
			return IsValid();
		}

		void Reset()
		{
			Mesh = nullptr;
			Indices.Empty();
			PositionBuffer = nullptr;
		}
	};

	namespace Defaults
	{
		constexpr FLinearColor ManualEditNoDeltaColor(0.0f, 0.55f, 0.45f, 0.3f);
		constexpr FLinearColor ManualEditNoDeltaSelectedColor(0.4f, 1.0f, 0.9f, 0.5f);

		FLinearColor GetNoDeltaColor(const bool bIsSelected)
		{
			return bIsSelected ? ManualEditNoDeltaSelectedColor : ManualEditNoDeltaColor;
		}
	}

	namespace Helpers
	{
		/** Looks up an element's delta in the collection via the extension registry. */
		FDeltaStateResult FindDeltaFromCollection(const FPCGDeltaCollection& Collection, const FTransform& Transform, const FPCGDeltaViewportExtensionRegistry& Registry)
		{
			FDeltaStateResult Result;

			// @todo_pcg: Consider caching a DeltaKey -> Extension + View mapping to avoid resolving struct types on every lookup.
			Collection.ForEachDelta([&Result, &Transform, &Registry, &Collection](const FPCGDeltaKey& DeltaKey, const TInstancedStruct<FPCGDeltaBase>& DeltaValue) -> bool
			{
				const UScriptStruct* ScriptStruct = DeltaValue.GetScriptStruct();
				IPCGDeltaViewportExtension* Extension = Registry.GetExtension(ScriptStruct);
				if (!Extension)
				{
					return true;
				}

				// Create a read-only view so the extension can inspect the delta without a concrete type cast.
				const FConstStructView View(ScriptStruct, DeltaValue.GetMemory());
				if (Extension->MatchesSourceElement(Transform, View, Collection.Settings.SpatialTolerance))
				{
					Result.Key = DeltaKey;
					Result.Extension = Extension;
					Result.DeltaView = View;
					return false;
				}

				return true;
			});

			return Result;
		}

		void DrawMeshWireframe(FPrimitiveDrawInterface* PDI, const FTransform& Transform, const FCachedMeshWireframe& CachedDebugMesh, const FLinearColor& Color)
		{
			// Transform all vertices first, then draw them all.
			TArray<FVector, TInlineAllocator<256>> TransformedVertices;
			TransformedVertices.SetNumUninitialized(CachedDebugMesh.PositionBuffer->GetNumVertices());
			for (uint32 i = 0; i < CachedDebugMesh.PositionBuffer->GetNumVertices(); ++i)
			{
				TransformedVertices[i] = Transform.TransformPosition(FVector(CachedDebugMesh.PositionBuffer->VertexPosition(i)));
			}

			for (int32 i = 0; i + 2 < CachedDebugMesh.Indices.Num(); i += 3)
			{
				const FVector& P0 = TransformedVertices[CachedDebugMesh.Indices[i]];
				const FVector& P1 = TransformedVertices[CachedDebugMesh.Indices[i + 1]];
				const FVector& P2 = TransformedVertices[CachedDebugMesh.Indices[i + 2]];

				PDI->DrawLine(P0, P1, Color, SDPG_World);
				PDI->DrawLine(P1, P2, Color, SDPG_World);
				PDI->DrawLine(P2, P0, Color, SDPG_World);
			}
		}

		// Node and pin frames extracted from a pin stack.
		struct FNodeAndPinFromStack
		{
			const UPCGNode* Node = nullptr;
			const UPCGPin* Pin = nullptr;

			bool IsValid() const { return Node != nullptr && Pin != nullptr; }
		};

		// Extracts the node and pin frames from a hit proxy's pin stack.
		FNodeAndPinFromStack ExtractNodeAndPinFromStack(const FPCGStack& InPinStack)
		{
			FNodeAndPinFromStack Result;

			const TArray<FPCGStackFrame>& Frames = InPinStack.GetStackFrames();
			if (Frames.Num() < 2)
			{
				return Result;
			}

			Result.Node = Frames[Frames.Num() - 2].GetObject_AnyThread<UPCGNode>();
			Result.Pin = Frames.Last().GetObject_AnyThread<UPCGPin>();
			return Result;
		}

	}
} // PCG::ComponentVisualizer

// @todo_pcg: Revisit making this a user preference instead of a CVar
static TAutoConsoleVariable<int32> CVarPCGManualEditMeshComplexityThreshold(
	TEXT("pcg.Editor.ManualEdit.MeshComplexityThreshold"),
	1000,
	TEXT("Maximum triangle count for rendering custom debug meshes as wireframe in viewport editing. Meshes exceeding this fall back to the default point mesh."),
	ECVF_Default);

HPCGComponentVisualizerHitProxy::HPCGComponentVisualizerHitProxy(const UActorComponent* InComponent, const FPCGStack& InPinStack, const int32 InDataIndex, const int32 InPointIndex)
	: HComponentVisProxy(InComponent)
	, PinStack(InPinStack)
	, DataIndex(InDataIndex)
	, PointIndex(InPointIndex)
{}

HPCGComponentVisualizerHitProxy::HPCGComponentVisualizerHitProxy(const UActorComponent* InComponent, const FPCGStack& InPinStack, const FPCGDeltaKey& InInsertedDeltaKey, const int32 InInsertedElementIndex)
	: HComponentVisProxy(InComponent)
	, PinStack(InPinStack)
	, bIsInsertedElement(true)
	, InsertedElementIndex(InInsertedElementIndex)
	, InsertedDeltaKey(InInsertedDeltaKey)
{}

/** Define commands for the PCG component visualizer */
class FPCGComponentVisualizerCommands : public TCommands<FPCGComponentVisualizerCommands>
{
public:
	FPCGComponentVisualizerCommands() : TCommands<FPCGComponentVisualizerCommands>
		(
			"PCGComponentVisualizer",                                      // Context name for fast lookup
			LOCTEXT("PCGComponentVisualizer", "PCG Component Visualizer"), // Localized context name for displaying
			NAME_None,                                                     // Parent
			FAppStyle::GetAppStyleSetName()
			)
	{}

	virtual void RegisterCommands() override
	{
		UI_COMMAND(RemoveAllOverrides, "Remove All Overrides", "Removes all data overrides from the active collection.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(RemoveSelectedOverride, "Remove Selected Override", "Removes the currently selected data override.", EUserInterfaceActionType::Button, FInputChord());
	}

	TSharedPtr<FUICommandInfo> RemoveAllOverrides;
	TSharedPtr<FUICommandInfo> RemoveSelectedOverride;
};

FPCGComponentVisualizer::FPCGComponentVisualizer()
{
	FPCGComponentVisualizerCommands::Register();

	ComponentVisualizerActions = MakeShareable(new FUICommandList);

	CachedWireframe = MakeUnique<PCG::ComponentVisualizer::FCachedMeshWireframe>();
}


FPCGComponentVisualizer::~FPCGComponentVisualizer()
{
	for (auto& [Component, Cache] : ComponentCaches)
	{
		TeardownManualEditComponentCacheCallbacks(Cache, Component.Get());
	}

	ComponentCaches.Empty();

	ResetGizmo();
}

void FPCGComponentVisualizer::OnRegister()
{
	FComponentVisualizer::OnRegister();

	const FPCGComponentVisualizerCommands& Commands = FPCGComponentVisualizerCommands::Get();

	ComponentVisualizerActions->MapAction(
		Commands.RemoveAllOverrides,
		FExecuteAction::CreateSP(this, &FPCGComponentVisualizer::RemoveAllDeltas));
	ComponentVisualizerActions->MapAction(
		Commands.RemoveSelectedOverride,
		FExecuteAction::CreateSP(this, &FPCGComponentVisualizer::RemoveSelectedDeltas),
		FCanExecuteAction::CreateSP(this, &FPCGComponentVisualizer::HasSelection));
}

FPCGComponentVisualizer::FManualEditComponentCache* FPCGComponentVisualizer::FindOrCreateManualEditComponentCache(const UPCGComponent* InComponent)
{
	if (!InComponent)
	{
		return nullptr;
	}

	const TWeakObjectPtr<const UPCGComponent> Key(InComponent);
	if (FManualEditComponentCache* FoundCache = ComponentCaches.Find(Key))
	{
		return FoundCache;
	}

	// Remove stale entries before adding a new cache.
	for (auto It = ComponentCaches.CreateIterator(); It; ++It)
	{
		if (!It.Key().IsValid())
		{
			TeardownManualEditComponentCacheCallbacks(It.Value(), It.Key().Get());
			It.RemoveCurrent();
		}
	}

	FManualEditComponentCache& NewCache = ComponentCaches.Add(Key);
	SetupManualEditComponentCacheCallbacks(NewCache, InComponent);

	return &NewCache;
}

void FPCGComponentVisualizer::SetupManualEditComponentCacheCallbacks(FManualEditComponentCache& Cache, const UPCGComponent* InComponent)
{
	if (!InComponent)
	{
		return;
	}

	UPCGComponent* MutableComponent = const_cast<UPCGComponent*>(InComponent);
	auto MarkDirty = [this](const UPCGComponent* Component)
	{
		if (FManualEditComponentCache* Found = ComponentCaches.Find(Component))
		{
			Found->bDirty = true;
		}
	};

	Cache.GraphGeneratedHandle = MutableComponent->OnPCGGraphGeneratedDelegate.AddSPLambda(this, MarkDirty);
	Cache.GraphCleanedHandle = MutableComponent->OnPCGGraphCleanedDelegate.AddSPLambda(this, MarkDirty);
}

void FPCGComponentVisualizer::TeardownManualEditComponentCacheCallbacks(FManualEditComponentCache& Cache, const UPCGComponent* InComponent)
{
	if (UPCGComponent* MutableComp = const_cast<UPCGComponent*>(InComponent))
	{
		if (Cache.GraphGeneratedHandle.IsValid())
		{
			MutableComp->OnPCGGraphGeneratedDelegate.Remove(Cache.GraphGeneratedHandle);
		}

		if (Cache.GraphCleanedHandle.IsValid())
		{
			MutableComp->OnPCGGraphCleanedDelegate.Remove(Cache.GraphCleanedHandle);
		}
	}

	Cache.GraphGeneratedHandle.Reset();
	Cache.GraphCleanedHandle.Reset();
}

void FPCGComponentVisualizer::BuildExecutedNodeStacksCache(FManualEditComponentCache& Cache, const UPCGComponent* InComponent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGComponentVisualizer::BuildExecutedNodeStacksCache);

	Cache.ExecutedNodeStacks.Reset();
	Cache.bDirty = false;

	if (!InComponent)
	{
		return;
	}

	const FPCGGraphExecutionInspection& Inspection = InComponent->GetExecutionState().GetInspection();

	Inspection.ForEachExecutedNodeStack(
		[&Cache](const UPCGNode* Node, const TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData>& NotificationDataSet)
		{
			FManualEditComponentCache::FNodeStacks& Entry = Cache.ExecutedNodeStacks.AddDefaulted_GetRef();
			Entry.Node = Node;
			Entry.Stacks.Reserve(NotificationDataSet.Num());

			for (const FPCGGraphExecutionInspection::FNodeExecutedNotificationData& StackData : NotificationDataSet)
			{
				Entry.Stacks.Add(StackData.Stack);
			}

			return true;
		});
}

void FPCGComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGComponentVisualizer::DrawVisualization);

	using namespace PCG::ComponentVisualizer;
	using namespace PCG::ComponentVisualizer::Defaults;

	ON_SCOPE_EXIT { FComponentVisualizer::DrawVisualization(Component, View, PDI); };

	{
		const UPCGComponent* PCGComp = Cast<const UPCGComponent>(Component);
		if (!PCGComp)
		{
			return;
		}

		const FPCGSourceDataContainer* DataContainer = PCGComp->GetExecutionState().GetSourceDataContainer();
		if (!DataContainer)
		{
			return;
		}

		FPCGEditorModule* EditorModule = FModuleManager::GetModulePtr<FPCGEditorModule>("PCGEditor");
		const UPCGNode* PanelActiveNode = nullptr;
		bool bPanelIsExternallyControlled = false;
		if (EditorModule)
		{
			if (TSharedPtr<SPCGManualEditPanel> Panel = EditorModule->GetManualEditPanel())
			{
				PanelActiveNode = Panel->GetActiveNode();
				bPanelIsExternallyControlled = (Panel->GetMode() == EPCGManualEditPanelMode::ExternallyControlled);
			}
		}

		FManualEditComponentCache* Cache = FindOrCreateManualEditComponentCache(PCGComp);

		// These need to be reset each frame. They are members used outside of this scope.
		ActiveEditingNode.Reset();
		CachedDeltaContext.Reset();

		if (!Cache)
		{
			return;
		}

		const FPCGGraphExecutionInspection& Inspection = PCGComp->GetExecutionState().GetInspection();

		if (!PCGComp->IsGenerating())
		{
			const uint64 CurrentStacksGeneration = Inspection.GetExecutedStacksGeneration();
			if (CurrentStacksGeneration != Cache->PreviousStacksGeneration)
			{
				Cache->bDirty = true;
				Cache->PreviousStacksGeneration = CurrentStacksGeneration;
			}

			if (Cache->bDirty)
			{
				BuildExecutedNodeStacksCache(*Cache, PCGComp);
			}
		}

		if (Cache->ExecutedNodeStacks.IsEmpty())
		{
			return;
		}

		// Not ideal to const_cast here, but we need to modify the component.
		LastPCGComponent = const_cast<UPCGComponent*>(PCGComp);

		const FPCGDeltaViewportExtensionRegistry& ExtensionRegistry = FPCGEditorModule::GetConstDeltaViewportExtensionRegistry();

		const bool bEnableCulling = GEnableCulling;
		const double CullingDistance2DSquared = GCullingDistance2D * GCullingDistance2D;
		const bool bFrustumCulling = GFrustumCulling;
		const FVector ViewOrigin = View->ViewMatrices.GetViewOrigin();
		const FVector ViewDirection = View->GetViewDirection().GetSafeNormal();
		const FConvexVolume& CullingFrustum = View->GetCullingFrustum();

		auto IsCulled = [bEnableCulling, bFrustumCulling, CullingDistance2DSquared, ViewOrigin, ViewDirection, CullingFrustum](const FVector& Position)
		{
			if (!bEnableCulling)
			{
				return false;
			}

			const double Dist2D = FVector::DistSquaredXY(ViewOrigin, Position);
			if (Dist2D > CullingDistance2DSquared)
			{
				return true;
			}
				
			// Remove points behind the camera
			if (ViewDirection.Dot(Position - ViewOrigin) < 0)
			{
				return true;
			}
				
			return bFrustumCulling && !CullingFrustum.IntersectPoint(Position);
		};

		for (const FManualEditComponentCache::FNodeStacks& Entry : Cache->ExecutedNodeStacks)
		{
			const UPCGNode* Node = Entry.Node.Get();
			const UPCGSettingsInterface* NodeSettings = Node ? Node->GetSettingsInterface() : nullptr;
			if (!NodeSettings)
			{
				continue;
			}

			const bool bHasManualEditFlag = NodeSettings->IsTemporaryManualEditingEnabled() || NodeSettings->IsMarkedForManualEditing();
			const bool bIsActiveNode = (Node == PanelActiveNode);
			if (!bIsActiveNode && !(bPanelIsExternallyControlled && bHasManualEditFlag))
			{
				continue;
			}

			FPCGElementPtr Element = Node->GetSettings() ? Node->GetSettings()->GetElement() : nullptr;
			if (!Element)
			{
				continue;
			}

			// Only override pre or post execute.
			EPCGDataOverridePhase Phase = Element->GetDataOverridePhase();
			if (Phase != EPCGDataOverridePhase::PrepareData && Phase != EPCGDataOverridePhase::PostExecute)
			{
				continue;
			}

			if (bIsActiveNode)
			{
				// Const cast needed because the inspection map stores const keys.
				ActiveEditingNode = const_cast<UPCGNode*>(Node);
			}

			// Resolve and cache the debug mesh wireframe geometry for this node.
			// @todo_pcg: Allow selecting the debug mesh via a attribute selector for the selection proxy.
			const FPCGDebugVisualizationSettings& DebugSettings = Node->GetSettingsInterface()->DebugSettings;
			const UStaticMesh* DebugMesh = DebugSettings.PointMesh.Get();
			if (!DebugMesh)
			{
				if (!CachedDefaultPointMesh.IsValid())
				{
					CachedDefaultPointMesh = Cast<UStaticMesh>(TSoftObjectPtr<UStaticMesh>(PCGDebugVisConstants::DefaultPointMesh).LoadSynchronous());
				}
				DebugMesh = CachedDefaultPointMesh.Get();
			}

			const int32 ComplexityThreshold = CVarPCGManualEditMeshComplexityThreshold.GetValueOnGameThread();
			if (ComplexityThreshold > 0 && FCachedMeshWireframe::GetTriangleCount(DebugMesh) > ComplexityThreshold)
			{
				DebugMesh = CachedDefaultPointMesh.Get();
			}

			if (!CachedWireframe->TryCache(DebugMesh))
			{
				continue;
			}
			
			TArray<TTuple<FTransform, HPCGComponentVisualizerHitProxy*, FLinearColor>, TInlineAllocator<512>> ElementsToDraw;

			const TArray<TObjectPtr<UPCGPin>>& Pins = (Phase == EPCGDataOverridePhase::PrepareData) ? Node->GetInputPins() : Node->GetOutputPins();
			if (Pins.Num() > 0)
			{
				for (const FPCGStack& Stack : Entry.Stacks)
				{
					const UPCGPin* PinUsed = Pins[0];
					const FName PinLabel = PinUsed->Properties.Label;

					FPCGSourceDataStorageKey StorageKey(PCG::DataOverride::Constants::DefaultOverrideLabel, PCG::DataOverride::Helpers::ComputePinOverrideKey(Stack, Node, PinLabel).Hash);

					// Push context and selection state to the panel only for the active node.
					if (bIsActiveNode && EditorModule)
					{
						if (TSharedPtr<SPCGManualEditPanel> Panel = EditorModule->GetManualEditPanel())
						{
							Panel->SetDeltaContext(LastPCGComponent.Get(), StorageKey);
							if (EditTarget.IsSet())
							{
								Panel->SetSelectionState(true, EditTarget->DeltaKey, EditTarget->Transform, EditTarget->SelectedElementIndex, EditTarget->OriginalPointIndex);
							}
							else
							{
								Panel->SetSelectionState(false, FPCGDeltaKey{});
							}
						}
					}

					FPCGStack PinStack = Stack;
					PinStack.PushFrame(Node);
					PinStack.PushFrame(PinUsed);

					// Cache the collection lookup for action methods. Only the active node's context drives the gizmo.
					if (bIsActiveNode)
					{
						CachedDeltaContext.DataContainer = LastPCGComponent->GetExecutionState().GetSourceDataContainer();
						CachedDeltaContext.StorageKey = StorageKey;
						CachedDeltaContext.PinStack = PinStack;
						CachedDeltaContext.PCGComponent = LastPCGComponent;
					}

					// Retrieve the delta collection
					FConstSharedStruct SharedStruct = DataContainer->Get<FPCGDeltaCollection>(StorageKey);
					const FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<const FPCGDeltaCollection>() : nullptr;
					TSharedPtr<FPCGInspectionData> InspectionData = Inspection.GetInspectionDataPtr(PinStack);

					if (InspectionData && InspectionData->Data)
					{
						const TArray<FPCGTaggedData>& DataArray = InspectionData->Data->GetAllInputs();

						{
							TRACE_CPUPROFILER_EVENT_SCOPE(FPCGComponentVisualizer::DrawVisualization::PrepareElementsToDraw);
							for (int DataIndex = 0; DataIndex < DataArray.Num(); DataIndex++)
							{
								const FPCGTaggedData& Input = DataArray[DataIndex];
								// @todo_pcg: For now only point data is supported. On a future pass, this should be expanded to include other data types.
								if (const UPCGBasePointData* BasePointData = Cast<UPCGBasePointData>(Input.Data))
								{
									TConstPCGValueRange<FTransform> TransformRange = BasePointData->GetConstTransformValueRange();
									ElementsToDraw.Reserve(ElementsToDraw.Num() + BasePointData->GetNumPoints());
									for (int Index = 0; Index < BasePointData->GetNumPoints(); Index++)
									{
										const FTransform& SourceElementTransform = TransformRange[Index];

										// Delta-type-aware lookup via extension registry
										FDeltaStateResult StateResult;
										if (Collection)
										{
											StateResult = Helpers::FindDeltaFromCollection(*Collection, SourceElementTransform, ExtensionRegistry);
										}

										const bool bHasDelta = StateResult.Key.Hash != 0;
										const bool bIsSelectedDelta = bHasDelta && EditTarget && EditTarget->DeltaKey == StateResult.Key && EditTarget->Stack == PinStack;
										const bool bIsSelectedSource = !bHasDelta && EditTarget && EditTarget->OriginalPointIndex == Index && EditTarget->Stack == PinStack;
										const bool bIsSelected = bIsSelectedDelta || bIsSelectedSource;

										// @todo_pcg: Gizmo sync with delta data for external changes (e.g. graph re-execution) is not yet implemented.
										// The sync block needs a dedicated virtual method to read the current transform from the delta struct,
										// since GetDisplayTransform is designed for the drawing loop and returns the wrong value here.

										// While dragging, the selected element is not visited by the main loop. Draw it here for visual feedback.
										if (bIsSelected && EditTarget && EditTarget->bGizmoDragging && ActiveEditingNode.IsValid() && CachedWireframe->IsValid())
										{
											const FLinearColor DragColor = EditTarget->ActiveExtension
												? EditTarget->ActiveExtension->GetDisplayColor(/*bIsSelected=*/true)
												: GetNoDeltaColor(/*bIsSelected=*/true);

											ElementsToDraw.Emplace(EditTarget->Transform, nullptr, DragColor);
										}

										// Extension provides the display transform (e.g. deletions show at original location)
										const FTransform& Transform = StateResult.Extension ? StateResult.Extension->GetDisplayTransform(SourceElementTransform, StateResult.DeltaView) : SourceElementTransform;

										if (IsCulled(Transform.GetLocation()))
										{
											continue;
										}

										const FLinearColor ProxyColor = StateResult.Extension ? StateResult.Extension->GetDisplayColor(bIsSelected) : GetNoDeltaColor(bIsSelected);

										ElementsToDraw.Emplace(Transform, new HPCGComponentVisualizerHitProxy(Component, PinStack, DataIndex, Index), ProxyColor);
									}
									
									// Render inserted elements provided by extensions
									if (Collection)
									{
										const FPCGDeltaViewportExtensionRegistry& Registry = ExtensionRegistry;
										TArray<FInsertedViewportElement> InsertedElements;
										for (const UScriptStruct* DeltaType : Registry.GetRegisteredDeltaTypes())
										{
											IPCGDeltaViewportExtension* Extension = Registry.GetExtension(DeltaType);
											if (!Extension)
											{
												continue;
											}

											InsertedElements.Reset();
											Extension->GetInsertedElements(*Collection, InsertedElements);

											ElementsToDraw.Reserve(ElementsToDraw.Num() + InsertedElements.Num());

											for (const FInsertedViewportElement& Inserted : InsertedElements)
											{
												if (IsCulled(Inserted.Transform.GetLocation()))
												{
													continue;
												}

												const bool bIsInsertedSelected = EditTarget
													&& EditTarget->ActiveExtension == Extension
													&& EditTarget->DeltaKey == Inserted.DeltaKey
													&& EditTarget->SelectedElementIndex == Inserted.ElementIndex;

												FLinearColor InsertedColor = Extension->GetDisplayColor(bIsInsertedSelected);

												ElementsToDraw.Emplace(Inserted.Transform, new HPCGComponentVisualizerHitProxy(Component, PinStack, Inserted.DeltaKey, Inserted.ElementIndex), InsertedColor);
											}
										}
									}
								}
							}
						}
					}
				}
			}
			
			// Draw can't happen outside of the loop as the CachedWireframe might be different between inputs.
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGComponentVisualizer::DrawVisualization::Allocate);
				const int32 NumLines = CachedWireframe->Indices.Num();
				PDI->AddReserveLines(SDPG_World, NumLines * ElementsToDraw.Num());
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGComponentVisualizer::DrawVisualization::Draw);
				for (auto& [Transform, HitProxy, Color] : ElementsToDraw)
				{
					PDI->SetHitProxy(HitProxy);
					Helpers::DrawMeshWireframe(PDI, Transform, *CachedWireframe, Color);
					PDI->SetHitProxy(nullptr);
				}
			}

			// UserControlled draws a single eligible node; ExternallyControlled iterates them all.
			if (!bPanelIsExternallyControlled)
			{
				break;
			}
		}
	}
}

void FPCGComponentVisualizer::DrawVisualizationHUD(const UActorComponent* Component, const FViewport* Viewport, const FSceneView* View,
												   FCanvas* Canvas)
{
	FComponentVisualizer::DrawVisualizationHUD(Component, Viewport, View, Canvas);
}

bool FPCGComponentVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click)
{
	using namespace PCG::ComponentVisualizer;
	using namespace PCG::ComponentVisualizer::Helpers;
	using namespace PCG::DataOverride;
	using namespace PCG::DataOverride::Helpers;

	FPCGEditorModule* EditorModule = FModuleManager::GetModulePtr<FPCGEditorModule>("PCGEditor");

	if (HPCGComponentVisualizerHitProxy* HitProxy = HitProxyCast<HPCGComponentVisualizerHitProxy>(VisProxy))
	{
		UPCGComponent* MutablePCGComponent = const_cast<UPCGComponent*>(CastChecked<const UPCGComponent>(HitProxy->Component));

		FPCGSourceDataContainer* DataContainer = MutablePCGComponent->GetExecutionState().GetSourceDataContainer();
		check(DataContainer);

		UModeManagerInteractiveToolsContext* ToolsContext = InViewportClient->GetModeTools() ? InViewportClient->GetModeTools()->GetInteractiveToolsContext() : nullptr;
		UInteractiveGizmoManager* GizmoManager = ToolsContext ? ToolsContext->GizmoManager.Get() : nullptr;
		if (!GizmoManager)
		{
			return false;
		}

		const FNodeAndPinFromStack Selected = ExtractNodeAndPinFromStack(HitProxy->PinStack);
		if (!Selected.IsValid())
		{
			return false;
		}

		// When the panel is pinned, proxy click syncs the panel highlight to show which spawner is being edited.
		if (EditorModule)
		{
			if (TSharedPtr<SPCGManualEditPanel> Panel = EditorModule->GetManualEditPanel())
			{
				if (Panel->GetMode() == EPCGManualEditPanelMode::ExternallyControlled && Panel->GetActiveNode() != Selected.Node)
				{
					Panel->SelectNode(Selected.Node);
				}
			}
		}

		const FName HitPinLabel = Selected.Pin->Properties.Label;
		FPCGStack HitExecutionStack = HitProxy->PinStack;
		HitExecutionStack.PopFrame(); // Pin
		HitExecutionStack.PopFrame(); // Node
		FPCGSourceDataStorageKey StorageKey(Constants::DefaultOverrideLabel, ComputePinOverrideKey(HitExecutionStack, Selected.Node, HitPinLabel).Hash);

		// @todo_pcg: Delegate click handling to extensions to decouple behavior from the visualizer.
		// --- Handle inserted element click ---
		if (HitProxy->bIsInsertedElement)
		{
			FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(StorageKey);
			FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
			if (!Collection)
			{
				return false;
			}

			TInstancedStruct<FPCGDeltaBase>* DeltaInstancedStruct = Collection->Find(HitProxy->InsertedDeltaKey);
			if (!DeltaInstancedStruct)
			{
				return false;
			}

			const UScriptStruct* DeltaStructType = DeltaInstancedStruct->GetScriptStruct();
			IPCGDeltaViewportExtension* Extension = FPCGEditorModule::GetMutableDeltaViewportExtensionRegistry().GetExtension(DeltaStructType);
			if (!Extension)
			{
				return false;
			}

			FTransform InsertedTransform = FTransform::Identity;
			const int32 TargetElementIndex = HitProxy->InsertedElementIndex;

			TArray<FInsertedViewportElement> InsertedElements;
			Extension->GetInsertedElements(*Collection, InsertedElements);
			for (const FInsertedViewportElement& Inserted : InsertedElements)
			{
				if (Inserted.DeltaKey == HitProxy->InsertedDeltaKey && Inserted.ElementIndex == TargetElementIndex)
				{
					InsertedTransform = Inserted.Transform;
					break;
				}
			}

			EditTarget.Emplace(MutablePCGComponent, HitProxy->PinStack, StorageKey, HitProxy->InsertedDeltaKey, /*bWasInDelta=*/true, InsertedTransform);
			EditTarget->OriginalTransform = InsertedTransform;
			EditTarget->ActiveExtension = Extension;
			EditTarget->DeltaStructType = DeltaStructType;
			EditTarget->SelectedElementIndex = HitProxy->InsertedElementIndex;

			ResetGizmo();
			if (Extension->UsesTRSGizmo())
			{
				SetupGizmo(GizmoManager, InsertedTransform);
			}

			RefreshKeyBindings(Extension);
			return true;
		}

		// --- Handle regular point click ---
		const FPCGGraphExecutionInspection& Inspection = MutablePCGComponent->GetExecutionState().GetInspection();
		TSharedPtr<FPCGInspectionData> InspectionData = Inspection.GetInspectionDataPtr(HitProxy->PinStack);
		if (!InspectionData || !InspectionData->Data)
		{
			return false;
		}

		const TArray<FPCGTaggedData>& DataArray = InspectionData->Data->GetAllInputs();
		if (!ensure(DataArray.IsValidIndex(HitProxy->DataIndex)))
		{
			return false;
		}

		// @todo_pcg: For now only point data is supported. On a future pass, this should be expanded to include other data types.
		const FPCGTaggedData& Input = DataArray[HitProxy->DataIndex];
		if (const UPCGBasePointData* BasePointData = Cast<UPCGBasePointData>(Input.Data))
		{
			FPCGPointTransform::ConstValueRange TransformRange = BasePointData->GetConstTransformValueRange();
			if (!ensure(TransformRange.IsValidIndex(HitProxy->PointIndex)))
			{
				return false;
			}

			const FTransform& CurrentTransform = TransformRange[HitProxy->PointIndex];
			FPCGDeltaCollection* Collection = nullptr;
			FPCGDeltaKey DeltaKey = {};
			bool bWasInDelta = false;
			FDeltaStateResult StateResult;

			// Check if the point is already a product of a delta.
			{
				FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(StorageKey);
				Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
				if (Collection)
				{
					StateResult = FindDeltaFromCollection(*Collection, CurrentTransform, FPCGEditorModule::GetConstDeltaViewportExtensionRegistry());
					DeltaKey = StateResult.Key;
					bWasInDelta = (DeltaKey.Hash != 0);
				}
			}

			// Duplicate the clicked element as an inserted point on Alt+click
			if (Click.IsAltDown())
			{
				if (!Collection)
				{
					FPCGDeltaCollection NewCollection;
					NewCollection.Settings.KeyPolicy = EPCGDataOverrideKeyPolicy::Spatial;
					NewCollection.Settings.KeyTarget = EPCGSpatialKeyTarget::Position;
					NewCollection.Settings.SpatialTolerance = Constants::SpatialToleranceDefault;
					DataContainer->Store(StorageKey, NewCollection);

					FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(StorageKey);
					Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
				}
				check(Collection);

				// Use the original transform for the duplicate (not post-apply zero-scale for deletions)
				const FTransform& SourceTransform = (StateResult.Extension && StateResult.DeltaView.IsValid())
					? StateResult.Extension->GetDisplayTransform(CurrentTransform, StateResult.DeltaView)
					: CurrentTransform;

				// Delegate to the active extension's OnAltDrag handler.
				IPCGDeltaViewportExtension* AltDragExtension = nullptr;
				if (EditorModule)
				{
					if (TSharedPtr<SPCGManualEditPanel> Panel = EditorModule->GetManualEditPanel())
					{
						if (const UScriptStruct* DeltaType = Panel->GetActiveDeltaType())
						{
							AltDragExtension = FPCGEditorModule::GetMutableDeltaViewportExtensionRegistry().GetExtension(DeltaType);
						}
					}
				}

				if (!AltDragExtension)
				{
					return false;
				}

				FScopedTransaction Transaction(LOCTEXT("InsertPointDelta", "PCG Manual Edit: Insert Point"));
				MutablePCGComponent->Modify();

				FPCGDeltaKey InsertionDeltaKey;
				int32 NewElementIndex = AltDragExtension->OnAltDrag(*Collection, SourceTransform, InsertionDeltaKey);
				if (NewElementIndex == INDEX_NONE)
				{
					return false;
				}

				EditTarget.Emplace(MutablePCGComponent, HitProxy->PinStack, StorageKey, InsertionDeltaKey, /*bWasInDelta=*/true, SourceTransform);
				EditTarget->OriginalTransform = SourceTransform;
				EditTarget->ActiveExtension = AltDragExtension;
				const TInstancedStruct<FPCGDeltaBase>* FoundDelta = Collection->Find(InsertionDeltaKey);
				EditTarget->DeltaStructType = FoundDelta ? FoundDelta->GetScriptStruct() : nullptr;
				EditTarget->SelectedElementIndex = NewElementIndex;
				EditTarget->bModified = true;

				ResetGizmo();

				if (AltDragExtension->UsesTRSGizmo())
				{
					SetupGizmo(GizmoManager, SourceTransform);
				}

				PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(MutablePCGComponent, DataContainer);

				RefreshKeyBindings(AltDragExtension);

				return true;
			}

			// Clicking an element with an existing delta that doesn't use a TRS gizmo
			if (bWasInDelta && StateResult.Extension && !StateResult.Extension->UsesTRSGizmo())
			{
				const FTransform DisplayTransform = StateResult.Extension->GetDisplayTransform(CurrentTransform, StateResult.DeltaView);
				EditTarget.Emplace(MutablePCGComponent, HitProxy->PinStack, StorageKey, DeltaKey, /*bWasInDelta=*/true, DisplayTransform);
				EditTarget->OriginalTransform = DisplayTransform;
				EditTarget->ActiveExtension = StateResult.Extension;
				EditTarget->DeltaStructType = StateResult.DeltaView.GetScriptStruct();
				EditTarget->OriginalPointIndex = HitProxy->PointIndex;

				ResetGizmo();
				RefreshKeyBindings(StateResult.Extension);

				return true;
			}

			// Resolve extension for deferred delta creation
			IPCGDeltaViewportExtension* DeferredExtension = nullptr;
			const UScriptStruct* DeferredDeltaType = nullptr;

			// It was not part of a delta...
			if (!bWasInDelta)
			{
				// Create collection if there was not already one.
				if (!Collection)
				{
					FPCGDeltaCollection NewCollection;
					NewCollection.Settings.KeyPolicy = EPCGDataOverrideKeyPolicy::Spatial;
					NewCollection.Settings.KeyTarget = EPCGSpatialKeyTarget::Position;
					NewCollection.Settings.SpatialTolerance = Constants::SpatialToleranceDefault;

					DataContainer->Store(StorageKey, NewCollection);

					FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(StorageKey);
					Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
				}

				check(Collection);

				// Defer delta creation until an explicit user action (gizmo drag, Delete key, Insert button).
				// Resolve the extension from the panel's selected delta type for key computation.
				FName DeltaName = NAME_None;
				if (EditorModule)
				{
					if (TSharedPtr<SPCGManualEditPanel> Panel = EditorModule->GetManualEditPanel())
					{
						if (const UScriptStruct* DeltaType = Panel->GetActiveDeltaType())
						{
							DeferredExtension = FPCGEditorModule::GetMutableDeltaViewportExtensionRegistry().GetExtension(DeltaType);
							DeferredDeltaType = DeltaType;
							if (DeferredExtension)
							{
								DeltaName = DeferredExtension->GetDeltaName();
							}
						}
					}
				}

				// Use composite key with signature settings from the hit node's configuration.
				bool bSignaturePosition = true;
				bool bSignatureRotation = false;
				bool bSignatureScale = false;
				double SignatureTolerance = Collection->Settings.SpatialTolerance;
				if (EditorModule)
				{
					if (TSharedPtr<SPCGManualEditPanel> Panel = EditorModule->GetManualEditPanel())
					{
						if (TSharedPtr<const FPCGManualEditNodeConfiguration> NodeConfiguration = Panel->GetOrCreateConfigurationForNode(Selected.Node))
						{
							bSignaturePosition = NodeConfiguration->bSignaturePosition;
							bSignatureRotation = NodeConfiguration->bSignatureRotation;
							bSignatureScale = NodeConfiguration->bSignatureScale;
							SignatureTolerance = NodeConfiguration->SignatureTolerance;
						}
					}
				}

				DeltaKey = PCG::DataOverride::Keys::FPCGCompositeTransformDeltaKey(CurrentTransform, SignatureTolerance, bSignaturePosition, bSignatureRotation, bSignatureScale, DeltaName);
			}

			EditTarget.Emplace(MutablePCGComponent, HitProxy->PinStack, StorageKey, DeltaKey, bWasInDelta, CurrentTransform);
			EditTarget->OriginalTransform = CurrentTransform;
			EditTarget->OriginalPointIndex = HitProxy->PointIndex;

			if (bWasInDelta && StateResult.Extension)
			{
				EditTarget->ActiveExtension = StateResult.Extension;
				EditTarget->DeltaStructType = StateResult.DeltaView.GetScriptStruct();
			}
			else if (!bWasInDelta)
			{
				EditTarget->bDeltaDeferred = true;
				EditTarget->ActiveExtension = DeferredExtension;
				EditTarget->DeltaStructType = DeferredDeltaType;
			}

			ResetGizmo();
			if (!EditTarget->ActiveExtension || EditTarget->ActiveExtension->UsesTRSGizmo())
			{
				SetupGizmo(GizmoManager, CurrentTransform);
			}

			RefreshKeyBindings(EditTarget->ActiveExtension);

			return true;
		}
	}

	return false;
}

void FPCGComponentVisualizer::ResetGizmo()
{
	if (Gizmo.IsValid())
	{
		Gizmo->SetVisibility(false);
		Gizmo->ClearActiveTarget();

		if (UInteractiveGizmoManager* GizmoManager = Gizmo->GetGizmoManager())
		{
			GizmoManager->DestroyGizmo(Gizmo.Get());
		}
		Gizmo.Reset();
	}
}

void FPCGComponentVisualizer::SetupGizmo(UInteractiveGizmoManager* GizmoManager, const FTransform& InitialTransform)
{
	ResetGizmo();

	if (!bRegisteredGizmoContextObject)
	{
		if (UModeManagerInteractiveToolsContext* ToolsContext = GLevelEditorModeTools().GetInteractiveToolsContext())
		{
			UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(ToolsContext);
			bRegisteredGizmoContextObject = true;
		}
	}

	ETransformGizmoSubElements TransformElements = ETransformGizmoSubElements::FullTranslateRotateScale;
	Gizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager, TransformElements, this, FString());
	if (!Gizmo.IsValid())
	{
		return;
	}

	UTransformProxy* Proxy = NewObject<UTransformProxy>(Gizmo.Get());
	Proxy->SetTransform(InitialTransform);
	Gizmo->SetActiveTarget(Proxy, GizmoManager);
	Gizmo->ReinitializeGizmoTransform(InitialTransform);
	Gizmo->SetVisibility(true);

	Proxy->OnTransformChanged.AddLambda([this](UTransformProxy* InProxy, const FTransform& NewTransform)
	{
		OnGizmoDragged(NewTransform);
	});

	Proxy->OnEndTransformEdit.AddLambda([this](UTransformProxy* InProxy)
	{
		OnGizmoReleased(InProxy->GetTransform());
	});
}

void FPCGComponentVisualizer::RefreshKeyBindings(IPCGDeltaViewportExtension* Extension)
{
	ActiveKeyBindings.Empty();

	if (!Extension)
	{
		return;
	}

	ActiveKeyBindings = Extension->GetKeyBindings();
}

void FPCGComponentVisualizer::EndEditing()
{
	ActiveKeyBindings.Empty();
	ClearActiveSelection();

	// In ExternallyControlled mode an external owner controls temp-flag lifetime; skip the wipe so the set survives.
	bool bPanelIsExternallyControlled = false;
	if (const FPCGEditorModule* EditorModule = FModuleManager::GetModulePtr<FPCGEditorModule>("PCGEditor"))
	{
		if (const TSharedPtr<SPCGManualEditPanel> Panel = EditorModule->GetManualEditPanel())
		{
			bPanelIsExternallyControlled = (Panel->GetMode() == EPCGManualEditPanelMode::ExternallyControlled);
		}
	}

	// Clear transient manual editing flags on all nodes via the graph.
	bool bAnyFlagCleared = false;
	const UPCGComponent* Component = LastPCGComponent.Get();
	if (!bPanelIsExternallyControlled && Component)
	{
		if (const UPCGGraph* PCGGraph = Component->GetGraph())
		{
			for (UPCGNode* Node : PCGGraph->GetNodes())
			{
				UPCGSettingsInterface* Settings = Node ? Node->GetSettingsInterface() : nullptr;
				if (Settings && Settings->IsTemporaryManualEditingEnabled())
				{
					Settings->SetTemporaryManualEditingEnabled(false);
					Node->OnNodeChangedDelegate.Broadcast(Node, EPCGChangeType::Cosmetic);
					bAnyFlagCleared = true;
				}
			}
		}
	}

	// Avoid disrupting the panel's active node state during normal editing.
	if (bAnyFlagCleared)
	{
		if (FPCGEditorModule* EditorModule = FModuleManager::GetModulePtr<FPCGEditorModule>("PCGEditor"))
		{
			EditorModule->UpdateManualEditPanelVisibility();
		}
	}

	ActiveEditingNode.Reset();
	CachedDeltaContext.Reset();

	LastPCGComponent.Reset();

	// Deregister here guaranteeing the context object is not left orphaned in the shared store.
	if (bRegisteredGizmoContextObject)
	{
		if (UModeManagerInteractiveToolsContext* ToolsContext = GLevelEditorModeTools().GetInteractiveToolsContext())
		{
			UE::TransformGizmoUtil::DeregisterTransformGizmoContextObject(ToolsContext);
		}

		bRegisteredGizmoContextObject = false;
	}

	FComponentVisualizer::EndEditing();
}

bool FPCGComponentVisualizer::GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const
{
	if (EditTarget.IsSet())
	{
		OutLocation = EditTarget->Transform.GetLocation();
		return true;
	}

	return false;
}

bool FPCGComponentVisualizer::GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const
{
	return FComponentVisualizer::GetCustomInputCoordinateSystem(ViewportClient, OutMatrix);
}

bool FPCGComponentVisualizer::HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate,
											   FRotator& DeltaRotate, FVector& DeltaScale)
{
	return FComponentVisualizer::HandleInputDelta(ViewportClient, Viewport, DeltaTranslate, DeltaRotate, DeltaScale);
}

bool FPCGComponentVisualizer::HandleInputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FKey Key, const EInputEvent Event)
{
	if (Event == IE_Pressed && EditTarget.IsSet())
	{
		for (const FPCGDeltaKeyBinding& Binding : ActiveKeyBindings)
		{
			if (Binding.Key == Key && Binding.Action)
			{
				Binding.Action();
				return true;
			}
		}
	}

	return FComponentVisualizer::HandleInputKey(ViewportClient, Viewport, Key, Event);
}

TSharedPtr<SWidget> FPCGComponentVisualizer::GenerateContextMenu() const
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, ComponentVisualizerActions);

	MenuBuilder.AddMenuEntry(FPCGComponentVisualizerCommands::Get().RemoveAllOverrides);
	MenuBuilder.AddMenuEntry(FPCGComponentVisualizerCommands::Get().RemoveSelectedOverride);

	TSharedPtr<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	return MenuWidget;
}

void FPCGComponentVisualizer::OnGizmoDragged(const FTransform& NewTransform)
{
	if (FEditState* Selection = EditTarget.GetPtrOrNull())
	{
		Selection->Transform = NewTransform;
		Selection->bGizmoDragging = true;
	}
}

void FPCGComponentVisualizer::OnGizmoReleased(const FTransform& FinalTransform)
{
	using namespace PCG::ComponentVisualizer;

	FEditState* Selection = EditTarget.GetPtrOrNull();
	if (!Selection || !EditTarget->PCGComponent.IsValid())
	{
		return;
	}

	Selection->bGizmoDragging = false;

	Selection->Transform = FinalTransform;

	FScopedTransaction Transaction(LOCTEXT("MovePointDelta", "PCG Manual Edit: Move Point"));
	EditTarget->PCGComponent->Modify();

	FPCGSourceDataContainer* DataContainer = EditTarget->PCGComponent->GetExecutionState().GetSourceDataContainer();
	if (!DataContainer)
	{
		return;
	}

	FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(EditTarget->StorageKey);
	FPCGDeltaCollection* Collection = SharedStruct.IsValid()
										  ? SharedStruct.GetPtr<FPCGDeltaCollection>()
										  : nullptr;

	// Lazy delta creation: first gizmo release materializes the deferred delta via the extension
	if (Selection->bDeltaDeferred && Collection && Selection->ActiveExtension)
	{
		Selection->bDeltaDeferred = false;

		FPCGDeltaCreateContext CreateContext;
		CreateContext.ComponentBounds = EditTarget->PCGComponent->GetTotalBounds();
		CreateContext.OriginalElementIndex = Selection->OriginalPointIndex;

		Selection->ActiveExtension->CreateNewDelta(EditTarget->DeltaKey, *Collection, Selection->OriginalTransform, FinalTransform, CreateContext);
	}

	TInstancedStruct<FPCGDeltaBase>* DeltaInstancedStruct = Collection ? Collection->Find(EditTarget->DeltaKey) : nullptr;

	if (DeltaInstancedStruct && Selection->ActiveExtension)
	{
		if (Selection->ActiveExtension->ApplyGizmoTransform(FinalTransform, *DeltaInstancedStruct, Selection->SelectedElementIndex))
		{
			EditTarget->bModified = true;
			PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(EditTarget->PCGComponent.Get(), DataContainer);
		}
	}
}

void FPCGComponentVisualizer::RestoreDelta(const FPCGDeltaKey& Key)
{
	if (!CachedDeltaContext.DataContainer)
	{
		return;
	}

	FSharedStruct SharedStruct = CachedDeltaContext.DataContainer->GetMutable<FPCGDeltaCollection>(CachedDeltaContext.StorageKey);
	FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
	if (!Collection)
	{
		return;
	}

	if (UPCGComponent* PCGComponent = CachedDeltaContext.PCGComponent.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("RestoreDelta", "PCG Manual Edit: Restore Element"));
		PCGComponent->Modify();

		if (Collection->Remove(Key))
		{
			if (EditTarget.IsSet() && EditTarget->DeltaKey == Key)
			{
				EditTarget.Reset();
				ResetGizmo();
			}

			PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, CachedDeltaContext.DataContainer);
		}
	}
}

void FPCGComponentVisualizer::RestoreAllDeltas()
{
	if (!CachedDeltaContext.DataContainer)
	{
		return;
	}

	FSharedStruct SharedStruct = CachedDeltaContext.DataContainer->GetMutable<FPCGDeltaCollection>(CachedDeltaContext.StorageKey);
	FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
	if (!Collection)
	{
		return;
	}

	TArray<FPCGDeltaKey> RestorableKeys;
	const FPCGDeltaViewportExtensionRegistry& Registry = FPCGEditorModule::GetConstDeltaViewportExtensionRegistry();
	for (const UScriptStruct* DeltaType : Registry.GetRegisteredDeltaTypes())
	{
		if (IPCGDeltaViewportExtension* Extension = Registry.GetExtension(DeltaType))
		{
			Extension->CollectRestorableKeys(*Collection, RestorableKeys);
		}
	}

	if (RestorableKeys.Num() > 0)
	{
		if (UPCGComponent* PCGComponent = CachedDeltaContext.PCGComponent.Get())
		{
			FScopedTransaction Transaction(LOCTEXT("RestoreAllElements", "PCG Manual Edit: Restore All Elements"));
			PCGComponent->Modify();

			for (const FPCGDeltaKey& Key : RestorableKeys)
			{
				Collection->Remove(Key);
			}

			// If the current selection was one of the removed deltas, clear it
			if (EditTarget.IsSet())
			{
				for (const FPCGDeltaKey& Key : RestorableKeys)
				{
					if (EditTarget->DeltaKey == Key)
					{
						EditTarget.Reset();
						ResetGizmo();
						break;
					}
				}
			}

			PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, CachedDeltaContext.DataContainer);
		}
	}
}

void FPCGComponentVisualizer::SelectElement(const FPCGDeltaKey& DeltaKey, int32 ElementIndex)
{
	if (!CachedDeltaContext.DataContainer)
	{
		return;
	}

	FSharedStruct SharedStruct = CachedDeltaContext.DataContainer->GetMutable<FPCGDeltaCollection>(CachedDeltaContext.StorageKey);
	FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr;
	if (!Collection)
	{
		return;
	}

	TInstancedStruct<FPCGDeltaBase>* DeltaStruct = Collection->Find(DeltaKey);
	if (!DeltaStruct)
	{
		return;
	}

	const UScriptStruct* DeltaStructType = DeltaStruct->GetScriptStruct();
	IPCGDeltaViewportExtension* Extension = FPCGEditorModule::GetMutableDeltaViewportExtensionRegistry().GetExtension(DeltaStructType);
	if (!Extension)
	{
		return;
	}

	// Resolve the transform for the element
	FTransform ElementTransform = FTransform::Identity;
	const FConstStructView DeltaView(DeltaStructType, DeltaStruct->GetMemory());

	if (ElementIndex != INDEX_NONE)
	{
		TArray<FInsertedViewportElement> InsertedElements;
		Extension->GetInsertedElements(*Collection, InsertedElements);
		for (const FInsertedViewportElement& Inserted : InsertedElements)
		{
			if (Inserted.DeltaKey == DeltaKey && Inserted.ElementIndex == ElementIndex)
			{
				ElementTransform = Inserted.Transform;
				break;
			}
		}
	}
	else // Single-element delta without a specific element index. Read the transform directly from the delta.
	{
		ElementTransform = Extension->GetDisplayTransform(FTransform::Identity, DeltaView);
	}

	EditTarget.Emplace(CachedDeltaContext.PCGComponent.Get(), CachedDeltaContext.PinStack, CachedDeltaContext.StorageKey, DeltaKey, /*bWasInDelta=*/true, ElementTransform);
	EditTarget->OriginalTransform = ElementTransform;
	EditTarget->ActiveExtension = Extension;
	EditTarget->DeltaStructType = DeltaStructType;
	EditTarget->SelectedElementIndex = ElementIndex;

	ResetGizmo();
	UModeManagerInteractiveToolsContext* ToolsContext = GLevelEditorModeTools().GetInteractiveToolsContext();
	UInteractiveGizmoManager* GizmoManager = ToolsContext ? ToolsContext->GizmoManager.Get() : nullptr;
	if (Extension->UsesTRSGizmo() && GizmoManager)
	{
		SetupGizmo(GizmoManager, ElementTransform);
	}

	RefreshKeyBindings(Extension);
}

void FPCGComponentVisualizer::RemoveAllDeltas()
{
	// @todo_pcg: Only removes deltas for the active node's storage key. If multiple nodes have been edited, enumerate
	// all delta storage keys to clear them all.
	if (UPCGComponent* PCGComponent = CachedDeltaContext.IsValid() ? CachedDeltaContext.PCGComponent.Get() : nullptr)
	{
		FScopedTransaction Transaction(LOCTEXT("RemoveAllElements", "PCG Manual Edit: Remove All Elements"));
		PCGComponent->Modify();

		CachedDeltaContext.DataContainer->Remove(CachedDeltaContext.StorageKey);
		PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, CachedDeltaContext.DataContainer);
		CachedDeltaContext.Reset();
	}

	ClearActiveSelection();
}

void FPCGComponentVisualizer::RemoveSelectedDeltas()
{
	if (EditTarget.IsSet())
	{
		if (UPCGComponent* PCGComponent = EditTarget->PCGComponent.Get())
		{
			FScopedTransaction Transaction(LOCTEXT("RemoveSelectedElement", "PCG Manual Edit: Remove Selected Element"));
			PCGComponent->Modify();

			FPCGSourceDataContainer* DataContainer = PCGComponent->GetExecutionState().GetSourceDataContainer();
			if (!DataContainer)
			{
				return;
			}

			FSharedStruct SharedStruct = DataContainer->GetMutable<FPCGDeltaCollection>(EditTarget->StorageKey);
			if (FPCGDeltaCollection* Collection = SharedStruct.IsValid() ? SharedStruct.GetPtr<FPCGDeltaCollection>() : nullptr)
			{
				Collection->Remove(EditTarget->DeltaKey);
			}

			EditTarget.Reset();
			ClearActiveSelection();

			PCG::DeltaViewportExtension::Helpers::MarkComponentDirty(PCGComponent, DataContainer);
		}
	}
}

bool FPCGComponentVisualizer::HasSelection() const
{
	return EditTarget.IsSet() && EditTarget->PCGComponent.IsValid();
}

void FPCGComponentVisualizer::ClearActiveSelection()
{
	if (EditTarget.IsSet())
	{
		if (!EditTarget->bWasInDelta && !EditTarget->bModified && !EditTarget->bDeltaDeferred)
		{
			// Delta was created but never modified — remove it.
			// If bDeltaDeferred is true, no delta was ever stored, so nothing to remove.
			RemoveSelectedDeltas();
		}
	}

	EditTarget.Reset();
	ResetGizmo();
}

#undef LOCTEXT_NAMESPACE
