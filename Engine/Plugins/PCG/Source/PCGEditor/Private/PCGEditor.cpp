// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditor.h"

#include "AssetEditorMode/PCGAssetEdMode.h"
#include "AssetEditorMode/PCGAssetEditorToolRegistry.h"
#include "AssetEditorMode/Tools/PCGAssetEditorInteractiveTool.h"
#include "PCGComponent.h"
#include "PCGDefaultExecutionSource.h"
#include "PCGDefaultWorldObjectExecutionSource.h"
#include "PCGEdge.h"
#include "PCGEditorModule.h"
#include "PCGGraph.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGGraphFactory.h"
#include "PCGPin.h"
#include "PCGSubgraph.h"
#include "PCGSubsystem.h"
#include "Editor/IPCGEditorModule.h"
#include "Elements/PCGReroute.h"
#include "Helpers/PCGSubgraphHelpers.h"
#include "Rendering/SlateRenderer.h"
#include "Subsystems/PCGEngineSubsystem.h"
#include "Tests/Determinism/PCGDeterminismNativeTests.h"
#include "Tests/Determinism/PCGDeterminismTestBlueprintBase.h"
#include "Tests/Determinism/PCGDeterminismTestsCommon.h"

#include "PCGEditorCommands.h"
#include "PCGEditorDefaultMode.h"
#include "PCGEditorGraph.h"
#include "PCGEditorMenuContext.h"
#include "PCGEditorSettings.h"
#include "PCGEditorStyle.h"
#include "PCGEditorTabFactories.h"
#include "PCGEditorUtils.h"
#include "Managers/PCGEditorInspectionDataManager.h"
#include "Nodes/PCGEditorGraphNode.h"
#include "Nodes/PCGEditorGraphNodeInput.h"
#include "Nodes/PCGEditorGraphNodeOutput.h"
#include "Nodes/PCGEditorGraphNodeReroute.h"
#include "Schema/PCGEditorGraphSchema.h"
#include "Schema/PCGEditorGraphSchemaActions.h"

#include "SPCGManualEditPanel.h" // @todo_pcg: To be moved into Widgets/
#include "Widgets/SPCGCodeEditor.h"
#include "Widgets/SPCGEditorGraphActionWidget.h"
#include "Widgets/SPCGEditorGraphAttributeListView.h"
#include "Widgets/SPCGEditorGraphDebugObjectTree.h"
#include "Widgets/SPCGEditorGraphDetailsView.h"
#include "Widgets/SPCGEditorGraphDeterminism.h"
#include "Widgets/SPCGEditorGraphEmbeddedSubgraphsView.h"
#include "Widgets/SPCGEditorGraphFind.h"
#include "Widgets/SPCGEditorGraphLogView.h"
#include "Widgets/SPCGEditorGraphNodePalette.h"
#include "Widgets/SPCGEditorGraphParamsView.h"
#include "Widgets/SPCGEditorGraphProfilingView.h"
#include "Widgets/SPCGEditorGraphTitleBar.h"
#include "Widgets/AssetEditorViewport/SPCGEditorViewport.h"

#include "AdvancedPreviewScene.h"
#include "AssetDefinition.h"
#include "AssetEditorModeManager.h"
#include "AssetToolsModule.h"
#include "ContextObjectStore.h"
#include "EdGraphUtilities.h"
#include "EditorAssetLibrary.h"
#include "EditorModeManager.h"
#include "GraphEditorActions.h"
#include "IAssetTools.h"
#include "LevelEditor.h"
#include "ScopedTransaction.h"
#include "SGraphEditorActionMenu.h"
#include "ShaderCore.h"
#include "SNodePanel.h"
#include "SourceCodeNavigation.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "UnrealEdGlobals.h"
#include "Algo/AnyOf.h"
#include "Algo/TopologicalSort.h"
#include "Tools/EdModeInteractiveToolsContext.h"

#include "Editor/UnrealEdEngine.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/ITransaction.h"
#include "Misc/MessageDialog.h"
#include "Misc/TransactionObjectEvent.h"
#include "Overlay/SDraggableBoxOverlay.h"
#include "Preferences/UnrealEdOptions.h"
#include "Widgets/SPCGEditorGraphDataOverrides.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"

#define LOCTEXT_NAMESPACE "PCGGraphEditor"

namespace FPCGEditor_private
{
	const FName AllDefaultPanels[] =
	{
		PCGEditorTabs::GraphEditorID,
		PCGEditorTabs::PropertyDetailsID[0],
		PCGEditorTabs::PropertyDetailsID[1],
		PCGEditorTabs::PropertyDetailsID[2],
		PCGEditorTabs::PropertyDetailsID[3],
		PCGEditorTabs::PaletteID,
		PCGEditorTabs::DebugObjectID,
		PCGEditorTabs::AttributesID[0],
		PCGEditorTabs::AttributesID[1],
		PCGEditorTabs::AttributesID[2],
		PCGEditorTabs::AttributesID[3],
		PCGEditorTabs::FindID,
		PCGEditorTabs::DeterminismID,
		PCGEditorTabs::ProfilingID,
		PCGEditorTabs::LogID,
		PCGEditorTabs::CodeEditorID,
		PCGEditorTabs::UserParamsID,
		PCGEditorTabs::ViewportID[0],
		PCGEditorTabs::ViewportID[1],
		PCGEditorTabs::ViewportID[2],
		PCGEditorTabs::ViewportID[3],
		PCGEditorTabs::EmbeddedSubgraphsID,
		PCGEditorTabs::DataOverridesID
	};

	const FName AllRestrictedPanels[] =
	{
		PCGEditorTabs::CodeEditorID,
	};

	constexpr int32 DefaultFixedNodeWidth = 250;
	constexpr int32 DefaultFixedNodeHeight = 200;
	constexpr int32 NodeLayoutHorizontalGap = 200;
	constexpr int32 NodeLayoutVerticalGap = 50;
}

UPCGEditorGraph* FPCGEditor::GetPCGEditorGraph(UPCGGraph* InGraph)
{
	return InGraph ? InGraph->PCGEditorGraph : nullptr;
}

UPCGEditorGraph* FPCGEditor::GetPCGEditorGraph(const UPCGNode* InNode)
{
	UPCGGraph* PCGGraph = InNode ? Cast<UPCGGraph>(InNode->GetOuter()) : nullptr;
	return GetPCGEditorGraph(PCGGraph);
}

UPCGEditorGraph* FPCGEditor::GetPCGEditorGraph(const UPCGSettings* InSettings)
{
	UPCGNode* PCGNode = InSettings ? Cast<UPCGNode>(InSettings->GetOuter()) : nullptr;
	return GetPCGEditorGraph(PCGNode);
}

void FPCGEditor::InitializePCGEditorGraph(UPCGGraph* InGraph)
{
	// Initializes the UPCGEditorGraph if needed
	const TSubclassOf<UPCGEditorGraphSchema> SchemaClass = GetSchemaClass();
	const bool bShouldUpdateSchema = !InGraph->PCGEditorGraph || InGraph->PCGEditorGraph->Schema != SchemaClass || !InGraph->PCGEditorGraph->GetPCGGraph();
	if (!InGraph->PCGEditorGraph)
	{
		InGraph->PCGEditorGraph = NewObject<UPCGEditorGraph>(InGraph, UPCGEditorGraph::StaticClass(), NAME_None, RF_Transactional | RF_Transient);
	}

	if (bShouldUpdateSchema)
	{
		InGraph->PCGEditorGraph->Schema = SchemaClass;
		InGraph->PCGEditorGraph->InitFromNodeGraph(InGraph);
	}

	// Only set editor if this graph is owned by us
	if (InGraph == MainGraph || InGraph->GetEmbeddedParentGraph() == MainGraph)
	{
		InGraph->PCGEditorGraph->SetEditor(SharedThis(this));
	}
}

UPCGEditorGraph* FPCGEditor::GetOrCreatePCGEditorGraph(UPCGGraph* InPCGGraph)
{
	UPCGEditorGraph* PCGEditorGraph = GetPCGEditorGraph(InPCGGraph);
	if (!PCGEditorGraph)
	{
		InitializePCGEditorGraph(InPCGGraph);
	}
	
	return GetPCGEditorGraph(InPCGGraph);
}

void FPCGEditor::Initialize(const EToolkitMode::Type InMode, const TSharedPtr<class IToolkitHost>& InToolkitHost, UPCGGraph* InPCGGraph, UObject* InObjectToEdit)
{
	InObjectToEdit = InObjectToEdit != nullptr ? InObjectToEdit : InPCGGraph;

	InspectionDataManager.OnInspectedStackChangedDelegate.AddSP(SharedThis(this), &FPCGEditor::UpdateAfterInspectedStackChanged);

	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		PCGEditorModule->OnPermissionModeChanged().AddRaw(this, &FPCGEditor::OnPermissionChanged);
	}

	FPCGModule::GetGraphExecutionRegistry().OnGraphExecutionSourcesChanged().AddRaw(this, &FPCGEditor::OnGraphExecutionSourcesChanged);

	MainGraph = InPCGGraph;

	UpdateDefaultExecutionSource();

	InitializePCGEditorGraph(MainGraph);

	GetOrCreateCodeEditorWidget();

	BindCommands();

	MainGraph->OnGraphChangedDelegate.AddRaw(this, &FPCGEditor::OnGraphChanged);
	MainGraph->OnNodeSourceCompiledDelegate.AddRaw(this, &FPCGEditor::OnNodeSourceCompiled);

	// Hook to map change / delete actor to refresh debug object selection list, to help prevent it going stale.
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.OnMapChanged().AddRaw(this, &FPCGEditor::OnMapChanged);

	// Hook to PIE start/end to keep callbacks up to date.
	FEditorDelegates::PostPIEStarted.AddRaw(this, &FPCGEditor::OnPostPIEStarted);
	FEditorDelegates::EndPIE.AddRaw(this, &FPCGEditor::OnEndPIE);

	if (UPCGEngineSubsystem* EngineSubsystem = UPCGEngineSubsystem::Get())
	{
		EngineSubsystem->GetOnPCGSourceGenerationDone().AddRaw(this, &FPCGEditor::OnSourceGenerationDone);
	}

	if (GEditor)
	{
		RegisterDelegatesForWorld(GEditor->GetEditorWorldContext().World());

		// In case the editor is opened while in PIE, we should try setting up callbacks for the PIE world.
		RegisterDelegatesForWorld(GEditor->PlayWorld.Get());
	}
	
	DocumentManager = MakeShared<FDocumentTracker>(PCGEditorTabs::GraphEditorID);

	const FName PCGGraphEditorAppName = FName(TEXT("PCGEditorApp"));
	InitAssetEditor(InMode, InToolkitHost, PCGGraphEditorAppName, FTabManager::FLayout::NullLayout, /*bCreateDefaultStandaloneMenu=*/ true, /*bCreateDefaultToolbar=*/ true, InObjectToEdit);

	// Clear inspection flag on all nodes.
	for (UEdGraphNode* EdGraphNode : GetMainEditorGraph()->Nodes)
	{
		if (UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(EdGraphNode))
		{
			PCGEditorGraphNode->SetInspected(false);
		}
	}
	
	TSharedPtr<FPCGEditor> ThisPtr = SharedThis<FPCGEditor>(this);
	DocumentManager->Initialize(AsShared());
	TSharedPtr<FDocumentTabFactory> DocumentTabFactoryPtr = MakeShareable(new FPCGGraphEditorDocumentTabFactory(ThisPtr,
		FPCGGraphEditorDocumentTabFactory::FOnCreateGraphEditorWidget::CreateSP(this, &FPCGEditor::CreateGraphEditorWidget)));

	DocumentTabFactory = DocumentTabFactoryPtr;
	DocumentManager->RegisterDocumentFactory(DocumentTabFactoryPtr);

	AddApplicationMode(FPCGEditorDefaultMode::StaticName(), MakeShared<FPCGEditorDefaultMode>(ThisPtr));
	SetCurrentMode(FPCGEditorDefaultMode::StaticName());

	NotifyModuleManualEditStateChanged();
}

UPCGGraph* FPCGEditor::GetMainGraph() const
{
	return MainGraph;
}

UPCGGraph* FPCGEditor::GetFocusedGraph() const
{
	if (TSharedPtr<SGraphEditor> GraphEditor = GetFocusedGraphEditorWidget())
	{
		if (UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(GraphEditor->GetCurrentGraph()))
		{
			return EditorGraph->GetPCGGraph();
		}
	}

	return GetMainGraph();
}

UPCGEditorGraph* FPCGEditor::GetMainEditorGraph() const
{
	return MainGraph ? MainGraph->PCGEditorGraph : nullptr;
}

UPCGEditorGraph* FPCGEditor::GetFocusedEditorGraph() const
{
	UPCGGraph* FocusedGraph = GetFocusedGraph();
	return FocusedGraph ? FocusedGraph->PCGEditorGraph.Get() : GetMainEditorGraph();
}

UPCGAssetEditorMode* FPCGEditor::GetAssetEditorMode(int32 Index) const
{
	if (Index >= 0 && Index < PCGEditorTabs::NumViewportTabs && ViewportModeManagers[Index].IsValid())
	{
		return Cast<UPCGAssetEditorMode>(ViewportModeManagers[Index]->GetActiveScriptableMode(UPCGAssetEditorMode::EM_PCGAssetEditorModeId));
	}
	else
	{
		return nullptr;
	}
}

void FPCGEditor::NavigateTab(FDocumentTracker::EOpenDocumentCause InCause)
{
	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(nullptr);
	DocumentManager->OpenDocument(Payload, InCause);
}

TSharedPtr<SDockTab> FPCGEditor::OpenDocument(UPCGGraph* InPCGGraph, FDocumentTracker::EOpenDocumentCause InCause)
{
	check(InPCGGraph);

	InitializePCGEditorGraph(InPCGGraph);

	UPCGEditorGraph* PCGEditorGraph = InPCGGraph->GetEditorGraph();
	if (!PCGEditorGraph)
	{
		return nullptr;
	}

	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(PCGEditorGraph);
	return DocumentManager->OpenDocument(Payload, InCause);
}

void FPCGEditor::CloseDocument(UPCGGraph* InPCGGraph)
{
	check(InPCGGraph);

	UPCGEditorGraph* PCGEditorGraph = InPCGGraph->GetEditorGraph();
	if (!PCGEditorGraph)
	{
		return;
	}

	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(PCGEditorGraph);
	DocumentManager->CloseTab(Payload);
}

FPCGEditor* FPCGEditor::OpenEditorForGraph(UPCGGraph* InPCGGraph)
{
	if (MainGraph == InPCGGraph || MainGraph->ContainsEmbeddedSubgraph(InPCGGraph))
	{
		OpenDocument(InPCGGraph, FDocumentTracker::EOpenDocumentCause::OpenNewDocument);
		return this;
	}
	else if(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InPCGGraph))
	{
		TSharedPtr<FPCGEditor> PCGEditor = InPCGGraph->GetEditorGraph() && InPCGGraph->GetEditorGraph()->GetEditor().IsValid() ? InPCGGraph->GetEditorGraph()->GetEditor().Pin() : nullptr;
		PCGEditor->OpenDocument(InPCGGraph, FDocumentTracker::OpenNewDocument);
		return PCGEditor.Get();
	}

	return nullptr;
}

void FPCGEditor::OpenAssets(const FAssetOpenArgs& OpenArgs)
{
	for (UPCGGraph* PCGGraph : OpenArgs.LoadObjects<UPCGGraph>())
	{
		UPCGGraph* GraphToOpen = PCGGraph;
		if (UPCGGraph* MainGraph = PCGGraph->GetEmbeddedParentGraph())
		{
			GraphToOpen = MainGraph;
		}

		TSharedPtr<FPCGEditor> PCGEditor = GraphToOpen->GetEditorGraph() && GraphToOpen->GetEditorGraph()->GetEditor().IsValid() ? GraphToOpen->GetEditorGraph()->GetEditor().Pin() : nullptr;
		if (!PCGEditor)
		{
			PCGEditor = MakeShared<FPCGEditor>();
			PCGEditor->Initialize(EToolkitMode::Standalone, OpenArgs.ToolkitHost, GraphToOpen);
			GraphToOpen->GetEditorGraph()->SetEditor(PCGEditor);
		}
		
		// Now that we have the editor, if we are opening an embedded subgraph, open the tab
		if (PCGGraph != GraphToOpen)
		{
			PCGEditor->OpenDocument(PCGGraph, FDocumentTracker::OpenNewDocument);
		}
	}
}

void FPCGEditor::SetStackBeingInspectedFromAnotherEditor(const FPCGStack& FullStack)
{
	if (DebugObjectTreeWidget)
	{
		DebugObjectTreeWidget->SetDebugObjectFromStackFromAnotherEditor(FullStack);
	}
}

void FPCGEditor::SetStackBeingInspected(const FPCGStack& FullStack)
{
	InspectionDataManager.SetStackBeingInspected(FullStack);
}

void FPCGEditor::OnSourceGenerated(IPCGGraphExecutionSource* InSource)
{
	if (DebugObjectTreeWidget)
	{
		DebugObjectTreeWidget->RequestRefresh();
	}

	InspectionDataManager.OnSourceGenerated(InSource);
}

bool FPCGEditor::OnValidateNodeTitle(const FText& NewName, UEdGraphNode* GraphNode, FText& OutErrorMessage)
{
	if (UPCGEditorGraphNode* PCGGraphNode = Cast<UPCGEditorGraphNode>(GraphNode))
	{
		return PCGGraphNode->OnValidateNodeTitle(NewName, OutErrorMessage);
	}
	else if (GraphNode && GraphNode->IsA<UEdGraphNode_Comment>())
	{
		return true;
	}

	return false;
}

void FPCGEditor::OnPermissionChanged(EPCGEditorPermissionMode InPermissionMode)
{
	// Menus and tabs can change based on permissions so we close the editor instead of dealing with refreshing the UX
	CloseWindow(EAssetEditorCloseReason::EditorRefreshRequested);
}

void FPCGEditor::UpdateAfterInspectedStackChanged(const FPCGStack& FullStack)
{
	const bool bValidStack = !FullStack.GetStackFrames().IsEmpty();
	
	IPCGGraphExecutionSource* Source = InspectionDataManager.GetPCGSourceBeingInspected();

	if (bValidStack)
	{
		if (MainGraph)
		{
			MainGraph->EnableInspection(FullStack);
		}

		if (Source)
		{
			// Implementation note: if we're inspecting and have not pre-run the graph, then it probably makes sense to enable inspection by default. 
			// TODO This could be selected with a cvar though.
			const bool bHasBeenGeneratedThisSession = Source->GetExecutionState().IsGenerated() && Source->GetExecutionState().WasGeneratedThisSession();
			const bool bWasAborted = LastExecutionStatus.IsSet() && LastExecutionStatus.GetValue().Key == Source && LastExecutionStatus.GetValue().Value == EPCGGenerationStatus::Aborted;
			const bool bWasInspecting = Source->GetExecutionState().GetInspection().IsInspecting();
			const bool bNeedsInspection = Algo::AnyOf(AttributesWidgets, [](const TSharedPtr<SPCGEditorGraphAttributeListView>& ALV) { return ALV.IsValid() && ALV->GetNodeBeingInspected() != nullptr; });

			if (!bHasBeenGeneratedThisSession || (bNeedsInspection && !bWasInspecting))
			{
				Source->GetExecutionState().GetInspection().EnableInspection();

				// Making sure to not re-trigger a new generation if the previous one was aborted, to avoid infinite loops
				if (!bWasAborted)
				{
					UpdateDebugAfterSourceSelection(Source, Source, true);
				}
			}
		}
	}
	else if (MainGraph)
	{
		MainGraph->DisableInspection();
	}

	// If we are debugging the graph cache then we need to refresh the cache count displayed in the title after every generation.
	IPCGBaseSubsystem* Subsystem = GetSubsystem();
	const bool CacheDebuggingEnabled = Subsystem && Subsystem->IsGraphCacheDebuggingEnabled();
	const EPCGChangeType InitialChangeType = CacheDebuggingEnabled ? EPCGChangeType::Cosmetic : EPCGChangeType::None;

	RefreshEditorNodeVisualization(Source, &FullStack, InitialChangeType);
}

void FPCGEditor::RefreshEditorNodeVisualization(const IPCGGraphExecutionSource* InSource, const FPCGStack* InStack, EPCGChangeType InitialChangeType) const
{
	const auto RefreshNodes = [InSource, InStack, InitialChangeType](UPCGEditorGraph* EditorGraph)
	{
		if (!EditorGraph)
		{
			return;
		}
		for (UEdGraphNode* Node : EditorGraph->Nodes)
		{
			if (UPCGEditorGraphNodeBase* PCGNode = Cast<UPCGEditorGraphNodeBase>(Node))
			{
				EPCGChangeType ChangeType = InitialChangeType;
				ChangeType |= PCGNode->UpdateErrorsAndWarnings();
				ChangeType |= PCGNode->UpdateStructuralVisualization(InSource, InStack);
				ChangeType |= PCGNode->UpdateGPUVisualization(InSource, InStack);
				ChangeType |= PCGNode->UpdateOverrideVisualization(InSource, InStack);

				if (ChangeType != EPCGChangeType::None)
				{
					PCGNode->ReconstructNode();
				}
			}
		}
	};

	check(GetMainEditorGraph());
	RefreshNodes(GetMainEditorGraph());

	if (MainGraph)
	{
		for (const UPCGGraph* EmbeddedSubgraph : MainGraph->GetEmbeddedSubgraphs())
		{
			RefreshNodes(EmbeddedSubgraph ? EmbeddedSubgraph->PCGEditorGraph.Get() : nullptr);
		}
	}
}

void FPCGEditor::ClearStackBeingInspected()
{
	if (GetStackBeingInspected())
	{
		SetStackBeingInspected(FPCGStack());
	}
}

UPCGComponent* FPCGEditor::GetPCGComponentBeingInspected() const
{
	return const_cast<UPCGComponent*>(Cast<UPCGComponent>(GetPCGSourceBeingInspected()));
}

IPCGGraphExecutionSource* FPCGEditor::GetPCGSourceBeingInspected() const
{
	return InspectionDataManager.GetPCGSourceBeingInspected();
}

void FPCGEditor::UpdateDebugAfterSourceSelection(IPCGGraphExecutionSource* InOldSource, IPCGGraphExecutionSource* InNewSource, bool bInNewSourceStartedInspecting)
{
	if (!ensure(MainGraph))
	{
		return;
	}

	auto RefreshSource = [](IPCGGraphExecutionSource* Source)
	{
		if (!ensure(Source))
		{
			return;
		}

		// GenerateAtRuntime sources should be refreshed through the runtime gen scheduler.
		if (Source->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			if (UPCGSubsystem* WorldSubsystem = GetWorldSubsystem())
			{
				// We don't want to do a full cleanup if we're setting the debug object, since full cleanup destroys the source, which is the debug object itself!
				WorldSubsystem->RefreshRuntimeGenExecutionSource(Source);
			}
		}
		else
		{
			IPCGGraphExecutionState::FGenerateParams GenerateParams;
			GenerateParams.bEvenIfAlreadyGenerated = true;

			Source->GetExecutionState().Generate(GenerateParams);
		}
	};

	// If individual component debugging is disabled, just generate the new component if required.
	if (!MainGraph->DebugFlagAppliesToIndividualComponents())
	{
		if (InNewSource && bInNewSourceStartedInspecting)
		{
			RefreshSource(InNewSource);
		}

		return;
	}

	// Trigger necessary generation(s) for per-source debugging.
	if (!InOldSource)
	{
		if (InNewSource && bInNewSourceStartedInspecting)
		{
			// Transition from 'null' to 'any component not already inspecting' - generate to create debug/inspection info.
			// If we have null selected, all sources are displaying debug. Go to Original source so that all refresh.
			RefreshSource(InNewSource->GetExecutionState().GetOriginalSource());
		}
	}
	else
	{
		const bool bDebugFlagSetOnAnyNode = Algo::AnyOf(MainGraph->GetNodes(), [](const UPCGNode* InNode)
		{
			return InNode && InNode->GetSettings() && InNode->GetSettings()->bDebug;
		});

		// Regenerate to clear debug info if switching sources, or if changing from a source to null.
		if (InNewSource != InOldSource && (InNewSource || bDebugFlagSetOnAnyNode))
		{
			// Use original source - debug can be displayed both by the local source and parent local sources.
			RefreshSource(InOldSource->GetExecutionState().GetOriginalSource());
		}

		// Debug new source if it wasn't already
		if (InNewSource && bInNewSourceStartedInspecting)
		{
			// Use original source - debug can be displayed both by the local source and parent local sources.
			RefreshSource(InNewSource->GetExecutionState().GetOriginalSource());
		}
	}
}

const FPCGStack* FPCGEditor::GetStackBeingInspected() const
{
	const FPCGStack& StackBeingInspected = InspectionDataManager.GetStackBeingInspected();
	return StackBeingInspected.GetStackFrames().IsEmpty() ? nullptr : &StackBeingInspected;
}

void FPCGEditor::SetSourceEditorTargetObject(UObject* InObject)
{
	if (CodeEditorWidget.IsValid())
	{
		CodeEditorWidget->SetTextProviderObject(InObject);
	}
}

void FPCGEditor::JumpToNode(const UEdGraphNode* InNode)
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		FocusedGraphWidget->JumpToNode(InNode);
	}
}

UPCGEditorGraphNodeBase* FPCGEditor::GetEditorNode(const UPCGNode* InNode)
{
	if (!InNode)
	{
		return nullptr;
	}

	UPCGGraph* OuterGraph = InNode->GetTypedOuter<UPCGGraph>();
	if (!OuterGraph || !OuterGraph->GetEditorGraph())
	{
		return nullptr;
	}

	for (UEdGraphNode* EdGraphNode : OuterGraph->GetEditorGraph()->Nodes)
	{
		if (UPCGEditorGraphNodeBase* PCGEdGraphNode = Cast<UPCGEditorGraphNodeBase>(EdGraphNode))
		{
			if (PCGEdGraphNode->GetPCGNode() == InNode)
			{
				return PCGEdGraphNode;
			}
		}
	}

	return nullptr;
}

void FPCGEditor::JumpToNode(const UPCGNode* InNode)
{
	if (const UPCGEditorGraphNodeBase* EditorNode = GetEditorNode(InNode))
	{
		JumpToNode(EditorNode);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName FPCGEditor::GetPanelID(const EPCGEditorPanel Panel) const
{
	switch (Panel)
	{
		case EPCGEditorPanel::Attributes1:
			return PCGEditorTabs::AttributesID[0];
		case EPCGEditorPanel::Attributes2:
			return PCGEditorTabs::AttributesID[1];
		case EPCGEditorPanel::Attributes3:
			return PCGEditorTabs::AttributesID[2];
		case EPCGEditorPanel::Attributes4:
			return PCGEditorTabs::AttributesID[3];
		case EPCGEditorPanel::DebugObjectTree:
			return PCGEditorTabs::DebugObjectID;
		case EPCGEditorPanel::Determinism:
			return PCGEditorTabs::DeterminismID;
		case EPCGEditorPanel::Find:
			return PCGEditorTabs::FindID;
		case EPCGEditorPanel::GraphEditor:
			return PCGEditorTabs::GraphEditorID;
		case EPCGEditorPanel::Log:
			return PCGEditorTabs::LogID;
		case EPCGEditorPanel::NodePalette:
			return PCGEditorTabs::PaletteID;
		case EPCGEditorPanel::NodeSource:
			return PCGEditorTabs::CodeEditorID;
		case EPCGEditorPanel::Profiling:
			return PCGEditorTabs::ProfilingID;
		case EPCGEditorPanel::PropertyDetails1:
			return PCGEditorTabs::PropertyDetailsID[0];
		case EPCGEditorPanel::PropertyDetails2:
			return PCGEditorTabs::PropertyDetailsID[1];
		case EPCGEditorPanel::PropertyDetails3:
			return PCGEditorTabs::PropertyDetailsID[2];
		case EPCGEditorPanel::PropertyDetails4:
			return PCGEditorTabs::PropertyDetailsID[3];
		case EPCGEditorPanel::UserParams:
			return PCGEditorTabs::UserParamsID;
		case EPCGEditorPanel::Viewport1:
			return PCGEditorTabs::ViewportID[0];
		case EPCGEditorPanel::Viewport2:
			return PCGEditorTabs::ViewportID[1];
		case EPCGEditorPanel::Viewport3:
			return PCGEditorTabs::ViewportID[2];
		case EPCGEditorPanel::Viewport4:
			return PCGEditorTabs::ViewportID[3];
		default:
			return NAME_None;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FPCGEditor::BringFocusToPanel(FName PanelID) const
{
	if (PanelID != NAME_None)
	{
		if (const TSharedPtr<SDockTab> Tab = TabManager->TryInvokeTab(PanelID))
		{
			Tab->DrawAttention(); // Bring the panel to focus and flash the tab
		}
	}
}

void FPCGEditor::CloseGraphPanel(FName PanelID) const
{
	if (PanelID != NAME_None)
	{
		if (const TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(PanelID))
		{
			Tab->RequestCloseTab();
		}
	}
}

bool FPCGEditor::IsPanelCurrentlyOpen(FName PanelID) const
{
	return TabManager.IsValid() && TabManager->FindExistingLiveTab(PanelID);
}

bool FPCGEditor::IsPanelCurrentlyForeground(FName PanelID) const
{
	const TSharedPtr<SDockTab> DockTab = TabManager.IsValid() ? TabManager->FindExistingLiveTab(PanelID) : nullptr;
	return DockTab.IsValid() && DockTab->IsForeground();
}

bool FPCGEditor::IsPanelAvailable(const FName PanelID) const
{
	if (PanelID == PCGEditorTabs::EmbeddedSubgraphsID && (!GetMainGraph() || !GetMainGraph()->SupportsEmbeddedSubgraphs()))
	{
		return false;
	}

	bool bIsPanelAvailable = Algo::AnyOf(FPCGEditor_private::AllDefaultPanels, [PanelID](const FName& It) { return It == PanelID; });

	IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get();
	EPCGEditorPermissionMode PermissionMode = PCGEditorModule ? PCGEditorModule->GetPermissionMode() : EPCGEditorPermissionMode::None;
	if (bIsPanelAvailable && PermissionMode != EPCGEditorPermissionMode::All)
	{
		bIsPanelAvailable = Algo::NoneOf(FPCGEditor_private::AllRestrictedPanels, [PanelID](const FName& It) { return It == PanelID; });
	}

	return bIsPanelAvailable;
}

void FPCGEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
	FWorkflowCentricApplication::RegisterTabSpawners(InTabManager);

	DocumentManager->SetTabManager(InTabManager);
}

void FPCGEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(MainGraph);
	Collector.AddReferencedObject(PCGDefaultExecutionSource);

	InspectionDataManager.AddReferencedObjects(Collector);

	for (TSharedPtr<SPCGEditorGraphAttributeListView> ALV : AttributesWidgets)
	{
		if (ALV.IsValid())
		{
			ALV->AddReferencedObjects(Collector);
		}
	}
}

bool FPCGEditor::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	if (InContext.Context == FPCGEditorCommon::ContextIdentifier)
	{
		return true;
	}

	// This is done to catch transaction blocks made outside PCG editor code were we need to trigger PostUndo for our context, i.e. UPCGEditorGraphSchema::TryCreateConnection
	for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectContext : TransactionObjectContexts)
	{
		const UObject* Object = TransactionObjectContext.Key;
		while (Object != nullptr)
		{
			if (Object == MainGraph)
			{
				return true;
			}
			Object = Object->GetOuter();
		}
	}

	return false;
}

void FPCGEditor::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		if (MainGraph)
		{
			// Deepest change type to catch all types of change (like redoing adding a grid size node or etc).
			MainGraph->NotifyGraphChanged(EPCGChangeType::Structural | EPCGChangeType::GenerationGrid);
		}

		CloseInvalidDocuments();
		DocumentManager->RefreshAllTabs();

		if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
		{
			FocusedGraphWidget->ClearSelectionSet();
			FocusedGraphWidget->NotifyGraphChanged();

			FSlateApplication::Get().DismissAllMenus();
		}
	}
}

FText FPCGEditor::GetToolkitName() const
{
	return MainGraph ? MainGraph->GetDisplayName() : FWorkflowCentricApplication::GetToolkitName();
}

FName FPCGEditor::GetToolkitFName() const
{
	return FName(TEXT("PCGEditor"));
}

FText FPCGEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "PCG Editor");
}

FLinearColor FPCGEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

FString FPCGEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "PCG ").ToString();
}

void FPCGEditor::RegisterToolbar(FToolBarBuilder& ToolbarBuilder) const
{
	ToolbarBuilder.BeginSection("PCGToolbar");
	RegisterToolbarInternal(ToolbarBuilder);
	ToolbarBuilder.EndSection();
}

void FPCGEditor::RegisterToolbarInternal(FToolBarBuilder& ToolbarBuilder) const
{
	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::Find);
	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::PauseRegen);
	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::ForceRegen);
	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::CancelExecution);
	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::OpenDebugObjectTreeTab);

	ToolbarBuilder.AddSeparator(NAME_None);

	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::Graph);
	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::GraphParams);
	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::GraphSettings);
	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::AutoLayoutNodes);
}

void FPCGEditor::RegisterToolbarButton(FToolBarBuilder& ToolbarBuilder, EPCGToolbarButtons Button) const
{
	const FPCGEditorCommands& PCGEditorCommands = FPCGEditorCommands::Get();
	
	switch (Button)
	{
	case EPCGToolbarButtons::Find:
		{
			ToolbarBuilder.AddToolBarButton(PCGEditorCommands.Find, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
				FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Command.Find"));
			break;
		}
	case EPCGToolbarButtons::PauseRegen:
		{
			ToolbarBuilder.AddToolBarButton(PCGEditorCommands.PauseAutoRegeneration, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
				FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Command.PauseRegen"));
			break;
		}
	case EPCGToolbarButtons::ForceRegen:
		{
			ToolbarBuilder.AddToolBarButton(PCGEditorCommands.ForceGraphRegeneration, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
				TAttribute<FSlateIcon>::CreateLambda([]()
				{
					static const FSlateIcon ForceRegen = FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Command.ForceRegen");
					static const FSlateIcon ForceRegenClearCache = FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Command.ForceRegenClearCache");

					FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
					return ModifierKeys.IsControlDown() ? ForceRegenClearCache : ForceRegen;
				}));
			break;
		}
	case EPCGToolbarButtons::AutoLayoutNodes:
		{
			ToolbarBuilder.AddToolBarButton(PCGEditorCommands.AutoLayoutNodes);
			break;
		}
	case EPCGToolbarButtons::CancelExecution:
		{
			ToolbarBuilder.AddToolBarButton(PCGEditorCommands.CancelExecution, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
				FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Command.StopRegen"));
			break;
		}
	case EPCGToolbarButtons::OpenDebugObjectTreeTab:
		{
			if (IsPanelAvailable(PCGEditorTabs::DebugObjectID))
			{
				ToolbarBuilder.AddToolBarButton(PCGEditorCommands.OpenDebugObjectTreeTab, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
					FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Command.OpenDebugTreeTab"));
			}

			break;
		}
	case EPCGToolbarButtons::GraphParams:
		{
			ToolbarBuilder.AddToolBarButton(PCGEditorCommands.ToggleGraphParams, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
				FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Command.OpenGraphParams"));
			break;
		}
	case EPCGToolbarButtons::GraphSettings:
		{
			ToolbarBuilder.AddToolBarButton(PCGEditorCommands.EditGraphSettings, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
				FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Command.GraphSettings"));
			break;
		}
	case EPCGToolbarButtons::Graph:
		{
			ToolbarBuilder.AddToolBarButton(PCGEditorCommands.EditGraph, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
				FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Command.EditGraph"));
			break;
		}
	}
}

void FPCGEditor::BindCommands()
{
	CreateGraphEditorCommands();

	const FPCGEditorCommands& PCGEditorCommands = FPCGEditorCommands::Get();

	ToolkitCommands->MapAction(
		PCGEditorCommands.Find,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnFind));

	ToolkitCommands->MapAction(
		PCGEditorCommands.ShowSelectedDetails,
		FExecuteAction::CreateSP(this, &FPCGEditor::OpenDetailsView));

	ToolkitCommands->MapAction(
		PCGEditorCommands.PauseAutoRegeneration,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnPauseAutomaticRegeneration_Clicked),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPCGEditor::IsAutomaticRegenerationPaused));

	ToolkitCommands->MapAction(
		PCGEditorCommands.ForceGraphRegeneration,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnForceGraphRegeneration_Clicked));

	ToolkitCommands->MapAction(
		PCGEditorCommands.AutoLayoutNodes,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnAutoLayoutNodes_Clicked));

	ToolkitCommands->MapAction(
		PCGEditorCommands.CancelExecution,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnCancelExecution_Clicked),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::IsCurrentlyGenerating));

	// Left on UI as a disabled button if debug object tree tab already open. This is a deliberate
	// hint for 5.4 to help direct users to use the tree.
	ToolkitCommands->MapAction(
		PCGEditorCommands.OpenDebugObjectTreeTab,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnOpenDebugObjectTreeTab_Clicked),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::IsDebugObjectTreeTabClosed));

	ToolkitCommands->MapAction(
		PCGEditorCommands.RunDeterminismGraphTest,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnDeterminismGraphTest),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanRunDeterminismGraphTest));

	ToolkitCommands->MapAction(
		PCGEditorCommands.ToggleGraphParams,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnToggleGraphParamsPanel),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPCGEditor::IsToggleGraphParamsToggled));

	ToolkitCommands->MapAction(
		PCGEditorCommands.EditGraphSettings,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnEditGraphSettings),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPCGEditor::IsEditGraphSettingsToggled));

	ToolkitCommands->MapAction(
		PCGEditorCommands.EditGraph,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnEditGraph_Clicked),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::IsEditGraphTabClosed));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.CollapseNodes,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnCollapseNodesInSubgraph, false),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanCollapseNodesInSubgraph, false));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.CollapseNodesToEmbeddedSubgraph,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnCollapseNodesInSubgraph, true),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanCollapseNodesInSubgraph, true));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.ExportNodes,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnExportNodes),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanExportNodes));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.ConvertToStandaloneNodes,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnConvertToStandaloneNodes),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanConvertToStandaloneNodes));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.ToggleInspect,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnToggleInspected),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanToggleInspected),
		FGetActionCheckState::CreateSP(this, &FPCGEditor::GetInspectedCheckState));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.RunDeterminismNodeTest,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnDeterminismNodeTest),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanRunDeterminismNodeTest));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.ToggleEnabled,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnToggleEnabled),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanToggleEnabled),
		FGetActionCheckState::CreateSP(this, &FPCGEditor::GetEnabledCheckState));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.ToggleDebug,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnToggleDebug),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanToggleDebug),
		FGetActionCheckState::CreateSP(this, &FPCGEditor::GetDebugCheckState));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.DebugOnlySelected,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnDebugOnlySelected),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanToggleDebug));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.DisableDebugOnAllNodes,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnDisableDebugOnAllNodes),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanToggleDebug));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.EditInViewport,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnEditInViewport),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanToggleViewportEditing));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.MarkForViewportEditing,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnMarkForViewportEditing),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanToggleViewportEditing),
		FGetActionCheckState::CreateSP(this, &FPCGEditor::GetMarkForViewportEditingCheckState));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.AddInputPin,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnAddDynamicPin, EPCGPinDirection::Input),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanAddDynamicPin, EPCGPinDirection::Input));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.AddOutputPin,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnAddDynamicPin, EPCGPinDirection::Output),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanAddDynamicPin, EPCGPinDirection::Output));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.RenameNode,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnRenameNode),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanRenameNode));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.SelectNamedRerouteUsages,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnSelectNamedRerouteUsages),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanSelectNamedRerouteUsages));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.SelectNamedRerouteDeclaration,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnSelectNamedRerouteDeclaration),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanSelectNamedRerouteDeclaration));

	GraphEditorCommands->MapAction(
		PCGEditorCommands.JumpToSource,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnJumpToSource));
}

void FPCGEditor::OnFind()
{
	if (TabManager.IsValid())
	{
		TabManager->TryInvokeTab(FTabId(PCGEditorTabs::FindID));
		if (FindWidget.IsValid())
		{
			FindWidget->FocusForUse();
		}
	}
}

void FPCGEditor::OpenDetailsView()
{
	if (TabManager.IsValid())
	{
		auto InvokeFirstUnlockedTab = [this](bool bVisibleOnly) -> bool
		{
			for (int DetailsViewIndex = 0; DetailsViewIndex < PCGEditorTabs::NumPropertyDetailTabs; ++DetailsViewIndex)
			{
				TSharedPtr<SPCGEditorGraphDetailsView> DetailsView = PropertyDetailsWidgets[DetailsViewIndex];
				if (!DetailsView.IsValid() || !DetailsView->IsLocked())
				{
					if (!bVisibleOnly || TabManager->FindExistingLiveTab(FTabId(PCGEditorTabs::PropertyDetailsID[DetailsViewIndex])))
					{
						TabManager->TryInvokeTab(FTabId(PCGEditorTabs::PropertyDetailsID[DetailsViewIndex]));
						return true;
					}
				}
			}

			return false;
		};

		if (InvokeFirstUnlockedTab(true) || InvokeFirstUnlockedTab(false))
		{
			return;
		}

		// Default to first if they are all locked
		TabManager->TryInvokeTab(FTabId(PCGEditorTabs::PropertyDetailsID[0]));
	}
}

void FPCGEditor::OnDetailsViewTabClosed(FName Id, int32 Index)
{
	check(Index >= 0 && Index < PCGEditorTabs::NumPropertyDetailTabs);
	check(PCGEditorTabs::PropertyDetailsID[Index] == Id);

	if (PropertyDetailsWidgets[Index].IsValid() && PropertyDetailsWidgets[Index]->IsLocked())
	{
		PropertyDetailsWidgets[Index]->SetIsLocked(false);
	}
}

void FPCGEditor::OnAttributeListViewTabClosed(FName Id, int32 Index)
{
	check(Index >= 0 && Index < PCGEditorTabs::NumAttributeTabs);
	check(PCGEditorTabs::AttributesID[Index] == Id);

	TSharedPtr<SPCGEditorGraphAttributeListView> AttributeListView = AttributesWidgets[Index];

	if (AttributeListView.IsValid())
	{
		if (AttributeListView->IsLocked())
		{
			AttributeListView->SetIsLocked(false);
		}

		UPCGEditorGraphNodeBase* NodeInspected = AttributeListView->GetNodeBeingInspected();
		AttributeListView->SetNodeBeingInspected(nullptr);

		if (NodeInspected)
		{
			bool bIsStillInspectedOnVisibleTabs = false;
			for (int OtherTabIndex = 0; OtherTabIndex < PCGEditorTabs::NumAttributeTabs; ++OtherTabIndex)
			{
				TSharedPtr<SPCGEditorGraphAttributeListView> ALV = AttributesWidgets[OtherTabIndex];
				if (ALV.IsValid() && ALV->GetNodeBeingInspected() == NodeInspected && TabManager->FindExistingLiveTab(FTabId(PCGEditorTabs::AttributesID[OtherTabIndex])))
				{
					bIsStillInspectedOnVisibleTabs = true;
					break;
				}
			}

			if (!bIsStillInspectedOnVisibleTabs)
			{
				NodeInspected->SetInspected(false);

				for (TSharedPtr<SPCGEditorGraphAttributeListView> ALV : AttributesWidgets)
				{
					if (ALV.IsValid() && ALV->GetNodeBeingInspected() == NodeInspected)
					{
						ALV->SetNodeBeingInspected(nullptr);
					}
				}
			}
		}
	}
}

void FPCGEditor::OnViewportViewTabClosed(FName Id, int32 Index)
{
	check(Index >= 0 && Index < PCGEditorTabs::NumViewportTabs);
	check(PCGEditorTabs::ViewportID[Index] == Id);
	
	if (ViewportWidgets[Index].IsValid())
	{
		ViewportWidgets[Index]->ResetScene();
	}
}

void FPCGEditor::OnPauseAutomaticRegeneration_Clicked()
{
	if (!MainGraph)
	{
		return;
	}

	MainGraph->ToggleUserPausedNotificationsForEditor();
}

bool FPCGEditor::IsAutomaticRegenerationPaused() const
{
	return MainGraph && MainGraph->NotificationsForEditorArePausedByUser();
}

void FPCGEditor::OnForceGraphRegeneration_Clicked()
{
	if (MainGraph)
	{
		EPCGChangeType ChangeType = EPCGChangeType::Structural;

		FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
		if (ModifierKeys.IsControlDown())
		{
			if (IPCGBaseSubsystem* Subsystem = GetSubsystem())
			{
				Subsystem->FlushCache();
			}

			ChangeType |= EPCGChangeType::GenerationGrid;

			ChangeType |= EPCGChangeType::ShaderSource;
		}

		MainGraph->ForceNotificationForEditor(ChangeType);
	}
}

void FPCGEditor::OnAutoLayoutNodes_Clicked()
{
	if (MainGraph)
	{
		AutoLayoutFullGraph();
	}
}

void FPCGEditor::OnCancelExecution_Clicked()
{
	IPCGBaseSubsystem* Subsystem = GetSubsystem();
	if (MainGraph && Subsystem)
	{
		Subsystem->CancelGeneration(MainGraph);
	}
}

bool FPCGEditor::IsCurrentlyGenerating() const
{
	IPCGBaseSubsystem* Subsystem = GetSubsystem();
	if (MainGraph && Subsystem)
	{
		return Subsystem->IsGraphCurrentlyExecuting(MainGraph);
	}

	return false;
}

bool FPCGEditor::IsDebugObjectTreeTabClosed() const
{
	return !TabManager.IsValid() || !TabManager->FindExistingLiveTab(FTabId(PCGEditorTabs::DebugObjectID)).IsValid();
}

void FPCGEditor::OnOpenDebugObjectTreeTab_Clicked()
{
	TabManager->TryInvokeTab(FTabId(PCGEditorTabs::DebugObjectID));
}

void FPCGEditor::OnEditGraph_Clicked()
{
	OpenDocument(GetMainGraph(), FDocumentTracker::EOpenDocumentCause::OpenNewDocument);
}

bool FPCGEditor::IsEditGraphTabClosed() const
{
	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(GetMainEditorGraph());

	TArray<TSharedPtr<SDockTab>> OpenedTabs;
	DocumentManager->FindMatchingTabs(Payload, OpenedTabs);

	return OpenedTabs.IsEmpty();
}

bool FPCGEditor::CanRunDeterminismNodeTest() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	check(FocusedGraphWidget.IsValid());

	for (const UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		if (Cast<const UPCGEditorGraphNodeBase>(Object) && !Cast<const UPCGEditorGraphNodeInput>(Object) && !Cast<const UPCGEditorGraphNodeOutput>(Object))
		{
			return true;
		}
	}

	return false;
}

void FPCGEditor::OnDeterminismNodeTest() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	check(FocusedGraphWidget.IsValid());

	// Lazy create
	const_cast<FPCGEditor*>(this)->GetOrCreateDeterminismWidget();

	if (!DeterminismWidget.IsValid() || !DeterminismWidget->WidgetIsConstructed())
	{
		return;
	}

	TMap<FName, FTestColumnInfo> TestsConducted;
	DeterminismWidget->ClearItems();
	DeterminismWidget->BuildBaseColumns();

	int64 TestIndex = 0;
	for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		check(Object);

		// Gets an appropriate width for each new column
		auto GetSlateTextWidth = [](const FText& Text) -> float
		{
			check(FSlateApplication::Get().GetRenderer());
			const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			// TODO: Verify the below property for this part of the UI
			FSlateFontInfo FontInfo(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
			constexpr float Padding = 30.f;
			return Padding + FontMeasure->Measure(Text, FontInfo).X;
		};

		if (!Object->IsA<UPCGEditorGraphNodeInput>() && !Object->IsA<UPCGEditorGraphNodeOutput>())
		{
			if (const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<const UPCGEditorGraphNodeBase>(Object))
			{
				const UPCGNode* PCGNode = PCGEditorGraphNode->GetPCGNode();
				check(PCGNode && PCGNode->GetSettings());

				TSharedPtr<FDeterminismTestResult> NodeResult = MakeShared<FDeterminismTestResult>();
				NodeResult->Index = TestIndex++;
				NodeResult->TestResultTitle = FName(*PCGNode->GetNodeTitle(EPCGNodeTitleType::ListView).ToString());
				NodeResult->TestResultName = PCGNode->GetName();
				NodeResult->Seed = PCGNode->GetSettings()->GetSeed();

				if (PCGNode->GetSettings()->DeterminismSettings.bNativeTests)
				{
					// If the settings has a native test suite
					if (TFunction<bool()> NativeTestSuite = PCGDeterminismTests::FNativeTestRegistry::GetNativeTestFunction(PCGNode->GetSettings()))
					{
						FName NodeName(PCGNode->GetName());

						bool bSuccess = NativeTestSuite();
						NodeResult->TestResults.Emplace(NodeName, bSuccess ? EDeterminismLevel::Basic : EDeterminismLevel::NoDeterminism);
						NodeResult->AdditionalDetails.Emplace(FString(TEXT("Native test conducted for - ")) + NodeName.ToString());
						NodeResult->bFlagRaised = !bSuccess;

						FText ColumnText = NSLOCTEXT("PCGDeterminism", "NativeTest", "Native Test");
						TestsConducted.FindOrAdd(NodeName, {NodeName, ColumnText, GetSlateTextWidth(ColumnText), HAlign_Center});
					}
					else // There is no native test suite, so run the basic tests
					{
						PCGDeterminismTests::FNodeTestInfo BasicTestInfo = PCGDeterminismTests::Defaults::DeterminismBasicTestInfo;
						PCGDeterminismTests::RunDeterminismTest(PCGNode, *NodeResult, BasicTestInfo);
						TestsConducted.FindOrAdd(BasicTestInfo.TestName, {BasicTestInfo.TestName, BasicTestInfo.TestLabel, BasicTestInfo.TestLabelWidth, HAlign_Center});

						PCGDeterminismTests::FNodeTestInfo OrderIndependenceTestInfo = PCGDeterminismTests::Defaults::DeterminismOrderIndependenceInfo;
						PCGDeterminismTests::RunDeterminismTest(PCGNode, *NodeResult, OrderIndependenceTestInfo);
						TestsConducted.FindOrAdd(OrderIndependenceTestInfo.TestName, {OrderIndependenceTestInfo.TestName, OrderIndependenceTestInfo.TestLabel, OrderIndependenceTestInfo.TestLabelWidth, HAlign_Center});
					}
				}

				// Custom tests
				if (PCGNode->GetSettings()->DeterminismSettings.bUseBlueprintDeterminismTest)
				{
					TSubclassOf<UPCGDeterminismTestBlueprintBase> Blueprint = PCGNode->GetSettings()->DeterminismSettings.DeterminismTestBlueprint;
					Blueprint.GetDefaultObject()->ExecuteTest(PCGNode, *NodeResult);
					FName BlueprintName(Blueprint->GetName());

					FText ColumnText = FText::FromString(Blueprint->GetName());
					TestsConducted.FindOrAdd(BlueprintName, {BlueprintName, ColumnText, GetSlateTextWidth(ColumnText), HAlign_Center});
				}

				DeterminismWidget->AddItem(NodeResult);
			}
		}
	}

	for (const TTuple<FName, FTestColumnInfo>& Test : TestsConducted)
	{
		DeterminismWidget->AddColumn(Test.Value);
	}

	DeterminismWidget->AddDetailsColumn();
	DeterminismWidget->RefreshItems();

	// Give focus to the Determinism Output Tab
	if (TabManager.IsValid())
	{
		TabManager->TryInvokeTab(FTabId(PCGEditorTabs::DeterminismID));
	}
}

bool FPCGEditor::CanRunDeterminismGraphTest() const
{
	return GetMainEditorGraph() && InspectionDataManager.GetPCGSourceBeingInspected();
}

void FPCGEditor::OnDeterminismGraphTest() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	check(FocusedGraphWidget.IsValid());

	// Lazy create
	const_cast<FPCGEditor*>(this)->GetOrCreateDeterminismWidget();

	const IPCGGraphExecutionSource* PCGSourceBeingInspected = InspectionDataManager.GetPCGSourceBeingInspected();

	if (!DeterminismWidget.IsValid() || !DeterminismWidget->WidgetIsConstructed() || !MainGraph || !PCGSourceBeingInspected)
	{
		return;
	}

	if (PCGSourceBeingInspected->GetExecutionState().GetGraph() != MainGraph)
	{
		// TODO: Should we alert the user more directly or disable this altogether?
		UE_LOGF(LogPCGEditor, Warning, "Running Determinism on a PCG Component with different/no attached PCG Graph");
	}

	DeterminismWidget->ClearItems();
	DeterminismWidget->BuildBaseColumns();

	FTestColumnInfo ColumnInfo({PCGDeterminismTests::Defaults::GraphResultName, NSLOCTEXT("PCGDeterminism", "Result", "Result"), 120.f, HAlign_Center});
	DeterminismWidget->AddColumn(ColumnInfo);

	TSharedPtr<FDeterminismTestResult> TestResult = MakeShared<FDeterminismTestResult>();
	TestResult->Index = 0;
	TestResult->TestResultTitle = TEXT("Full Graph Test");
	TestResult->TestResultName = MainGraph->GetDisplayName().ToString();
	TestResult->Seed = PCGSourceBeingInspected->GetExecutionState().GetSeed();

	PCGDeterminismTests::RunDeterminismTest(MainGraph, PCGSourceBeingInspected, *TestResult);

	DeterminismWidget->AddItem(TestResult);
	DeterminismWidget->AddDetailsColumn();
	DeterminismWidget->RefreshItems();

	// Give focus to the Determinism Output Tab
	if (TabManager.IsValid())
	{
		TabManager->TryInvokeTab(FTabId(PCGEditorTabs::DeterminismID));
	}
}

TSubclassOf<UPCGEditorGraphSchema> FPCGEditor::GetSchemaClass() const
{
	return UPCGEditorGraphSchema::StaticClass();
}

TAttribute<FGraphAppearanceInfo> FPCGEditor::GetAppearanceInfo() const
{
	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("PCGGraphEditorCornerText", "PCG Graph");

	return AppearanceInfo;
}

void FPCGEditor::OnEditGraphSettings()
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	check(FocusedGraphWidget.IsValid());

	// Clear any selected nodes.
	FocusedGraphWidget->ClearSelectionSet();

	for (TSharedPtr<SPCGEditorGraphDetailsView> PropertyDetailsWidget : PropertyDetailsWidgets)
	{
		if (PropertyDetailsWidget.IsValid())
		{
			PropertyDetailsWidget->SetObject(GetFocusedGraph());
		}
	}

	OpenDetailsView();
}

bool FPCGEditor::IsEditGraphSettingsToggled() const
{
	if (!TabManager.IsValid())
	{
		return false;
	}

	for (int32 WidgetIndex = 0; WidgetIndex < PCGEditorTabs::NumPropertyDetailTabs; ++WidgetIndex)
	{
		if (!PropertyDetailsWidgets[WidgetIndex].IsValid())
		{
			continue;
		}

		const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyDetailsWidgets[WidgetIndex]->GetSelectedObjects();
		// The only object selected should be the graph. If there is no details view panel open, leave it disabled.
		if (SelectedObjects.Num() == 1 && SelectedObjects[0] == GetFocusedGraph())
		{
			if (const TSharedPtr<SDockTab>& Tab = TabManager->FindExistingLiveTab(FTabId(PCGEditorTabs::PropertyDetailsID[WidgetIndex])))
			{
				if (Tab.IsValid() && Tab->IsForeground())
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FPCGEditor::OnToggleGraphParamsPanel() const
{
	if (IsPanelCurrentlyForeground(PCGEditorTabs::UserParamsID))
	{
		CloseGraphPanel(PCGEditorTabs::UserParamsID);
	}
	else
	{
		BringFocusToPanel(PCGEditorTabs::UserParamsID);
	}
}

bool FPCGEditor::IsToggleGraphParamsToggled() const
{
	return IsPanelCurrentlyOpen(PCGEditorTabs::UserParamsID);
}

bool FPCGEditor::CanCollapseNodesInSubgraph(bool bCollapseToEmbeddedSubgraph) const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return false;
	}

	bool HasPCGNode = false;

	for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		check(Object);

		// Exclude input and output nodes from the subgraph.
		if (Object->IsA<UPCGEditorGraphNodeInput>() || Object->IsA<UPCGEditorGraphNodeOutput>())
		{
			continue;
		}

		if (Object->IsA<UPCGEditorGraphNodeBase>())
		{
			if (HasPCGNode)
			{
				return true;
			}

			HasPCGNode = true;
		}
	}

	return false;
}

void FPCGEditor::OnAddDynamicPin(const EPCGPinDirection Direction)
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	check(FocusedGraphWidget.IsValid());

	const FGraphPanelSelectionSet SelectedNodes = FocusedGraphWidget->GetSelectedNodes();

	if (!ensure(SelectedNodes.Num() == 1))
	{
		UE_LOGF(LogPCGEditor, Warning, "Attempting to add new dynamic pin to multiple nodes.");
		return;
	}

	UPCGEditorGraphNodeBase* Node = CastChecked<UPCGEditorGraphNodeBase>(*SelectedNodes.CreateConstIterator());
	Node->OnUserAddDynamicPin(Direction);
}

bool FPCGEditor::CanAddDynamicPin(const EPCGPinDirection Direction) const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	check(FocusedGraphWidget.IsValid());

	const FGraphPanelSelectionSet SelectedNodes = FocusedGraphWidget->GetSelectedNodes();
	if (SelectedNodes.Num() != 1)
	{
		return false;
	}

	const UPCGEditorGraphNodeBase* Node = Cast<const UPCGEditorGraphNodeBase>(*SelectedNodes.CreateConstIterator());
	return Node && Node->CanUserAddRemoveDynamicPins(Direction);
}

void FPCGEditor::OnRenameNode()
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	check(FocusedGraphWidget.IsValid());

	const FGraphPanelSelectionSet SelectedNodes = FocusedGraphWidget->GetSelectedNodes();

	if (!ensure(SelectedNodes.Num() == 1))
	{
		UE_LOGF(LogPCGEditor, Warning, "Attempting to rename multiple nodes.");
		return;
	}

	const UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*SelectedNodes.CreateConstIterator());
	if (SelectedNode != nullptr && SelectedNode->GetCanRenameNode())
	{
		FocusedGraphWidget->IsNodeTitleVisible(SelectedNode, true);
	}
}

bool FPCGEditor::CanRenameNode() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	check(FocusedGraphWidget.IsValid());

	const FGraphPanelSelectionSet SelectedNodes = FocusedGraphWidget->GetSelectedNodes();

	// You cannot enter renaming mode on multiple nodes at once, since they will not all enter synchronously.
	// Simultaneous editing of multiple InlineEditableTextBlocks may not even be possible with default behavior.
	if (SelectedNodes.Num() != 1)
	{
		return false;
	}

	UObject* SelectedObject = *SelectedNodes.CreateConstIterator();
	if (const UPCGEditorGraphNode* SelectedNode = Cast<UPCGEditorGraphNode>(SelectedObject))
	{
		return SelectedNode->GetCanRenameNode();
	}
	else if (SelectedObject && SelectedObject->IsA<UEdGraphNode_Comment>())
	{
		return true;
	}
	else
	{
		return false;
	}
}

bool FPCGEditor::InternalValidationOnAction()
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	UPCGEditorGraph* FocusedEditorGraph = GetFocusedEditorGraph();
	if (!FocusedGraphWidget || FocusedEditorGraph == nullptr)
	{
		UE_LOGF(LogPCGEditor, Error, "FocusedGraphWidget or FocusedEditorGraph is null, aborting");
		return false;
	}

	UPCGGraph* PCGGraph = FocusedEditorGraph->GetPCGGraph();
	if (PCGGraph == nullptr)
	{
		UE_LOGF(LogPCGEditor, Error, "PCGGraph is null, aborting");
		return false;
	}

	return true;
}

void FPCGEditor::OnSelectNamedRerouteUsages()
{
	if (!InternalValidationOnAction())
	{
		return;
	}

	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	const FGraphPanelSelectionSet SelectedNodes = FocusedGraphWidget->GetSelectedNodes();

	if (SelectedNodes.Num() != 1)
	{
		return;
	}

	const UPCGEditorGraphNodeNamedRerouteDeclaration* DeclarationNode = nullptr;

	for (const UObject* Object : SelectedNodes)
	{
		DeclarationNode = Cast<UPCGEditorGraphNodeNamedRerouteDeclaration>(Object);
	}

	if (!DeclarationNode || !DeclarationNode->GetPCGNode())
	{
		return;
	}

	FocusedGraphWidget->ClearSelectionSet();

	// Some assumptions below - that only usages are connected to the invisible pin.
	if (const UPCGPin* InvisiblePin = DeclarationNode->GetPCGNode()->GetOutputPin(PCGNamedRerouteConstants::InvisiblePinLabel))
	{
		for (const UPCGEdge* Edge : InvisiblePin->Edges)
		{
			if (const UPCGNode* Usage = Edge->OutputPin->Node)
			{
				FocusedGraphWidget->SetNodeSelection(GetEditorNode(Usage), true);
			}
		}
	}

	FocusedGraphWidget->ZoomToFit(true);
}

bool FPCGEditor::CanSelectNamedRerouteUsages() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return false;
	}

	if (FocusedGraphWidget->GetSelectedNodes().Num() != 1)
	{
		return false;
	}

	if (auto It = FocusedGraphWidget->GetSelectedNodes().CreateConstIterator(); It)
	{
		const UObject* Object = *It;
		return Object && Object->IsA<UPCGEditorGraphNodeNamedRerouteDeclaration>();
	}

	return false;
}

void FPCGEditor::OnSelectNamedRerouteDeclaration()
{
	if (!InternalValidationOnAction())
	{
		return;
	}

	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	const FGraphPanelSelectionSet SelectedNodes = FocusedGraphWidget->GetSelectedNodes();

	if (SelectedNodes.Num() != 1)
	{
		return;
	}

	for (const UObject* Object : SelectedNodes)
	{
		const UPCGEditorGraphNodeNamedRerouteUsage* UsageNode = Cast<UPCGEditorGraphNodeNamedRerouteUsage>(Object);

		if (!UsageNode)
		{
			continue;
		}

		FocusedGraphWidget->ClearSelectionSet();

		if (!UsageNode->GetPCGNode())
		{
			continue;
		}

		// Find the declaration node that matches the settings in the Usage node.
		if (UPCGNamedRerouteUsageSettings* UsageSettings = Cast<UPCGNamedRerouteUsageSettings>(UsageNode->GetPCGNode()->GetSettings()))
		{
			if (UsageSettings->Declaration && UsageSettings->Declaration->GetOuter() && UsageSettings->Declaration->GetOuter()->IsA<UPCGNode>())
			{
				JumpToNode(Cast<UPCGNode>(UsageSettings->Declaration->GetOuter()));
				break;
			}
		}
	}
}

bool FPCGEditor::CanSelectNamedRerouteDeclaration() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return false;
	}

	if (FocusedGraphWidget->GetSelectedNodes().Num() != 1)
	{
		return false;
	}

	if (auto It = FocusedGraphWidget->GetSelectedNodes().CreateConstIterator(); It)
	{
		const UObject* Object = *It;
		return Object && Object->IsA<UPCGEditorGraphNodeNamedRerouteUsage>();
	}

	return false;
}

void FPCGEditor::OnJumpToSource()
{
	if (!InternalValidationOnAction())
	{
		return;
	}

	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return;
	}

	for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<const UPCGEditorGraphNodeBase>(Object);
		const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
		const UPCGSettings* Settings = PCGNode ? PCGNode->GetSettings() : nullptr;

		if (Settings)
		{
			JumpToDefinition(Settings->GetClass());
		}
	}
}

FReply FPCGEditor::OnSpawnNodeByShortcut(FInputChord InChord, const FVector2D& InPosition, UPCGEditorGraph* InGraph)
{
	return OnSpawnNodeByShortcut(InChord, UE::Slate::CastToVector2f(InPosition), InGraph);
}

FReply FPCGEditor::OnSpawnNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UPCGEditorGraph* InGraph)
{
	const TSharedPtr<FEdGraphSchemaAction> Action = FPCGEditorSpawnNodeCommands::Get().GetGraphActionByChord(InChord);
	if (Action.IsValid())
	{
		TArray<UEdGraphPin*> DummyPins;
		Action->PerformAction(InGraph, DummyPins, UE::Slate::FDeprecateVector2DParameter(InPosition));
		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FActionMenuContent FPCGEditor::OnCreateActionMenu(UEdGraph* InGraph, const FVector2f& Location, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed OnMenuClosed)
{
	TSharedRef<SGraphEditorActionMenu> Menu = SNew(SGraphEditorActionMenu)
		.GraphObj(InGraph)
		.NewNodePosition(Location)
		.DraggedFromPins(InDraggedPins)
		.AutoExpandActionMenu(bAutoExpand)
		.OnClosedCallback(OnMenuClosed)
		.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateLambda(
			[](const FCreateWidgetForActionData* const Data)
			{
				return SNew(SPCGGraphActionWidget, Data);
			}));

	// No need for the contextual menu options if it is not dragged from pin(s).
	if (InDraggedPins.IsEmpty())
	{
		return FActionMenuContent(Menu, Menu->GetFilterTextBox());
	}

	TSharedRef<SWidget> MenuWithOptions =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4, 4, 4, 0)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			.AutoWidth()
			.Padding(0)
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					return IsContextualFilteringEnabled(PCGActionsHelpers::ECompatibilityMask::Compatible) ? LOCTEXT("PCGAllNodesShown", "Compatible Nodes") : LOCTEXT("PCGOnlyCompatibleNodesShown", "All Nodes");
				})
			]
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.HAlign(HAlign_Center)
			.Padding(0)
			[
				// Left blank to separate the text from the buttons
				SNullWidget::NullWidget
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(0)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("PCGShowContextual", "Show contextual nodes only, e.g. nodes that can be connected directly."))
				.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
				.OnCheckStateChanged(FOnCheckStateChanged::CreateSP(this, &FPCGEditor::OnContextualFilteringChanged, Menu.ToWeakPtr(), PCGActionsHelpers::ECompatibilityMask::Compatible))
				.IsChecked(TAttribute<ECheckBoxState>::CreateSP(this, &FPCGEditor::IsContextualFilteringChecked, PCGActionsHelpers::ECompatibilityMask::Compatible))
				.Padding(FMargin(2.0f, 0))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PCGContextualMenuActivated", "Contextual"))
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(0)
			[
				// Accept filters
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("PCGAcceptFiltering", "Show nodes that would apply a filter"))
				.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
				.OnCheckStateChanged(FOnCheckStateChanged::CreateSP(this, &FPCGEditor::OnContextualFilteringChanged, Menu.ToWeakPtr(), PCGActionsHelpers::ECompatibilityMask::Filter))
				.IsChecked(TAttribute<ECheckBoxState>::CreateSP(this, &FPCGEditor::IsContextualFilteringChecked, PCGActionsHelpers::ECompatibilityMask::Filter))
				.IsEnabled(TAttribute<bool>::CreateSP(this, &FPCGEditor::IsContextualFilteringEnabled, PCGActionsHelpers::ECompatibilityMask::Compatible))
				.Padding(FMargin(2.0f, 0))
				[
					SNew(SImage)
					.Image(FPCGEditorStyle::Get().GetBrush(PCGEditorStyleConstants::NodeFilter))
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(0)
			[
				// Accept conversions
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("PCGAcceptConversion", "Show nodes that would apply a conversion"))
				.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
				.OnCheckStateChanged(FOnCheckStateChanged::CreateSP(this, &FPCGEditor::OnContextualFilteringChanged, Menu.ToWeakPtr(), PCGActionsHelpers::ECompatibilityMask::Conversion))
				.IsChecked(TAttribute<ECheckBoxState>::CreateSP(this, &FPCGEditor::IsContextualFilteringChecked, PCGActionsHelpers::ECompatibilityMask::Conversion))
				.IsEnabled(TAttribute<bool>::CreateSP(this, &FPCGEditor::IsContextualFilteringEnabled, PCGActionsHelpers::ECompatibilityMask::Compatible))
				.Padding(FMargin(2.0f, 0))
				[
					SNew(SImage)
					.Image(FPCGEditorStyle::Get().GetBrush(PCGEditorStyleConstants::NodeConvert))
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(0)
		.AutoHeight()
		[
			Menu
		];

	return FActionMenuContent(MenuWithOptions, Menu->GetFilterTextBox());
}

PCGActionsHelpers::ECompatibilityMask FPCGEditor::GetContextualCompatibilityMask() const
{
	// Behavior note: there is compatibility at all unless the Compatible flag is enabled, hence why we return NoMask when this happens.
	return !!(ContextualActionMask & PCGActionsHelpers::ECompatibilityMask::Compatible) ? ContextualActionMask : PCGActionsHelpers::ECompatibilityMask::NoMask;
}

void FPCGEditor::OnContextualFilteringChanged(ECheckBoxState NewState, const TWeakPtr<SGraphEditorActionMenu> WeakActionMenu, PCGActionsHelpers::ECompatibilityMask AffectedMask)
{
	if (NewState == ECheckBoxState::Checked)
	{
		ContextualActionMask |= AffectedMask;
	}
	else
	{
		ContextualActionMask &= ~AffectedMask;
	}

	if (TSharedPtr<SGraphEditorActionMenu> ActionMenu = WeakActionMenu.Pin())
	{
		ActionMenu->RefreshAllActions();
	}
}

bool FPCGEditor::IsContextualFilteringEnabled(PCGActionsHelpers::ECompatibilityMask AffectedMask) const
{
	return !!(ContextualActionMask & AffectedMask);
}

ECheckBoxState FPCGEditor::IsContextualFilteringChecked(PCGActionsHelpers::ECompatibilityMask AffectedMask) const
{
	return IsContextualFilteringEnabled(AffectedMask) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FPCGEditor::OnCollapseNodesInSubgraph(bool bCollapseToEmbeddedSubgraph)
{
	if (!InternalValidationOnAction())
	{
		return;
	}

	UPCGGraph* PCGGraph = GetFocusedGraph();
	check(PCGGraph);

	// Gather all nodes that will be included in the subgraph, and the extra nodes
	TArray<UPCGNode*> NodesToCollapse;
	TArray<UObject*> ExtraNodesToCollapse;

	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	check(FocusedGraphWidget);
	for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		check(Object);

		// Exclude input and output nodes from the subgraph.
		if (Object->IsA<UPCGEditorGraphNodeInput>() || Object->IsA<UPCGEditorGraphNodeOutput>())
		{
			continue;
		}

		if (UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Object))
		{
			UPCGNode* PCGNode = PCGEditorGraphNode->GetPCGNode();
			check(PCGNode);
			NodesToCollapse.Add(PCGNode);
		}
		else if (UEdGraphNode* GraphNode = Cast<UEdGraphNode>(Object))
		{
			ExtraNodesToCollapse.Add(GraphNode);
		}
	}

	// If we have at most 1 node to collapse, just exit
	if (NodesToCollapse.Num() <= 1)
	{
		UE_LOGF(LogPCGEditor, Warning, "There were less than 2 PCG nodes selected, abort");
		return;
	}

	// Create a new subgraph, by creating a new PCGGraph asset.
	TObjectPtr<UPCGGraph> NewPCGGraph = nullptr;

	if(!bCollapseToEmbeddedSubgraph)
	{
		IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		TObjectPtr<UPCGGraphFactory> Factory = NewObject<UPCGGraphFactory>();
		Factory->bSkipTemplateSelection = true;

		FString NewPackageName;
		FString NewAssetName;
		PCGEditorUtils::GetParentPackagePathAndUniqueName(PCGGraph, LOCTEXT("NewPCGSubgraphAsset", "NewPCGSubgraph").ToString(), NewPackageName, NewAssetName);

		NewPCGGraph = Cast<UPCGGraph>(AssetTools.CreateAssetWithDialog(NewAssetName, NewPackageName, PCGGraph->GetClass(), Factory, "PCGEditor_CollapseInSubgraph"));

		if (NewPCGGraph == nullptr)
		{
			UE_LOGF(LogPCGEditor, Warning, "Subgraph asset creation was aborted or failed, abort.");
			return;
		}
	}

	{
		FScopedTransaction Transaction(LOCTEXT("PCGCollapseInSubgraphMessage", "[PCG] Collapse into Subgraph"));

		if (bCollapseToEmbeddedSubgraph)
		{
			UPCGGraph* RootGraph = PCGGraph;
			if (PCGGraph->IsEmbeddedSubgraph())
			{
				RootGraph = PCGGraph->GetEmbeddedParentGraph();
			}
			NewPCGGraph = RootGraph->AddNewEmbeddedSubgraph();
		}

		FText OutFailReason;
		const UPCGEditorGraphSchema* PCGSchema = CastChecked<UPCGEditorGraphSchema>(PCGGraph->GetEditorGraph()->GetSchema());
		NewPCGGraph = FPCGSubgraphHelpers::CollapseIntoSubgraphWithReason(PCGGraph, NodesToCollapse, ExtraNodesToCollapse, OutFailReason, NewPCGGraph, PCGSchema->GetSubgraphSettingsClass());

		if (NewPCGGraph == nullptr)
		{
			FMessageDialog::Open(EAppMsgType::Ok, OutFailReason, LOCTEXT("PCGCollapseInSubgraphFailed", "PCG Subgraph Collapse Failed"));
			Transaction.Cancel();
			return;
		}

		// Force a refresh
		PCGGraph->GetEditorGraph()->ReconstructGraph();
	}

	if (NewPCGGraph)
	{
		if (!NewPCGGraph->IsEmbeddedSubgraph())
		{
			// Save the new asset
			UEditorAssetLibrary::SaveLoadedAsset(NewPCGGraph);
		}

		// Notify the widget
		FocusedGraphWidget->NotifyGraphChanged();
	}
}

bool FPCGEditor::CanExportNodes() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return false;
	}

	for (const UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		check(Object);

		// Exclude input and output nodes from the subgraph.
		if (Object->IsA<UPCGEditorGraphNodeInput>() || Object->IsA<UPCGEditorGraphNodeOutput>())
		{
			continue;
		}

		// Also exclude reroute nodes
		if (Object->IsA<UPCGEditorGraphNodeReroute>() || Object->IsA<UPCGEditorGraphNodeNamedRerouteBase>())
		{
			continue;
		}

		if (const UPCGEditorGraphNodeBase* GraphNode = Cast<UPCGEditorGraphNodeBase>(Object))
		{
			// Exclude embedded subgraph nodes
			if (const UPCGBaseSubgraphNode* SubgraphNode = Cast<UPCGBaseSubgraphNode>(GraphNode->GetPCGNode()))
			{
				if (SubgraphNode->GetSubgraph() && SubgraphNode->GetSubgraph()->IsEmbeddedSubgraph())
				{
					continue;
				}
			}

			return true;
		}
	}

	return false;
}

void FPCGEditor::OnExportNodes()
{
	UPCGEditorGraph* FocusedEditorGraph = GetFocusedEditorGraph();
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedEditorGraph || !FocusedGraphWidget)
	{
		UE_LOGF(LogPCGEditor, Error, "FocusedGraphWidget or Focused editor graph is null, aborting");
		return;
	}

	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		check(Object);

		// Exclude input and output nodes from the subgraph.
		if (Object->IsA<UPCGEditorGraphNodeInput>() || Object->IsA<UPCGEditorGraphNodeOutput>())
		{
			continue;
		}

		UPCGSettings* Settings = nullptr;

		if (UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Object))
		{
			UPCGNode* PCGNode = PCGEditorGraphNode->GetPCGNode();
			check(PCGNode);

			// Exclude embedded subgraph nodes
			if (const UPCGBaseSubgraphNode* SubgraphNode = Cast<UPCGBaseSubgraphNode>(PCGNode))
			{
				if (SubgraphNode->GetSubgraph() && SubgraphNode->GetSubgraph()->IsEmbeddedSubgraph())
				{
					continue;
				}
			}

			Settings = PCGNode->GetSettings();
		}

		if (!Settings)
		{
			continue;
		}

		// Create new settings asset
		FString NewPackageName;
		FString NewAssetName;
		PCGEditorUtils::GetParentPackagePathAndUniqueName(GetFocusedGraph(), LOCTEXT("NewPCGSettingsAsset", "NewPCGSettings").ToString(), NewPackageName, NewAssetName);

		UObject* NewSettings = AssetTools.DuplicateAssetWithDialogAndTitle(NewAssetName, NewPackageName, Settings, NSLOCTEXT("PCGEditor_ExportNodes", "PCGEditor_ExportNodesTitle", "Export Settings As..."));

		if (NewSettings == nullptr)
		{
			UE_LOGF(LogPCGEditor, Warning, "Settings asset creation was aborted or failed, abort.");
			return;
		}
	}
}

void FPCGEditor::OnConvertToStandaloneNodes()
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return;
	}

	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorConvertToStandaloneMessage", "PCG Editor: Converting instanced nodes to standalone"), nullptr);

	for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		check(Object);
		// Exclude input and output nodes from the subgraph.
		if (Object->IsA<UPCGEditorGraphNodeInput>() || Object->IsA<UPCGEditorGraphNodeOutput>())
		{
			continue;
		}

		if (UPCGEditorGraphNodeBase* Node = Cast<UPCGEditorGraphNodeBase>(Object))
		{
			if (UPCGNode* PCGNode = Node->GetPCGNode())
			{
				if (PCGNode->IsInstance())
				{
					PCGNode->Modify();

					UPCGSettings* SourceSettings = PCGNode->GetSettings();
					check(SourceSettings);

					UPCGSettings* SettingsCopy = DuplicateObject(SourceSettings, PCGNode);
					SettingsCopy->SetFlags(RF_Transactional);

					PCGNode->SetSettingsInterface(SettingsCopy);
				}
			}

			Node->ReconstructNode();
		}
	}

	// Notify the widget
	FocusedGraphWidget->NotifyGraphChanged();
}

bool FPCGEditor::CanConvertToStandaloneNodes() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return false;
	}

	for (const UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		check(Object);

		// Exclude input and output nodes from the subgraph.
		if (Object->IsA<UPCGEditorGraphNodeInput>() || Object->IsA<UPCGEditorGraphNodeOutput>())
		{
			continue;
		}

		if (const UPCGEditorGraphNodeBase* Node = Cast<const UPCGEditorGraphNodeBase>(Object))
		{
			if (const UPCGNode* PCGNode = Node->GetPCGNode())
			{
				if (PCGNode->IsInstance())
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FPCGEditor::OnToggleInspected()
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return;
	}

	UEdGraphNode* GraphNode = FocusedGraphWidget->GetSingleSelectedNode();
	UPCGEditorGraphNodeBase* PCGGraphNodeBase = Cast<UPCGEditorGraphNodeBase>(GraphNode);

	const UPCGNode* PCGNode = PCGGraphNodeBase ? PCGGraphNodeBase->GetPCGNode() : nullptr;
	const UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;

	// Switch value.
	SetNodeInspected(PCGGraphNodeBase, /*bValue=*/ PCGGraphNodeBase ? !PCGGraphNodeBase->GetInspected() : false);
}

void FPCGEditor::SetNodeInspected(UPCGEditorGraphNodeBase* InspectedNode, bool bValue)
{
	if (InspectedNode && InspectedNode->GetInspected() == bValue)
	{
		// Nothing to do.
		return;
	}
	
	const UPCGNode* PCGNode = InspectedNode ? InspectedNode->GetPCGNode() : nullptr;
	const UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;

	if (PCGSettingsInterface && !PCGSettingsInterface->CanBeDebugged())
	{
		return;
	}

	TArray<UPCGEditorGraphNodeBase*, TInlineAllocator<4>> InspectedNodesBefore;
	for (TSharedPtr<SPCGEditorGraphAttributeListView> AttributeListView : AttributesWidgets)
	{
		if (AttributeListView.IsValid() && AttributeListView->GetNodeBeingInspected())
		{
			InspectedNodesBefore.Add(AttributeListView->GetNodeBeingInspected());
		}
	}

	bool bIsInspecting = false;

	const auto FindInspectedNodePredicate = [PCGNode](const UPCGEditorGraphNodeBase* InNode) { return InNode && InNode->GetPCGNode() == PCGNode; };

	// If the selected node was previously inspected, stop inspecting it, and unselect it from the attribute list views
	if (InspectedNodesBefore.ContainsByPredicate(FindInspectedNodePredicate))
	{
		InspectedNode->SetInspected(false);

		for (TSharedPtr<SPCGEditorGraphAttributeListView> AttributeListView : AttributesWidgets)
		{
			if (AttributeListView.IsValid() &&
				AttributeListView->GetNodeBeingInspected() &&
				AttributeListView->GetNodeBeingInspected()->GetPCGNode() == PCGNode)
			{
				AttributeListView->SetNodeBeingInspected(nullptr);
			}
		}

		if (DataOverridesWidget.IsValid())
		{
			DataOverridesWidget->SetNodeBeingInspected(nullptr);
		}
	}
	else
	{
		TArray<UPCGEditorGraphNodeBase*, TInlineAllocator<4>> InspectedNodesAfter;

		for (int32 Index = 0; Index < PCGEditorTabs::NumAttributeTabs; ++Index)
		{
			// If not yet created do that now
			if (!AttributesWidgets[Index].IsValid())
			{
				GetOrCreateAttributesWidget(Index);
				check(AttributesWidgets[Index].IsValid());
			}

			if (!AttributesWidgets[Index]->IsLocked())
			{
				AttributesWidgets[Index]->SetNodeBeingInspected(InspectedNode);
			
				InspectedNodesAfter.Add(AttributesWidgets[Index]->GetNodeBeingInspected());
			}
		}

		GetOrCreateDataOverridesWidget()->SetNodeBeingInspected(InspectedNode);

		for (UPCGEditorGraphNodeBase* BeforeNode : InspectedNodesBefore)
		{
			if (BeforeNode &&
				!InspectedNodesAfter.ContainsByPredicate(
					[&BeforeNode](const UPCGEditorGraphNodeBase* AfterNode)
						{
							return BeforeNode ? BeforeNode->GetPCGNode() == AfterNode->GetPCGNode() : false;
						}
				))
			{
				BeforeNode->SetInspected(false);
			}
		}
		
		if (InspectedNode && InspectedNodesAfter.Contains(InspectedNode))
		{
			InspectedNode->SetInspected(true);
			bIsInspecting = true;
		}
	}

	if (bIsInspecting)
	{
		// Summon the first attribute list view that is inspecting this node
		auto InvokeFirstTab = [this, InspectedNode](bool bVisibleOnly) -> bool
		{
			for (int AttributeListViewIndex = 0; AttributeListViewIndex < PCGEditorTabs::NumAttributeTabs; ++AttributeListViewIndex)
			{
				TSharedPtr<SPCGEditorGraphAttributeListView> AttributeListView = AttributesWidgets[AttributeListViewIndex];
				if (AttributeListView.IsValid() && AttributeListView->GetNodeBeingInspected() == InspectedNode)
				{
					if (!bVisibleOnly || TabManager->FindExistingLiveTab(FTabId(PCGEditorTabs::AttributesID[AttributeListViewIndex])))
					{
						if (IsPanelAvailable(PCGEditorTabs::AttributesID[AttributeListViewIndex]))
						{
							GetTabManager()->TryInvokeTab(FTabId(PCGEditorTabs::AttributesID[AttributeListViewIndex]));
							return true;
						}
					}
				}
			}

			return false;
		};

		// If the Data Overrides tab is already active, don't switch focus to an ALV.
		const bool bDataOverridesActive = TabManager->FindExistingLiveTab(FTabId(PCGEditorTabs::DataOverridesID)).IsValid();
		const bool bTabSummoned = !bDataOverridesActive && (InvokeFirstTab(true) || InvokeFirstTab(false));

		// Default to first if they are all locked
		if (!bTabSummoned && !bDataOverridesActive && IsPanelAvailable(PCGEditorTabs::AttributesID[0]))
		{
			GetTabManager()->TryInvokeTab(FTabId(PCGEditorTabs::AttributesID[0]));
		}

		if (DebugObjectTreeWidget.IsValid())
		{
			DebugObjectTreeWidget->SetNodeBeingInspected(PCGNode);
		}
	}
	else if(DebugObjectTreeWidget.IsValid())
	{
		DebugObjectTreeWidget->SetNodeBeingInspected(nullptr);
	}

	// Turn on "inspecting" on graph if we now have at least one inspected node and had none before
	UpdateAfterInspectedStackChanged(InspectionDataManager.GetStackBeingInspected());
}

bool FPCGEditor::CanToggleInspected() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return false;
	}

	const FGraphPanelSelectionSet& SelectedNodes = FocusedGraphWidget->GetSelectedNodes();
	if (SelectedNodes.Num() != 1)
	{
		// Can only inspect one node.
		return false;
	}

	for (const UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Object);
		if (!PCGEditorGraphNode)
		{
			return false;
		}

		const UPCGSettingsInterface* PCGSettingsInterface = PCGEditorGraphNode->GetPCGNode() ? PCGEditorGraphNode->GetPCGNode()->GetSettingsInterface() : nullptr;
		if (PCGSettingsInterface && PCGSettingsInterface->CanBeDebugged())
		{
			return true;
		}
	}

	return false;
}

ECheckBoxState FPCGEditor::GetInspectedCheckState() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return ECheckBoxState::Unchecked;
	}

	const FGraphPanelSelectionSet& SelectedNodes = FocusedGraphWidget->GetSelectedNodes();

	if (SelectedNodes.IsEmpty())
	{
		return ECheckBoxState::Unchecked;
	}

	bool bAllEnabled = true;
	bool bAnyEnabled = false;

	for (UObject* Object : SelectedNodes)
	{
		const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Object);
		if (!PCGEditorGraphNode)
		{
			continue;
		}

		bAllEnabled &= PCGEditorGraphNode->GetInspected();
		bAnyEnabled |= PCGEditorGraphNode->GetInspected();
	}

	if (bAllEnabled)
	{
		return ECheckBoxState::Checked;
	}
	else if (bAnyEnabled)
	{
		return ECheckBoxState::Undetermined;
	}

	return ECheckBoxState::Unchecked;
}

void FPCGEditor::OnToggleEnabled()
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return;
	}

	const ECheckBoxState CheckState = GetEnabledCheckState();
	const bool bNewCheckState = !(CheckState != ECheckBoxState::Unchecked);

	// To prevent the changes on the editor node from being in the transaction, we delay reconstruction.
	TArray<FPCGDeferNodeReconstructScope> DeferredEditorNodes;

	FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorToggleEnableTransactionMessage", "PCG Editor: Toggle Enable Nodes"), nullptr);

	UPCGGraph* PCGGraph = GetFocusedGraph();
	if (!ensure(PCGGraph))
	{
		return;
	}

	PCGGraph->DisableNotificationsForEditor();

	bool bChanged = false;
	for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Object);
		UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
		UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;

		if (!PCGSettingsInterface || !PCGSettingsInterface->CanBeDisabled())
		{
			continue;
		}

		if (PCGSettingsInterface->bEnabled != bNewCheckState)
		{
			DeferredEditorNodes.Emplace(PCGEditorGraphNode);
			PCGSettingsInterface->Modify();
			PCGSettingsInterface->SetEnabled(bNewCheckState);
			bChanged = true;
		}
	}

	PCGGraph->EnableNotificationsForEditor();

	if (bChanged)
	{
		FocusedGraphWidget->NotifyGraphChanged();
	}
	else
	{
		Transaction.Cancel();
	}
}

bool FPCGEditor::CanToggleEnabled() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return false;
	}

	for (const UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Object);
		const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
		if (!PCGNode)
		{
			continue;
		}

		if (PCGNode->GetSettingsInterface() && PCGNode->GetSettingsInterface()->CanBeDisabled())
		{
			return true;
		}
	}

	// Could not toggle enabled on anything in selection.
	return false;
}

ECheckBoxState FPCGEditor::GetEnabledCheckState() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return ECheckBoxState::Unchecked;
	}

	bool bAllEnabled = true;
	bool bAnyEnabled = false;

	for (const UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Object);
		const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
		const UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;

		if (!PCGSettingsInterface || !PCGSettingsInterface->CanBeDisabled())
		{
			continue;
		}

		bAllEnabled &= PCGSettingsInterface->bEnabled;
		bAnyEnabled |= PCGSettingsInterface->bEnabled;
	}

	if (bAllEnabled)
	{
		return ECheckBoxState::Checked;
	}
	else if (bAnyEnabled)
	{
		return ECheckBoxState::Undetermined;
	}

	return ECheckBoxState::Unchecked;
}

void FPCGEditor::OnToggleDebug()
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return;
	}

	const ECheckBoxState CheckState = GetDebugCheckState();
	const bool bNewCheckState = !(CheckState != ECheckBoxState::Unchecked);

	FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorToggleDebugTransactionMessage", "PCG Editor: Toggle Debug Nodes"), nullptr);

	bool bChanged = false;
	for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Object);
		UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
		UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;

		if (!PCGSettingsInterface || !PCGSettingsInterface->CanBeDebugged())
		{
			continue;
		}

		bChanged |= PCGSettingsInterface->SetDebugged(bNewCheckState);
	}

	if (!bChanged)
	{
		Transaction.Cancel();
	}
}

bool FPCGEditor::CanToggleDebug() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return false;
	}

	for (const UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Object);
		const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;

		if (PCGNode && PCGNode->GetSettingsInterface()->CanBeDebugged())
		{
			return true;
		}
	}

	// Could not toggle debug on anything in selection.
	return false;
}

void FPCGEditor::OnDebugOnlySelected()
{
	UPCGEditorGraph* FocusedEditorGraph = GetFocusedEditorGraph();
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedEditorGraph || !FocusedGraphWidget)
	{
		return;
	}

	bool bChanged = false;

	const FGraphPanelSelectionSet& SelectedNodes = FocusedGraphWidget->GetSelectedNodes();

	FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorDebugOnlySelectedTransactionMessage", "PCG Editor: Debug only selected nodes"), nullptr);

	bool bAnyNonSelectedNodesDebugged = false;
	bool bAllSelectedNodesDebugged = true;

	// Initial pass - inspect state of selected and non-selected nodes.
	for (const UEdGraphNode* Node : FocusedEditorGraph->Nodes)
	{
		const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Node);
		const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
		const UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;
		if (!PCGSettingsInterface)
		{
			continue;
		}

		if (SelectedNodes.Contains(PCGEditorGraphNode))
		{
			bAllSelectedNodesDebugged &= PCGSettingsInterface->bDebug;
		}
		else
		{
			bAnyNonSelectedNodesDebugged |= PCGSettingsInterface->bDebug;
		}
	}

	// The selected nodes should be debugged if any non-selected nodes are being debugged, or if the selected
	// nodes are partially being debugged.
	const bool bTargetDebugState = bAnyNonSelectedNodesDebugged || !bAllSelectedNodesDebugged;

	for (UEdGraphNode* Node : FocusedEditorGraph->Nodes)
	{
		UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Node);
		UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
		UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;

		if (!PCGSettingsInterface || !PCGSettingsInterface->CanBeDebugged())
		{
			continue;
		}

		// Selected set to target state, non-selected should not be debugged.
		const bool bShouldBeDebug = SelectedNodes.Contains(PCGEditorGraphNode) ? bTargetDebugState : false;

		bChanged |= PCGSettingsInterface->SetDebugged(bShouldBeDebug);
	}

	if (!bChanged)
	{
		Transaction.Cancel();
	}
}

void FPCGEditor::OnDisableDebugOnAllNodes()
{
	UPCGEditorGraph* FocusedEditorGraph = GetFocusedEditorGraph();
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedEditorGraph || !FocusedGraphWidget)
	{
		return;
	}

	bool bChanged = false;
	FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorDisableDebugAllNodesTransactionMessage", "PCG Editor: Disable debug on all nodes"), nullptr);

	for (UEdGraphNode* Node : FocusedEditorGraph->Nodes)
	{
		UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Node);
		UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
		UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;
		if (!PCGSettingsInterface)
		{
			continue;
		}

		bChanged |= PCGSettingsInterface->SetDebugged(false);
	}

	if (!bChanged)
	{
		Transaction.Cancel();
	}
}

ECheckBoxState FPCGEditor::GetDebugCheckState() const
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return ECheckBoxState::Unchecked;
	}

	bool bAllDebug = true;
	bool bAnyDebug = false;

	for (const UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Object);
		const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
		const UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;

		if (!PCGSettingsInterface || !PCGSettingsInterface->CanBeDebugged())
		{
			continue;
		}

		bAllDebug &= PCGSettingsInterface->bDebug;
		bAnyDebug |= PCGSettingsInterface->bDebug;
	}

	if (bAllDebug)
	{
		return ECheckBoxState::Checked;
	}
	else if (bAnyDebug)
	{
		return ECheckBoxState::Undetermined;
	}

	return ECheckBoxState::Unchecked;
}

void FPCGEditor::OnEditInViewport()
{
	UPCGEditorGraph* FocusedEditorGraph = GetFocusedEditorGraph();
	const TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedEditorGraph || !FocusedGraphWidget)
	{
		return;
	}

	UEdGraphNode* SelectedGraphNode = FocusedGraphWidget->GetSingleSelectedNode();
	UPCGEditorGraphNodeBase* SelectedEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(SelectedGraphNode);
	const UPCGNode* SelectedPCGNode = SelectedEditorGraphNode ? SelectedEditorGraphNode->GetPCGNode() : nullptr;
	UPCGSettingsInterface* SelectedSettingsInterface = SelectedPCGNode ? SelectedPCGNode->GetSettingsInterface() : nullptr;

	if (!SelectedSettingsInterface || !SelectedSettingsInterface->CanBeDebugged())
	{
		return;
	}

	if (!SelectedSettingsInterface->IsTemporaryManualEditingEnabled())
	{
		// Only one node can have the temporary viewport editing active at a time
		for (UEdGraphNode* Node : FocusedEditorGraph->Nodes)
		{
			UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Node);
			const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
			UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;
			if (PCGSettingsInterface && PCGSettingsInterface != SelectedSettingsInterface && PCGSettingsInterface->IsTemporaryManualEditingEnabled())
			{
				PCGSettingsInterface->SetTemporaryManualEditingEnabled(false);
				PCGEditorGraphNode->OnNodeChangedDelegate.ExecuteIfBound();
			}
		}

		SelectedSettingsInterface->SetTemporaryManualEditingEnabled(true);
		SelectedEditorGraphNode->OnNodeChangedDelegate.ExecuteIfBound();

		NotifyModuleManualEditStateChanged();
	}

	// Re-select the owning actor and frame the viewport on it
	if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(GetPCGSourceBeingInspected()))
	{
		// Taken largely from SPCGEditorGraphDebugObjectTree::FocusOnDebugObject_OnClicked
		// @todo_pcg: In follow up CL, abstract this into PCGEditorUtils. There are multiple iterations in PCG.
		AActor* Actor = PCGComponent->GetOwner();
		if (Actor && GEditor && GUnrealEd && GEditor->CanSelectActor(Actor, /*bInSelected=*/true))
		{
			GEditor->SelectNone(/*bNoteSelectionChange=*/false, /*bDeselectBSPSurfs=*/true, /*WarnAboutManyActors=*/false);
			GEditor->SelectActor(Actor, /*bInSelected=*/true, /*bNotify=*/true, /*bSelectEvenIfHidden=*/true);
			GUnrealEd->Exec(Actor->GetWorld(), TEXT("CAMERA ALIGN ACTIVEVIEWPORTONLY"));
			GEditor->SelectComponent(PCGComponent, /*bInSelected=*/true, /*bNotify=*/true, /*bSelectEvenIfHidden=*/true);
		}
	}
}

bool FPCGEditor::CanToggleViewportEditing() const
{
	const TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return false;
	}

	const FGraphPanelSelectionSet& SelectedNodes = FocusedGraphWidget->GetSelectedNodes();
	if (SelectedNodes.Num() != 1)
	{
		return false;
	}

	for (const UObject* NodeObject : SelectedNodes)
	{
		const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(NodeObject);
		const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
		const UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;
		if (PCGSettingsInterface && PCGSettingsInterface->CanBeDebugged())
		{
			return true;
		}
	}

	return false;
}

void FPCGEditor::OnMarkForViewportEditing()
{
	const TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return;
	}

	UEdGraphNode* SelectedGraphNode = FocusedGraphWidget->GetSingleSelectedNode();
	UPCGEditorGraphNodeBase* SelectedEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(SelectedGraphNode);
	const UPCGNode* SelectedPCGNode = SelectedEditorGraphNode ? SelectedEditorGraphNode->GetPCGNode() : nullptr;
	UPCGSettingsInterface* SelectedSettingsInterface = SelectedPCGNode ? SelectedPCGNode->GetSettingsInterface() : nullptr;

	if (!SelectedSettingsInterface || !SelectedSettingsInterface->CanBeDebugged())
	{
		return;
	}

	{
		FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorMarkForViewportEditingTransactionMessage", "PCG Editor: Mark for Viewport Editing"), nullptr);

		SelectedSettingsInterface->Modify();
		SelectedSettingsInterface->SetMarkedForManualEditing(!SelectedSettingsInterface->IsMarkedForManualEditing());
	}

	SelectedEditorGraphNode->OnNodeChangedDelegate.ExecuteIfBound();
	NotifyModuleManualEditStateChanged();
}

ECheckBoxState FPCGEditor::GetMarkForViewportEditingCheckState() const
{
	const TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return ECheckBoxState::Unchecked;
	}

	const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(FocusedGraphWidget->GetSingleSelectedNode());
	const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;
	const UPCGSettingsInterface* PCGSettingsInterface = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;

	if (PCGSettingsInterface && PCGSettingsInterface->IsMarkedForManualEditing())
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

void FPCGEditor::SelectAllNodes()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		FocusedGraphWidget->SelectAllNodes();
	}
}

bool FPCGEditor::CanSelectAllNodes() const
{
	return !!GetFocusedGraphEditorWidget();
}

void FPCGEditor::DeleteSelectedNodes()
{
	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return;
	}

	UPCGGraph* PCGGraph = GetFocusedGraph();
	check(PCGGraph && PCGGraph->GetEditorGraph());

	// DeleteSelectedNodes is called directly from UI command 
	PCGGraph->PrimeGraphCompilationCache();

	bool bChanged = false;

	{
		const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorDeleteTransactionMessage", "PCG Editor: Delete"), nullptr);
		PCGGraph->GetEditorGraph()->Modify();

		TArray<UPCGNode*> NodesToRemove;

		for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
		{
			if (UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<UPCGEditorGraphNodeBase>(Object))
			{
				if (PCGEditorGraphNode->CanUserDeleteNode())
				{
					UPCGNode* PCGNode = PCGEditorGraphNode->GetPCGNode();
					check(PCGNode);

					NodesToRemove.Add(PCGNode);

					PCGEditorGraphNode->DestroyNode();
					bChanged = true;
				}
			}
			else if (UEdGraphNode* GraphNode = Cast<UEdGraphNode>(Object))
			{
				if (GraphNode->CanUserDeleteNode())
				{
					GraphNode->DestroyNode();
					bChanged = true;
				}
			}
		}

		if (bChanged)
		{
			// Need to modify the pcg graph so comments are also caught.
			PCGGraph->Modify();
			PCGGraph->RemoveNodes(NodesToRemove);
		}
	}

	if (bChanged)
	{
		FocusedGraphWidget->ClearSelectionSet();
		FocusedGraphWidget->NotifyGraphChanged();
	}
}

bool FPCGEditor::CanDeleteSelectedNodes() const
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
		{
			UEdGraphNode* GraphNode = CastChecked<UEdGraphNode>(Object);

			if (GraphNode->CanUserDeleteNode())
			{
				return true;
			}
		}
	}

	return false;
}

void FPCGEditor::CopySelectedNodes()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		const FGraphPanelSelectionSet SelectedNodes = FocusedGraphWidget->GetSelectedNodes();

		//TODO: evaluate creating a clipboard object instead of ownership hack
		for (UObject* SelectedNode : SelectedNodes)
		{
			UEdGraphNode* GraphNode = CastChecked<UEdGraphNode>(SelectedNode);
			GraphNode->PrepareForCopying();
		}

		FString ExportedText;
		FEdGraphUtilities::ExportNodesToText(SelectedNodes, ExportedText);
		FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

		for (UObject* SelectedNode : SelectedNodes)
		{
			if (UPCGEditorGraphNodeBase* PCGGraphNode = Cast<UPCGEditorGraphNodeBase>(SelectedNode))
			{
				PCGGraphNode->PostCopy();
			}
		}
	}
}

bool FPCGEditor::CanCopySelectedNodes() const
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
		{
			if (UEdGraphNode* GraphNode = CastChecked<UEdGraphNode>(Object))
			{
				if (GraphNode->CanDuplicateNode())
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FPCGEditor::CutSelectedNodes()
{
	CopySelectedNodes();
	DeleteSelectedNodes();
}

bool FPCGEditor::CanCutSelectedNodes() const
{
	return CanCopySelectedNodes() && CanDeleteSelectedNodes();
}

void FPCGEditor::PasteNodes()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		PasteNodesHere(FocusedGraphWidget->GetPasteLocation2f());
	}
}

void FPCGEditor::PasteNodesHere(const FVector2D& Location)
{
	UPCGEditorGraph* FocusedEditorGraph = GetFocusedEditorGraph();
	if (!FocusedEditorGraph)
	{
		return;
	}

	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	if (!FocusedGraphWidget)
	{
		return;
	}

	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorPasteTransactionMessage", "PCG Editor: Paste"), nullptr);
	FocusedEditorGraph->Modify();

	// Clear the selection set (newly pasted stuff will be selected)
	FocusedGraphWidget->ClearSelectionSet();

	// Grab the text to paste from the clipboard.
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Import the nodes
	TSet<UEdGraphNode*> PastedNodes;
	FEdGraphUtilities::ImportNodesFromText(FocusedEditorGraph, TextToImport, /*out*/ PastedNodes);

	//Average position of nodes so we can move them while still maintaining relative distances to each other
	FVector2D AvgNodePosition(0.0f, 0.0f);

	// Number of nodes used to calculate AvgNodePosition
	int32 AvgCount = 0;

	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		if (PastedNode)
		{
			AvgNodePosition.X += PastedNode->NodePosX;
			AvgNodePosition.Y += PastedNode->NodePosY;
			++AvgCount;
		}
	}

	if (AvgCount > 0)
	{
		float InvNumNodes = 1.0f / float(AvgCount);
		AvgNodePosition.X *= InvNumNodes;
		AvgNodePosition.Y *= InvNumNodes;
	}

	TArray<UPCGNode*> NodesToPaste;

	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		FocusedGraphWidget->SetNodeSelection(PastedNode, true);

		PastedNode->NodePosX = (PastedNode->NodePosX - AvgNodePosition.X) + Location.X;
		PastedNode->NodePosY = (PastedNode->NodePosY - AvgNodePosition.Y) + Location.Y;

		PastedNode->SnapToGrid(SNodePanel::GetSnapGridSize());

		PastedNode->CreateNewGuid();

		UPCGEditorGraphNodeBase* PastedPCGGraphNode = Cast<UPCGEditorGraphNodeBase>(PastedNode);
		if (UPCGNode* PastedPCGNode = PastedPCGGraphNode ? PastedPCGGraphNode->GetPCGNode() : nullptr)
		{
			NodesToPaste.Add(PastedPCGNode);
		}
	}

	// Need to modify the pcg graph so comments are also caught.
	FocusedEditorGraph->GetPCGGraph()->Modify();
	FocusedEditorGraph->GetPCGGraph()->AddNodes(NodesToPaste);

	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		UPCGEditorGraphNodeBase* PastedPCGGraphNode = Cast<UPCGEditorGraphNodeBase>(PastedNode);
		if (UPCGNode* PastedPCGNode = PastedPCGGraphNode ? PastedPCGGraphNode->GetPCGNode() : nullptr)
		{
			PastedPCGGraphNode->RebuildAfterPaste();
		}
	}

	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		UPCGEditorGraphNodeBase* PastedPCGGraphNode = Cast<UPCGEditorGraphNodeBase>(PastedNode);
		if (UPCGNode* PastedPCGNode = PastedPCGGraphNode ? PastedPCGGraphNode->GetPCGNode() : nullptr)
		{
			PastedPCGGraphNode->PostPaste();

			if (UPCGSettings* Settings = PastedPCGNode->GetSettings())
			{
				Settings->PostPaste();
			}
		}
	}

	FocusedGraphWidget->NotifyGraphChanged();
}

bool FPCGEditor::CanPasteNodes() const
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return FEdGraphUtilities::CanImportNodesFromText(GetFocusedEditorGraph(), ClipboardContent);
}

void FPCGEditor::DuplicateNodes()
{
	CopySelectedNodes();
	PasteNodes();
}

bool FPCGEditor::CanDuplicateNodes() const
{
	return CanCopySelectedNodes();
}

void FPCGEditor::OnAlignTop()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		FocusedGraphWidget->OnAlignTop();
	}
}

void FPCGEditor::OnAlignMiddle()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		FocusedGraphWidget->OnAlignMiddle();
	}
}

void FPCGEditor::OnAlignBottom()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		FocusedGraphWidget->OnAlignBottom();
	}
}

void FPCGEditor::OnAlignLeft()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		FocusedGraphWidget->OnAlignLeft();
	}
}

void FPCGEditor::OnAlignCenter()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		FocusedGraphWidget->OnAlignCenter();
	}
}

void FPCGEditor::OnAlignRight()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		FocusedGraphWidget->OnAlignRight();
	}
}

void FPCGEditor::OnStraightenConnections()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		FocusedGraphWidget->OnStraightenConnections();
	}
}

void FPCGEditor::OnDistributeNodesH()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		FocusedGraphWidget->OnDistributeNodesH();
	}
}

void FPCGEditor::OnDistributeNodesV()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget())
	{
		FocusedGraphWidget->OnDistributeNodesV();
	}
}

void FPCGEditor::OnCreateComment()
{
	if (UPCGEditorGraph* FocusedEditorGraph = GetFocusedEditorGraph())
	{
		FPCGEditorGraphSchemaAction_NewComment CommentAction;

		TSharedPtr<SGraphEditor> GraphEditorPtr = SGraphEditor::FindGraphEditorForGraph(FocusedEditorGraph);
		FVector2f Location = FVector2f::ZeroVector;
		if (GraphEditorPtr)
		{
			Location = GraphEditorPtr->GetPasteLocation2f();
		}

		CommentAction.PerformAction(FocusedEditorGraph, nullptr, Location);
	}
}

void FPCGEditor::CreateGraphEditorCommands()
{
	if (GraphEditorCommands.IsValid())
	{
		return;
	}

	GraphEditorCommands = MakeShareable(new FUICommandList);

	// Editing commands
	GraphEditorCommands->MapAction(FGenericCommands::Get().SelectAll,
		FExecuteAction::CreateSP(this, &FPCGEditor::SelectAllNodes),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanSelectAllNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &FPCGEditor::DeleteSelectedNodes),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanDeleteSelectedNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &FPCGEditor::CopySelectedNodes),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanCopySelectedNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &FPCGEditor::CutSelectedNodes),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanCutSelectedNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &FPCGEditor::PasteNodes),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanPasteNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &FPCGEditor::DuplicateNodes),
		FCanExecuteAction::CreateSP(this, &FPCGEditor::CanDuplicateNodes));

	// Alignment Commands
	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesTop,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnAlignTop)
	);

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesMiddle,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnAlignMiddle)
	);

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesBottom,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnAlignBottom)
	);

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesLeft,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnAlignLeft)
	);

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesCenter,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnAlignCenter)
	);

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesRight,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnAlignRight)
	);

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().StraightenConnections,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnStraightenConnections)
	);

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().CreateComment,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnCreateComment)
	);

	// Distribution Commands
	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesHorizontally,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnDistributeNodesH)
	);

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesVertically,
		FExecuteAction::CreateSP(this, &FPCGEditor::OnDistributeNodesV)
	);
}

TSharedPtr<SGraphEditor> FPCGEditor::GetFocusedGraphEditorWidget() const
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEditorPtr = FocusedGraphEditor.Pin())
	{
		return FocusedGraphEditorPtr;
	}
	else
	{
		return GraphEditorWidget;
	}
}

TSharedRef<SGraphEditor> FPCGEditor::CreateGraphEditorWidget(TSharedPtr<FTabInfo> InTabInfo, UPCGEditorGraph* InGraph)
{
	if (!InGraph)
	{
		InGraph = GetMainEditorGraph();
	}

	TSharedPtr<SWidget> TitleBarWidget = SNew(SPCGGraphEditorTitleBar)
		.Graph(InGraph)
		.Editor(SharedThis(this))
		.HistoryNavigationWidget(InTabInfo->CreateHistoryNavigationWidget());

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FPCGEditor::OnSelectedNodesChanged);
	InEvents.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &FPCGEditor::OnValidateNodeTitle);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FPCGEditor::OnNodeTitleCommitted);
	InEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &FPCGEditor::OnNodeDoubleClicked);
	InEvents.OnSpawnNodeByShortcutAtLocation = SGraphEditor::FOnSpawnNodeByShortcutAtLocation::CreateSP(this, &FPCGEditor::OnSpawnNodeByShortcut, InGraph);
	InEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &FPCGEditor::OnCreateActionMenu);

	TSharedRef<SGraphEditor> GraphWidget = SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(true)
		.Appearance(GetAppearanceInfo())
		.GraphToEdit(InGraph)
		.GraphEvents(InEvents)
		.ShowGraphStateOverlay(false)
		.TitleBar(TitleBarWidget)
		.OnNavigateHistoryBack(FSimpleDelegate::CreateSP(this, &FPCGEditor::NavigateTab, FDocumentTracker::EOpenDocumentCause::NavigateBackwards))
		.OnNavigateHistoryForward(FSimpleDelegate::CreateSP(this, &FPCGEditor::NavigateTab, FDocumentTracker::EOpenDocumentCause::NavigateForwards));

	if (InGraph == GetMainEditorGraph())
	{
		GraphEditorWidget = GraphWidget;
	}

	return GraphWidget;
}

void FPCGEditor::NotifyModuleManualEditStateChanged()
{
	FModuleManager::GetModuleChecked<FPCGEditorModule>("PCGEditor").UpdateManualEditPanelVisibility();
}

void FPCGEditor::OnClose()
{
	FWorkflowCentricApplication::OnClose();

	// Hide the manual edit panel now that this editor is closing.
	NotifyModuleManualEditStateChanged();

	if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
	{
		LevelEditor->OnMapChanged().RemoveAll(this);
	}

	FEditorDelegates::PostPIEStarted.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);

	FAssetEditorToolkit::OnClose();

	InspectionDataManager.Cleanup();

	if (MainGraph)
	{
		MainGraph->OnGraphChangedDelegate.RemoveAll(this);
		MainGraph->OnNodeSourceCompiledDelegate.RemoveAll(this);

		if (MainGraph->IsInspecting())
		{
			MainGraph->DisableInspection();
		}

		if (MainGraph->NotificationsForEditorArePausedByUser())
		{
			MainGraph->ToggleUserPausedNotificationsForEditor();
		}
	}

	if (UPCGEngineSubsystem* EngineSubsystem = UPCGEngineSubsystem::Get())
	{
		EngineSubsystem->GetOnPCGSourceGenerationDone().RemoveAll(this);
	}

	if (GEditor)
	{
		UnregisterDelegatesForWorld(GEditor->GetEditorWorldContext().World());
		UnregisterDelegatesForWorld(GEditor->PlayWorld.Get());
	}

	// Lose the default execution source
	ReleaseDefaultExecutionSource();

	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		PCGEditorModule->OnPermissionModeChanged().RemoveAll(this);
	}

	FPCGModule::GetGraphExecutionRegistry().OnGraphExecutionSourcesChanged().RemoveAll(this);
}

void FPCGEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);

	UPCGEditorMenuContext* Context = NewObject<UPCGEditorMenuContext>();
	Context->PCGEditor = SharedThis(this);
	MenuContext.AddObject(Context);
}

void FPCGEditor::SaveEditedObjectState()
{
	UPCGGraph* Graph = GetMainGraph();
	if (!Graph)
	{
		return;
	}
	
	Graph->LastEditedDocuments.Empty();

	DocumentManager->SaveAllState();
}

void FPCGEditor::RestoreEditedObjectState()
{
	UPCGGraph* Graph = GetMainGraph();
	if (!Graph)
	{
		return;
	}

	auto RestoreDocument = [this](UPCGGraph* InGraph, const FPCGGraphDocumentInfo* Info)
	{
		TSharedPtr<SDockTab> TabWithGraph = OpenDocument(InGraph, FDocumentTracker::EOpenDocumentCause::RestorePreviousDocument);
		if (TabWithGraph)
		{
			TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(TabWithGraph->GetContent());
			
			if (Info)
			{
				GraphEditor->SetViewLocation(Info->SavedViewOffset, Info->SavedZoomAmount);
			}
		}
	};

	// Find Main graph info
	const FPCGGraphDocumentInfo* MainGraphInfo = Graph->LastEditedDocuments.FindByPredicate([Graph](const FPCGGraphDocumentInfo& Info)
	{
		return (Info.EditedObjectPath == Graph);
	});

	// Restore main graph first (so it is first in the tab list)
	RestoreDocument(Graph, MainGraphInfo);

	// Restore other graph(s)
	for (const FPCGGraphDocumentInfo& Info : Graph->LastEditedDocuments)
	{
		if (UPCGGraph* EditedGraph = Cast<UPCGGraph>(Info.EditedObjectPath.ResolveObject()))
		{
			if (EditedGraph != Graph)
			{
				RestoreDocument(EditedGraph, &Info);
			}
		}
	}

	// Focus on main tab
	OpenDocument(Graph, FDocumentTracker::EOpenDocumentCause::RestorePreviousDocument);
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreatePaletteTabFactory()
{
	FPCGGenericTabFactoryParams Params(PCGEditorTabs::PaletteID, SharedThis<FPCGEditor>(this), [this]() { return GetOrCreatePaletteWidget(); });
	Params.Label = LOCTEXT("PaletteLabel", "Palette");
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedRef<SPCGEditorGraphNodePalette> FPCGEditor::GetOrCreatePaletteWidget()
{
	if (!PaletteWidget.IsValid())
	{
		PaletteWidget = SNew(SPCGEditorGraphNodePalette, SharedThis(this));
	}

	return PaletteWidget.ToSharedRef();
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateDebugObjectTreeTabFactory()
{
	FPCGGenericTabFactoryParams Params(PCGEditorTabs::DebugObjectID, SharedThis<FPCGEditor>(this), [this]() { return GetOrCreateDebugObjectTreeWidget(); });
	Params.Label = LOCTEXT("DebugLabel", "Debug Object");
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedRef<SPCGEditorGraphDebugObjectTree> FPCGEditor::GetOrCreateDebugObjectTreeWidget()
{
	if (!DebugObjectTreeWidget.IsValid())
	{
		DebugObjectTreeWidget = SNew(SPCGEditorGraphDebugObjectTree, SharedThis(this));
	}

	return DebugObjectTreeWidget.ToSharedRef();
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateFindTabFactory()
{
	FPCGGenericTabFactoryParams Params(PCGEditorTabs::FindID, SharedThis<FPCGEditor>(this), [this]() { return GetOrCreateFindWidget(); });
	Params.Label = LOCTEXT("FindLabel", "Find");
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedRef<SPCGEditorGraphFind> FPCGEditor::GetOrCreateFindWidget()
{
	if (!FindWidget.IsValid())
	{
		FindWidget = SNew(SPCGEditorGraphFind, SharedThis(this));
	}

	return FindWidget.ToSharedRef();
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateAttributesTabFactory(int32 Index, TSharedRef<FWorkspaceItem> Group)
{
	check(Index >= 0 && Index < PCGEditorTabs::NumAttributeTabs);

	FPCGGenericTabFactoryParams Params(PCGEditorTabs::AttributesID[Index], SharedThis<FPCGEditor>(this), [this, Index]() { return GetOrCreateAttributesWidget(Index); });
	
	static FText AttributeText[4] =
	{
		LOCTEXT("AttributeLabel1", "Attributes"),
		LOCTEXT("AttributeLabel2", "Attributes 2"),
		LOCTEXT("AttributeLabel3", "Attributes 3"),
		LOCTEXT("AttributeLabel4", "Attributes 4")
	};
	
	Params.Label = AttributeText[Index];
	Params.Group = Group;
	Params.OnTabClosed = FPCGGenericTabClosed::CreateSP(this, &FPCGEditor::OnAttributeListViewTabClosed, Index);
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedRef<SPCGEditorGraphAttributeListView> FPCGEditor::GetOrCreateAttributesWidget(int32 Index)
{
	check(Index >= 0 && Index < PCGEditorTabs::NumAttributeTabs);

	static_assert(FPCGEditorInspectionDataManager::NumberOfEntries == PCGEditorTabs::NumAttributeTabs, "FPCGEditorInspectionDataManager::NumberOfEntries == PCGEditorTabs::NumAttributeTabs");

	if (!AttributesWidgets[Index].IsValid())
	{
		AttributesWidgets[Index] = SNew(SPCGEditorGraphAttributeListView, SharedThis(this))
			.WidgetEntryNumber(Index);
		AttributesWidgets[Index]->SetViewportWidget(GetOrCreateViewportWidget(Index), PCGEditorTabs::ViewportID[Index]);
	}

	return AttributesWidgets[Index].ToSharedRef();
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateDeterminismTabFactory()
{
	FPCGGenericTabFactoryParams Params(PCGEditorTabs::DeterminismID, SharedThis<FPCGEditor>(this), [this]() { return GetOrCreateDeterminismWidget(); });
	Params.Label = LOCTEXT("DeterminismLabel", "Determinism");
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedRef<SPCGEditorGraphDeterminismListView> FPCGEditor::GetOrCreateDeterminismWidget()
{
	if (!DeterminismWidget.IsValid())
	{
		DeterminismWidget = SNew(SPCGEditorGraphDeterminismListView, SharedThis(this));
	}

	return DeterminismWidget.ToSharedRef();
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateProfilingTabFactory()
{
	FPCGGenericTabFactoryParams Params(PCGEditorTabs::ProfilingID, SharedThis<FPCGEditor>(this), [this]() { return GetOrCreateProfilingWidget(); });
	Params.Label = LOCTEXT("ProfilingLabel", "Profiling");
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedRef<SPCGEditorGraphProfilingView> FPCGEditor::GetOrCreateProfilingWidget()
{
	if (!ProfilingWidget.IsValid())
	{
		ProfilingWidget = SNew(SPCGEditorGraphProfilingView, SharedThis(this));
	}

	return ProfilingWidget.ToSharedRef();
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateLogTabFactory()
{
	FPCGGenericTabFactoryParams Params(PCGEditorTabs::LogID, SharedThis<FPCGEditor>(this), [this]() { return GetOrCreateLogWidget(); });
	Params.Label = LOCTEXT("FindLog", "Log");
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedRef<SPCGEditorGraphLogView> FPCGEditor::GetOrCreateLogWidget()
{
	if (!LogWidget.IsValid())
	{
		LogWidget = SNew(SPCGEditorGraphLogView, SharedThis(this));
	}

	return LogWidget.ToSharedRef();
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateCodeEditorTabFactory()
{
	FPCGGenericTabFactoryParams Params(PCGEditorTabs::CodeEditorID, SharedThis<FPCGEditor>(this), [this]() { return GetOrCreateCodeEditorWidget(); });
	Params.Label = LOCTEXT("CodeEditorLabel", "Code Editor");
	return MakeShared<FPCGGenericTabFactory>(Params);
}

// Deprecated 5.8
TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateNodeSourceTabFactory()
{
	return CreateCodeEditorTabFactory();
}

TSharedRef<SPCGCodeEditor> FPCGEditor::GetOrCreateCodeEditorWidget()
{
	if (!CodeEditorWidget.IsValid())
	{
		CodeEditorWidget = SNew(SPCGCodeEditor);
	}
	
	return CodeEditorWidget.ToSharedRef();
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateGraphParamsTabFactory()
{
	FPCGGenericTabFactoryParams Params(PCGEditorTabs::UserParamsID, SharedThis<FPCGEditor>(this), [this]() { return GetOrCreateGraphParamsWidget(); });
	Params.Label = LOCTEXT("UserParamsLabel", "Graph Parameters");
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedRef<SPCGEditorGraphUserParametersView> FPCGEditor::GetOrCreateGraphParamsWidget()
{
	if (!UserParamsWidget.IsValid())
	{
		UserParamsWidget = SNew(SPCGEditorGraphUserParametersView, SharedThis(this));
	}

	return UserParamsWidget.ToSharedRef();
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreatePropertyDetailsTabFactory(int32 Index, TSharedRef<FWorkspaceItem> Group)
{
	check(Index >= 0 && Index < PCGEditorTabs::NumPropertyDetailTabs);

	FPCGGenericTabFactoryParams Params(PCGEditorTabs::PropertyDetailsID[Index], SharedThis<FPCGEditor>(this), [this, Index]() { return GetOrCreatePropertyDetailsWidget(Index); });

	static FText PropertyDetailText[4] =
	{
		LOCTEXT("PropertyDetailsLabel1", "Details"),
		LOCTEXT("PropertyDetailsLabel2", "Details 2"),
		LOCTEXT("PropertyDetailsLabel3", "Details 3"),
		LOCTEXT("PropertyDetailsLabel4", "Details 4")
	};

	Params.Label = PropertyDetailText[Index];
	Params.Group = Group;
	Params.OnTabClosed = FPCGGenericTabClosed::CreateSP(this, &FPCGEditor::OnDetailsViewTabClosed, Index);
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedRef<SPCGEditorGraphDetailsView> FPCGEditor::GetOrCreatePropertyDetailsWidget(int32 Index)
{
	check(Index >= 0 && Index < PCGEditorTabs::NumPropertyDetailTabs);

	if (!PropertyDetailsWidgets[Index].IsValid())
	{
		PropertyDetailsWidgets[Index] = SNew(SPCGEditorGraphDetailsView);
		PropertyDetailsWidgets[Index]->SetEditor(SharedThis(this));
		PropertyDetailsWidgets[Index]->SetObject(GetFocusedGraph());
	}

	return PropertyDetailsWidgets[Index].ToSharedRef();
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateViewportTabFactory(int32 Index, TSharedRef<FWorkspaceItem> Group)
{
	check(Index >= 0 && Index < PCGEditorTabs::NumViewportTabs);

	FPCGGenericTabFactoryParams Params(PCGEditorTabs::ViewportID[Index], SharedThis<FPCGEditor>(this), [this, Index]() 
	{
		// Attribute Widget is tied to Viewport Widget so create it to (to make sure Viewport gets refreshed properly through Attribute Widget)
		GetOrCreateAttributesWidget(Index);
		return GetOrCreateViewportWidget(Index); 
	});

	static FText ViewportText[4] =
	{
		LOCTEXT("ViewportLabel1", "Viewport"),
		LOCTEXT("ViewportLabel2", "Viewport 2"),
		LOCTEXT("ViewportLabel3", "Viewport 3"),
		LOCTEXT("ViewportLabel4", "Viewport 4")
	};

	Params.Label = ViewportText[Index];
	Params.Group = Group;
	Params.OnTabClosed = FPCGGenericTabClosed::CreateSP(this, &FPCGEditor::OnViewportViewTabClosed, Index);
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedPtr<SPCGEditorViewport> FPCGEditor::GetViewportWidget(int32 Index) const
{
	check(Index >= 0 && Index < PCGEditorTabs::NumViewportTabs);
	return ViewportWidgets[Index];
}

bool FPCGEditor::AutoLayoutFullGraph()
{
	FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorAutoLayout", "PCG Editor: Auto Layout"), nullptr);

	TSharedPtr<SGraphEditor> FocusedGraphWidget = GetFocusedGraphEditorWidget();
	check(FocusedGraphWidget.IsValid());

	UPCGNode* InputNode = MainGraph->InputNode;
	check(InputNode);
	UPCGNode* OutputNode = MainGraph->OutputNode;
	check(OutputNode);

	// Check if some nodes are selected first. If so, we only layout these nodes.
	TArray<UPCGNode*> NodesToLayout;
	for (UObject* Object : FocusedGraphWidget->GetSelectedNodes())
	{
		if (UPCGEditorGraphNode * EditorNode = Cast<UPCGEditorGraphNode>(Object))
		{
			if (EditorNode->GetPCGNode())
			{
				NodesToLayout.Add(EditorNode->GetPCGNode());
			}
		}
	}

	bool bInputShouldBeLayout = false;
	bool bOutputShouldBeLayout = false;
	if (NodesToLayout.IsEmpty())
	{
		NodesToLayout = MainGraph->GetNodes();
		NodesToLayout.Add(InputNode);
		NodesToLayout.Add(OutputNode);
		bInputShouldBeLayout = true;
		bOutputShouldBeLayout = true;
	}
	else
	{
		bInputShouldBeLayout = NodesToLayout.Contains(InputNode);
		bOutputShouldBeLayout = NodesToLayout.Contains(OutputNode);
	}
	
	// Precompute predecessor since we use it more than once
	TMap<UPCGNode*, TSet<UPCGNode*>> PredecessorNodes;
	for (UPCGNode* Node : NodesToLayout)
	{
		TSet<UPCGNode*>& CurPred = PredecessorNodes.Add(Node);
		for (UPCGPin* Pin : Node->GetInputPins())
		{
			for (UPCGEdge* Edge : Pin->Edges)
			{
				if (Edge && Edge->InputPin && Edge->InputPin->Node)
				{
					CurPred.Add(Edge->InputPin->Node);
				}
			}
		}
	}
	
	auto GetNodePredecessor = [&PredecessorNodes](const UPCGNode* Node) -> TSet<UPCGNode*>
	{
		return PredecessorNodes[Node];
	};
	
	// Use the min position as the base position to keep the graph approx. where it is
	FIntVector2 BasePosition{NodesToLayout[0]->PositionX, NodesToLayout[0]->PositionY};
	for (const UPCGNode* Node : NodesToLayout)
	{
		FIntVector2 CurPosition{Node->PositionX, Node->PositionY};
		BasePosition.X = FMath::Min(BasePosition.X, CurPosition.X);
		BasePosition.Y = FMath::Min(BasePosition.Y, CurPosition.Y);
	}
	
	bool bNodeMovementOccurred = false;
	auto SetNodePosition = [&bNodeMovementOccurred, this](UPCGNode* Node, int32 NewPositionX, int32 NewPositionY)
	{
		int32 CurPositionX = 0;
		int32 CurPositionY = 0;
		Node->GetNodePosition(CurPositionX, CurPositionY);
		if (CurPositionX != NewPositionX || CurPositionY != NewPositionY)
		{
			bNodeMovementOccurred = true;
			Node->Modify();
			Node->SetNodePosition(NewPositionX, NewPositionY);
			if (UPCGEditorGraphNodeBase* EditorNode = GetEditorNode(Node))
			{
				EditorNode->Modify();
				EditorNode->SetNodePosX(NewPositionX);
				EditorNode->SetNodePosY(NewPositionY);
			}
		}
	};
	
	// Extract lonely nodes (Nodes without any connection), we'll place them manually
	auto IsLonely = [](const UPCGNode* Node) -> bool
	{
		return !Node->GetFirstConnectedInputPin() && !Node->GetFirstConnectedOutputPin();
	};
	
	TArray<UPCGNode*> LonelyNodes;
	TArray<UPCGNode*> SortedAttachedNodes;
	for (UPCGNode* Node : NodesToLayout)
	{
		if (Node != InputNode && Node != OutputNode)
		{
			if (IsLonely(Node))
			{
				LonelyNodes.Add(Node);
			}
			else
			{
				SortedAttachedNodes.Add(Node);
			}
		}
	}
	
	const bool bInputIsLonely = IsLonely(InputNode);
	const bool bOutputIsLonely = IsLonely(OutputNode);
	
	if (!bInputIsLonely && NodesToLayout.Contains(InputNode))
	{
		SortedAttachedNodes.Add(InputNode);
	}
	
	if (!bOutputIsLonely && NodesToLayout.Contains(OutputNode))
	{
		SortedAttachedNodes.Add(OutputNode);
	}

	// Give space for the input node
	if (bInputIsLonely && bInputShouldBeLayout)
	{
		BasePosition.X += (FPCGEditor_private::DefaultFixedNodeWidth + FPCGEditor_private::NodeLayoutHorizontalGap);
	}

	Algo::TopologicalSort(SortedAttachedNodes, GetNodePredecessor);
	
	// Associate a layer to each node
	int MaxLayerIndex = 0;
	TMap<UPCGNode*, int> NodeToLayerIndex;
	for (UPCGNode* Node : SortedAttachedNodes)
	{
		NodeToLayerIndex.Add(Node, 0);
	}

	for (UPCGNode* Node : SortedAttachedNodes)
	{
		int MaxLayerOfPred = 0;
		for (UPCGNode* PredNode: GetNodePredecessor(Node))
		{
			if (NodesToLayout.Contains(PredNode))
			{
				MaxLayerOfPred = FMath::Max(MaxLayerOfPred, NodeToLayerIndex[PredNode] + 1);
			}
		}
		NodeToLayerIndex[Node] = MaxLayerOfPred;
		MaxLayerIndex = FMath::Max(MaxLayerIndex, MaxLayerOfPred);
	}
	
	TArray<TArray<UPCGNode*>> LayerMembersByIndex;
	LayerMembersByIndex.SetNum(MaxLayerIndex+1);
	for (UPCGNode* Node : SortedAttachedNodes)
	{
		LayerMembersByIndex[NodeToLayerIndex[Node]].Add(Node);
	}

	// Set Positions for attached nodes
	int32 CurrentLayerPosX = BasePosition.X;
	int32 MaxPositionY = BasePosition.Y;
	for (TArray<UPCGNode*>& CurLayer : LayerMembersByIndex)
	{
		if (CurLayer.IsEmpty())
		{
			continue;
		}
		
		int32 CurLayerPosY = BasePosition.Y;
		int32 MaxNodeWidthForLayer = 0;
		for (UPCGNode* Node : CurLayer)
		{
			SetNodePosition(Node, CurrentLayerPosX, CurLayerPosY);
			
			// Update next Pos Y for this layer
			CurLayerPosY += FPCGEditor_private::DefaultFixedNodeHeight + FPCGEditor_private::NodeLayoutVerticalGap;
			
			MaxNodeWidthForLayer = FMath::Max(MaxNodeWidthForLayer, FPCGEditor_private::DefaultFixedNodeWidth);
			MaxPositionY = FMath::Max(MaxPositionY, CurLayerPosY);
		}
		
		//Update position X for the next layer
		CurrentLayerPosX += MaxNodeWidthForLayer + FPCGEditor_private::NodeLayoutHorizontalGap;
	}
	
	if (bInputIsLonely && bInputShouldBeLayout)
	{
		SetNodePosition(InputNode, BasePosition.X - (FPCGEditor_private::DefaultFixedNodeWidth + FPCGEditor_private::NodeLayoutHorizontalGap), BasePosition.Y);
	}
	
	if (bOutputIsLonely && bOutputShouldBeLayout)
	{
		SetNodePosition(OutputNode, CurrentLayerPosX, BasePosition.Y);
	}
	
	int32 LonelyLayerPosX = BasePosition.X;
	for (UPCGNode* LonelyNode : LonelyNodes)
	{
		SetNodePosition(LonelyNode, LonelyLayerPosX, MaxPositionY);
		LonelyLayerPosX += FPCGEditor_private::DefaultFixedNodeWidth + FPCGEditor_private::NodeLayoutHorizontalGap;
	}
	
	if (!bNodeMovementOccurred)
	{
		// Do not spam the undo buffer if the graph was already well positioned.
		Transaction.Cancel();
	}
	
	return bNodeMovementOccurred;
}

void FPCGEditor::OpenDataOverridesAndInspect(UPCGEditorGraphNodeBase* InNode)
{
	if (!InNode)
	{
		return;
	}

	if (!InNode->GetInspected())
	{
		SetNodeInspected(InNode, true);
	}

	GetOrCreateDataOverridesWidget()->SetNodeBeingInspected(InNode);

	BringFocusToPanel(PCGEditorTabs::DataOverridesID);
}

void FPCGEditor::FocusOwningActorInLevelViewport() const
{
	const UPCGComponent* PCGComponent = Cast<UPCGComponent>(GetPCGSourceBeingInspected());
	AActor* Actor = PCGComponent ? PCGComponent->GetOwner() : nullptr;
	if (!Actor || !GEditor || !GEditor->CanSelectActor(Actor, /*bInSelected=*/true))
	{
		return;
	}

	GEditor->SelectNone(/*bNoteSelectionChange=*/false, /*bDeselectBSPSurfs=*/true, /*WarnAboutManyActors=*/false);
	GEditor->SelectActor(Actor, /*bInSelected=*/true, /*bNotify=*/true, /*bSelectEvenIfHidden=*/true);
	GEditor->MoveViewportCamerasToActor(*Actor, /*bActiveViewportOnly=*/false);

	FGlobalTabmanager::Get()->TryInvokeTab(FTabId("LevelEditor"));
}

void FPCGEditor::SelectManualEditNode(UPCGEditorGraphNodeBase* InNode)
{
	if (!InNode)
	{
		return;
	}

	if (const FPCGEditorModule* EditorModule = FModuleManager::GetModulePtr<FPCGEditorModule>("PCGEditor"))
	{
		if (const TSharedPtr<SPCGManualEditPanel> Panel = EditorModule->GetManualEditPanel())
		{
			Panel->SelectNodeInList(InNode->GetPCGNode());
		}
	}
}

void FPCGEditor::CreateEditorModeManager()
{
	// Create a shared FAssetEditorModeManager for viewport 0.
	// ViewportModeManagers[0] and the toolkit's EditorModeManager share ownership of the same object.
	TSharedPtr<FAssetEditorModeManager> NewManager = MakeShared<FAssetEditorModeManager>();
	ViewportModeManagers[0] = NewManager;
	EditorModeManager = NewManager;
}

TSharedRef<SPCGEditorViewport> FPCGEditor::CreateViewportWidget(int32 Index)
{
	// Viewport 0's manager is always pre-created by CreateEditorModeManager().
	// Viewports 1-3 get their own managers created on demand here.
	if (!ViewportModeManagers[Index].IsValid())
	{
		ViewportModeManagers[Index] = MakeShared<FAssetEditorModeManager>();
		ViewportModeManagers[Index]->SetToolkitHost(ToolkitHost.Pin().ToSharedRef());
	}

	FAssetEditorModeManager* ModeManager = ViewportModeManagers[Index].Get();
	TSharedRef<SPCGEditorViewport> Viewport = SNew(SPCGEditorViewport).ModeTools(ModeManager);
	ModeManager->SetSupportsViewportITF(true);
	ModeManager->SetPreviewScene(Viewport->GetAdvancedPreviewScene());
	ModeManager->ActivateMode(UPCGAssetEditorMode::EM_PCGAssetEditorModeId);
	
	SetupModeTools(ModeManager);
	
	const TSharedPtr<FPCGEditor> ThisPtr = SharedThis<FPCGEditor>(this);
	if (UPCGAssetEditorMode* const PCGAssetEditorMode = GetAssetEditorMode(Index))
	{
		PCGAssetEditorMode->SetPCGEditor(ThisPtr);
	}

	return Viewport;
}

void FPCGEditor::SetupModeTools(FAssetEditorModeManager* InModeTools)
{
	check(InModeTools);
	if (const UModeManagerInteractiveToolsContext* InteractiveToolsContext = InModeTools->GetInteractiveToolsContext())
	{
		UContextObjectStore* ContextObjectStore = InteractiveToolsContext->ContextObjectStore;
		UPCGNodeToolContext* PCGNodeToolContext = NewObject<UPCGNodeToolContext>(ContextObjectStore);
		ContextObjectStore->AddContextObject(PCGNodeToolContext);
	}
}

FAssetEditorModeManager* FPCGEditor::GetViewportModeManager(int32 Index) const
{
	check(Index >= 0 && Index < PCGEditorTabs::NumViewportTabs);
	return ViewportModeManagers[Index].Get();
}

TSharedRef<SPCGEditorViewport> FPCGEditor::GetOrCreateViewportWidget(int32 Index)
{
	check(Index >= 0 && Index < PCGEditorTabs::NumViewportTabs);

	if (!ViewportWidgets[Index].IsValid())
	{
		ViewportWidgets[Index] = CreateViewportWidget(Index);
	}

	return ViewportWidgets[Index].ToSharedRef();
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateEmbeddedSubgraphsTabFactory()
{
	FPCGGenericTabFactoryParams Params(PCGEditorTabs::EmbeddedSubgraphsID, SharedThis<FPCGEditor>(this), [this]() { return GetOrCreateEmbeddedSubgraphsWidget(); });
	Params.Label = LOCTEXT("EmbeddedSubgraphsLabel", "Embedded Graphs");
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedPtr<FWorkflowTabFactory> FPCGEditor::CreateDataOverridesTabFactory()
{
	FPCGGenericTabFactoryParams Params(PCGEditorTabs::DataOverridesID, SharedThis<FPCGEditor>(this), [this]() { return GetOrCreateDataOverridesWidget(); });
	Params.Label = LOCTEXT("DataOverridesLabel", "Data Overrides");
	return MakeShared<FPCGGenericTabFactory>(Params);
}

TSharedRef<SPCGEditorGraphEmbeddedSubgraphsView> FPCGEditor::GetOrCreateEmbeddedSubgraphsWidget()
{
	if (!EmbeddedSubgraphsWidget.IsValid())
	{
		EmbeddedSubgraphsWidget = SNew(SPCGEditorGraphEmbeddedSubgraphsView, SharedThis(this));
	}

	return EmbeddedSubgraphsWidget.ToSharedRef();
}

TSharedRef<SPCGEditorGraphDataOverridesView> FPCGEditor::GetOrCreateDataOverridesWidget()
{
	if (!DataOverridesWidget.IsValid())
	{
		DataOverridesWidget = SNew(SPCGEditorGraphDataOverridesView, SharedThis(this));
	}

	return DataOverridesWidget.ToSharedRef();
}

void FPCGEditor::OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	FocusedGraphEditor = InGraphEditor;
	OnSelectedNodesChanged(InGraphEditor->GetSelectedNodes());

	if (UserParamsWidget)
	{
		UserParamsWidget->SetGraph(GetFocusedGraph());
	}
}

void FPCGEditor::OnGraphEditorBackgrounded(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	FocusedGraphEditor = nullptr;
	
	OnSelectedNodesChanged(FGraphPanelSelectionSet());

	if (UserParamsWidget)
	{
		UserParamsWidget->SetGraph(GetFocusedGraph());
	}
}

void FPCGEditor::OnGraphPreSave(const UPCGGraph* InGraph)
{
	if (InGraph == GetMainGraph())
	{
		SaveEditedObjectState();
	}
}

void FPCGEditor::OnSelectedNodesChanged(const TSet<UObject*>& NewSelection)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;

	if (NewSelection.Num() == 0)
	{
		SelectedObjects.Add(GetFocusedGraph());
	}
	else
	{
		for (UObject* Object : NewSelection)
		{
			if (UEdGraphNode* GraphNode = Cast<UEdGraphNode>(Object))
			{
				SelectedObjects.Add(GraphNode);
			}
		}
	}

	for (TSharedPtr<SPCGEditorGraphDetailsView> PropertyDetailsWidget : PropertyDetailsWidgets)
	{
		if (PropertyDetailsWidget.IsValid())
		{
			PropertyDetailsWidget->SetObjects(SelectedObjects, /*bForceRefresh=*/true);
		}
	}

	// Give a single selected node with valid settings to the source editor, or give it null so it can clear the UI.
	UPCGEditorGraphNode* SelectedNode = (NewSelection.Num() == 1) ? Cast<UPCGEditorGraphNode>(*NewSelection.CreateConstIterator()) : nullptr;
	UPCGNode* PCGNode = SelectedNode ? SelectedNode->GetPCGNode() : nullptr;
	SetSourceEditorTargetObject(PCGNode ? PCGNode->GetSettings() : nullptr);
}

void FPCGEditor::OnNodeToolStarted(UPCGEditorGraphNodeBase* InteractiveNode)
{
	if (!InteractiveNode)
	{
		return;
	}
	
	const UPCGNode* PCGNode = InteractiveNode->GetPCGNode();
	if (!ensure(PCGNode))
	{
		return;
	}
	
	UPCGSettings* NodeSettings = PCGNode->GetSettings();
	if (!ensure(NodeSettings))
	{
		return;
	}
	
	if (NodeSettings->IsNodeToolActive())
	{
		// Already active tool. Nothing to do
		return;
	}
	
	// Select the Interactive Node
	const TSharedPtr<SGraphEditor> GraphWidget = GetFocusedGraphEditorWidget();
	check(GraphWidget.IsValid());
	GraphWidget->ClearSelectionSet();
	GraphWidget->SetNodeSelection(InteractiveNode, /*bSelect=*/true);

	// Start inspecting the Interactive Node
	if (!InteractiveNode->GetInspected() && CanToggleInspected())
	{
		OnToggleInspected();
	}

	ActiveToolViewportIndex = 0;
	for (int32 Index = 0; Index < PCGEditorTabs::NumAttributeTabs; ++Index)
	{
		// If not yet created do that now
		if (!AttributesWidgets[Index].IsValid())
		{
			GetOrCreateAttributesWidget(Index);
			check(AttributesWidgets[Index].IsValid());
		}

		if (!AttributesWidgets[Index]->IsLocked())
		{
			ActiveToolViewportIndex = Index;
			break;
		}
	}

	const UPCGAssetEditorMode* PCGAssetEditorMode = GetAssetEditorMode(ActiveToolViewportIndex);
	if (!PCGAssetEditorMode)
	{
		return;
	}

	if (const TSubclassOf<UPCGAssetEditorInteractiveTool>* ToolClass = FPCGAssetEditorToolRegistry::Get().FindToolForSettings(NodeSettings->GetClass()))
	{
		if (UEditorInteractiveToolsContext* ToolsContext = PCGAssetEditorMode->GetInteractiveToolsContext())
		{
			if (const auto SourceBeingInspected = GetPCGSourceBeingInspected())
			{
				if (UPCGNodeToolContext* ToolContext = ToolsContext->ContextObjectStore->FindContext<UPCGNodeToolContext>())
				{
					ToolContext->NodeSettings = NodeSettings;
					ToolContext->ExecutionSourceObject = Cast<UObject>(SourceBeingInspected);
					ToolContext->GraphInstance = SourceBeingInspected->GetExecutionState().GetGraphInstance();

					ToolsContext->StartTool((*ToolClass)->GetName());
					
					NodeSettings->SetNodeToolActive(true);
				}
			}
		}
	}
}

void FPCGEditor::OnNodeToolEnded(UPCGEditorGraphNodeBase* InteractiveNode)
{
	if (InteractiveNode)
	{
		if (UPCGSettings* NodeSettings = InteractiveNode->GetSettings())
		{
			NodeSettings->SetNodeToolActive(false);
		}
	}

	if (ActiveToolViewportIndex != INDEX_NONE)
	{
		if (const UPCGAssetEditorMode* AssetEdMode = GetAssetEditorMode(ActiveToolViewportIndex))
		{
			if (UEditorInteractiveToolsContext* ToolsContext = AssetEdMode->GetInteractiveToolsContext())
			{
				if (ToolsContext->HasActiveTool())
				{
					ToolsContext->EndTool(EToolShutdownType::Accept);
				}
			}
		}
		ActiveToolViewportIndex = INDEX_NONE;
	}
}

void FPCGEditor::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	UPCGGraph* PCGGraph = GetFocusedGraph();
	check(PCGGraph);

	if (NodeBeingChanged)
	{
		if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
		{
			FText ErrorText;
			if (OnValidateNodeTitle(NewText, NodeBeingChanged, ErrorText))
			{
				const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorRenameNode", "PCG Editor: Rename Node"), nullptr);

				// Implementation detail: In UPCGEditorGraphNode we only set the title under certain conditions, so it calls Modify() itself.
				// However, UEdGraphNode does not call Modify() on its own, so we should still call it in this case.
				if (!NodeBeingChanged->IsA<UPCGEditorGraphNode>())
				{
					NodeBeingChanged->Modify();
					// Modify the graph as well, as non-pcg editor nodes (like the comment nodes) are serialized in UPCGGraph::ExtraEditorNodes.
					PCGGraph->Modify();
				}

				NodeBeingChanged->OnRenameNode(NewText.ToString());
			}
			else
			{
				UE_LOGF(LogPCGEditor, Warning, "%ls", *FText::Format(LOCTEXT("UnableToRenameNode", "Unable to rename node {0}. Reason: {1}"), NodeBeingChanged->GetNodeTitle(ENodeTitleType::FullTitle), std::move(ErrorText)).ToString());
			}
		}

		if (const UPCGEditorGraphNodeBase* PCGEditorNode = Cast<UPCGEditorGraphNodeBase>(NodeBeingChanged))
		{
			PCGEditorNode->OnNodeChangedDelegate.ExecuteIfBound();
		}
	}
}

void FPCGEditor::OnNodeDoubleClicked(UEdGraphNode* Node)
{
	if (Node != nullptr)
	{
		UObject* Object = Node->GetJumpTargetForDoubleClick();

		// "Normal" node
		if (const UPCGSettings* PCGSettings = Cast<UPCGSettings>(Object))
		{
			// Functions may require the GraphEditorWidget's node selection, so set it manually to be safe.
			GetFocusedGraphEditorWidget()->SetNodeSelection(Node, /*bSelect=*/true);

			switch (GetDefault<UPCGEditorSettings>()->NodeDoubleClickAction)
			{
				case EPCGEditorDoubleClickAction::ToggleInspectNode:
					if (CanToggleInspected())
					{
						OnToggleInspected();
					}
					break;
				case EPCGEditorDoubleClickAction::ToggleDebugNode:
					if (CanToggleDebug())
					{
						OnToggleDebug();
					}
					break;
				case EPCGEditorDoubleClickAction::JumpToSourceFile:
					JumpToDefinition(PCGSettings->GetClass());
					break;
				case EPCGEditorDoubleClickAction::DoNothing: // fall-through
				default:
					break;
			}
		}
		else // Special options with non-UPCGSettings based targets.
		{
			const UPCGEditorGraphNodeBase* PCGEditorGraphNode = Cast<const UPCGEditorGraphNodeBase>(Node);
			const UPCGNode* PCGNode = PCGEditorGraphNode ? PCGEditorGraphNode->GetPCGNode() : nullptr;

			FPCGStack StackToInspect;

			// If we're inspecting, we'll try to find a match in the stacks for subgraphs instead of relying on the static/template subgraph
			if (GetStackBeingInspected() && DebugObjectTreeWidget)
			{
				if (DebugObjectTreeWidget->GetFirstStackFromSelection(PCGNode, /*Graph=*/nullptr, StackToInspect))
				{
					Object = const_cast<UPCGGraph*>(StackToInspect.GetGraphForCurrentFrame());
				}
			}

			if (UPCGGraph* PCGGraph = Cast<UPCGGraph>(Object))
			{
				FPCGEditor* CurrentPCGEditor = OpenEditorForGraph(PCGGraph);
				if (CurrentPCGEditor && CurrentPCGEditor != this)
				{
					CurrentPCGEditor->SetStackBeingInspectedFromAnotherEditor(StackToInspect);
				}
			}
			else if (Object)
			{
				// Fallback: open the asset editor for any other UObject target (e.g. UBlueprint from a Blueprint element node).
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Object);
			}
		}
	}
}

void FPCGEditor::JumpToDefinition(const UClass* Class) const
{
	if (ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
		FSourceCodeNavigation::NavigateToClass(Class);
	}
}

void FPCGEditor::OnSourceUnregistered(IPCGGraphExecutionSource* Source)
{
	// Refresh the debug object tree to avoid stale entries from sources that have been unregistered.
	if (!Source || Source->GetExecutionState().GetGraph() == MainGraph)
	{
		if (DebugObjectTreeWidget.IsValid())
		{
			DebugObjectTreeWidget->RequestRefresh();
		}
	}

	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		PCGEditorModule->GetNodeVisualLogsMutable().ClearLogs(Source);
	}
}

void FPCGEditor::OnSourceGenerationDone(IPCGBaseSubsystem* Subsystem, IPCGGraphExecutionSource* Source, EPCGGenerationStatus Status)
{
	const FPCGStack& StackBeingInspected = InspectionDataManager.GetStackBeingInspected();
	const IPCGGraphExecutionSource* PCGSourceBeingInspected = InspectionDataManager.GetPCGSourceBeingInspected();

	// We want to refresh if the component that is done generating has generated the current graph being edited,
	// or if it is the root of the current stack being inspected (for subgraphs to also be refreshed).
	// If we don't have a component, we refresh nonetheless.
	bool bShouldRefresh = !Source || StackBeingInspected.GetRootSource() == Source || Source->GetExecutionState().GetGraph() == MainGraph;

	// Additionally, if we are not inspecting but the component that's done executing contains this graph, then we should also update.
	IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get();
	if (!bShouldRefresh && !PCGSourceBeingInspected && MainGraph && PCGEditorModule)
	{
		TArray<FPCGStackSharedPtr> ExecutedStacksContainingThisGraph = PCGEditorModule->GetExecutedStacksPtrs(Source, MainGraph);
		bShouldRefresh |= !ExecutedStacksContainingThisGraph.IsEmpty();
	}

	if (!bShouldRefresh)
	{
		return;
	}

	LastExecutionStatus = { Source, Status };

	OnSourceGenerated(Source);

	LastExecutionStatus.Reset();
}

IPCGBaseSubsystem* FPCGEditor::GetSubsystem() const
{
	return GetWorldSubsystem();
}

UPCGSubsystem* FPCGEditor::GetWorldSubsystem()
{
	UWorld* World = (GEditor ? (GEditor->PlayWorld ? GEditor->PlayWorld.Get() : GEditor->GetEditorWorldContext().World()) : nullptr);
	return UPCGSubsystem::GetInstance(World);
}

void FPCGEditor::RegisterDelegatesForWorld(UWorld* World)
{
	UnregisterDelegatesForWorld(World);

	if (UPCGSubsystem* Subsystem = UPCGSubsystem::GetInstance(World))
	{
		Subsystem->GetOnPCGSourceUnregistered().AddRaw(this, &FPCGEditor::OnSourceUnregistered);
		Subsystem->GetOnPCGSourceGenerationDone().AddRaw(this, &FPCGEditor::OnSourceGenerationDone);
	}
}

void FPCGEditor::UnregisterDelegatesForWorld(UWorld* World)
{
	if (UPCGSubsystem* Subsystem = UPCGSubsystem::GetInstance(World))
	{
		Subsystem->GetOnPCGSourceUnregistered().RemoveAll(this);
		Subsystem->GetOnPCGSourceGenerationDone().RemoveAll(this);
	}
}

void FPCGEditor::OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType)
{
	UpdateDefaultExecutionSource();

	if (!!(ChangeType & EPCGChangeType::Structural))
	{
		if (EmbeddedSubgraphsWidget.IsValid())
		{
			EmbeddedSubgraphsWidget->RequestRefresh();
		}
	}

	if (!!(ChangeType & EPCGChangeType::ShaderSource))
	{
		// Flush the shader file cache in case we are editing engine or data interface shaders.
		// We could make the user do this manually, but that makes iterating on data interfaces really painful.
		FlushShaderFileCache();
	}

	if (!!(ChangeType & (EPCGChangeType::GraphCustomization | EPCGChangeType::Cosmetic)))
	{
		if (PaletteWidget)
		{
			PaletteWidget->RequestRefresh();
		}
	}

	if (!!(ChangeType & EPCGChangeType::Edge))
	{
		for (TSharedPtr<SPCGEditorGraphDetailsView> Widget : PropertyDetailsWidgets)
		{
			if (Widget.IsValid() && Widget->GetVisibility() == EVisibility::Visible)
			{
				const TSharedPtr<IDetailsView> DetailsViewPtr = Widget->GetDetailsView();
				DetailsViewPtr->ForceRefresh();
			}
		}
	}

	CloseInvalidDocuments();
}

void FPCGEditor::OnNodeSourceCompiled(const UPCGNode* InNode, const FPCGCompilerDiagnostics& InDiagnostics)
{
	check(CodeEditorWidget);

	const UPCGSettings* Settings = InNode ? InNode->GetSettings() : nullptr;
	if (Settings && CodeEditorWidget->GetTextProviderObject() == Settings)
	{
		CodeEditorWidget->OnDiagnosticsUpdated(InDiagnostics);
	}
}

void FPCGEditor::OnMapChanged(UWorld* InWorld, EMapChangeType InMapChangedType)
{
	if (InMapChangedType != EMapChangeType::SaveMap)
	{
		if (InMapChangedType == EMapChangeType::TearDownWorld)
		{
			ReleaseDefaultExecutionSource();
		}
		else
		{
			UpdateDefaultExecutionSource();
		}

		InspectionDataManager.Cleanup();

		RefreshViews();

		// Subsystem has been torn down and rebuilt.
		if (GEditor)
		{
			RegisterDelegatesForWorld(GEditor->GetEditorWorldContext().World());
			RegisterDelegatesForWorld(GEditor->PlayWorld.Get());
		}
	}
}

void FPCGEditor::OnPostPIEStarted(bool bIsSimulating)
{
	RegisterDelegatesForWorld(GEditor ? GEditor->PlayWorld.Get() : nullptr);
}

void FPCGEditor::OnEndPIE(bool bIsSimulating)
{
	UnregisterDelegatesForWorld(GEditor ? GEditor->PlayWorld.Get() : nullptr);
}

void FPCGEditor::OnGraphExecutionSourcesChanged()
{
	if (DebugObjectTreeWidget.IsValid())
	{
		DebugObjectTreeWidget->RequestRefresh();
	}
}

void FPCGEditor::RefreshViews()
{
	if (DebugObjectTreeWidget.IsValid())
	{
		DebugObjectTreeWidget->RequestRefresh();
	}

	for (TSharedPtr<SPCGEditorGraphAttributeListView>& AttributeWidget : AttributesWidgets)
	{
		if (AttributeWidget.IsValid())
		{
			AttributeWidget->RequestRefresh();
		}
	}
}

void FPCGEditor::ReleaseDefaultExecutionSource(bool bCollectGarbage)
{
	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		PCGEditorModule->ClearExecutionMetadata(PCGDefaultExecutionSource);
	}
	PCGDefaultExecutionSource = nullptr;

	if (bCollectGarbage)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}	
}

void FPCGEditor::UpdateDefaultExecutionSource()
{
	const EPCGGraphUsage PreviousUsage = (PCGDefaultExecutionSource ? (Cast<UPCGDefaultWorldObjectExecutionSource>(PCGDefaultExecutionSource) != nullptr ? EPCGGraphUsage::Level : EPCGGraphUsage::Asset) : EPCGGraphUsage::Standard);
	const EPCGGraphUsage CurrentUsage = MainGraph->GetGraphUsage();
	bool bNeedsRefreshView = false;

	if (PreviousUsage != CurrentUsage && PCGDefaultExecutionSource)
	{
		ReleaseDefaultExecutionSource(/*bCollectGarbage=*/true);
		bNeedsRefreshView = true;
	}

	if (CurrentUsage != EPCGGraphUsage::Standard && !PCGDefaultExecutionSource)
	{
		if (CurrentUsage == EPCGGraphUsage::Asset)
		{
			PCGDefaultExecutionSource = IPCGBaseSubsystem::CreateExecutionSource<UPCGDefaultExecutionSource>({ .GraphInterface = MainGraph });
		}
		else if (CurrentUsage == EPCGGraphUsage::Level)
		{
			FPCGDefaultWorldObjectExecutionSourceParams Params;
			Params.GraphInterface = MainGraph;
			Params.WorldObject = UPCGSubsystem::GetSubsystemForCurrentWorld();
			PCGDefaultExecutionSource = IPCGBaseSubsystem::CreateExecutionSource<UPCGDefaultWorldObjectExecutionSource>(Params);
		}

		bNeedsRefreshView = true;
	}

	if (bNeedsRefreshView)
	{
		RefreshViews();
	}
}

TSharedRef<FTabManager::FLayout> FPCGEditor::GetDefaultLayout() const
{
	return PCGEditorDefaultMode::GetTabLayout().ToSharedRef();
}

void FPCGEditor::CloseInvalidDocuments()
{
	TArray<TSharedPtr<SDockTab>> AllDocumentTabs = DocumentManager->GetAllDocumentTabs();
	for (TSharedPtr<SDockTab> Tab : AllDocumentTabs)
	{
		if (Tab->GetLayoutIdentifier() == FTabId(PCGEditorTabs::GraphEditorID))
		{
			TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
			if (UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(GraphEditor->GetCurrentGraph()))
			{
				if (!EditorGraph->GetPCGGraph())
				{
					TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(EditorGraph);
					DocumentManager->CloseTab(Payload);
				}
			}
		}
	}

	DocumentManager->CleanInvalidTabs();
}

#undef LOCTEXT_NAMESPACE