// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorCommands.h"

#include "Schema/PCGEditorGraphSchemaActions.h"

#include "PCGSettings.h"

#include "Misc/ConfigCacheIni.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "PCGEditorCommands"

FPCGEditorCommands::FPCGEditorCommands()
	: TCommands<FPCGEditorCommands>(
		"PCGEditor",
		NSLOCTEXT("Contexts", "PCGEditor", "PCG Editor"),
		NAME_None,
		FAppStyle::GetAppStyleSetName())
{
}

void FPCGEditorCommands::RegisterCommands()
{
	UI_COMMAND(CollapseNodes, "Graph Asset", "Collapse selected nodes into a separate PCGGraph asset.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::J));
	UI_COMMAND(CollapseNodesToEmbeddedSubgraph, "Embedded Graph", "Collapse selected nodes into an embedded subgraph.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportNodes, "Export nodes to PCGSettings", "Exports selected nodes to separate and reusable PCGSettings assets.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertToStandaloneNodes, "Convert to Standalone Nodes", "Converts instanced nodes to standalone nodes.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Find, "Find", "Finds PCG nodes and comments in the current graph.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F));
	UI_COMMAND(ShowSelectedDetails, "Show Node Details", "Opens a details panel for the selected nodes.", EUserInterfaceActionType::Button, FInputChord(EKeys::F4));
	UI_COMMAND(PauseAutoRegeneration, "Pause Regen", "Pause automatic regeneration of the current graph.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::R));
	UI_COMMAND(ForceGraphRegeneration, "Force Regen", "Manually force a regeneration of the current graph.\nCtrl-click will also perform a flush cache before the regeneration.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AutoLayoutNodes, "Auto Layout", "Automatically reposition the nodes in the graph for better layout.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenDebugObjectTreeTab, "Debug Object Tree", "Open the Debug Object Tree tab to display and select graph invocations to debug.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RunDeterminismNodeTest, "Run Determinism Test on Node", "Evaluate the current node for determinism.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::T));
	UI_COMMAND(RunDeterminismGraphTest, "Graph Determinism Test", "Evaluate the current graph for determinism.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(EditGraphSettings, "Graph Settings", "Edit the graph settings.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EditGraph, "Graph", "Edit the graph.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleGraphParams, "Graph Parameters", "Open the graph settings panel.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(CancelExecution, "Cancel Execution", "Cancels the execution of the current graph", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Escape));
	UI_COMMAND(ToggleEnabled, "Toggle Enabled", "Toggle node enabled state for selected nodes.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::E));
	UI_COMMAND(ToggleDebug, "Toggle Debug", "Toggle node debug state for selected nodes", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::D));
	UI_COMMAND(DisableDebugOnAllNodes, "Disable Debug on all nodes", "Disable debug state for all nodes", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::D));
	UI_COMMAND(ToggleInspect, "Toggle Inspection", "Toggle node inspection for selected node", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::A));
	UI_COMMAND(AddInputPin, "Add Input Pin", "Add an input pin to the current node", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddOutputPin, "Add Output Pin", "Add an output pin to the current node", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RenameNode, "Rename Node", "Rename the selected node", EUserInterfaceActionType::Button, FInputChord(EKeys::F2));
	UI_COMMAND(SelectNamedRerouteUsages, "Select Named Reroute Usages", "Selects all usages of this Named Reroute Declaration", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectNamedRerouteDeclaration, "Select Named Reroute Declaration", "Selects the associated Named Reroute Declaration matching this Named Reroute Usage", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(JumpToSource, "Jump to Source", "Jumps to the associated source file.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AutoFocusViewport, "Auto Focus", "When inspection updates the viewport will re-focus to the data.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EditInViewport, "Edit in Viewport", "Begin manual viewport editing for the selected node, then select its owning actor and frame the viewport on it", EUserInterfaceActionType::Button, FInputChord(EKeys::V));
	UI_COMMAND(MarkForViewportEditing, "Mark for Viewport Editing", "Mark selected node for persistent viewport editing", EUserInterfaceActionType::ToggleButton, FInputChord());

	//~ PLATFORM SPECIFIC ~//
#if PLATFORM_MAC
	UI_COMMAND(DebugOnlySelected, "Debug Only Selected", "Enable node debug state for selected nodes and disable debug state for the others", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Command | EModifierKey::Shift, EKeys::D));
#else
	UI_COMMAND(DebugOnlySelected, "Debug Only Selected", "Enable node debug state for selected nodes and disable debug state for the others", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::D));
#endif
}

TSharedPtr<FEdGraphSchemaAction> FPCGSpawnNodeCommandInfo::GetAction() const
{
	TSharedPtr<FPCGEditorGraphSchemaAction_NewNativeElement> NewAction(new FPCGEditorGraphSchemaAction_NewNativeElement);
	NewAction->SettingsClass = PCGSettingsClass;
	NewAction->PreconfiguredInfo = PreconfiguredInfo;
	return NewAction;
}

void FPCGEditorSpawnNodeCommands::RegisterCommands()
{
	const FString ConfigSection = TEXT("PCGEditorSpawnNodes");
	const FString SettingsLabel = TEXT("Node");
	TArray<FString> SpawnNodeConfigEntries;

	GConfig->GetArray(*ConfigSection, *SettingsLabel, SpawnNodeConfigEntries, GEditorPerProjectIni);

	for (const FString& CurrentEntry : SpawnNodeConfigEntries)
	{
		FString ClassName;
		if (!FParse::Value(*CurrentEntry, TEXT("Class="), ClassName))
		{
			// Could not find a class name
			continue;
		}

		UClass* FoundClass = UClass::TryFindTypeSlow<UClass>(ClassName, EFindFirstObjectOptions::ExactClass);
		if (!FoundClass || !FoundClass->IsChildOf(UPCGSettings::StaticClass()))
		{
			continue;
		}

		TSharedPtr<FPCGSpawnNodeCommandInfo> SpawnCommandInfo = MakeShareable(new FPCGSpawnNodeCommandInfo(FoundClass));

		// If command info was created, set up a UI Command for keybinding.
		if (SpawnCommandInfo.IsValid())
		{
			FKey Key;

			// Parse the keybinding information.
			FString KeyString;
			if (FParse::Value(*CurrentEntry, TEXT("Key="), KeyString))
			{
				Key = *KeyString;
			}

			FInputChord Chord;
			// Parse the chord.
			if (Key.IsValid())
			{
				bool bShift = false;
				bool bCtrl = false;
				bool bAlt = false;

				FParse::Bool(*CurrentEntry, TEXT("Shift="), bShift);
				FParse::Bool(*CurrentEntry, TEXT("Alt="), bAlt);
				FParse::Bool(*CurrentEntry, TEXT("Ctrl="), bCtrl);
				Chord = FInputChord(Key, EModifierKey::FromBools(bCtrl, bAlt, bShift, false));
			}

			FParse::Value(*CurrentEntry, TEXT("Index="), SpawnCommandInfo->PreconfiguredInfo.PreconfiguredIndex);
			FString OutValue;
			const bool bOverrideLabel = FParse::Value(*CurrentEntry, TEXT("Label="), OutValue, /*bShouldStopOnSeparator=*/true);
			SpawnCommandInfo->PreconfiguredInfo.Label = FText::FromString(OutValue);

			const UPCGSettings* CDO = CastChecked<UPCGSettings>(FoundClass->GetDefaultObject(false));
			const FText CommandLabelText = bOverrideLabel ? SpawnCommandInfo->PreconfiguredInfo.Label : CDO->GetDefaultNodeTitle();
			const FText Description = FText::Format(NSLOCTEXT("PCGEditor", "SpawnNodeDescription", "Hold down the bound keys and left click in the graph panel to spawn a {0} node."), CommandLabelText);

			FUICommandInfo::MakeCommandInfo(/*InContext=*/this->AsShared(),
				SpawnCommandInfo->CommandInfo,
				FName(FText::Format(INVTEXT("SpawnNode_{0}"), CommandLabelText).ToString()),
				CommandLabelText,
				Description,
				/*InIcon=*/{},
				EUserInterfaceActionType::Button,
				Chord);

			SpawnNodeCommands.Add(SpawnCommandInfo);
		}
	}
}

TSharedPtr<FEdGraphSchemaAction> FPCGEditorSpawnNodeCommands::GetGraphActionByChord(const FInputChord& InChord) const
{
	if (InChord.IsValidChord())
	{
		for (const TSharedPtr<FPCGSpawnNodeCommandInfo>& Command : SpawnNodeCommands)
		{
			if (Command.IsValid() && Command->CommandInfo.IsValid() && Command->CommandInfo->HasActiveChord(InChord))
			{
				return Command->GetAction();
			}
		}
	}

	return TSharedPtr<FEdGraphSchemaAction>();
}

#undef LOCTEXT_NAMESPACE
