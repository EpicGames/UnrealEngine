// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorCommon)

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

namespace FPCGEditorCommon
{
	TAutoConsoleVariable<bool> CVarShowAdvancedAttributesFields(
		TEXT("pcg.graph.ShowAdvancedAttributes"),
		false,
		TEXT("Control whether advanced attributes/properties are shown in the PCG graph editor"));

	void Helpers::DispatchEditorToast(const FText& Text, const FText& SubText, const float Duration)
	{
		FNotificationInfo ToastInfo(Text);
		ToastInfo.ExpireDuration = Duration;
		ToastInfo.bFireAndForget = true;
		ToastInfo.SubText = SubText;

		FSlateNotificationManager::Get().AddNotification(ToastInfo);
	}
}
