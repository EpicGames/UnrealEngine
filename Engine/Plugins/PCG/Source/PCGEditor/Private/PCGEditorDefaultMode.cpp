// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorDefaultMode.h"

#include "PCGEditor.h"
#include "PCGEditorTabFactories.h"

#include "Delegates/DelegateCombinations.h"

#define LOCTEXT_NAMESPACE "PCGGraphEditor"

namespace PCGEditorDefaultMode
{
	DECLARE_DELEGATE_RetVal(TSharedPtr<FWorkflowTabFactory>, FCreateTabFactoryDelegate);

	FText GetLocalizedMode(FName ModeName)
	{
		return LOCTEXT("DefaultModeName", "PCG");
	}

	TSharedPtr<FTabManager::FLayout> GetTabLayout()
	{
		return FTabManager::NewLayout("Standalone_PCGGraphEditor_DefaultLayout_v1.0")
			->AddArea // Main PCG Graph Editor Area
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
				->Split // Top Section - Graph, Data Viewport, HLSL Source Editor, and Details View
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
					->SetSizeCoefficient(0.65f)
					->Split // Graph Palette
					(
						FTabManager::NewStack()
						->AddTab(PCGEditorTabs::PaletteID, ETabState::SidebarTab, ESidebarLocation::Left, /*SidebarSizeCoefficient=*/0.13f)
					)
					->Split // Data Viewport/HLSL Source Editor
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.2f)
						->AddTab(PCGEditorTabs::ViewportID[0], ETabState::OpenedTab)
						->AddTab(PCGEditorTabs::CodeEditorID, ETabState::OpenedTab)
						->SetForegroundTab(FName(PCGEditorTabs::ViewportID[0]))
					)
					->Split // Node Graph
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.6f)
						->AddTab(PCGEditorTabs::GraphEditorID, ETabState::OpenedTab)
						->SetHideTabWell(true)
					)
					->Split // Details View
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.2f)
						->AddTab(PCGEditorTabs::PropertyDetailsID[0], ETabState::OpenedTab)
					)
				)
				->Split // Bottom Section - Debug/Params and ALV
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
					->SetSizeCoefficient(0.35f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.2f)
						->AddTab(PCGEditorTabs::DebugObjectID, ETabState::OpenedTab)
						->AddTab(PCGEditorTabs::UserParamsID, ETabState::OpenedTab)
						->AddTab(PCGEditorTabs::EmbeddedSubgraphsID, ETabState::OpenedTab)
						->SetForegroundTab(FTabId(PCGEditorTabs::DebugObjectID))
					)
					->Split // ALV, Profiling, Find, Determinism
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.8f)
						->SetHideTabWell(false)
						->AddTab(PCGEditorTabs::AttributesID[0], ETabState::OpenedTab)
						->AddTab(PCGEditorTabs::ProfilingID, ETabState::OpenedTab)
						->AddTab(PCGEditorTabs::FindID, ETabState::OpenedTab)
						->AddTab(PCGEditorTabs::DeterminismID, ETabState::ClosedTab)
						->AddTab(PCGEditorTabs::DataOverridesID, ETabState::ClosedTab)
						->SetForegroundTab(FTabId(PCGEditorTabs::AttributesID[0]))
					)
				)
			);
	}
}

FPCGEditorDefaultMode::FPCGEditorDefaultMode(TSharedPtr<FPCGEditor> InPCGEditor)
	: FApplicationMode(StaticName(), PCGEditorDefaultMode::GetLocalizedMode)
{
	TabLayout = InPCGEditor->GetDefaultLayout();

	using CreateFactoryFunc = decltype(PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreatePaletteTabFactory));
		
	struct TabInfo
	{
		FName ID;
		CreateFactoryFunc CreateFactory;
		TSharedPtr<FWorkspaceItem> Group;
	};

	TSharedRef<FWorkspaceItem> PropertyDetailsGroup = GetWorkspaceMenuCategory()->AddGroup(LOCTEXT("WorkspaceMenu_PCGEditor_Details", "Details"));
	TSharedRef<FWorkspaceItem> AttributesGroup = GetWorkspaceMenuCategory()->AddGroup(LOCTEXT("WorkspaceMenu_PCGEditor_Attributes", "Attributes"));
	TSharedRef<FWorkspaceItem> ViewportGroup = GetWorkspaceMenuCategory()->AddGroup(LOCTEXT("WorkspaceMenu_PCGEditor_Viewport", "Data Viewport"));

	TabInfo DefaultTabInfo[] =
	{
		{PCGEditorTabs::PropertyDetailsID[0], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreatePropertyDetailsTabFactory, 0, PropertyDetailsGroup)},
		{PCGEditorTabs::PropertyDetailsID[1], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreatePropertyDetailsTabFactory, 1, PropertyDetailsGroup)},
		{PCGEditorTabs::PropertyDetailsID[2], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreatePropertyDetailsTabFactory, 2, PropertyDetailsGroup)},
		{PCGEditorTabs::PropertyDetailsID[3], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreatePropertyDetailsTabFactory, 3, PropertyDetailsGroup)},
		{PCGEditorTabs::PaletteID, PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreatePaletteTabFactory)},
		{PCGEditorTabs::DebugObjectID, PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateDebugObjectTreeTabFactory)},
		{PCGEditorTabs::AttributesID[0], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateAttributesTabFactory, 0, AttributesGroup)},
		{PCGEditorTabs::AttributesID[1], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateAttributesTabFactory, 1, AttributesGroup)},
		{PCGEditorTabs::AttributesID[2], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateAttributesTabFactory, 2, AttributesGroup)},
		{PCGEditorTabs::AttributesID[3], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateAttributesTabFactory, 3, AttributesGroup)},
		{PCGEditorTabs::FindID, PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateFindTabFactory)},
		{PCGEditorTabs::DeterminismID, PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateDeterminismTabFactory)},
		{PCGEditorTabs::ProfilingID, PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateProfilingTabFactory)},
		{PCGEditorTabs::LogID, PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateLogTabFactory)},
		{PCGEditorTabs::CodeEditorID, PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateCodeEditorTabFactory)},
		{PCGEditorTabs::UserParamsID, PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateGraphParamsTabFactory)},
		{PCGEditorTabs::ViewportID[0], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateViewportTabFactory, 0, ViewportGroup)},
		{PCGEditorTabs::ViewportID[1], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateViewportTabFactory, 1, ViewportGroup)},
		{PCGEditorTabs::ViewportID[2], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateViewportTabFactory, 2, ViewportGroup)},
		{PCGEditorTabs::ViewportID[3], PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateViewportTabFactory, 3, ViewportGroup)},
		{PCGEditorTabs::EmbeddedSubgraphsID, PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateEmbeddedSubgraphsTabFactory)},
		{PCGEditorTabs::DataOverridesID, PCGEditorDefaultMode::FCreateTabFactoryDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::CreateDataOverridesTabFactory)}
	};

	for (TabInfo& Info : DefaultTabInfo)
	{
		if (!InPCGEditor->IsPanelAvailable(Info.ID))
		{
			continue;
		}

		TabFactories.RegisterFactory(Info.CreateFactory.Execute());
	}

	InPCGEditor->RegisterExtraTabFactories(ExtraTabFactories);

	ToolbarExtender = MakeShared<FExtender>();
	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		InPCGEditor->GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(InPCGEditor.Get(), &FPCGEditor::RegisterToolbar)
	);
}

FName FPCGEditorDefaultMode::StaticName()
{
	return FName("PCGEditorDefaultMode");
}

void FPCGEditorDefaultMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	if (TSharedPtr<FPCGEditor> PCGEditor = StaticCastSharedPtr<FPCGEditor>(GetHost()))
	{
		PCGEditor->PushTabFactories(TabFactories);
		PCGEditor->PushTabFactories(ExtraTabFactories);
	}
}

void FPCGEditorDefaultMode::PreDeactivateMode()
{
	if (TSharedPtr<FPCGEditor> PCGEditor = StaticCastSharedPtr<FPCGEditor>(GetHost()))
	{
		PCGEditor->SaveEditedObjectState();
	}

	FApplicationMode::PreDeactivateMode();
}

void FPCGEditorDefaultMode::PostActivateMode()
{
	FApplicationMode::PostActivateMode();

	if (TSharedPtr<FPCGEditor> PCGEditor = StaticCastSharedPtr<FPCGEditor>(GetHost()))
	{
		PCGEditor->RestoreEditedObjectState();
	}
}

#undef LOCTEXT_NAMESPACE