// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Docking/TabManager.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FPCGEditor;

namespace PCGEditorDefaultMode
{
	TSharedPtr<FTabManager::FLayout> GetTabLayout();
}

class FPCGEditorDefaultMode : public FApplicationMode
{
public:
	static FName StaticName();

	FPCGEditorDefaultMode(TSharedPtr<FPCGEditor> InPCGEditor);

	/** FApplicationMode interface */
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	virtual void PreDeactivateMode() override;
	virtual void PostActivateMode() override;

private:
	FWorkflowAllowedTabSet TabFactories;
	FWorkflowAllowedTabSet ExtraTabFactories;
};