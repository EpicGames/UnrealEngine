// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorGraph.h"

#include "PCGEdge.h"
#include "PCGEditorModule.h"
#include "PCGGraph.h"
#include "PCGPin.h"
#include "Elements/PCGReroute.h"
#include "Elements/PCGUserParameterGet.h"
#include "Nodes/PCGEditorGraphNode.h"
#include "Nodes/PCGEditorGraphNodeComment.h"
#include "Nodes/PCGEditorGraphNodeGetUserParameter.h"
#include "Nodes/PCGEditorGraphNodeInput.h"
#include "Nodes/PCGEditorGraphNodeOutput.h"
#include "Nodes/PCGEditorGraphNodeReroute.h"
#include "Schema/PCGEditorGraphSchemaActions.h"

#include "EdGraph/EdGraphPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorGraph)

namespace PCGEditorGraphUtils
{
	void GetInspectablePin(const UPCGNode* InNode, const UPCGPin* InPin, const UPCGNode*& OutNode, const UPCGPin*& OutPin)
	{
		OutNode = InNode;
		OutPin = InPin;

		// Basically, this is needed so we can go up the graph when the selected node/pin combo is on a reroute node.
		while (OutPin && OutPin->IsOutputPin() &&
			OutNode && OutNode->GetSettings() && OutNode->GetSettings()->IsA<UPCGRerouteSettings>())
		{
			// Since it's a reroute node, we can look at the inbound edge (if any) on the reroute node and go up there
			check(OutNode->GetInputPin(PCGPinConstants::DefaultInputLabel));
			const TArray<TObjectPtr<UPCGEdge>>& Edges = OutNode->GetInputPin(PCGPinConstants::DefaultInputLabel)->Edges;
			// A reroute node can have at most one inbound edge, but we still need to make sure it exists
			if (Edges.Num() == 1)
			{
				OutPin = Edges[0]->InputPin;
				OutNode = OutPin->Node;
			}
			else
			{
				break;
			}
		}
	}

	UPCGPin* GetPCGPinFromEdGraphPin(const UEdGraphPin* Pin)
	{
		UEdGraphNode* GraphNode = Pin ? Pin->GetOwningNodeUnchecked() : nullptr;
		UPCGEditorGraphNodeBase* PCGGraphNode = GraphNode ? CastChecked<UPCGEditorGraphNodeBase>(GraphNode, ECastCheckedType::NullAllowed) : nullptr;
		UPCGNode* PCGNode = PCGGraphNode ? PCGGraphNode->GetPCGNode() : nullptr;
		return PCGNode ? (Pin->Direction == EGPD_Input ? PCGNode->GetInputPin(Pin->PinName) : PCGNode->GetOutputPin(Pin->PinName)) : nullptr;
	}
}

void UPCGEditorGraph::InitFromNodeGraph(UPCGGraph* InPCGGraph)
{
	check(InPCGGraph && WeakPCGGraph.IsExplicitlyNull());
	WeakPCGGraph = InPCGGraph;

	InPCGGraph->OnGraphParametersChangedDelegate.AddUObject(this, &UPCGEditorGraph::OnGraphUserParametersChanged);
	InPCGGraph->OnGraphChangedDelegate.AddUObject(this, &UPCGEditorGraph::OnGraphChanged);

	ReconstructGraph();
}

void UPCGEditorGraph::ReconstructGraph()
{
	UPCGGraph* PCGGraph = GetPCGGraph();
	check(PCGGraph);

	// If there are already some nodes, remove all of them.
	if (!Nodes.IsEmpty())
	{
		Modify();

		TArray<TObjectPtr<class UEdGraphNode>> NodesCopy = Nodes;
		for (UEdGraphNode* Node : NodesCopy)
		{
			RemoveNode(Node);
		}
	}

	TMap<UPCGNode*, UPCGEditorGraphNodeBase*> NodeLookup;
	constexpr bool bSelectNewNode = false;

	auto ConstructNode = [this, &NodeLookup]<typename NodeType>(UPCGNode* PCGNode, TSubclassOf<NodeType> NodeClass = NodeType::StaticClass())
	{
		check(PCGNode && NodeClass);
		if (!PCGNode->IsHidden())
		{
			FGraphNodeCreator<NodeType> NodeCreator(*this);
			NodeType* GraphNode = NodeCreator.CreateNode(bSelectNewNode, NodeClass);
			GraphNode->Construct(PCGNode);
			NodeCreator.Finalize();

			// Need to reset it there because Finalize will override the guid.
			GraphNode->InitNodeGuid();
			NodeLookup.Add(PCGNode, GraphNode);
		}
	};

	// Create input and output nodes directly.
	ConstructNode.operator()<UPCGEditorGraphNodeInput>(PCGGraph->GetInputNode());
	ConstructNode.operator()<UPCGEditorGraphNodeOutput>(PCGGraph->GetOutputNode());

	for (UPCGNode* PCGNode : PCGGraph->GetNodes())
	{
		if (!IsValid(PCGNode) || PCGNode->IsHidden())
		{
			continue;
		}

		// Create other nodes based on settings.
		const UPCGSettings* PCGSettings = PCGNode->GetSettings();
		const TSubclassOf<UPCGEditorGraphNodeBase> PCGGraphNodeClass =
			PCGSettings
				? GetGraphNodeClassFromPCGSettings(PCGSettings)
				: TSubclassOf<UPCGEditorGraphNodeBase>(UPCGEditorGraphNode::StaticClass());

		ConstructNode(PCGNode, PCGGraphNodeClass);
	}

	for (const auto& NodeLookupIt : NodeLookup)
	{
		UPCGEditorGraphNodeBase* GraphNode = NodeLookupIt.Value;
		CreateLinks(GraphNode, /*bCreateInbound=*/false, /*bCreateOutbound=*/true, NodeLookup);
	}

	for (const UObject* ExtraNode : PCGGraph->GetExtraEditorNodes())
	{
		if (const UEdGraphNode* ExtraGraphNode = Cast<UEdGraphNode>(ExtraNode))
		{
			UEdGraphNode* NewNode = DuplicateObject(ExtraGraphNode, /*Outer=*/this);
			AddNode(NewNode, /*bIsUserAction=*/false, bSelectNewNode);
		}
	}
	
	for (const FPCGGraphCommentNodeData& CommentData : PCGGraph->GetCommentNodes())
	{
		UPCGEditorGraphNodeComment* NewNode = NewObject<UPCGEditorGraphNodeComment>(this, NAME_None, RF_Transactional);
		NewNode->InitializeFromNodeData(CommentData);
		AddNode(NewNode, /*bIsUserAction=*/false, bSelectNewNode);
	}

	// Ensure graph structure visualization is nice and fresh upon opening.
	UpdateVisualizations(nullptr, nullptr);
}

void UPCGEditorGraph::BeginDestroy()
{
	Super::BeginDestroy();

	SaveState();

	if (UPCGGraph* PCGGraph = GetPCGGraph())
	{
		PCGGraph->OnGraphChangedDelegate.RemoveAll(this);
		PCGGraph->OnGraphParametersChangedDelegate.RemoveAll(this);
	}
}

void UPCGEditorGraph::SaveState()
{
	ReplicateExtraNodes();
}

void UPCGEditorGraph::CreateLinks(UPCGEditorGraphNodeBase* GraphNode, bool bCreateInbound, bool bCreateOutbound)
{
	check(GraphNode);

	// Build pcg node to pcg editor graph node map
	TMap<UPCGNode*, UPCGEditorGraphNodeBase*> PCGNodeToPCGEditorNodeMap;
	for (const TObjectPtr<UEdGraphNode>& EdGraphNode : Nodes)
	{
		if (UPCGEditorGraphNodeBase* SomeGraphNode = Cast<UPCGEditorGraphNodeBase>(EdGraphNode))
		{
			PCGNodeToPCGEditorNodeMap.Add(SomeGraphNode->GetPCGNode(), SomeGraphNode);
		}
	}

	// Forward the call
	CreateLinks(GraphNode, bCreateInbound, bCreateOutbound, PCGNodeToPCGEditorNodeMap);
}

void UPCGEditorGraph::ReplicateExtraNodes() const
{
	if (UPCGGraph* PCGGraph = GetPCGGraph())
	{
		TArray<TObjectPtr<const UObject>> ExtraNodes;
		TArray<FPCGGraphCommentNodeData> CommentData;
		for (const UEdGraphNode* GraphNode : Nodes)
		{
			check(GraphNode);
			if (const UEdGraphNode_Comment* CommentNode = Cast<const UEdGraphNode_Comment>(GraphNode))
			{
				CommentData.Emplace_GetRef().InitializeFromCommentNode(*CommentNode);
			}
			else if (!GraphNode->IsA<UPCGEditorGraphNodeBase>())
			{
				ExtraNodes.Add(GraphNode);
			}

		}

		PCGGraph->SetExtraEditorNodes(ExtraNodes);
		PCGGraph->SetCommentNodes(std::move(CommentData));
	}
}

void UPCGEditorGraph::UpdateVisualizations(IPCGGraphExecutionSource* PCGSourceBeingInspected, const FPCGStack* PCGStackBeingInspected)
{
	for (UEdGraphNode* EditorNode : Nodes)
	{
		if (UPCGEditorGraphNodeBase* PCGEditorNode = Cast<UPCGEditorGraphNodeBase>(EditorNode))
		{
			EPCGChangeType ChangeType = EPCGChangeType::None;
			ChangeType |= PCGEditorNode->UpdateStructuralVisualization(PCGSourceBeingInspected, PCGStackBeingInspected);
			ChangeType |= PCGEditorNode->UpdateGPUVisualization(PCGSourceBeingInspected, PCGStackBeingInspected);
			ChangeType |= PCGEditorNode->UpdateOverrideVisualization(PCGSourceBeingInspected, PCGStackBeingInspected);

			if (ChangeType != EPCGChangeType::None)
			{
				PCGEditorNode->ReconstructNode();
			}
		}
	}
}

FText UPCGEditorGraph::GetDisplayName() const
{
	FGraphDisplayInfo DisplayInfo;
	GetSchema()->GetGraphDisplayInformation(*this, DisplayInfo);

	return DisplayInfo.DisplayName;
}

const UPCGEditorGraphNodeBase* UPCGEditorGraph::GetEditorNodeFromPCGNode(const UPCGNode* InPCGNode) const
{
	if (ensure(InPCGNode))
	{
		for (const UEdGraphNode* EdGraphNode : Nodes)
		{
			if (const UPCGEditorGraphNodeBase* PCGEdGraphNode = Cast<UPCGEditorGraphNodeBase>(EdGraphNode))
			{
				if (PCGEdGraphNode->GetPCGNode() == InPCGNode)
				{
					return PCGEdGraphNode;
				}
			}
		}
	}

	return nullptr;
}

UPCGEditorGraphNodeBase* UPCGEditorGraph::GetEditorNodeFromPCGNode(const UPCGNode* InPCGNode)
{
	return const_cast<UPCGEditorGraphNodeBase*>(static_cast<const UPCGEditorGraph*>(this)->GetEditorNodeFromPCGNode(InPCGNode));
}

TSubclassOf<UPCGEditorGraphNodeBase> UPCGEditorGraph::GetGraphNodeClassFromPCGSettings(const UPCGSettings* Settings)
{
	if (Settings->IsA<UPCGNamedRerouteDeclarationSettings>())
	{
		return UPCGEditorGraphNodeNamedRerouteDeclaration::StaticClass();
	}
	else if (Settings->IsA<UPCGNamedRerouteUsageSettings>())
	{
		return UPCGEditorGraphNodeNamedRerouteUsage::StaticClass();
	}
	else if (Settings->IsA<UPCGRerouteSettings>())
	{
		return UPCGEditorGraphNodeReroute::StaticClass();
	}
	else if (Settings->IsA<UPCGUserParameterGetSettings>())
	{
		return UPCGEditorGraphGetUserParameter::StaticClass();
	}
	else // All other settings.
	{
		return UPCGEditorGraphNode::StaticClass();
	}
}

void UPCGEditorGraph::CreateLinks(UPCGEditorGraphNodeBase* GraphNode, bool bCreateInbound, bool bCreateOutbound, const TMap<UPCGNode*, UPCGEditorGraphNodeBase*>& InPCGNodeToPCGEditorNodeMap)
{
	check(GraphNode);
	const UPCGNode* PCGNode = GraphNode->GetPCGNode();
	check(PCGNode);

	if (bCreateInbound)
	{
		for (UPCGPin* InputPin : PCGNode->GetInputPins())
		{
			if (!InputPin || InputPin->Properties.bInvisiblePin)
			{
				continue;
			}

			UEdGraphPin* InPin = GraphNode->FindPin(InputPin->Properties.Label, EEdGraphPinDirection::EGPD_Input);
			if (!InPin)
			{
				UE_LOGF(LogPCGEditor, Error, "Invalid InputPin for %ls", *InputPin->Properties.Label.ToString());
				ensure(false);
				continue;
			}

			for (const UPCGEdge* InboundEdge : InputPin->Edges)
			{
				if (!InboundEdge || !InboundEdge->IsValid())
				{
					UE_LOGF(LogPCGEditor, Error, "Invalid inbound edge for %ls", *InputPin->Properties.Label.ToString());
					ensure(false);
					continue;
				}

				const UPCGNode* InboundNode = InboundEdge->InputPin ? InboundEdge->InputPin->Node : nullptr;
				if (!ensure(InboundNode))
				{
					UE_LOGF(LogPCGEditor, Error, "Invalid inbound node for %ls", *InputPin->Properties.Label.ToString());
					continue;
				}

				if (UPCGEditorGraphNodeBase* const* ConnectedGraphNode = InboundNode ? InPCGNodeToPCGEditorNodeMap.Find(InboundNode) : nullptr)
				{
					if (UEdGraphPin* OutPin = (*ConnectedGraphNode)->FindPin(InboundEdge->InputPin->Properties.Label, EGPD_Output))
					{
						OutPin->MakeLinkTo(InPin);
					}
					else
					{
						UE_LOGF(LogPCGEditor, Error, "Could not find inbound pin '%ls' to link to input pin '%ls' from node '%ls'",
							*InboundEdge->InputPin->Properties.Label.ToString(),
							*InputPin->Properties.Label.ToString(),
							*InboundNode->GetFName().ToString());
						ensure(false);
					}
				}
				else
				{
					/**
					 * Note: The ConnectedGraphNode may be in a state where it is not yet in the Nodes list. Ex. The
					 * graph has been reconstructed after a cancelled generation, but before the reconstruction that
					 * creates the UEdGraphNode.
					 * 
					 * All cases seem to be user driven actions that happen during generation, where a reconstruct is
					 * invoked as part of a cancelled generation during the user action, but before the user action's
					 * reconstruction--which then correctly links the edges.
					 * 
					 * @todo_pcg: As part of an overall effort to clean up the graph construction process, investigate
					 * this with better Editor tracing.
					 */
				}
			}
		}
	}

	if (bCreateOutbound)
	{
		for (UPCGPin* OutputPin : PCGNode->GetOutputPins())
		{
			if (!OutputPin || OutputPin->Properties.bInvisiblePin)
			{
				continue;
			}

			UEdGraphPin* OutPin = GraphNode->FindPin(OutputPin->Properties.Label, EEdGraphPinDirection::EGPD_Output);
			if (!OutPin)
			{
				UE_LOGF(LogPCGEditor, Error, "Invalid OutputPin for %ls", *OutputPin->Properties.Label.ToString());
				ensure(false);
				continue;
			}

			for (const UPCGEdge* OutboundEdge : OutputPin->Edges)
			{
				if (!OutboundEdge || !OutboundEdge->IsValid())
				{
					UE_LOGF(LogPCGEditor, Error, "Invalid outbound edge for %ls", *OutputPin->Properties.Label.ToString());
					ensure(false);
					continue;
				}

				const UPCGNode* OutboundNode = OutboundEdge->OutputPin ? OutboundEdge->OutputPin->Node : nullptr;
				if (!ensure(OutboundNode))
				{
					UE_LOGF(LogPCGEditor, Error, "Invalid outbound node for %ls", *OutputPin->Properties.Label.ToString());
					continue;
				}

				if (UPCGEditorGraphNodeBase* const* ConnectedGraphNode = OutboundNode ? InPCGNodeToPCGEditorNodeMap.Find(OutboundNode) : nullptr)
				{
					if (UEdGraphPin* InPin = ConnectedGraphNode ? (*ConnectedGraphNode)->FindPin(OutboundEdge->OutputPin->Properties.Label, EGPD_Input) : nullptr)
					{
						OutPin->MakeLinkTo(InPin);
					}
					else
					{
						UE_LOGF(LogPCGEditor, Error, "Could not find outbound pin '%ls' to link to input pin '%ls' on node '%ls'",
							*OutboundEdge->OutputPin->Properties.Label.ToString(),
							*OutputPin->Properties.Label.ToString(),
							*OutboundNode->GetFName().ToString());
						ensure(false);
					}
				}
				else
				{
					// See above comment in Inbound version
				}
			}
		}
	}
}

void UPCGEditorGraph::OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType)
{
	constexpr EPCGChangeType IgnoredChangeTypes = EPCGChangeType::Cosmetic | EPCGChangeType::Input | EPCGChangeType::ShaderSource | EPCGChangeType::GraphCustomization;
	if (!!(ChangeType & ~IgnoredChangeTypes))
	{
		CachedEdgeUsedByExecution.Reset();

		for (UEdGraphNode* EdGraphNode : Nodes)
		{
			if (UPCGEditorGraphNodeBase* PCGEditorNode = Cast<UPCGEditorGraphNodeBase>(EdGraphNode))
			{
				PCGEditorNode->ClearPinColorCache();
			}
		}
	}

	if (!!(ChangeType & EPCGChangeType::ExternalModification))
	{
		//Changes caused by external tools: Editor graph should be reconstructed.
		ReconstructGraph();
	}
}

void UPCGEditorGraph::OnGraphUserParametersChanged(UPCGGraphInterface* InGraph, EPCGGraphParameterEvent ChangeType, FName ChangedPropertyName)
{
	UPCGGraph* PCGGraph = GetPCGGraph();
	if (InGraph != PCGGraph)
	{
		return;
	}

	// Those change types can result in a parameter node being removed from the graph. (same condition found in UPCGGraph::OnGraphParametersChanged)
	const bool bCheckForRemovedNodes = ChangeType == EPCGGraphParameterEvent::RemovedUnused
		|| ChangeType == EPCGGraphParameterEvent::RemovedUsed
		|| ChangeType == EPCGGraphParameterEvent::PropertyRenamed
		|| ChangeType == EPCGGraphParameterEvent::PropertyTypeModified;
	
	if (!bCheckForRemovedNodes)
	{
		return;
	}

	// If a parameter was removed, just look for getter nodes that do exists in the editor graph, but not in the PCG graph.
	TArray<UPCGEditorGraphNodeBase*> NodesToRemove;
	for (UEdGraphNode* EditorNode : Nodes)
	{
		if (UPCGEditorGraphNodeBase* PCGEditorNode = Cast<UPCGEditorGraphNodeBase>(EditorNode))
		{
			if (UPCGNode* PCGNode = PCGEditorNode->GetPCGNode())
			{
				if (UPCGUserParameterGetSettings* Settings = Cast<UPCGUserParameterGetSettings>(PCGNode->GetSettings()))
				{
					if (!PCGGraph->Contains(PCGNode))
					{
						NodesToRemove.Add(PCGEditorNode);
					}
				}
			}
		}
	}

	if (NodesToRemove.IsEmpty())
	{
		return;
	}

	Modify();

	for (UPCGEditorGraphNodeBase* NodeToRemove : NodesToRemove)
	{
		NodeToRemove->DestroyNode();
	}
}

bool UPCGEditorGraph::CanReceivePropertyBagDetailsDropOnGraphPin(const UEdGraphPin* Pin) const
{
	const UPCGPin* PCGPin = PCGEditorGraphUtils::GetPCGPinFromEdGraphPin(Pin);
	return PCGPin
		&& !PCGPin->IsOutputPin()
		&& (PCGPin->EdgeCount() == 0 || PCGPin->AllowsMultipleConnections())
		&& PCGPin->IsDownstreamPinTypeCompatible(EPCGDataType::Param);
}

bool UPCGEditorGraph::CanReceivePropertyBagDetailsDropOnGraphNode(const UEdGraphNode* Node) const
{
	// Currently no useful way to interpret dropping a user parameter on a node.
	return false;
}

bool UPCGEditorGraph::CanReceivePropertyBagDetailsDropOnGraph(const UEdGraph* Graph) const
{
	// Anywhere on the graph panel should be fine for creating a new get user parameter node.
	return true;
}

FReply UPCGEditorGraph::OnPropertyBagDetailsDropOnGraphPin(const FPropertyBagPropertyDesc& PropertyDesc, UEdGraphPin* Pin, const FVector2f& GraphPosition) const
{
	const UEdGraphNode* Node = Pin ? Pin->GetOwningNode() : nullptr;
	if (Node && Node->GetGraph() && PropertyDesc.ID.IsValid() && PropertyDesc.CachedProperty)
	{
		FPCGEditorGraphSchemaAction_NewGetParameterElement Action;
		Action.SettingsClass = UPCGUserParameterGetSettings::StaticClass();
		Action.PropertyDesc = PropertyDesc;
		Action.PerformAction(Node->GetGraph(), Pin, GraphPosition, /*bSelectNewNode=*/true);
	}

	return FReply::Handled();
}

FReply UPCGEditorGraph::OnPropertyBagDetailsDropOnGraph(const FPropertyBagPropertyDesc& PropertyDesc, UEdGraph* Graph, const FVector2f& GraphPosition) const
{
	if (Graph && PropertyDesc.ID.IsValid() && PropertyDesc.CachedProperty)
	{
		FPCGEditorGraphSchemaAction_NewGetParameterElement Action;
		Action.SettingsClass = UPCGUserParameterGetSettings::StaticClass();
		Action.PropertyDesc = PropertyDesc;
		Action.PerformAction(Graph, nullptr, GraphPosition, /*bSelectNewNode=*/true);
	}

	return FReply::Handled();
}

bool UPCGEditorGraph::IsEdgeUsedByExecution(const UPCGEdge* InEdge) const
{
	if (const bool* Cached = CachedEdgeUsedByExecution.Find(InEdge))
	{
		return *Cached;
	}

	// IsUsedByExecution dictated by downstream input pin.
	const UPCGNode* Node = InEdge && InEdge->OutputPin ? InEdge->OutputPin->Node : nullptr;
	const bool bUsed = !Node || Node->IsEdgeUsedByNodeExecution(InEdge);
	CachedEdgeUsedByExecution.Add(InEdge, bUsed);

	return bUsed;
}
