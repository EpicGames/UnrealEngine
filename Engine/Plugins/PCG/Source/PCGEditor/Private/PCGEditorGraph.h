// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyBagEditorGraph.h"
#include "EdGraph/EdGraph.h"

#include "PCGGraph.h"

#include "UObject/ObjectKey.h"

#include "PCGEditorGraph.generated.h"

class FPCGEditor;
class UPCGEdge;
class UPCGNode;
class UPCGPin;
class UPCGEditorGraphNodeBase;

namespace PCGEditorGraphUtils
{
	/** Goes up the graph to the first non-reroute node it can, assuming it's properly connected. */
	void GetInspectablePin(const UPCGNode* InNode, const UPCGPin* InPin, const UPCGNode*& OutNode, const UPCGPin*& OutPin);
}

UCLASS()
class UPCGEditorGraph : public UEdGraph, public IPropertyBagEdGraph
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	virtual void BeginDestroy() override;
	// ~End UObject interface

	/** Initialize the editor graph from a PCGGraph */
	void InitFromNodeGraph(UPCGGraph* InPCGGraph);

	/** If the underlying graph changed without UI interaction, use this function to reconstruct the UI elements. */
	void ReconstructGraph();

	/** When the editor is closing */
	void SaveState();

	/** Creates the links for a given node */
	void CreateLinks(UPCGEditorGraphNodeBase* InGraphNode, bool bCreateInbound, bool bCreateOutbound);

	/** To be called everytime we need to replicate our extra nodes to the underlying PCGGraph */
	void ReplicateExtraNodes() const;

	UPCGGraph* GetPCGGraph() const { return WeakPCGGraph.Get(); }

	void SetEditor(TWeakPtr<FPCGEditor> InEditor) { PCGEditor = InEditor; }
	TWeakPtr<FPCGEditor> GetEditor() const { return PCGEditor; }

	FText GetDisplayName() const;

	/** Updates the grid size visualization in the editor. */
	void UpdateVisualizations(IPCGGraphExecutionSource* PCGSourceBeingInspected, const FPCGStack* PCGStackBeingInspected);

	/** Returns the PCG editor graph node corresponding to the given PCG node. */
	const UPCGEditorGraphNodeBase* GetEditorNodeFromPCGNode(const UPCGNode* InPCGNode) const;
	UPCGEditorGraphNodeBase* GetEditorNodeFromPCGNode(const UPCGNode* InPCGNode);

	/** Returns whether the edge is used by the downstream node's execution. Cached per graph change. */
	bool IsEdgeUsedByExecution(const UPCGEdge* InEdge) const;

	/** Returns the PCG Editor Graph Node that should be spawned from any given PCG Settings. */
	static TSubclassOf<UPCGEditorGraphNodeBase> GetGraphNodeClassFromPCGSettings(const UPCGSettings* Settings);

protected:
	void CreateLinks(UPCGEditorGraphNodeBase* InGraphNode, bool bCreateInbound, bool bCreateOutbound, const TMap<UPCGNode*, UPCGEditorGraphNodeBase*>& InPCGNodeToPCGEditorNodeMap);

	void OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType);
	void OnGraphUserParametersChanged(UPCGGraphInterface* InGraph, EPCGGraphParameterEvent ChangeType, FName ChangedPropertyName);

	// ~Begin IPropertyBagEdGraph interface
	virtual bool CanReceivePropertyBagDetailsDropOnGraphPin(const UEdGraphPin* Pin) const override;
	virtual bool CanReceivePropertyBagDetailsDropOnGraphNode(const UEdGraphNode* Node) const override;
	virtual bool CanReceivePropertyBagDetailsDropOnGraph(const UEdGraph* Graph) const override;
	virtual FReply OnPropertyBagDetailsDropOnGraphPin(const FPropertyBagPropertyDesc& PropertyDesc, UEdGraphPin* Pin, const FVector2f& GraphPosition) const override;
	virtual FReply OnPropertyBagDetailsDropOnGraph(const FPropertyBagPropertyDesc& PropertyDesc, UEdGraph* Graph, const FVector2f& GraphPosition) const override;
	// ~End IPropertyBagEdGraph interface

private:
	UPROPERTY()
	TWeakObjectPtr<UPCGGraph> WeakPCGGraph;

	TWeakPtr<FPCGEditor> PCGEditor = nullptr;

	mutable TMap<TObjectKey<UPCGEdge>, bool> CachedEdgeUsedByExecution;
};
