// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "PCGEditorMenuContext.generated.h"

class FPCGEditor;

UCLASS()
class UPCGEditorMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<FPCGEditor> PCGEditor;
};
