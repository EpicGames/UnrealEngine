// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorMenuUtils.h"

#include "PCGAssetExporterUtils.h"
#include "PCGEditorStyle.h"
#include "PCGLevelToAsset.h"

#include "Engine/World.h"
#include "Misc/ScopedSlowTask.h"

#define LOCTEXT_NAMESPACE "PCGEditorMenuUtils"

namespace PCGEditorMenuUtils
{
	FToolMenuSection& CreatePCGSection(UToolMenu* Menu)
	{
		const FName LevelSectionName = TEXT("PCG");
		FToolMenuSection* SectionPtr = Menu->FindSection(LevelSectionName);
		if (!SectionPtr)
		{
			SectionPtr = &(Menu->AddSection(LevelSectionName, LOCTEXT("PCGSectionLabel", "Procedural (PCG)")));
		}

		FToolMenuSection& Section = *SectionPtr;
		return Section;
	}

	void CreateOrUpdatePCGAssetFromMenu(UToolMenu* Menu, TArray<FAssetData>& InAssets)
	{
		TArray<FAssetData> TempWorldAssets;
		TArray<FAssetData> TempPCGAssets;

		for (const FAssetData& Asset : InAssets)
		{
			if (Asset.IsInstanceOf<UWorld>())
			{
				TempWorldAssets.Add(Asset);
			}

			if (Asset.IsInstanceOf<UPCGDataAsset>())
			{
				TempPCGAssets.Add(Asset);
			}
		}

		if (TempWorldAssets.IsEmpty() && TempPCGAssets.IsEmpty())
		{
			return;
		}

		FToolMenuSection& Section = CreatePCGSection(Menu);

		if (!TempWorldAssets.IsEmpty())
		{
			FToolUIAction UIAction;
			UIAction.ExecuteAction.BindLambda([WorldAssets = MoveTemp(TempWorldAssets)](const FToolMenuContext& MenuContext)
			{
				FScopedSlowTask SlowTask(0.0f, LOCTEXT("CreateOrUpdatePCGAssetsInProgress", "Creating PCG Assets from Level(s)..."));
				UPCGLevelToAsset::CreateOrUpdatePCGAssets(WorldAssets);
			});

			Section.AddMenuEntry(
				"CreatePCGAssetFromMenu",
				LOCTEXT("CreatePCGAssetFromMenu", "Create PCG Assets from Level(s)"),
				TAttribute<FText>(),
				FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "ClassIcon.PCGDataAsset"),
				UIAction);
		}

		if (!TempPCGAssets.IsEmpty())
		{
			FToolUIAction UIAction;
			UIAction.ExecuteAction.BindLambda([PCGAssets = MoveTemp(TempPCGAssets)](const FToolMenuContext& MenuContext)
			{
				FScopedSlowTask SlowTask(0.0f, LOCTEXT("UpdatePCGAssetsInProgress", "Updating PCG Assets..."));
				UPCGAssetExporterUtils::UpdateAssets(PCGAssets);
			});

			Section.AddMenuEntry(
				"UpdatePCGAssetFromMenu",
				LOCTEXT("UpdatePCGAssetFromMenu", "Update PCG Assets"),
				TAttribute<FText>(),
				FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "ClassIcon.PCGDataAsset"),
				UIAction);
		}

	}
}

#undef LOCTEXT_NAMESPACE