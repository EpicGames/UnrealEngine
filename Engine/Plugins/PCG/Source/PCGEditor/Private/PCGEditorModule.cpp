// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorModule.h"

#include "PCGComponent.h"
#include "PCGComponentVisualizer.h"
#include "PCGEngineSettings.h"
#include "PCGGraph.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "PCGParamData.h"
#include "PCGSettings.h"
#include "PCGSubsystem.h"
#include "PCGVolumeFactory.h"
#include "PCGWorldActor.h"
#include "Data/PCGCollisionShapeData.h"
#include "Data/PCGCollisionWrapperData.h"
#include "Data/PCGDynamicMeshData.h"
#include "Data/PCGLandscapeData.h"
#include "Data/PCGPolygon2DData.h"
#include "Data/PCGPolygon2DInteriorData.h"
#include "Data/PCGPrimitiveData.h"
#include "Data/PCGRenderTargetData.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGStaticMeshResourceData.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Data/PCGTextureData.h"
#include "Data/PCGVirtualTextureData.h"
#include "Data/PCGVolumeData.h"
#include "Elements/IO/PCGLoadAssetElement.h"
#include "Grid/PCGPartitionActor.h"
#include "WorldPartitionBuilder/PCGWorldPartitionBuilder.h"

#include "PCGEditor.h"
#include "PCGEditorCommands.h"
#include "PCGEditorGraph.h"
#include "PCGEditorMenuUtils.h"
#include "PCGEditorProgressNotification.h"
#include "PCGEditorSettings.h"
#include "PCGEditorStyle.h"
#include "PCGEditorUtils.h"
#include "AssetEditorMode/PCGAssetEditorToolRegistry.h"
#include "AssetEditorMode/Tools/PCGAssetEditorInteractiveTool.h"
#include "ChangeTracking/PCGActorTracker.h"
#include "ChangeTracking/PCGComponentChangeHandler.h"
#include "ChangeTracking/PCGLandscapeTracker.h"
#include "ChangeTracking/PCGMiscTracker.h"
#include "ChangeTracking/PCGRuntimeGenChangeHandler.h"
#include "Compute/Data/PCGRawBufferData.h"
#include "Data/DataView/PCGDataViewData.h"
#include "DataVisualizations/PCGBaseTextureDataVisualization.h"
#include "DataVisualizations/PCGDataViewVisualization.h"
#include "DataVisualizations/PCGLandscapeDataVisualization.h"
#include "DataVisualizations/PCGParamDataVisualization.h"
#include "DataVisualizations/PCGRawBufferDataVisualization.h"
#include "DataVisualizations/PCGPolygon2DDataVisualization.h"
#include "DataVisualizations/PCGSpatialDataVisualization.h"
#include "DataVisualizations/PCGSplineDataVisualization.h"
#include "DataVisualizations/PCGStaticMeshDataVisualization.h"
#include "DataVisualizations/PCGTexture2DArrayDataVisualization.h"
#include "DataVisualizations/PCGVolumeDataVisualization.h"
#include "DeltaViewportExtensions/PCGPointDeletionViewportExtension.h"
#include "DeltaViewportExtensions/PCGPointInsertionViewportExtension.h"
#include "DeltaViewportExtensions/PCGPointTransformOffsetViewportExtension.h"
#include "DeltaViewportExtensions/PCGPointTransformViewportExtension.h"
#include "DeltaVisualizations/PCGDeltaBaseVisualization.h"
#include "DeltaVisualizations/PCGDeltaPointVisualization.h"
#include "Details/EnumSelectorDetails.h"
#include "Details/PCGAttributePropertySelectorDetails.h"
#include "Details/PCGBlueprintSettingsDetails.h"
#include "Details/PCGComponentDetails.h"
#include "Details/PCGComputeSourceDetails.h"
#include "Details/PCGCustomHLSLSettingsDetails.h"
#include "Details/PCGDataTypeIdentifierDetails.h"
#include "Details/PCGEditableUserParameterDetails.h"
#include "Details/PCGGraphDetails.h"
#include "Details/PCGGraphInstanceDetails.h"
#include "Details/PCGInstancedPropertyBagOverrideDetails.h"
#include "Details/PCGInteractiveToolSettingsDetails.h"
#include "Details/PCGRaycastFilterRuleCollectionDetails.h"
#include "Details/PCGVolumeDetails.h"
#include "EditorMode/PCGEdMode.h"
#include "EditorMode/Tools/PCGInteractiveToolSettings.h"
#include "Graph/DataOverride/PCGDataOverridePoints.h"
#include "Nodes/PCGEditorGraphNodeFactory.h"
#include "PCGManualEditPanelManager.h"
#include "Schema/PCGEditorGraphSchema.h"
#include "Schema/PCGGraphParametersSchema.h"

#include "ComponentVisualizers.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "EditorBuildUtils.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "IContentBrowserSingleton.h"
#include "ISettingsModule.h"
#include "LevelEditor.h"
#include "LevelEditorMenuContext.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "SEnumCombo.h"
#include "SLevelViewport.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor/SceneOutliner/Public/ISceneOutliner.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HierarchyEditor/PropertyBagHierarchyViewModel.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectHash.h"
#include "Widgets/SPCGCodeEditorTextBox.h"

#define LOCTEXT_NAMESPACE "FPCGEditorModule"

namespace PCGEditorGraphSwitches
{
	TAutoConsoleVariable<bool> CVarEnableColorsOnParamPins{
		TEXT("pcg.Editor.EnableColorsOnParamPins"),
		true,
		TEXT("Enable coloring of param pins based on their underlying types.")
	};
}

namespace PCGEditorModule
{
	static const FName PCGBuildType(TEXT("PCG"));
}

void FPCGEditorModule::StartupModule()
{
	RegisterDetailsCustomizations();
	RegisterSettings();
	RegisterPCGDataVisualizations();
	RegisterDeltaVisualizations();
	RegisterDeltaViewportExtensions();
	RegisterPinColorAndIcons();
	RegisterTrackerFactories();

	LoadDataAssetChangeTracker = MakeUnique<FPCGLoadDataAssetChangeTracker>();

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FPCGEditorModule::RegisterMenuExtensions));

	FPCGEditorStyle::Register();
	FPCGEditorCommands::Register();
	FPCGEditorSpawnNodeCommands::Register();

	FEditorDelegates::OnEditorInitialized.AddRaw(this, &FPCGEditorModule::OnEditorInitialized);

	FPCGCodeEditorTextBoxCommands::Register();

	UPCGEditorMode::RegisterEditorMode();

	GraphNodeFactory = MakeShareable(new FPCGEditorGraphNodeFactory());
	FEdGraphUtilities::RegisterVisualNodeFactory(GraphNodeFactory);

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		OnLevelEditorCreated(FirstLevelEditor);
	}
	else
	{
		LevelEditorModule.OnLevelEditorCreated().AddRaw(this, &FPCGEditorModule::OnLevelEditorCreated);
	}

	FEditorBuildUtils::RegisterCustomBuildType(PCGEditorModule::PCGBuildType,
		FCanDoEditorBuildDelegate::CreateStatic(&UPCGWorldPartitionBuilder::CanBuild),
		FDoEditorBuildDelegate::CreateStatic(&UPCGWorldPartitionBuilder::Build),
		/*BuildAllExtensionPoint*/NAME_None,
		/*MenuEntryLabel*/LOCTEXT("BuildPCG", "Build PCG"),
		/*MenuSectionLabel*/LOCTEXT("PCG", "PCG"),
		/*bExternalProcess*/true);

	IPCGEditorModule::SetEditorModule(this);

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FPCGEditorModule::OnPostEngineInit);

	// Register onto the PreExit, because we need the class/struct to be still valid to remove them from the mapping
	FCoreDelegates::OnPreExit.AddRaw(this, &FPCGEditorModule::OnPreExit);
}

void FPCGEditorModule::ShutdownModule()
{
	FCoreDelegates::OnPreExit.RemoveAll(this);

	IPCGEditorModule::SetEditorModule(nullptr);

	LoadDataAssetChangeTracker.Reset();

	ManualEditPanelManager.Reset();

	UnregisterTrackerFactories();
	UnregisterDeltaViewportExtensions();
	UnregisterDeltaVisualizations();
	UnregisterPCGDataVisualizations();
	UnregisterSettings();
	UnregisterDetailsCustomizations();
	UnregisterMenuExtensions();

	UPCGEditorMode::UnregisterEditorMode();

	FPCGCodeEditorTextBoxCommands::Unregister();
	FPCGEditorSpawnNodeCommands::Unregister();
	FPCGEditorCommands::Unregister();
	FPCGEditorStyle::Unregister();

	FPCGAssetEditorToolRegistry::TearDown();

	FEdGraphUtilities::UnregisterVisualNodeFactory(GraphNodeFactory);

	if (GEditor)
	{
		GEditor->ActorFactories.RemoveAll([](const UActorFactory* ActorFactory) { return ActorFactory->IsA<UPCGVolumeFactory>(); });

		GEditor->ShouldDisableCPUThrottlingDelegates.RemoveAll([this](const UEditorEngine::FShouldDisableCPUThrottling& Delegate)
		{
			return Delegate.GetHandle() == ShouldDisableCPUThrottlingDelegateHandle;
		});

		GEditor->OnSceneMaterialsModifiedEvent().RemoveAll(this);
	}

	if (GUnrealEd)
	{
		for (const FName& Name : VisualizersToUnregisterOnShutdown)
		{
			GUnrealEd->UnregisterComponentVisualizer(Name);
		}
		VisualizersToUnregisterOnShutdown.Empty();
	}

	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.OnLevelEditorCreated().RemoveAll(this);
	}

	FEditorBuildUtils::UnregisterCustomBuildType(PCGEditorModule::PCGBuildType);
	
	FEditorDelegates::OnEditorInitialized.RemoveAll(this);

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

void FPCGEditorModule::ExtendShowFlagsMenu()
{
	check(GEditor);

	UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ViewportToolbar.Show");
	if (!Menu)
	{
		return;
	}
	
	FToolMenuSection& AllShowFlagsSection = Menu->FindOrAddSection("AllShowFlags");

	AllShowFlagsSection.AddEntry(FToolMenuEntry::InitSubMenu(
		"ShowFlagsPCG",
		LOCTEXT("ShowFlagsPCGMenu", "PCG"),
		LOCTEXT("ShowFlagsPCGMenu_ToolTip", "PCG Specific show flags"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu)
			{
				TSharedPtr<::SLevelViewport> Viewport = ULevelViewportContext::GetLevelViewport(Submenu);
				if (!Viewport)
				{
					return;
				}

				if (!Viewport->GetWorld())
				{
					return;
				}

				{
					FToolMenuSection& Section = Submenu->AddSection("ShowFlagsRuntimePCG", LOCTEXT("ShowFlagsRuntimePCGLabel", "Runtime"));

					auto CreateTreatViewportAsGenerationSourceWidget = []()
					{
						return SNew(SEnumComboBox, StaticEnum<EPCGTreatViewportAsGenerationSource>())
								.CurrentValue_Static(&FPCGEditorModule::GetTreatEditorViewportAsGenerationSourceValue)
								.OnEnumSelectionChanged_Static(&FPCGEditorModule::OnTreatEditorViewportAsGenerationSourceValueChanged);
					};

					FToolMenuEntry TreatEditorViewportAsGenerationSourceMenuEntry = FToolMenuEntry::InitWidget(
						"TreatEditorViewportAsGenerationSource",
						CreateTreatViewportAsGenerationSourceWidget(),
						LOCTEXT("TreatEditorViewportAsGenerationSourceLabel", "Treat Editor Viewport as Generation Source")
					);
					Section.AddEntry(TreatEditorViewportAsGenerationSourceMenuEntry);
				}
			}
		),
		false,
		FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.EditorIcon")
	));
}

int32 FPCGEditorModule::GetTreatEditorViewportAsGenerationSourceValue()
{
	EPCGTreatViewportAsGenerationSource Value = EPCGTreatViewportAsGenerationSource::PCGWorldActor;
	if (const UPCGEditorSettings* EditorSettings = GetDefault<UPCGEditorSettings>())
	{
		Value = EditorSettings->TreatViewportAsGenerationSource;
	}
	return (int32)Value;
}

void FPCGEditorModule::OnTreatEditorViewportAsGenerationSourceValueChanged(int32 Value, ESelectInfo::Type SelectInfo)
{
	EPCGTreatViewportAsGenerationSource NewValue = static_cast<EPCGTreatViewportAsGenerationSource>(Value);
	if (UPCGEditorSettings* EditorSettings = GetMutableDefault<UPCGEditorSettings>())
	{
		EditorSettings->TreatViewportAsGenerationSource = NewValue;
		EditorSettings->SaveConfig();
	}
}

void FPCGEditorModule::OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor)
{
	if (GEditor)
	{
		// Factory should be auto-discovered by UEditorEngine::InitEditor
		check(GEditor->ActorFactories.FindItemByClass<UPCGVolumeFactory>());

		if (!IsRunningCommandlet())
		{
			GEditor->ShouldDisableCPUThrottlingDelegates.Add(UEditorEngine::FShouldDisableCPUThrottling::CreateRaw(this, &FPCGEditorModule::ShouldDisableCPUThrottling));
			ShouldDisableCPUThrottlingDelegateHandle = GEditor->ShouldDisableCPUThrottlingDelegates.Last().GetHandle();
		}

		GEditor->OnSceneMaterialsModifiedEvent().AddRaw(this, &FPCGEditorModule::OnSceneMaterialsModified);

		ExtendShowFlagsMenu();

		if (!IsRunningCommandlet())
		{
			ManualEditPanelManager = MakeUnique<FPCGManualEditPanelManager>();
		}
	}
}

bool FPCGEditorModule::ShouldDisableCPUThrottling()
{
	if (const UPCGEditorSettings* EditorSettings = GetDefault<UPCGEditorSettings>(); EditorSettings && EditorSettings->bDisableCPUThrottlingDuringGraphExecution)
	{
		if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
		{
			return PCGSubsystem->IsAnyGraphCurrentlyExecuting();
		}
	}

	return false;
}

bool FPCGEditorModule::ShouldTreatViewportAsGenerationSourceInternal(const APCGWorldActor* InPCGWorldActor) const
{
	EPCGTreatViewportAsGenerationSource Value = EPCGTreatViewportAsGenerationSource::PCGWorldActor;

	if (const UPCGEditorSettings* EditorSettings = GetDefault<UPCGEditorSettings>())
	{
		Value = EditorSettings->TreatViewportAsGenerationSource;
	}

	switch (Value)
	{
		case EPCGTreatViewportAsGenerationSource::PCGWorldActor:
			return InPCGWorldActor && InPCGWorldActor->bTreatEditorViewportAsGenerationSource;
		case EPCGTreatViewportAsGenerationSource::Always:
			return true;
		case EPCGTreatViewportAsGenerationSource::Never:
			return false;
		default:
			check(false);
	}

	return false;
}

void FPCGEditorModule::OnEditorInitialized(double Duration)
{
	TArray<UClass*> DerivedToolClasses;
	GetDerivedClasses(UPCGAssetEditorInteractiveTool::StaticClass(), DerivedToolClasses, /*bRecursive=*/true);

	FPCGAssetEditorToolRegistry& ToolRegistry = FPCGAssetEditorToolRegistry::Get();
	for (UClass* ToolClass : DerivedToolClasses)
	{
		if (ToolClass->HasAnyClassFlags(CLASS_Abstract))
		{
			continue;
		}
		const UPCGAssetEditorInteractiveTool* ToolCDO = ToolClass->GetDefaultObject<UPCGAssetEditorInteractiveTool>();
		for (UClass* SettingsClass : ToolCDO->GetSupportedSettingsClasses())
		{
			if (SettingsClass)
			{
				ToolRegistry.RegisterTool(SettingsClass, ToolClass);
			}
		}
	}
}

void FPCGEditorModule::OnPostEngineInit()
{
	FComponentVisualizersModule& VisualisersModule = FModuleManager::LoadModuleChecked<FComponentVisualizersModule>("ComponentVisualizers");
	VisualisersModule.RegisterComponentVisualizer(UPCGComponent::StaticClass()->GetFName(), MakeShared<FPCGComponentVisualizer>());
	VisualizersToUnregisterOnShutdown.Add(UPCGComponent::StaticClass()->GetFName());
}

void FPCGEditorModule::OnPreExit()
{
	UnregisterPinColorAndIcons();
}

void FPCGEditorModule::UpdateManualEditPanelVisibility()
{
	if (ManualEditPanelManager)
	{
		ManualEditPanelManager->UpdateManualEditPanelVisibility();
	}
}

TSharedPtr<SPCGManualEditPanel> FPCGEditorModule::GetManualEditPanel() const
{
	return ManualEditPanelManager ? ManualEditPanelManager->GetManualEditPanel() : nullptr;
}

void FPCGEditorModule::OnSceneMaterialsModified()
{
	// Currently, there is no explicit persistence of instance data in the GPU scene and procedural instances are lost when the GPU Scene is flushed.
	if (UPCGSubsystem* Subsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
	{
		Subsystem->DirtyRuntimeGenExecutionSources(EPCGChangeType::None, ERuntimeGenRefreshReason::RenderStateRefresh);
	}
}

bool FPCGEditorModule::SupportsDynamicReloading()
{
	return true;
}

TWeakPtr<IPCGEditorProgressNotification> FPCGEditorModule::CreateProgressNotification(const FTextFormat& TextFormat, bool bCanCancel)
{
	if (!FSlateNotificationManager::Get().AreNotificationsAllowed())
	{
		return nullptr;
	}

	TSharedPtr<IPCGEditorProgressNotification> NewNotification = MakeShared<FPCGEditorProgressNotification>(TextFormat, bCanCancel);
	ActiveNotifications.Add(NewNotification);
	return NewNotification.ToWeakPtr();
}

void FPCGEditorModule::ReleaseProgressNotification(TWeakPtr<IPCGEditorProgressNotification> InNotification)
{
	if (InNotification.IsValid())
	{
		if (TSharedPtr<IPCGEditorProgressNotification> SharedPtr = InNotification.Pin(); ActiveNotifications.Contains(SharedPtr))
		{
			ActiveNotifications.Remove(SharedPtr);
		}
	}
}

void FPCGEditorModule::SetOutlinerUIRefreshDelay(float InDelay)
{
	TWeakPtr<class ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();
	if (LevelEditor.IsValid())
	{
		TArray<TWeakPtr<class ISceneOutliner>> SceneOutlinerPtrs = LevelEditor.Pin()->GetAllSceneOutliners();

		for (TWeakPtr<class ISceneOutliner> SceneOutlinerPtr : SceneOutlinerPtrs)
		{
			if (TSharedPtr<class ISceneOutliner> SceneOutlinerPin = SceneOutlinerPtr.Pin())
			{
				SceneOutlinerPin->SetNextUIRefreshDelay(InDelay);
			}
		}
	}
}

bool FPCGEditorModule::CanSelectPartitionActors() const
{
	return GetDefault<UPCGEditorSettings>()->bCanSelectPartitionActors;
}

void FPCGEditorModule::SetPermissionMode(EPCGEditorPermissionMode InPermissionMode)
{
	if (InPermissionMode == PermissionMode)
	{
		return;
	}

	PermissionMode = InPermissionMode;
	PermissionModeChangedEvent.Broadcast(PermissionMode);
}

void FPCGEditorModule::TransferPropertyBagMetadataIntoHierarchy(UPCGGraph* Graph) const
{
	if (!Graph || !Graph->UserParameterHierarchyRoot || !Graph->UserParameterHierarchyRoot->IsA<UPropertyBagHierarchyRoot>())
	{
		return;
	}
	
	GetDefault<UPCGGraphParametersSchema>()->TransferPropertyBagMetadataIntoHierarchy(Graph->UserParameters, *Cast<UPropertyBagHierarchyRoot>(Graph->UserParameterHierarchyRoot));
}

UObject* FPCGEditorModule::CreatePropertyBagHierarchyRootObject(UPCGGraph* Graph) const
{
	return NewObject<UPropertyBagHierarchyRoot>(Graph, TEXT("UserParametersHierarchyRoot"));
}

void FPCGEditorModule::ConnectPropertyBagHierarchyRootObjectModifyDelegate(UPCGGraph* Graph) const
{
	check(Graph);
	if (UPropertyBagHierarchyRoot* PropertyBagHierarchyRootObject = Cast<UPropertyBagHierarchyRoot>(Graph->UserParameterHierarchyRoot))
	{
		if (!PropertyBagHierarchyRootObject->OnHierarchyModified.IsBoundToObject(Graph))
		{
			PropertyBagHierarchyRootObject->OnHierarchyModified.AddUObject(Graph, &UPCGGraph::OnUserParameterHierarchyModified);
		}
	}
}

void FPCGEditorModule::GraphIsBeingDestroyed(UPCGGraph* Graph) const
{
	check(Graph);
	if (UPropertyBagHierarchyRoot* PropertyBagHierarchyRootObject = Cast<UPropertyBagHierarchyRoot>(Graph->UserParameterHierarchyRoot))
	{
		PropertyBagHierarchyRootObject->OnHierarchyModified.RemoveAll(Graph);
	}
}

void FPCGEditorModule::RegisterDetailsCustomizations()
{
	FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditor.RegisterCustomClassLayout("PCGBlueprintSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FPCGBlueprintSettingsDetails::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("PCGComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FPCGComponentDetails::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("PCGGraph", FOnGetDetailCustomizationInstance::CreateStatic(&FPCGGraphDetails::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("PCGGraphInstance", FOnGetDetailCustomizationInstance::CreateStatic(&FPCGGraphInstanceDetails::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("PCGVolume", FOnGetDetailCustomizationInstance::CreateStatic(&FPCGVolumeDetails::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("PCGUserParameterGetSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FPCGEditableUserParameterDetails::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("PCGCustomHLSLSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FPCGCustomHLSLSettingsDetails::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("PCGComputeSource", FOnGetDetailCustomizationInstance::CreateStatic(&FPCGComputeSourceDetails::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("PCGInteractiveToolSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FPCGInteractiveToolSettingsDetails::MakeInstance));

	PropertyEditor.RegisterCustomPropertyTypeLayout("PCGAttributePropertySelector", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPCGAttributePropertySelectorDetails::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout("PCGAttributePropertyInputSelector", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPCGAttributePropertySelectorDetails::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout("PCGAttributePropertyOutputSelector", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPCGAttributePropertySelectorDetails::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout("PCGAttributePropertyOutputNoSourceSelector", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPCGAttributePropertySelectorDetails::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout("PCGOverrideInstancedPropertyBag", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPCGOverrideInstancedPropertyBagDetails::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout("EnumSelector", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FEnumSelectorDetails::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout("PCGDataTypeIdentifier", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPCGDataTypeIdentifierDetails::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout("PCGRaycastFilterRuleCollection", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPCGRaycastFilterRuleCollectionDetails::MakeInstance));
	
	PropertyEditor.NotifyCustomizationModuleChanged();
}

void FPCGEditorModule::UnregisterDetailsCustomizations()
{
	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyModule->UnregisterCustomClassLayout("PCGInteractiveToolSettings");
		PropertyModule->UnregisterCustomClassLayout("PCGComputeSource");
		PropertyModule->UnregisterCustomClassLayout("PCGCustomHLSLSettings");
		PropertyModule->UnregisterCustomClassLayout("PCGUserParameterGetSettings");
		PropertyModule->UnregisterCustomClassLayout("PCGVolume");
		PropertyModule->UnregisterCustomClassLayout("PCGGraphInstance");
		PropertyModule->UnregisterCustomClassLayout("PCGGraph");
		PropertyModule->UnregisterCustomClassLayout("PCGComponent");
		PropertyModule->UnregisterCustomClassLayout("PCGBlueprintSettings");

		PropertyModule->UnregisterCustomPropertyTypeLayout("PCGRaycastFilterRuleCollection");
		PropertyModule->UnregisterCustomPropertyTypeLayout("PCGDataTypeIdentifier");
		PropertyModule->UnregisterCustomPropertyTypeLayout("EnumSelector");
		PropertyModule->UnregisterCustomPropertyTypeLayout("PCGOverrideInstancedPropertyBag");
		PropertyModule->UnregisterCustomPropertyTypeLayout("PCGAttributePropertyOutputNoSourceSelector");
		PropertyModule->UnregisterCustomPropertyTypeLayout("PCGAttributePropertyOutputSelector");
		PropertyModule->UnregisterCustomPropertyTypeLayout("PCGAttributePropertyInputSelector");
		PropertyModule->UnregisterCustomPropertyTypeLayout("PCGAttributePropertySelector");

		PropertyModule->NotifyCustomizationModuleChanged();
	}
}

void FPCGEditorModule::RegisterMenuExtensions()
{
	FToolMenuOwnerScoped OwnerScoped(this);
	
	if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Tools"))
	{
		FToolMenuSection& Section = Menu->AddSection("PCGToolsSection", LOCTEXT("PCGToolsSection", "Procedural Generation Tools"));

		Section.AddSubMenu(
			"PCGToolsSubMenu",
			LOCTEXT("PCGSubMenu", "PCG Framework"),
			LOCTEXT("PCGSubMenu_Tooltip", "Procedural Content Generation (PCG) Framework related functionality"),
			FNewMenuDelegate::CreateRaw(this, &FPCGEditorModule::PopulateMenuActions),
			/*bInOpenSubMenuOnClick=*/false,
			FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.EditorIcon"));
	}

	if (UToolMenu* WorldAssetMenu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.AssetActionsSubMenu"))
	{
		// Use a dynamic section here because we might have plugins registering at a later time
		FToolMenuSection& Section = WorldAssetMenu->AddDynamicSection("PCG", FNewToolMenuDelegate::CreateLambda([this](UToolMenu* ToolMenu)
		{
			if (!GEditor || GEditor->GetPIEWorldContext() || !ToolMenu)
			{
				return;
			}

			if (UContentBrowserAssetContextMenuContext* AssetMenuContext = ToolMenu->Context.FindContext<UContentBrowserAssetContextMenuContext>())
			{
				PCGEditorMenuUtils::CreateOrUpdatePCGAssetFromMenu(ToolMenu, AssetMenuContext->SelectedAssets);
			}

		}), FToolMenuInsert(NAME_None, EToolMenuInsertType::Default));
	}

	if (UToolMenu* ComponentMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.ComponentContextMenu"))
	{
		FToolMenuSection& Section = ComponentMenu->AddDynamicSection("PCGComponent", FNewToolMenuDelegate::CreateLambda([this](UToolMenu* ToolMenu)
		{
			if (ULevelEditorContextMenuContext* LevelEditorContext = ToolMenu->FindContext<ULevelEditorContextMenuContext>())
			{
				if (LevelEditorContext->SelectedComponents.Num() == 1)
				{
					if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(LevelEditorContext->SelectedComponents[0]))
					{
						FToolMenuSection& Section = ToolMenu->AddSection("PCGComponentSection", LOCTEXT("PCGComponentSection", "PCG Component"));

						FUIAction SelectOriginalComponentAction(
							FExecuteAction::CreateLambda([PCGComponent]()
							{
								if(UPCGComponent* OriginalComponent = PCGComponent->GetOriginalComponent(); OriginalComponent && GEditor)
								{
									FScopedTransaction Transaction(LOCTEXT("SelectionOriginalPCGComponentTransaction", "Select Original PCG Component"));
									GEditor->SelectNone(true, true);
									GEditor->SelectComponent(OriginalComponent, true, true);
								}
							}),
							FCanExecuteAction::CreateLambda([PCGComponent]()
							{
								return GEditor && PCGComponent->GetOriginalComponent() != nullptr && PCGComponent->GetOriginalComponent() != PCGComponent;
							})
						);

						Section.AddMenuEntry(
							"SelectionOriginalPCGComponent",
							LOCTEXT("SelectionOriginalPCGComponentLabel", "Select Original PCG Component"),
							LOCTEXT("SelectionOriginalPCGComponentToolTip", "Selects the original PCG Component for the currently selected PCG Component"),
							FSlateIcon(),
							SelectOriginalComponentAction
						);
					}
				}
			}
			
		}), FToolMenuInsert(NAME_None, EToolMenuInsertType::Default));
	}

	if (UToolMenu* ToolbarMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.PlayToolBar"))
	{
		FToolMenuSection& Section = ToolbarMenu->FindOrAddSection("Play");

		if (const UPCGEditorSettings* EditorSettings = GetDefault<UPCGEditorSettings>())
		{
			if (EditorSettings->bShowPauseButton)
			{
				FToolMenuEntry PCGPauseButton = FToolMenuEntry::InitToolBarButton(
					"PCGPauseButton",
					FUIAction(
						FExecuteAction::CreateLambda([]()
						{
							const bool bWasPaused = PCGSystemSwitches::CVarPausePCGExecution.GetValueOnAnyThread();

							if (bWasPaused)
							{
								bool bShouldCancelAll = false;

								if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
								{
									bShouldCancelAll = true;
								}
								else if (FSlateApplication::Get().GetModifierKeys().IsAltDown())
								{
									bShouldCancelAll = false;
								}
								else if (const UPCGEditorSettings* EditorSettings = GetDefault<UPCGEditorSettings>())
								{
									bShouldCancelAll = EditorSettings->bUnpauseCancelsAll;
								}

								if (bShouldCancelAll)
								{
									if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
									{
										PCGSubsystem->CancelAllGeneration();
									}
								}
							}

							PCGSystemSwitches::CVarPausePCGExecution->Set(!bWasPaused);
						}),
						FIsActionButtonVisible::CreateLambda([](){ return true; }),
						FIsActionChecked::CreateLambda([]()
						{
							return PCGSystemSwitches::CVarPausePCGExecution.GetValueOnAnyThread();
						})
					),
					TAttribute<FText>::CreateLambda([]()
					{
						const UPCGEditorSettings* EditorSettings = GetDefault<UPCGEditorSettings>();

						if(PCGSystemSwitches::CVarPausePCGExecution.GetValueOnAnyThread())
						{
							if (EditorSettings && EditorSettings->OverridePausedButtonLabel != NAME_None)
							{
								return FText::FromName(EditorSettings->OverridePausedButtonLabel);
							}
							else
							{
								return LOCTEXT("PCGPauseButton_Off", "Paused");
							}
						}
						else
						{
							if (EditorSettings && EditorSettings->OverrideNotPausedButtonLabel != NAME_None)
							{
								return FText::FromName(EditorSettings->OverrideNotPausedButtonLabel);
							}
							else
							{
								return LOCTEXT("PCGPauseButton_On", "PCG");
							}
						}
					}),
					TAttribute<FText>::CreateLambda([]()
					{
						const UPCGEditorSettings* EditorSettings = GetDefault<UPCGEditorSettings>();
						if (EditorSettings && !EditorSettings->OverridePausedButtonTooltip.IsEmpty())
						{
							return FText::FromString(EditorSettings->OverridePausedButtonTooltip);
						}
						else
						{
							return LOCTEXT("PCGPauseButton_Tooltip", "Toggles PCG processing on/off and will cancel tasks depending on settings.\nUse Ctrl to unpause and cancel all tasks.\nUse Alt to unpause without cancelling tasks.");
						}
					}),
					TAttribute<FSlateIcon>::CreateLambda([]()
					{
						const UPCGEditorSettings* EditorSettings = GetDefault<UPCGEditorSettings>();
						if (EditorSettings && EditorSettings->bUseAlternatePauseButton)
						{
							return FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Editor.AlternatePause");
						}
						else
						{
							return FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Editor.Pause");
						}
					}),
					EUserInterfaceActionType::ToggleButton
				);
				PCGPauseButton.StyleNameOverride = "CalloutToolbar"; // used to show the button text
				Section.AddEntry(PCGPauseButton);
			}
		}
	}
}

void FPCGEditorModule::UnregisterMenuExtensions()
{
	UToolMenus::UnregisterOwner(this);
}

void FPCGEditorModule::PopulateMenuActions(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CreateMissingPartitionActors", "Create missing Partition Grid Actors"),
		LOCTEXT("CreateMissingPartitionActors_Tooltip", "Will visit all Partitioned PCG Components and create the missing intersecting Partition Grid Actors"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([]() 
			{
				if (GEditor)
				{
					if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
					{
						FScopedTransaction Transaction(LOCTEXT("CreateMissingPartitionActorsTransaction", "Create missing Partition Grid Actors"));
						PCGSubsystem->CreateMissingPartitionActors();
					}
				}
			})),
		NAME_None);

	MenuBuilder.AddSubMenu(
		LOCTEXT("PCGSubMenuDelete", "Delete"), 
		LOCTEXT("PCGSubMenuDelete_Tooltip", "Editor commands to delete PCG Partition Actors & World Actors."),
		FNewMenuDelegate::CreateLambda([](FMenuBuilder& SubMenuBuilder)
		{
			SubMenuBuilder.AddMenuEntry(
				LOCTEXT("DeletePCGPartitionActors", "All PCG Partition Grid Actors & Generated Actors"),
				LOCTEXT("DeletePCGPartitionActors_Tooltip", "Deletes all PCG Partition Grid Actors and PCG Partition Generated Actors in the current world"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([]() 
					{
						if (GEditor)
						{
							if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
							{
								PCGSubsystem->DeleteSerializedPartitionActors(/*bOnlyDeleteUnused=*/false);
							}
						}
					})),
				NAME_None);

			SubMenuBuilder.AddMenuEntry(
				LOCTEXT("DeletePCGPartitionActorsChildren", "All PCG Partition Generated Actors"),
				LOCTEXT("DeletePCGPartitionActorsChildren_Tooltip", "Deletes all PCG Partition Generated Actors in the current world (not the Partition Grid Actors themselves)"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([]() 
					{
						if (GEditor)
						{
							if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
							{
								PCGSubsystem->DeleteSerializedPartitionActors(/*bOnlyDeleteUnused=*/false, /*bOnlyChildren=*/true);
							}
						}
					})),
				NAME_None);

			SubMenuBuilder.AddMenuEntry(
				LOCTEXT("DeletePCGWorldActor", "All PCG World Actors"),
				LOCTEXT("DeletePCGWorldActor_Tooltip", "Deletes all PCG World Actors (This also deletes all PCG Partition Grid Actors & Generated Actors)"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([]()
						{
							if (GEditor)
							{
								if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
								{
									PCGSubsystem->DestroyAllPCGWorldActors();
								}
							}
						})),
				NAME_None);

			SubMenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteUnusedPCGPartitionActors", "Unused PCG Partition Grid Actors"),
				LOCTEXT("DeleteUnusedPCGPartitionActors_Tooltip", "Deletes unused PCG Partition Grid Actors in the current world"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([]() 
					{
						if (GEditor)
						{
							if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
							{
								PCGSubsystem->DeleteSerializedPartitionActors(/*bOnlyDeleteUnused=*/true);
							}
						}
					})),
				NAME_None);
		}));

	MenuBuilder.AddSubMenu(
		LOCTEXT("PCGSubMenuLandscape", "Landscape"), 
		LOCTEXT("PCGSubMenuLandscape_Tooltip", "PCG Landscape cache related editor commands"),
		FNewMenuDelegate::CreateLambda([](FMenuBuilder& SubMenuBuilder)
		{
			SubMenuBuilder.AddMenuEntry(
				LOCTEXT("BuildLandscapeCache", "Build Cache"),
				LOCTEXT("BuildLandscapeCache_Tooltip", "Caches the landscape data in the PCG World Actor"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([]() 
					{
						if (GEditor)
						{
							if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
							{
								PCGSubsystem->BuildLandscapeCache();
							}
						}
					})),
				NAME_None);

			SubMenuBuilder.AddMenuEntry(
				LOCTEXT("ClearLandscapeCache", "Clear Cache"),
				LOCTEXT("ClearLandscapeCache_Tooltip", "Clears the landscape data cache in the PCG World Actor"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([]() 
					{
						if (GEditor)
						{
							if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
							{
								PCGSubsystem->ClearLandscapeCache();
							}
						}
					})),
				NAME_None);
		}));
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CancelAllGeneration", "Cancel all PCG tasks"),
		LOCTEXT("CancelAllGeneration_Tooltip", "Cancels all PCG tasks running"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([]() {
				if (GEditor)
				{
					if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
					{
						PCGSubsystem->CancelAllGeneration();
					}
				}
				})),
		NAME_None);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("UpdatePCGBlueprintVariableVisibility", "Make all PCG blueprint variables visible to instances"),
		LOCTEXT("UpdatePCGBlueprintVariableVisibility_Tooltip", "Will visit all PCG blueprints, update their Instance editable flag, unless there is already one variable that is visible"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([]() {
				PCGEditorUtils::ForcePCGBlueprintVariableVisibility();
				})),
		NAME_None);

	MenuBuilder.AddSubMenu(
		LOCTEXT("PCGToolsLoggingSubMenu", "Logging / Reporting"),
		LOCTEXT("PCGToolsLoggingSubMenu_Tooltip", "Logging and reporting related editor commands"),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& LoggingMenuBuilder)
		{
			LoggingMenuBuilder.AddMenuEntry(
			LOCTEXT("LogAbnormalComponentState", "Log abnormal component state (actor order)"),
			LOCTEXT("LogAbnormalComponentState_Tooltip", "Logs unusual PCG components state, for every loaded actor"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([]() {
					if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
					{
						PCGSubsystem->LogAbnormalComponentStates(/*bGroupByState=*/false);
					}
				})),
			NAME_None);

			LoggingMenuBuilder.AddMenuEntry(
			LOCTEXT("LogAbnormalComponentState_GroupedByState", "Log abnormal component state (grouped by state)"),
			LOCTEXT("LogAbnormalComponentState_GroupedByState_Tooltip", "Logs unusual PCG components, for every loaded actor, grouped by state"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([]() {
					if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
					{
						PCGSubsystem->LogAbnormalComponentStates(/*bGroupByState=*/true);
					}
				})),
			NAME_None);
		}),
		false,
		FSlateIcon());

	MenuBuilder.AddMenuEntry(
		LOCTEXT("RefreshRuntimeGen", "Refresh all runtime gen components"),
		LOCTEXT("RefreshRuntimeGen_Tooltip", "Cleans up and re-generates all GenerateAtRuntime PCG components, including their partition actors."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([]()
			{
				if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
				{
					PCGSubsystem->RefreshAllRuntimeGenExecutionSources(EPCGChangeType::GenerationGrid);
				}
			})),
		NAME_None);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("GenerateAllComponents", "Generate all PCG components"),
		LOCTEXT("GenerateAllComponents_Tooltip", "Generate or refresh all the loaded PCG components, whenever they are dirty or not."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([]()
			{
				if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
				{
					PCGSubsystem->GenerateAllPCGComponents(/*bForce=*/true);
				}
			})),
		NAME_None);

	MenuBuilder.AddMenuEntry(
	LOCTEXT("GenerateAllDirtyComponents", "Generate all dirty PCG components"),
	LOCTEXT("GenerateAllDirtyComponents_Tooltip", "Generate or refresh all the loaded and dirty PCG components."),
	FSlateIcon(),
	FUIAction(
		FExecuteAction::CreateLambda([]()
		{
			if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
			{
				PCGSubsystem->GenerateAllPCGComponents(/*bForce=*/false);
			}
		})),
	NAME_None);

	MenuBuilder.AddMenuEntry(
	LOCTEXT("CleanupAllComponents", "Cleanup all PCG components"),
	LOCTEXT("CleanupAllComponents_Tooltip", "Cleanup all the loaded PCG components."),
	FSlateIcon(),
	FUIAction(
		FExecuteAction::CreateLambda([]()
		{
			if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
			{
				PCGSubsystem->CleanupAllPCGComponents(/*bPurge=*/false);
			}
		})),
	NAME_None);

	MenuBuilder.AddMenuEntry(
	LOCTEXT("PurgeAllComponents", "Purge all PCG components"),
	LOCTEXT("PurgeAllComponents_Tooltip", "Cleanup all the loaded PCG components and also remove any ghost resources on the components owners."),
	FSlateIcon(),
	FUIAction(
		FExecuteAction::CreateLambda([]()
		{
			if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
			{
				PCGSubsystem->CleanupAllPCGComponents(/*bPurge=*/true);
			}
		})),
	NAME_None);

	MenuBuilder.AddMenuEntry(
	LOCTEXT("ClearPCGLinkForAllComponents", "Clear link for all PCG components"),
	LOCTEXT("ClearPCGLinkForAllComponents_Tooltip", "Clear PCG Link for all loaded PCG Components. Note that all components will be cleaned up after that."),
	FSlateIcon(),
	FUIAction(
		FExecuteAction::CreateLambda([]()
		{
			PCGEditorUtils::OpenConfirmationDialog(LOCTEXT("ClearPCGLinkForAllComponents_Confirmation", "This will clear the link for ALL loaded PCG components. It's a destructive operation. Are you sure?"),
			[]()
			{
				if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
				{
					PCGSubsystem->ClearLinkForAllPCGComponents(FPCGMoveResourceParams{.TemplateTargetClass=AActor::StaticClass()});
				}
			},
			[](){});
		})),
	NAME_None);
}

void FPCGEditorModule::RegisterSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings("Editor", "ContentEditors", "PCGEditor",
			LOCTEXT("PCGEditorSettingsName", "PCG Editor"),
			LOCTEXT("PCGEditorSettingsDescription", "Configure the look and feel of the PCG Editor."),
			GetMutableDefault<UPCGEditorSettings>());
	}
}

void FPCGEditorModule::UnregisterSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Editor", "ContentEditors", "PCGEditor");
	}
}

void FPCGEditorModule::RegisterPCGDataVisualizations()
{
	FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetMutablePCGDataVisualizationRegistry();
	DataVisRegistry.InternalRegistry.Add(UPCGParamData::StaticClass(), MakeUnique<const IPCGParamDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGSpatialData::StaticClass(), MakeUnique<const IPCGSpatialDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGSplineData::StaticClass(), MakeUnique<const IPCGSplineDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGVolumeData::StaticClass(), MakeUnique<const FPCGVolumeDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGPrimitiveData::StaticClass(), MakeUnique<const FPCGPrimitiveDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGCollisionShapeData::StaticClass(), MakeUnique<const FPCGCollisionShapeDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGCollisionWrapperData::StaticClass(), MakeUnique<const FPCGCollisionWrapperDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGLandscapeData::StaticClass(), MakeUnique<const IPCGLandscapeDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGStaticMeshResourceData::StaticClass(), MakeUnique<const IPCGStaticMeshDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGTexture2DSingleBaseData::StaticClass(), MakeUnique<const FPCGBaseTextureDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGTexture2DArrayData::StaticClass(), MakeUnique<const FPCGTexture2DArrayDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGPolygon2DData::StaticClass(), MakeUnique<const FPCGPolygon2DDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGPolygon2DInteriorSurfaceData::StaticClass(), MakeUnique<const FPCGPolygon2DDataVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGDataViewData::StaticClass(), MakeUnique<const FPCGDataViewVisualization>());
	DataVisRegistry.InternalRegistry.Add(UPCGRawBufferData::StaticClass(), MakeUnique<const FPCGRawBufferDataVisualization>());
}

void FPCGEditorModule::UnregisterPCGDataVisualizations()
{
	FPCGModule::GetMutablePCGDataVisualizationRegistry().InternalRegistry.Empty();
}

void FPCGEditorModule::RegisterDeltaVisualizations()
{
	DeltaVisualizationRegistry.InternalRegistry.Add(FPCGDeltaBase::StaticStruct(), MakeUnique<const FPCGDeltaBaseVisualization>());
	DeltaVisualizationRegistry.InternalRegistry.Add(FPCGPointTransformDelta::StaticStruct(), MakeUnique<const FPCGPointTransformDeltaVisualization>());
	DeltaVisualizationRegistry.InternalRegistry.Add(FPCGPointTransformOffsetDelta::StaticStruct(), MakeUnique<const FPCGPointTransformOffsetDeltaVisualization>());
	DeltaVisualizationRegistry.InternalRegistry.Add(FPCGPointDeletionDelta::StaticStruct(), MakeUnique<const FPCGPointDeletionDeltaVisualization>());
	DeltaVisualizationRegistry.InternalRegistry.Add(FPCGPointInsertionDelta::StaticStruct(), MakeUnique<const FPCGPointInsertionDeltaVisualization>());
}

void FPCGEditorModule::UnregisterDeltaVisualizations()
{
	DeltaVisualizationRegistry.InternalRegistry.Empty();
}

const FPCGDeltaVisualizationRegistry& FPCGEditorModule::GetConstDeltaVisualizationRegistry()
{
	return FModuleManager::GetModuleChecked<FPCGEditorModule>("PCGEditor").DeltaVisualizationRegistry;
}

FPCGDeltaVisualizationRegistry& FPCGEditorModule::GetMutableDeltaVisualizationRegistry()
{
	return FModuleManager::GetModuleChecked<FPCGEditorModule>("PCGEditor").DeltaVisualizationRegistry;
}

void FPCGEditorModule::RegisterDeltaViewportExtensions()
{
	DeltaViewportExtensionRegistry.RegisterInternalExtension(FPCGPointTransformDelta::StaticStruct(), MakeUnique<FPCGPointTransformViewportExtension>());
	DeltaViewportExtensionRegistry.RegisterInternalExtension(FPCGPointTransformOffsetDelta::StaticStruct(), MakeUnique<FPCGPointTransformOffsetViewportExtension>());
	DeltaViewportExtensionRegistry.RegisterInternalExtension(FPCGPointDeletionDelta::StaticStruct(), MakeUnique<FPCGPointDeletionViewportExtension>());
	DeltaViewportExtensionRegistry.RegisterInternalExtension(FPCGPointInsertionDelta::StaticStruct(), MakeUnique<FPCGPointInsertionViewportExtension>());
}

void FPCGEditorModule::UnregisterDeltaViewportExtensions()
{
	DeltaViewportExtensionRegistry.UnregisterAllInternalExtensions();
}

const FPCGDeltaViewportExtensionRegistry& FPCGEditorModule::GetConstDeltaViewportExtensionRegistry()
{
	return FModuleManager::GetModuleChecked<FPCGEditorModule>("PCGEditor").DeltaViewportExtensionRegistry;
}

FPCGDeltaViewportExtensionRegistry& FPCGEditorModule::GetMutableDeltaViewportExtensionRegistry()
{
	return FModuleManager::GetModuleChecked<FPCGEditorModule>("PCGEditor").DeltaViewportExtensionRegistry;
}

void FPCGEditorModule::RegisterTrackerFactories()
{
	FPCGChangeTrackingRegistry& TrackingRegistry = FPCGModule::GetMutableChangeTrackingRegistry();
	TrackingRegistry.RegisterTrackerFactory(FPCGActorTracker::GetName(), FPCGCreateTrackerInstance::CreateStatic(&FPCGActorTracker::MakeInstance));
	TrackingRegistry.RegisterTrackerFactory(FPCGLandscapeTracker::GetName(), FPCGCreateTrackerInstance::CreateStatic(&FPCGLandscapeTracker::MakeInstance));
	TrackingRegistry.RegisterTrackerFactory(FPCGMiscTracker::GetName(), FPCGCreateTrackerInstance::CreateStatic(&FPCGMiscTracker::MakeInstance));

	TrackingRegistry.RegisterHandlerFactory(FPCGComponentChangeHandler::GetName(), FPCGCreateHandlerInstance::CreateStatic(&FPCGComponentChangeHandler::MakeInstance));
	TrackingRegistry.RegisterHandlerFactory(FPCGRuntimeGenChangeHandler::GetName(), FPCGCreateHandlerInstance::CreateStatic(&FPCGRuntimeGenChangeHandler::MakeInstance));
}

void FPCGEditorModule::UnregisterTrackerFactories()
{
	FPCGChangeTrackingRegistry& TrackingRegistry = FPCGModule::GetMutableChangeTrackingRegistry();
	TrackingRegistry.UnregisterTrackerFactory(FPCGActorTracker::GetName());
	TrackingRegistry.UnregisterTrackerFactory(FPCGLandscapeTracker::GetName());
	TrackingRegistry.UnregisterTrackerFactory(FPCGMiscTracker::GetName());

	TrackingRegistry.UnregisterHandlerFactory(FPCGComponentChangeHandler::GetName());
	TrackingRegistry.UnregisterHandlerFactory(FPCGRuntimeGenChangeHandler::GetName());
}

void FPCGEditorModule::RegisterPinColorAndIcons()
{
	FPCGDataTypeRegistry& InRegistry = FPCGModule::GetMutableDataTypeRegistry();

	// Pin Colors
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoParam::AsId(), [](const FPCGDataTypeIdentifier& InType)
	{
		if (PCGEditorGraphSwitches::CVarEnableColorsOnParamPins.GetValueOnAnyThread())
		{
			return GetDefault<UPCGEditorGraphSchema>()->GetMetadataTypeColor(static_cast<EPCGMetadataTypes>(InType.CustomSubtype));
		}
		else
		{
			return GetDefault<UPCGEditorSettings>()->ParamDataPinColor;
		}
	});

	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoPoint::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->PointDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoConcrete::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->ConcreteDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoPolygon2D::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->Polygon2DDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoPolyline::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->PolyLineDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoLandscape::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->LandscapeDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoTexture2DSingleBase::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->BaseTextureDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoTexture2D::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->TextureDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoRenderTarget2D::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->RenderTargetDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoVirtualTexture::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->VirtualTextureDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoSurface::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->SurfaceDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoVolume::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->VolumeDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoPrimitive::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->PrimitiveDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoDynamicMesh::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->DynamicMeshPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoSpatial::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->SpatialDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoStaticMeshResource::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->StaticMeshResourcePinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoRawBuffer::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->RawBufferPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoDataView::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->DataViewPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfoOther::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->UnknownDataPinColor; });
	InRegistry.RegisterPinColorFunction(FPCGDataTypeInfo::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPCGEditorSettings>()->DefaultPinColor; });

	// PinIcons. Just have catch all as most of it will be the same icons.
	InRegistry.RegisterPinIconsFunction(FPCGDataTypeInfo::AsId(), [](const FPCGDataTypeIdentifier& InId, const FPCGPinProperties& InProperties, const bool bIsInput) -> TTuple<const FSlateBrush*, const FSlateBrush*>
	{
		const bool bIsMultiData = InProperties.bAllowMultipleData;
		const bool bIsMultiConnections = InProperties.AllowsMultipleConnections();

		FName ConnectedBrushName = NAME_None;
		FName DisconnectedBrushName = NAME_None;

		if (InId.IsSameType(FPCGDataTypeInfoParam::AsId()))
		{
			ConnectedBrushName = bIsInput ? PCGEditorStyleConstants::Pin_Param_IN_C : PCGEditorStyleConstants::Pin_Param_OUT_C;
			DisconnectedBrushName = bIsInput ? PCGEditorStyleConstants::Pin_Param_IN_DC : PCGEditorStyleConstants::Pin_Param_OUT_DC;
		}
		else if (InId.IsSameType(FPCGDataTypeInfoSpatial::AsId()))
		{
			ConnectedBrushName = bIsInput ? PCGEditorStyleConstants::Pin_Composite_IN_C : PCGEditorStyleConstants::Pin_Composite_OUT_C;
			DisconnectedBrushName = bIsInput ? PCGEditorStyleConstants::Pin_Composite_IN_DC : PCGEditorStyleConstants::Pin_Composite_OUT_DC;
		}
		else if (InProperties.Usage == EPCGPinUsage::DependencyOnly)
		{
			ConnectedBrushName = PCGEditorStyleConstants::Pin_Graph_Dependency_C;
			DisconnectedBrushName = PCGEditorStyleConstants::Pin_Graph_Dependency_DC;
		}
		else
		{
			// Node outputs are always single collection (SC).
			static const FName* PinBrushes[] =
			{
				&PCGEditorStyleConstants::Pin_SD_SC_IN_C,
				&PCGEditorStyleConstants::Pin_SD_SC_IN_DC,
				&PCGEditorStyleConstants::Pin_SD_MC_IN_C,
				&PCGEditorStyleConstants::Pin_SD_MC_IN_DC,
				&PCGEditorStyleConstants::Pin_MD_SC_IN_C,
				&PCGEditorStyleConstants::Pin_MD_SC_IN_DC,
				&PCGEditorStyleConstants::Pin_MD_MC_IN_C,
				&PCGEditorStyleConstants::Pin_MD_MC_IN_DC,
				&PCGEditorStyleConstants::Pin_SD_SC_OUT_C,
				&PCGEditorStyleConstants::Pin_SD_SC_OUT_DC,
				&PCGEditorStyleConstants::Pin_SD_SC_OUT_C,
				&PCGEditorStyleConstants::Pin_SD_SC_OUT_DC,
				&PCGEditorStyleConstants::Pin_MD_SC_OUT_C,
				&PCGEditorStyleConstants::Pin_MD_SC_OUT_DC,
				&PCGEditorStyleConstants::Pin_MD_SC_OUT_C,
				&PCGEditorStyleConstants::Pin_MD_SC_OUT_DC
			};

			const int32 ConnectedIndex = (bIsInput ? 0 : 8) + (bIsMultiData ? 4 : 0) + (bIsMultiConnections ? 2 : 0);
			const int32 DisconnectedIndex = ConnectedIndex + 1;

			ConnectedBrushName = *PinBrushes[ConnectedIndex];
			DisconnectedBrushName = *PinBrushes[DisconnectedIndex];
		}

		const FPCGEditorStyle& EditorStyle = FPCGEditorStyle::Get();

		return {EditorStyle.GetBrush(ConnectedBrushName), EditorStyle.GetBrush(DisconnectedBrushName)};
	});
}

void FPCGEditorModule::UnregisterPinColorAndIcons()
{
	FPCGDataTypeRegistry& InRegistry = FPCGModule::GetMutableDataTypeRegistry();
	
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfo::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoOther::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoStaticMeshResource::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoSpatial::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoDynamicMesh::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoPrimitive::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoVolume::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoSurface::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoVirtualTexture::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoRenderTarget2D::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoTexture2D::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoTexture2DSingleBase::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoLandscape::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoPolyline::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoPolygon2D::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoConcrete::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoPoint::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoParam::AsId());
	InRegistry.UnregisterPinColorFunction(FPCGDataTypeInfoDataView::AsId());

	// PinIcons. Just have catch all as most of it will be the same icons.
	InRegistry.UnregisterPinIconsFunction(FPCGDataTypeInfo::AsId());
}

void FPCGEditorModule::OnScheduleGraph(const FPCGStackContext& StackContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSubsystem::OnScheduleGraph);
	// Always clear any possibly related warnings/errors on schedule.
	for (const FPCGStack& Stack : StackContext.GetStacks())
	{
		GetNodeVisualLogsMutable().ClearLogs(Stack);
	}
	
	IPCGGraphExecutionSource* RootSource = nullptr;

	// Flush out all stacks that begin from the source / top graph as the existing dynamic
	// stacks may not be occur during the next execution.
	if (const FPCGStack* BaseStack = StackContext.GetStack(0))
	{
		RootSource = const_cast<IPCGGraphExecutionSource*>(BaseStack->GetRootSource());

		if (BaseStack->IsCurrentFrameInRootGraph())
		{
			ClearExecutionMetadata(RootSource);
		}
	}

	// Record executed stacks.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSubsystem::OnScheduleGraph::RecordExecutedStacks);
		PCG::TUniqueScopeLock WriteLock(ExecutedStacksLock);

		for (const FPCGStack& ExecutedStack : StackContext.GetStacks())
		{
			FPCGStackSharedPtr SharedStack(MakeShared<FPCGStack>(ExecutedStack));

			bool bIsAlreadyInSet = false;

			// Push to the set
			ExecutedStacks.Add(SharedStack, &bIsAlreadyInSet);

			if (!bIsAlreadyInSet && RootSource)
			{
				ExecutedStacksPerSource.FindOrAdd(RootSource).Add(SharedStack);
			}
		}
	}
}

void FPCGEditorModule::OnGraphPreSave(UPCGGraph* Graph, FObjectPreSaveContext ObjectSaveContext)
{
	if (!Graph || !Graph->PCGEditorGraph)
	{
		return;
	}

	// No need to do it on cooking
	if (!ObjectSaveContext.IsProceduralSave())
	{
		if (TSharedPtr<FPCGEditor> PCGEditor = Graph->PCGEditorGraph->GetEditor().Pin())
		{
			PCGEditor->OnGraphPreSave(Graph);
		}

		Graph->PCGEditorGraph->ReplicateExtraNodes();
	}
}

void FPCGEditorModule::ClearExecutionMetadata(IPCGGraphExecutionSource* InSource)
{
	if (InSource)
	{
		ClearExecutedStacks(InSource);
		InSource->GetExecutionState().GetInspection().ClearInspectionData();

		GetNodeVisualLogsMutable().ClearLogs(InSource);
	}
}

void FPCGEditorModule::ClearExecutedStacks(const IPCGGraphExecutionSource* InRootSource)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGEditorModule::ClearExecutedStacksWithComponent);
	if (InRootSource)
	{
		PCG::TUniqueScopeLock WriteLock(ExecutedStacksLock);

		if (TSet<FPCGStackSharedPtr>* AllStackForComponent = ExecutedStacksPerSource.Find(InRootSource))
		{
			for (const FPCGStackSharedPtr& Stack : *AllStackForComponent)
			{
				ExecutedStacks.Remove(Stack);
			}

			ExecutedStacksPerSource.Remove(InRootSource);
		}
	}
}

void FPCGEditorModule::ClearExecutedStacks(const UPCGGraph* InContainingGraph)
{
	// TODO - it's fine to take more time here since it'll happen in rare(r) occurrences.
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGEditorModule::ClearExecutedStacksWithGraph);
	if (!InContainingGraph)
	{
		return;
	}

	PCG::TUniqueScopeLock WriteLock(ExecutedStacksLock);
	for (auto Iterator = ExecutedStacks.CreateIterator(); Iterator; ++Iterator)
	{
		FPCGStackSharedPtr StackPtr = *Iterator;

		if (StackPtr->HasObject(InContainingGraph))
		{
			Iterator.RemoveCurrent();

			if (TSet<FPCGStackSharedPtr>* StacksForComponent = ExecutedStacksPerSource.Find(StackPtr->GetRootSource()))
			{
				StacksForComponent->Remove(StackPtr);
			}
		}
	}
}

TArray<FPCGStackSharedPtr> FPCGEditorModule::GetExecutedStacksPtrs(const FPCGStack& BeginningWithStack)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGEditorModule::GetExecutedStacksPtrsWithBeginning);
	TArray<FPCGStackSharedPtr> MatchingStacks;

	PCG::TSharedScopeLock ReadLock(ExecutedStacksLock);
	TSet<FPCGStackSharedPtr>* StacksToTest = ExecutedStacksPerSource.Find(BeginningWithStack.GetRootSource());

	// Stack doesn't have a root component (unlikely), we'll check everything then
	if (!StacksToTest)
	{
		StacksToTest = &ExecutedStacks;
	}

	for (const FPCGStackSharedPtr& Stack : *StacksToTest)
	{
		if (Stack->BeginsWith(BeginningWithStack))
		{
			MatchingStacks.Add(Stack);
		}
	}

	return MatchingStacks;
}

TArray<FPCGStackSharedPtr> FPCGEditorModule::GetExecutedStacksPtrs(const IPCGGraphExecutionSource* InSource, const UPCGGraph* InSubgraph, bool bOnlyWithSubgraphAsCurrentFrame)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGEditorModule::GetExecutedStacksPtrs);
	TArray<FPCGStackSharedPtr> MatchingStacks;
	PCG::TSharedScopeLock ReadLock(ExecutedStacksLock);

	TSet<FPCGStackSharedPtr>* ComponentSet = nullptr;
	TSet<FPCGStackSharedPtr>* GraphSet = nullptr;

	if (InSource)
	{
		ComponentSet = ExecutedStacksPerSource.Find(InSource);
	}

	if (!ComponentSet)
	{
		ComponentSet = &ExecutedStacks;
	}

	auto StackMatches = [bOnlyWithSubgraphAsCurrentFrame, InSubgraph](const FPCGStackSharedPtr& Stack) -> bool
	{
		return ((bOnlyWithSubgraphAsCurrentFrame && Stack->GetGraphForCurrentFrame() == InSubgraph) ||
			(!bOnlyWithSubgraphAsCurrentFrame && Stack->HasObject(InSubgraph)));
	};

	for (const FPCGStackSharedPtr& Stack : *ComponentSet)
	{
		if (StackMatches(Stack))
		{
			MatchingStacks.Add(Stack);
		}
	}

	return MatchingStacks;
}

void FPCGEditorModule::NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType)
{
	if (!!(ChangeType & (EPCGChangeType::Structural | EPCGChangeType::GenerationGrid)))
	{
		// If change was deep enough, clear out all executed stacks, and let them refresh upon next generate. Fixes
		// cases where stale stacks never got flushed.
		ClearExecutedStacks(InGraph);
	}
}

IMPLEMENT_MODULE(FPCGEditorModule, PCGEditor);

DEFINE_LOG_CATEGORY(LogPCGEditor);

#undef LOCTEXT_NAMESPACE
