// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorProgressNotification.h"
#include "PCGEditorStyle.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/INotificationWidget.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

namespace PCGEditorProgressNotification
{
	class SProgressNotificationWidget : public SCompoundWidget, public INotificationWidget
	{
	public:
		SLATE_BEGIN_ARGS(SProgressNotificationWidget) {}
			SLATE_ATTRIBUTE(TWeakPtr<IPCGEditorProgressNotification>, ProgressNotification)
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments & InArgs)
		{
			ProgressNotification = InArgs._ProgressNotification;
			if (TWeakPtr<IPCGEditorProgressNotification> ProgressNotificationWeakPtr = ProgressNotification.Get(); ProgressNotificationWeakPtr.IsValid())
			{
				if (TSharedPtr<IPCGEditorProgressNotification> ProgressNotificationPtr = ProgressNotificationWeakPtr.Pin())
				{
					TextFormat = ProgressNotificationPtr->GetTextFormat();
					NumTasks = ProgressNotificationPtr->GetNumTasks();
				}
			}

			ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 4.0f, 0.0f)
					[
						SNew(SImage)
						.Image(FPCGEditorStyle::Get().GetBrush("PCG.EditorIcon"))
					]
					+ SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 4.0f, 0.0f)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 3.0f, 0.0f, 2.0f)
						[
							SAssignNew(ProgressTextWidget, STextBlock)
							.Font(FAppStyle::Get().GetFontStyle("NotificationList.FontBold"))
						]
						+ SVerticalBox::Slot()
						[
							SNew(SBox)
							.HeightOverride(8.f)
							[
								SNew(SOverlay)
								+ SOverlay::Slot()
								.VAlign(VAlign_Center)
								.Padding(1.0f, 0.0f)
								[
									SAssignNew(ProgressBar, SProgressBar)
									.Percent(0.0f)
								]
								+ SOverlay::Slot()
								[
									SNew(SImage)
									.Image(FAppStyle::Get().GetBrush("StatusBar.ProgressOverlay"))
									.Visibility(EVisibility::HitTestInvisible)
								]
							]
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SAssignNew(PercentText, STextBlock)
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.Text(NSLOCTEXT("PCGEditorProgressNotification", "CancelButtonText", "Cancel"))
					.Visibility_Lambda([this]()
					{
						TWeakPtr<IPCGEditorProgressNotification> ProgressNotificationPtr = ProgressNotification.Get();
						if (ProgressNotificationPtr.IsValid() && ProgressNotificationPtr.Pin()->CanCancel())
						{
							return EVisibility::Visible;
						}
						return EVisibility::Collapsed;
					})
					.OnClicked_Lambda([this]()
					{
						TWeakPtr<IPCGEditorProgressNotification> ProgressNotificationPtr = ProgressNotification.Get();
						if (ProgressNotificationPtr.IsValid())
						{
							ProgressNotificationPtr.Pin()->Cancel();
						}
						return FReply::Handled();
					})
				]
			];
		}

		virtual void Tick(const FGeometry & AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
		{						
			uint32 NumTasksDone = NumTasks;
			float PercentDone = 1.0f;

			// Check if Notification is still valid
			TWeakPtr<IPCGEditorProgressNotification> ProgressNotificationWeakPtr = ProgressNotification.Get();
			TSharedPtr<IPCGEditorProgressNotification> ProgressNotificationPtr = ProgressNotificationWeakPtr.IsValid() ? ProgressNotificationWeakPtr.Pin() : nullptr;
			if (ProgressNotificationPtr)
			{
				NumTasks = ProgressNotificationPtr->GetNumTasks();
				NumTasksDone = ProgressNotificationPtr->GetNumTasksDone();
				PercentDone = NumTasks == 0 ? 1.0f : FMath::Clamp((float)NumTasksDone / NumTasks, 0.0f, 1.0f);
			}

			FFormatNamedArguments Args;
			
			Args.Add(TEXT("ProgressMessage"), FText::Format(TextFormat, FText::AsNumber(NumTasks)));
			Args.Add(TEXT("NumTasks"), FText::AsNumber(NumTasks));
			Args.Add(TEXT("NumTasksDone"), FText::AsNumber(NumTasksDone));

			ProgressTextWidget->SetText(FText::Format(NSLOCTEXT("PCGEditorProgressNotification", "ProgressText", "{ProgressMessage} ({NumTasksDone}/{NumTasks})"), Args));
			PercentText->SetText(FText::AsPercent(PercentDone));
			ProgressBar->SetPercent(PercentDone);
		}

		void SetProgressText(FText ProgressText)
		{
			ProgressTextWidget->SetText(ProgressText);
		}

		void SetProgressPercent(float Percent)
		{
			ProgressBar->SetPercent(Percent);
			PercentText->SetText(FText::AsPercent(Percent));
		}

		/** INotificationWidget interface */
		virtual void OnSetCompletionState(SNotificationItem::ECompletionState) override {}

		TSharedRef<SWidget> AsWidget() override
		{
			return AsShared();
		}
		
	private:
		TAttribute<TWeakPtr<IPCGEditorProgressNotification>> ProgressNotification;
		TSharedPtr<SProgressBar> ProgressBar;
		TSharedPtr<STextBlock> PercentText;
		TSharedPtr<STextBlock> ProgressTextWidget;

		// Cached values
		FTextFormat TextFormat;
		uint32 NumTasks = 0;
	};
}

FPCGEditorProgressNotification::FPCGEditorProgressNotification(const FTextFormat& InTextFormat, bool bInCanCancel)
	: NumTasks(0), NumTasksDone(0), NumTasksRemaining(0), bCanCancel(bInCanCancel), TextFormat(InTextFormat)
{
}

FPCGEditorProgressNotification::~FPCGEditorProgressNotification()
{
	if (NotificationItem.IsValid())
	{
		NotificationItem->ExpireAndFadeout();
		NotificationItem = nullptr;
	}
}

void FPCGEditorProgressNotification::Update(uint32 InNumRemainingTasks)
{
	uint32 NumNewTasks = FMath::Max(0, (int32)InNumRemainingTasks - (int32)NumTasksRemaining);
	
	NumTasks += NumNewTasks;
	
	NumTasksDone = NumTasks - InNumRemainingTasks;
	NumTasksRemaining = InNumRemainingTasks;

	if (NumTasks > 0 && !NotificationItem)
	{
		FNotificationInfo ProgressNotification(FText::GetEmpty());

		ProgressNotification.ContentWidget = SNew(PCGEditorProgressNotification::SProgressNotificationWidget).ProgressNotification(AsWeak());
		ProgressNotification.bAllowThrottleWhenFrameRateIsLow = false;
		ProgressNotification.bFireAndForget = false;
		ProgressNotification.ExpireDuration = 1.0f;
		ProgressNotification.FadeOutDuration = 0.5f;
		NotificationItem = FSlateNotificationManager::Get().AddNotification(ProgressNotification);
	}
}
