// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/IPCGEditorProgressNotification.h"

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "Widgets/Notifications/SNotificationList.h"

class FPCGEditorProgressNotification : public IPCGEditorProgressNotification
{
public:
	FPCGEditorProgressNotification(const FTextFormat& InTextFormat, bool bInCanCancel);
	virtual ~FPCGEditorProgressNotification();

	virtual void Update(uint32 InNumRemainingTasks) override;
		
	DECLARE_DERIVED_EVENT(FPCGEditorProgressNotification, IPCGEditorProgressNotification::FCancelTasksEvent, FCancelTasksEvent);
	virtual FCancelTasksEvent& OnCancelTasks() override { return CancelTasksEvent; }

	virtual uint32 GetNumTasks() const override { return NumTasks; }
	virtual uint32 GetNumTasksDone() const override { return NumTasksDone; }
	virtual const FTextFormat& GetTextFormat() const override { return TextFormat; }

	virtual void Cancel() override { OnCancelTasks().Broadcast(); }
	virtual bool CanCancel() const override { return bCanCancel; }
	
private:
	uint32 NumTasks = 0;
	uint32 NumTasksDone = 0;
	uint32 NumTasksRemaining = 0;
	
	bool bCanCancel = false;
	bool bIsCompleted = false;
	
	FTextFormat TextFormat;
	FCancelTasksEvent CancelTasksEvent;
	TSharedPtr<SNotificationItem> NotificationItem;
};