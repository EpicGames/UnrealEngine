// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorSettings.h"
#include "PCGEditorCommon.h"
#include "PCGModule.h"
#include "PCGSubgraph.h"

#include "EdGraph/EdGraphPin.h"
#include "PCGSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorSettings)

UPCGEditorProjectSettings::UPCGEditorProjectSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UPCGEditorSettings::UPCGEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DefaultNodeColor = FLinearColor(0.4f, 0.62f, 1.0f);
	InstancedNodeBodyTintColor = FLinearColor(0.5f, 0.5f, 0.5f);
	InputOutputNodeColor = FLinearColor(1.0f, 0.0f, 0.0f);
	SetOperationNodeColor = FLinearColor(1.0f, 0.2f, 1.0f);
	DensityOperationNodeColor = FLinearColor(0.2f, 0.59f, 0.99f);
	BlueprintNodeColor = FLinearColor(0.02f, 0.18f, 1.0f);
	MetadataNodeColor = FLinearColor(1.0f, 0.99f, 0.99f);
	FilterNodeColor = FLinearColor(0.24f, 0.09f, 0.85f);
	SamplerNodeColor = FLinearColor(0.0f, 0.0f, 0.0f);
	SpawnerNodeColor = FLinearColor(0.0f, 1.0f, 0.69f);
	SubgraphNodeColor = FLinearColor(1.0f, 0.05f, 0.05f);
	EmbeddedSubgraphNodeColor = FLinearColor(0.225f, 0.f, 0.f);
	ParamDataNodeColor = FLinearColor(1.0f, 0.38f, 0.02f);
	DebugNodeColor = FLinearColor(1.0f, 0.0f, 1.0f);
	ControlFlowNodeColor = FLinearColor(0.0f, 1.0f, 0.0f);
	PointOpsNodeColor = FLinearColor(0.0f, 0.04f, 0.23f);
	HierarchicalGenerationNodeColor = FLinearColor(1.0f, 0.132868f, 0.0f);
	GraphParametersNodeColor = FLinearColor::Yellow;
	RerouteNodeColor = FLinearColor(0.5f, 1.0f, 0.83f);
	DynamicMeshNodeColor = FLinearColor(0.42f, 0.25f, 1.0f);
	ResourceNodeColor = FLinearColor(1.0f, 0.0f, 1.0f);

	DefaultPinColor = FLinearColor(0.29f, 0.29f, 0.29f);
	SpatialDataPinColor = FLinearColor(1.0f, 1.0f, 1.0f);
	ConcreteDataPinColor = FLinearColor(0.45f, 0.38f, 0.96f);
	PointDataPinColor = FLinearColor(0.05f, 0.25f, 1.0f);
	PolyLineDataPinColor = FLinearColor(0.05f, 0.75f, 0.82f);
	Polygon2DDataPinColor = FLinearColor(0.055f, 0.65f, 0.515f);
	SurfaceDataPinColor = FLinearColor(0.06f, 0.55f, 0.21f);
	LandscapeDataPinColor = FLinearColor(0.66f, 0.66f, 0.07f);
	BaseTextureDataPinColor = FLinearColor(0.81f, 0.28f, 0.22f);
	TextureDataPinColor = FLinearColor(0.79f, 0.08f, 0.01f);
	DataViewPinColor = FLinearColor(0.69f, 0.92f, 0.23f);
	RenderTargetDataPinColor = FLinearColor(0.8f, 0.18f, 0.12f);
	VirtualTextureDataPinColor = FLinearColor(0.8f, 0.48f, 0.22f);
	VolumeDataPinColor = FLinearColor(0.79f, 0.06f, 0.5f);
	PrimitiveDataPinColor = FLinearColor(0.22f, 0.05f, 1.0f);
	DynamicMeshPinColor = FLinearColor(0.42f, 0.25f, 1.0f);
	StaticMeshResourcePinColor = FLinearColor(0.82f, 0.25f, 0.1f);
	RawBufferPinColor = FLinearColor(0.6f, 0.7f, 1.0f);

	ParamDataPinColor = FLinearColor(1.0f, 0.38f, 0.02f);
	UnknownDataPinColor = FLinearColor(0.3f, 0.3f, 0.3f);
}

FLinearColor UPCGEditorSettings::GetColor(UPCGSettings* Settings) const
{
	if (!Settings)
	{
		return DefaultNodeColor;
	}

	// First: check if there's an override
	if (const FLinearColor* Override = OverrideNodeColorByClass.Find(Settings->GetClass()))
	{
		return *Override;
	}

	// Otherwise, check against the classes we know
	switch (Settings->GetType())
	{
		case EPCGSettingsType::InputOutput:
			return InputOutputNodeColor;
		case EPCGSettingsType::Spatial:
			return SetOperationNodeColor;
		case EPCGSettingsType::Density:
			return DensityOperationNodeColor;
		case EPCGSettingsType::Blueprint:
			return BlueprintNodeColor;
		case EPCGSettingsType::Metadata:
			return MetadataNodeColor;
		case EPCGSettingsType::Filter:
			return FilterNodeColor;
		case EPCGSettingsType::Sampler:
			return SamplerNodeColor;
		case EPCGSettingsType::Spawner:
			return SpawnerNodeColor;
		case EPCGSettingsType::Subgraph:
		{
			if (UPCGBaseSubgraphSettings* SubGraphSettings = Cast<UPCGBaseSubgraphSettings>(Settings))
			{
				if (UPCGGraph* Subgraph = SubGraphSettings->GetSubgraph(); Subgraph && Subgraph->IsEmbeddedSubgraph())
				{
					return EmbeddedSubgraphNodeColor;
				}
			}
			
			return SubgraphNodeColor;
		}
		case EPCGSettingsType::Debug:
			return DebugNodeColor;
		case EPCGSettingsType::Param:
			return ParamDataNodeColor;
		case EPCGSettingsType::HierarchicalGeneration:
			return HierarchicalGenerationNodeColor;
		case EPCGSettingsType::ControlFlow:
			return ControlFlowNodeColor;
		case EPCGSettingsType::PointOps:
			return PointOpsNodeColor;
		case EPCGSettingsType::GraphParameters:
		{
			// For graph parameters, use the output pin
			const TArray<FPCGPinProperties> Properties = Settings->OutputPinProperties();
			return !Properties.IsEmpty() ? FPCGModule::GetConstDataTypeRegistry().GetPinColor(Properties[0].AllowedTypes) : GraphParametersNodeColor;
		}
		case EPCGSettingsType::Reroute:
			return RerouteNodeColor;
		case EPCGSettingsType::DynamicMesh:
			return DynamicMeshNodeColor;
		case EPCGSettingsType::Resource:
			return ResourceNodeColor;
		case EPCGSettingsType::Generic: // falls through
		default:
			// Finally, we couldn't find any match, so return the default value
			return DefaultNodeColor;
	}
}

FLinearColor UPCGEditorSettings::GetPinColor(const FEdGraphPinType& PinType) const
{
	if (PinType.PinCategory == FPCGEditorCommon::ConcreteDataType)
	{
		// Clauses below try to pick the narrowest type possible, falling back to Spatial
		if (PinType.PinSubCategory == FPCGEditorCommon::PointDataType)
		{
			return PointDataPinColor;
		}
		else if (PinType.PinSubCategory == FPCGEditorCommon::PolyLineDataType)
		{
			return PolyLineDataPinColor;
		}
		else if (PinType.PinSubCategory == FPCGEditorCommon::LandscapeDataType)
		{
			return LandscapeDataPinColor;
		}
		else if (PinType.PinSubCategory == FPCGEditorCommon::BaseTextureDataType)
		{
			return BaseTextureDataPinColor;
		}
		else if (PinType.PinSubCategory == FPCGEditorCommon::TextureDataType)
		{
			return TextureDataPinColor;
		}
		else if (PinType.PinSubCategory == FPCGEditorCommon::RenderTargetDataType)
		{
			return RenderTargetDataPinColor;
		}
		else if (PinType.PinSubCategory == FPCGEditorCommon::VirtualTextureDataType)
		{
			return VirtualTextureDataPinColor;
		}
		else if (PinType.PinSubCategory == FPCGEditorCommon::SurfaceDataType)
		{
			return SurfaceDataPinColor;
		}
		else if (PinType.PinSubCategory == FPCGEditorCommon::VolumeDataType)
		{
			return VolumeDataPinColor;
		}
		else if (PinType.PinSubCategory == FPCGEditorCommon::DynamicMeshDataType)
		{
			return DynamicMeshPinColor;
		}
		else if (PinType.PinSubCategory == FPCGEditorCommon::PrimitiveDataType)
		{
			return PrimitiveDataPinColor;
		}
		else
		{
			return ConcreteDataPinColor;
		}
	}
	else if (PinType.PinCategory == FPCGEditorCommon::SpatialDataType)
	{
		return SpatialDataPinColor;
	}
	else if (PinType.PinSubCategory == FPCGEditorCommon::StaticMeshResourceDataType)
	{
		return StaticMeshResourcePinColor;
	}
	else if (PinType.PinCategory == FPCGEditorCommon::ParamDataType)
	{
		return ParamDataPinColor;
	}
	else if (PinType.PinCategory == FPCGEditorCommon::OtherDataType)
	{
		return UnknownDataPinColor;
	}
	else
	{
		return DefaultPinColor;
	}
}
