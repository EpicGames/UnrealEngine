// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorStyle.h"

#include "PCGCommon.h"

#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

void FPCGEditorStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FPCGEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

FPCGEditorStyle::FPCGEditorStyle() : FSlateStyleSet("PCGEditorStyle")
{
	static const FVector2D Icon16x16(16.0f, 16.0f);
	static const FVector2D Icon20x20(20.0f, 20.0f);
	static const FVector2D Icon32x32(32.0f, 32.0f);
	static const FVector2D Icon64x64(64.0f, 64.0f);
	static const FVector2D Icon128x128(128.0f, 128.0f);
	
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetContentRoot(IPluginManager::Get().FindPlugin("PCG")->GetBaseDir() / TEXT("Content"));

	Set("PCG.NodeOverlay.Debug", new CORE_IMAGE_BRUSH_SVG("Starship/Blueprints/Breakpoint_Valid", Icon20x20, FSlateColor(FColor::Cyan)));
	Set("PCG.NodeOverlay.Inspect", new CORE_IMAGE_BRUSH_SVG("Starship/Blueprints/Breakpoint_Valid", Icon20x20, FSlateColor(FColor::Orange)));
	Set("PCG.NodeOverlay.OnInactiveBranch", new CORE_IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Fail_Badge", Icon20x20, FSlateColor(FColor::White)));
	Set("PCG.NodeOverlay.GPUUpload", new CORE_IMAGE_BRUSH("Common/UpArrow", Icon16x16, FSlateColor(FColor::Yellow)));
	Set("PCG.NodeOverlay.GPUReadback", new CORE_IMAGE_BRUSH("Common/DownArrow", Icon16x16, FSlateColor(FColor::Yellow)));

	static const FVector2D PinSize(22.0f, 22.0f);
	Set(PCGEditorStyleConstants::Pin_SD_SC_IN_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_SD_SC_IN", PinSize));
	Set(PCGEditorStyleConstants::Pin_SD_SC_IN_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_SD_SC_IN_Unplugged", PinSize));
	Set(PCGEditorStyleConstants::Pin_SD_SC_OUT_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_SD_SC_OUT", PinSize));
	Set(PCGEditorStyleConstants::Pin_SD_SC_OUT_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_SD_SC_OUT_Unplugged", PinSize));

	Set(PCGEditorStyleConstants::Pin_SD_MC_IN_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_SD_MC_IN", PinSize));
	Set(PCGEditorStyleConstants::Pin_SD_MC_IN_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_SD_MC_IN_Unplugged", PinSize));
	Set(PCGEditorStyleConstants::Pin_SD_MC_OUT_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_SD_MC_OUT", PinSize));
	Set(PCGEditorStyleConstants::Pin_SD_MC_OUT_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_SD_MC_OUT_Unplugged", PinSize));

	Set(PCGEditorStyleConstants::Pin_MD_SC_IN_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_MD_SC_IN", PinSize));
	Set(PCGEditorStyleConstants::Pin_MD_SC_IN_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_MD_SC_IN_Unplugged", PinSize));
	Set(PCGEditorStyleConstants::Pin_MD_SC_OUT_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_MD_SC_OUT", PinSize));
	Set(PCGEditorStyleConstants::Pin_MD_SC_OUT_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_MD_SC_OUT_Unplugged", PinSize));

	Set(PCGEditorStyleConstants::Pin_MD_MC_IN_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_MD_MC_IN", PinSize));
	Set(PCGEditorStyleConstants::Pin_MD_MC_IN_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_MD_MC_IN_Unplugged", PinSize));
	Set(PCGEditorStyleConstants::Pin_MD_MC_OUT_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_MD_MC_OUT", PinSize));
	Set(PCGEditorStyleConstants::Pin_MD_MC_OUT_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_MD_MC_OUT_Unplugged", PinSize));

	Set(PCGEditorStyleConstants::Pin_Param_IN_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Param_IN", PinSize));
	Set(PCGEditorStyleConstants::Pin_Param_IN_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Param_IN_Unplugged", PinSize));
	Set(PCGEditorStyleConstants::Pin_Param_OUT_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Param_OUT", PinSize));
	Set(PCGEditorStyleConstants::Pin_Param_OUT_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Param_OUT_Unplugged", PinSize));

	Set(PCGEditorStyleConstants::Pin_Composite_IN_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Composite_IN", PinSize));
	Set(PCGEditorStyleConstants::Pin_Composite_IN_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Composite_IN_Unplugged", PinSize));
	Set(PCGEditorStyleConstants::Pin_Composite_OUT_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Composite_OUT", PinSize));
	Set(PCGEditorStyleConstants::Pin_Composite_OUT_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Composite_OUT_Unplugged", PinSize));

	Set(PCGEditorStyleConstants::Pin_Required, new IMAGE_BRUSH_SVG("Style/PCG_Graph_RequiredPin_IN", FVector2D(8.0f, 22.0f)));

	Set(PCGEditorStyleConstants::Pin_Graph_Dependency_C, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Dependency", PinSize));
	Set(PCGEditorStyleConstants::Pin_Graph_Dependency_DC, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Dependency_Unplugged", PinSize));

	Set(PCGEditorStyleConstants::Node_Overlay_Inactive, new IMAGE_BRUSH_SVG("Style/PCG_Node_Overlay_Inactive", Icon20x20));

	Set(PCGNodeConstants::Icons::CompactNodeFilter, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Filter", FVector2D(28.0f, 28.0f)));
	Set(PCGNodeConstants::Icons::CompactNodeConvert, new IMAGE_BRUSH_SVG("Style/PCG_Graph_To", PinSize));

	Set(PCGEditorStyleConstants::NodeFilter, new IMAGE_BRUSH_SVG("Style/PCG_Graph_Filter", Icon16x16));
	Set(PCGEditorStyleConstants::NodeConvert, new IMAGE_BRUSH_SVG("Style/PCG_Graph_To", Icon16x16));

	Set(PCGEditorStyleConstants::Node_ToolStart, new IMAGE_BRUSH_SVG("Style/PCG_Node_ViewportToolStart", Icon20x20));
	Set(PCGEditorStyleConstants::Node_DataOverride, new IMAGE_BRUSH_SVG("Style/PCG_Node_DataOverride", Icon16x16));
	Set(PCGEditorStyleConstants::Node_ManualEdit, new IMAGE_BRUSH_SVG("Style/PCG_Node_ManualEdit", Icon16x16));

	FInlineEditableTextBlockStyle NodeTitleStyle = FInlineEditableTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("Graph.Node.NodeTitleInlineEditableText"));
	FTextBlockStyle GraphNodeItalicTitle = FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Graph.Node.NodeTitle"))
		.SetFont(DEFAULT_FONT("BoldCondensedItalic", 10));
	FTextBlockStyle CompactGraphNodeTitle = FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Graph.Node.NodeTitle"))
		.SetFont(DEFAULT_FONT("Regular", 10))
		.SetShadowColorAndOpacity(FLinearColor::Black * 0.8f)
		.SetShadowOffset(FVector2D::UnitVector);

	Set("PCG.Node.NodeTitleInlineEditableText", NodeTitleStyle);
	Set("PCG.Node.InstancedNodeTitleInlineEditableText", FInlineEditableTextBlockStyle(NodeTitleStyle).SetTextStyle(GraphNodeItalicTitle));
	Set("PCG.Node.CompactNodeTitle", FInlineEditableTextBlockStyle(NodeTitleStyle).SetTextStyle(CompactGraphNodeTitle));

	// Styles for higen grid size label. Hand-tweaked values to match UI target mockups.
	Set(PCGEditorStyleConstants::Node_Overlay_GridSizeLabel_Active_Border, new FSlateRoundedBoxBrush(
		FLinearColor::White,
		PCGEditorStyleConstants::Node_Overlay_GridSizeLabel_BorderRadius,
		FLinearColor::Black,
		PCGEditorStyleConstants::Node_Overlay_GridSizeLabel_BorderStroke));

	// Styles related to additional tokens on nodes (GPU, ...)
	FTextBlockStyle AdditionalOverlayWidgetTextStyle = FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetFont(FCoreStyle::GetDefaultFontStyle("Bold", 8))
		.SetColorAndOpacity(FLinearColor(218.0f / 255.0f, 218.0f / 255.0f, 218.0f / 255.0f))
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.7f));
	Set("PCG.Node.AdditionalOverlayWidgetText", AdditionalOverlayWidgetTextStyle);

	// Orphaned override row style for the Data Overrides panel.
	{
		FTableRowStyle OrphanedOverrideRowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("DataTableEditor.CellListViewRow");

		const FSlateColorBrush OrphanedOverrideEvenBackground(FLinearColor(0.135f, 0.105f, 0.090f, 1.0f));
		const FSlateColorBrush OrphanedOverrideEvenBackgroundHovered(FLinearColor(0.180f, 0.140f, 0.120f, 1.0f));
		const FSlateColorBrush OrphanedOverrideOddBackground(FLinearColor(0.075f, 0.055f, 0.045f, 1.0f));
		const FSlateColorBrush OrphanedOverrideOddBackgroundHovered(FLinearColor(0.110f, 0.080f, 0.070f, 1.0f));
		const FSlateColor OrphanedOverrideTextColor(FLinearColor(1.0f, 0.78f, 0.74f, 1.0f));

		OrphanedOverrideRowStyle.SetEvenRowBackgroundBrush(OrphanedOverrideEvenBackground);
		OrphanedOverrideRowStyle.SetEvenRowBackgroundHoveredBrush(OrphanedOverrideEvenBackgroundHovered);
		OrphanedOverrideRowStyle.SetOddRowBackgroundBrush(OrphanedOverrideOddBackground);
		OrphanedOverrideRowStyle.SetOddRowBackgroundHoveredBrush(OrphanedOverrideOddBackgroundHovered);
		OrphanedOverrideRowStyle.SetTextColor(OrphanedOverrideTextColor);
		OrphanedOverrideRowStyle.SetSelectedTextColor(OrphanedOverrideTextColor);

		Set("PCG.OrphanedOverrideRow", OrphanedOverrideRowStyle);
	}

	// Plugin icon/editor/component icons
	Set("ClassIcon.PCGComponent", new IMAGE_BRUSH_SVG("Icons/PCG_16", Icon16x16));
	Set("ClassThumbnail.PCGComponent", new IMAGE_BRUSH_SVG("Icons/PCG_64", Icon64x64));
	Set("PCG.PluginIcon", new IMAGE_BRUSH_SVG("Icons/PCG_128", Icon128x128));
	Set("PCG.EditorIcon", new IMAGE_BRUSH_SVG("Icons/PCG_16", Icon16x16));

	// Asset/Class icons
	Set("ClassIcon.PCGVolume", new IMAGE_BRUSH_SVG("Icons/PCG_16", Icon16x16));
	Set("ClassThumbnail.PCGVolume", new IMAGE_BRUSH_SVG("Icons/PCG_64", Icon64x64));
	Set("ClassIcon.PCGWorldActor", new IMAGE_BRUSH_SVG("Icons/PCG_16", Icon16x16));
	Set("ClassThumbnail.PCGWorldActor", new IMAGE_BRUSH_SVG("Icons/PCG_64", Icon64x64));
	Set("ClassIcon.PCGPartitionActor", new IMAGE_BRUSH_SVG("Icons/PCG_16", Icon16x16));
	Set("ClassThumbnail.PCGPartitionActor", new IMAGE_BRUSH_SVG("Icons/PCG_64", Icon64x64));

	Set("ClassIcon.PCGDataAsset", new IMAGE_BRUSH_SVG("Icons/PCGAsset_16", Icon16x16));
	Set("ClassThumbnail.PCGDataAsset", new IMAGE_BRUSH_SVG("Icons/PCGAsset_64", Icon64x64));
	Set("ClassIcon.PCGGraphInterface", new IMAGE_BRUSH_SVG("Icons/PCGGraph_16", Icon16x16));
	Set("ClassThumbnail.PCGGraphInterface", new IMAGE_BRUSH_SVG("Icons/PCGGraph_64", Icon64x64));
	Set("ClassIcon.PCGGraph", new IMAGE_BRUSH_SVG("Icons/PCGGraph_16", Icon16x16));
	Set("ClassThumbnail.PCGGraph", new IMAGE_BRUSH_SVG("Icons/PCGGraph_64", Icon64x64));
	Set("ClassIcon.PCGGraphInstance", new IMAGE_BRUSH_SVG("Icons/PCGGraphInstance_16", Icon16x16));
	Set("ClassThumbnail.PCGGraphInstance", new IMAGE_BRUSH_SVG("Icons/PCGGraphInstance_64", Icon64x64));
	Set("ClassIcon.PCGSettings", new IMAGE_BRUSH_SVG("Icons/PCGSettings_16", Icon16x16));
	Set("ClassThumbnail.PCGSettings", new IMAGE_BRUSH_SVG("Icons/PCGSettings_64", Icon64x64));

	// Command icons
	Set("PCG.Command.Find", new IMAGE_BRUSH_SVG("Style/PCG_Command_Find", Icon20x20));
	Set("PCG.Command.ForceRegen", new IMAGE_BRUSH_SVG("Style/PCG_Command_ForceRegen", Icon20x20));
	Set("PCG.Command.ForceRegenClearCache", new IMAGE_BRUSH_SVG("Style/PCG_Command_ForceRegenClearCache", Icon20x20));
	Set("PCG.Command.PauseRegen", new IMAGE_BRUSH_SVG("Style/PCG_Command_PauseRegen", Icon20x20));
	Set("PCG.Command.StopRegen", new IMAGE_BRUSH_SVG("Style/PCG_Command_StopRegen", Icon20x20));
	Set("PCG.Command.OpenGraphParams", new IMAGE_BRUSH_SVG("Style/PCG_Graph_OpenGraphParams", Icon20x20));
	Set("PCG.Command.GraphSettings", new IMAGE_BRUSH_SVG("Style/PCG_Command_GraphSettings", Icon20x20));
	Set("PCG.Command.EditGraph", new IMAGE_BRUSH_SVG("Style/PCG_Command_EditGraph", Icon20x20));
	Set("PCG.Command.OpenDebugTreeTab", new IMAGE_BRUSH_SVG("Style/PCG_Command_OpenDebugTreeTab", Icon20x20));
	Set("PCG.Command.RunDeterminismTest", new IMAGE_BRUSH_SVG("Style/PCG_Command_RunDeterminismTest", Icon20x20));

	Set("PCG.Editor.CopyToClipboard", new IMAGE_BRUSH("Style/PCG_Editor_CopyToClipboard", Icon32x32));
	Set("PCG.Editor.ZoomToSelection", new IMAGE_BRUSH_SVG("Style/PCG_Editor_ZoomToSelection", Icon16x16));
	Set("PCG.Editor.Pause", new IMAGE_BRUSH_SVG("Icons/PCG_16", Icon16x16));
	Set("PCG.Editor.AlternatePause", new IMAGE_BRUSH_SVG("Style/PCG_Editor_AlternatePause", Icon16x16));
	Set("PCG.Editor.JumpTo", new IMAGE_BRUSH_SVG("Style/PCG_DebugTree_JumpTo", Icon16x16));

	// Viewport overlay styles
	{
		FLinearColor ViewportOverlayBackgroundColor = FStyleColors::Foreground.GetSpecifiedColor();
		ViewportOverlayBackgroundColor.A = 0.4f;
		Set("PCG.OverlayBackgroundBrush", new FSlateRoundedBoxBrush(ViewportOverlayBackgroundColor, 8.0f, FStyleColors::Dropdown, 1.0f));

		FLinearColor InnerPanelColor = FStyleColors::Panel.GetSpecifiedColor();
		InnerPanelColor.A = 0.85f;
		Set("PCG.OverlayInnerPanelBrush", new FSlateRoundedBoxBrush(InnerPanelColor, 6.0f));
	}

	// Text editor styles
	{
		Set("TextEditor.Border", new BOX_BRUSH("Images/TextEditorBorder", FMargin(4.0f / 16.0f), FLinearColor(0.02f, 0.02f, 0.02f, 1)));

		FSearchBoxStyle SearchBoxStyle = FCoreStyle::Get().GetWidgetStyle<FSearchBoxStyle>("SearchBox");
		SearchBoxStyle.SetLeftAlignGlassImageAndClearButton(true);
		SearchBoxStyle.SetLeftAlignSearchResultButtons(false);
		Set("TextEditor.SearchBoxStyle", SearchBoxStyle);
	}
}

const FPCGEditorStyle& FPCGEditorStyle::Get()
{
	static FPCGEditorStyle Instance;
	return Instance;
}
