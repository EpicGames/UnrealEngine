// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FPCGEditorStyle : public FSlateStyleSet
{
public:
	static void Register();
	static void Unregister();

	static const FPCGEditorStyle& Get();

private:	
	FPCGEditorStyle();
};

namespace PCGEditorStyleConstants
{
	const FName Pin_SD_SC_IN_C = TEXT("PCG.Graph.SingleData.SingleCollection.In.Connected");
	const FName Pin_SD_SC_IN_DC = TEXT("PCG.Graph.SingleData.SingleCollection.In.Disconnected");
	const FName Pin_SD_SC_OUT_C = TEXT("PCG.Graph.SingleData.SingleCollection.Out.Connected");
	const FName Pin_SD_SC_OUT_DC = TEXT("PCG.Graph.SingleData.SingleCollection.Out.Disconnected");
	
	const FName Pin_SD_MC_IN_C = TEXT("PCG.Graph.SingleData.MultiCollection.In.Connected");
	const FName Pin_SD_MC_IN_DC = TEXT("PCG.Graph.SingleData.MultiCollection.In.Disconnected");
	const FName Pin_SD_MC_OUT_C = TEXT("PCG.Graph.SingleData.MultiCollection.Out.Connected");
	const FName Pin_SD_MC_OUT_DC = TEXT("PCG.Graph.SingleData.MultiCollection.Out.Disconnected");

	const FName Pin_MD_SC_IN_C = TEXT("PCG.Graph.MultiData.SingleCollection.In.Connected");
	const FName Pin_MD_SC_IN_DC = TEXT("PCG.Graph.MultiData.SingleCollection.In.Disconnected");
	const FName Pin_MD_SC_OUT_C = TEXT("PCG.Graph.MultiData.SingleCollection.Out.Connected");
	const FName Pin_MD_SC_OUT_DC = TEXT("PCG.Graph.MultiData.SingleCollection.Out.Disconnected");

	const FName Pin_MD_MC_IN_C = TEXT("PCG.Graph.MultiData.MultiCollection.In.Connected");
	const FName Pin_MD_MC_IN_DC = TEXT("PCG.Graph.MultiData.MultiCollection.In.Disconnected");
	const FName Pin_MD_MC_OUT_C = TEXT("PCG.Graph.MultiData.MultiCollection.Out.Connected");
	const FName Pin_MD_MC_OUT_DC = TEXT("PCG.Graph.MultiData.MultiCollection.Out.Disconnected");

	const FName Pin_Param_IN_C = TEXT("PCG.Graph.Param.In.Connected");
	const FName Pin_Param_IN_DC = TEXT("PCG.Graph.Param.In.Disconnected");
	const FName Pin_Param_OUT_C = TEXT("PCG.Graph.Param.Out.Connected");
	const FName Pin_Param_OUT_DC = TEXT("PCG.Graph.Param.Out.Disconnected");

	const FName Pin_Composite_IN_C = TEXT("PCG.Graph.Composite.In.Connected");
	const FName Pin_Composite_IN_DC = TEXT("PCG.Graph.Composite.In.Disconnected");
	const FName Pin_Composite_OUT_C = TEXT("PCG.Graph.Composite.Out.Connected");
	const FName Pin_Composite_OUT_DC = TEXT("PCG.Graph.Composite.Out.Disconnected");

	const FName Pin_Required = TEXT("PCG.Graph.Pin.Required");
	const FName Pin_Graph_Dependency_C = TEXT("PCG.Graph.Pin.Dependency.Connected");
	const FName Pin_Graph_Dependency_DC = TEXT("PCG.Graph.Pin.Dependency.Disconnected");

	const FName Node_Overlay_Inactive = TEXT("PCG.Node.Overlay.Inactive");

	const FName Node_Overlay_GridSizeLabel_Active_Border = TEXT("PCG.Node.Overlay.ThisGridSizeLabel.Active.Border");

	const FName NodeFilter = TEXT("PCG.Graph.NodeFilter");
	const FName NodeConvert = TEXT("PCG.Graph.NodeConvert");

	const FName Node_ToolStart = TEXT("PCG.Node.ToolStart");
	const FName Node_DataOverride = TEXT("PCG.Node.DataOverride");
	const FName Node_ManualEdit = TEXT("PCG.Node.ManualEdit");

	const float Node_Overlay_GridSizeLabel_BorderRadius = 11.0f;
	const float Node_Overlay_GridSizeLabel_BorderStroke = 1.5f;
}