// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorTabFactories.h"

#include "PCGGraph.h"
#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "PCGEditorStyle.h"

#include "Widgets/Docking/SDockTab.h"

#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "PCGGraphEditor"

FPCGGenericTabFactory::FPCGGenericTabFactory(const FPCGGenericTabFactoryParams& InParams)
	: FWorkflowTabFactory(InParams.Id, InParams.Editor)
{	
	EnableTabPadding();

	CreateTabBodyFunc = InParams.CreateTabBodyFunc;
	check(CreateTabBodyFunc.IsSet());

	OnTabClosedDelegate = InParams.OnTabClosed;

	bIsSingleton = InParams.bSingleton;

	if (InParams.Label.IsSet())
	{
		TabLabel = InParams.Label.GetValue();
	}

	if (InParams.Icon.IsSet())
	{
		TabIcon = InParams.Icon.GetValue();
	}

	if (InParams.Role.IsSet())
	{
		TabRole = InParams.Role.GetValue();
	}

	Group = InParams.Group;
}

TSharedRef<SDockTab> FPCGGenericTabFactory::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SDockTab> Tab = FWorkflowTabFactory::SpawnTab(Info);

	if (OnTabClosedDelegate.IsBound())
	{
		Tab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateSP(this, &FPCGGenericTabFactory::OnTabClosed));
	}

	return Tab;
}

TSharedRef<SWidget> FPCGGenericTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return CreateTabBodyFunc();
}

void FPCGGenericTabFactory::OnTabClosed(TSharedRef<SDockTab> DockTab) const
{
	OnTabClosedDelegate.ExecuteIfBound(GetIdentifier());
}

FPCGGraphEditorDocumentTabFactory::FPCGGraphEditorDocumentTabFactory(TSharedPtr<class FPCGEditor> InEditorPtr, FOnCreateGraphEditorWidget InCreateGraphEditorWidgetCallback)
	: FDocumentTabFactoryForObjects<UPCGEditorGraph>(PCGEditorTabs::GraphEditorID, InEditorPtr)
	, OnCreateGraphEditorWidget(InCreateGraphEditorWidgetCallback)
{
}

TSharedPtr<FPCGEditor> FPCGGraphEditorDocumentTabFactory::GetPCGEditor() const
{
	return StaticCastSharedPtr<FPCGEditor>(HostingApp.Pin());
}

void FPCGGraphEditorDocumentTabFactory::OnTabActivated(TSharedPtr<SDockTab> InTab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(InTab->GetContent());
	GetPCGEditor()->OnGraphEditorFocused(GraphEditor);
}

void FPCGGraphEditorDocumentTabFactory::OnTabBackgrounded(TSharedPtr<SDockTab> InTab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(InTab->GetContent());
	GetPCGEditor()->OnGraphEditorBackgrounded(GraphEditor);
}

void FPCGGraphEditorDocumentTabFactory::OnTabRefreshed(TSharedPtr<SDockTab> InTab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(InTab->GetContent());
	GraphEditor->NotifyGraphChanged();
}

void FPCGGraphEditorDocumentTabFactory::SaveState(TSharedPtr<SDockTab> InTab, TSharedPtr<FTabPayload> InPayload) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(InTab->GetContent());

	FVector2f ViewLocation;
	float ZoomAmount;
	GraphEditor->GetViewLocation(ViewLocation, ZoomAmount);

	UPCGEditorGraph* EditorGraph = InPayload.IsValid() && InPayload->IsValid() ? FTabPayload_UObject::CastChecked<UPCGEditorGraph>(InPayload) : nullptr;
	UPCGGraph* Graph = EditorGraph ? EditorGraph->GetPCGGraph() : nullptr;
	if (!Graph)
	{
		return;
	}
	
	UPCGGraph* OuterGraph = Graph->GetTypedOuter<UPCGGraph>();
	UPCGGraph* MainGraph = OuterGraph ? OuterGraph : Graph;

	if (MainGraph == Graph || MainGraph->ContainsEmbeddedSubgraph(Graph))
	{
		MainGraph->LastEditedDocuments.Add(FPCGGraphDocumentInfo(Graph, ViewLocation, ZoomAmount));

		EditorGraph->SaveState();
	}
}

FText FPCGGraphEditorDocumentTabFactory::GetTabTitle(TWeakObjectPtr<UPCGEditorGraph> InWeakEditorGraph)
{
	if (UPCGEditorGraph* EditorGraph = InWeakEditorGraph.Get())
	{
		return EditorGraph->GetDisplayName();
	}

	return LOCTEXT("GraphEditorDocumentTabInvalidTitle", "Invalid");
}

TAttribute<FText> FPCGGraphEditorDocumentTabFactory::ConstructTabNameForObject(UPCGEditorGraph* InEditorGraph) const
{
	return TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FPCGGraphEditorDocumentTabFactory::GetTabTitle, TWeakObjectPtr<UPCGEditorGraph>(InEditorGraph)));
}

TSharedRef<SWidget> FPCGGraphEditorDocumentTabFactory::CreateTabBodyForObject(const FWorkflowTabSpawnInfo& InInfo, UPCGEditorGraph* InEditorGraph) const
{
	return OnCreateGraphEditorWidget.Execute(InInfo.TabInfo, InEditorGraph);
}

const FSlateBrush* FPCGGraphEditorDocumentTabFactory::GetTabIconForObject(const FWorkflowTabSpawnInfo& InInfo, UPCGEditorGraph* InEditorGraph) const
{
	return FPCGEditorStyle::Get().GetBrush("ClassIcon.PCGGraph");
}

bool FPCGGraphEditorDocumentTabFactory::SupportsObjectType(UObject* InDocumentID) const
{
	const UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(InDocumentID);

	return EditorGraph && EditorGraph->GetPCGGraph();
}

#undef LOCTEXT_NAMESPACE