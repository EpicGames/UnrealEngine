// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEditorUtils.h"

#include "PCGDataAsset.h"
#include "PCGGraph.h"
#include "Elements/PCGExecuteBlueprint.h"

#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "FrontendFilterBase.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ScopedTransaction.h"
#include "SPrimaryButton.h"
#include "Algo/Transform.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/BlueprintSupport.h"
#include "Filters/SBasicFilterBar.h"
#include "Framework/Application/SlateApplication.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/PackageName.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Types/SlateEnums.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "PCGEditorUtils"

bool PCGEditorUtils::IsAssetPCGBlueprint(const FAssetData& InAssetData)
{
	if (InAssetData.AssetClassPath != UBlueprint::StaticClass()->GetClassPathName())
	{
		return false;
	}

	const FString InNativeParentClassName = InAssetData.GetTagValueRef<FString>(FBlueprintTags::NativeParentClassPath);
	const UClass* NativeParentClass = !InNativeParentClassName.IsEmpty() ? FSoftClassPath(InNativeParentClassName).ResolveClass() : nullptr;

	return NativeParentClass && NativeParentClass->IsChildOf<UPCGBlueprintBaseElement>();
}

void PCGEditorUtils::GetParentPackagePathAndUniqueName(const UObject* OriginalObject, const FString& NewAssetTentativeName, FString& OutPackagePath, FString& OutUniqueName)
{
	if (OriginalObject == nullptr)
	{
		return;
	}

	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	FString PackageRoot, PackagePath, PackageName;
	FPackageName::SplitLongPackageName(OriginalObject->GetPackage()->GetPathName(), PackageRoot, PackagePath, PackageName);

	OutPackagePath = PackageRoot / PackagePath;
	
	if (!FPackageName::IsValidObjectPath(OutPackagePath))
	{
		OutPackagePath = FPaths::ProjectContentDir();
	}
	
	FString DummyPackageName;
	AssetTools.CreateUniqueAssetName(OutPackagePath, NewAssetTentativeName, DummyPackageName, OutUniqueName);
}

void PCGEditorUtils::ForEachAssetData(const FARFilter& InFilter, TFunctionRef<bool(const FAssetData&)> InFunc)
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssets(InFilter, AssetDataList);

	for (const FAssetData& AssetData : AssetDataList)
	{
		if (!InFunc(AssetData))
		{
			break;
		}
	}
}

void PCGEditorUtils::ForEachPCGBlueprintAssetData(TFunctionRef<bool(const FAssetData&)> InFunc)
{
	FARFilter Filter;
	Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;
	
	TSet<FTopLevelAssetPath> NativeBaseClasses;
	
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	
	// Get all sub-classes of UPCGBlueprintBaseElement (Native & BP)
	AssetRegistryModule.Get().GetDerivedClassNames({ UPCGBlueprintBaseElement::StaticClass()->GetClassPathName() }, TSet<FTopLevelAssetPath>(), NativeBaseClasses);
	NativeBaseClasses.Add(UPCGBlueprintBaseElement::StaticClass()->GetClassPathName());

	// Filter based on if asset is subclassing one of those sub-classes
	AssetRegistryModule.Get().EnumerateAssets(Filter, [&InFunc, &NativeBaseClasses](const FAssetData& InAssetData)
	{
		if(UAssetRegistryHelpers::IsAssetDataBlueprintOfClassSet(InAssetData, NativeBaseClasses))
		{
			InFunc(InAssetData);
		}
		return true;
	});
}

void PCGEditorUtils::ForEachPCGSettingsAssetData(TFunctionRef<bool(const FAssetData&)> InFunc)
{
	FARFilter Filter;
	Filter.ClassPaths.Add(UPCGSettings::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;

	ForEachAssetData(Filter, InFunc);
}

void PCGEditorUtils::ForEachPCGGraphAssetData(TFunctionRef<bool(const FAssetData&)> InFunc)
{
	FARFilter Filter;
	Filter.ClassPaths.Add(UPCGGraphInterface::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;

	ForEachAssetData(Filter, InFunc);
}

void PCGEditorUtils::ForEachPCGAssetData(TFunctionRef<bool(const FAssetData&)> InFunc)
{
	FARFilter Filter;
	Filter.ClassPaths.Add(UPCGDataAsset::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;

	ForEachAssetData(Filter, InFunc);
}

void PCGEditorUtils::ForcePCGBlueprintVariableVisibility()
{
	ForEachPCGBlueprintAssetData([](const FAssetData& AssetData)
	{
		const FString GeneratedClass = AssetData.GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath);
		FSoftClassPath BlueprintClassPath = FSoftClassPath(GeneratedClass);
		TSubclassOf<UPCGBlueprintBaseElement> BlueprintClass = BlueprintClassPath.TryLoadClass<UPCGBlueprintBaseElement>();
		if (BlueprintClass)
		{
			if (UBlueprint* Blueprint = Cast<UBlueprint>(BlueprintClass->ClassGeneratedBy))
			{
				if (Blueprint->NewVariables.IsEmpty())
				{
					return true;
				}

				const bool bHasEditOnInstanceVariables = (Blueprint->NewVariables.FindByPredicate([](const FBPVariableDescription& VarDesc) { return !(VarDesc.PropertyFlags & CPF_DisableEditOnInstance); }) != nullptr);
				if (!bHasEditOnInstanceVariables)
				{
					Blueprint->Modify();

					for (FBPVariableDescription& VarDesc : Blueprint->NewVariables)
					{
						VarDesc.PropertyFlags &= ~CPF_DisableEditOnInstance;
					}

					FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
					FKismetEditorUtilities::CompileBlueprint(Blueprint, EBlueprintCompileOptions::SkipGarbageCollection);
				}
			}
		}

		return true;
	});
}

class FFrontendFilter_PCGGraphTemplate : public FFrontendFilter
{
public:
	FFrontendFilter_PCGGraphTemplate(TSharedPtr<FFrontendFilterCategory> FilterCategory, const FString& InCategory) 
		: FFrontendFilter(FilterCategory)
		, Category(InCategory)
	{
		TArray<FString> Tokens;
		Category.ParseIntoArray(Tokens, TEXT("|"), true);

		TArray<FString> DisplayNames;
		Algo::Transform(Tokens, DisplayNames, [](const FString& Token) { return FName::NameToDisplayString(Token, /*bIsBool=*/false); });

		const FString JoinDelimiter(" | ");
		DisplayName = FText::FromString(FString::Join(DisplayNames, *JoinDelimiter));
	}

	virtual FString GetName() const override { return Category; }
	virtual FText GetDisplayName() const override { return DisplayName; }
	virtual FText GetToolTipText() const override { return FText(); }
	virtual FLinearColor GetColor() const override { return FLinearColor(0.7f, 0.7f, 0.7f); }

	virtual bool PassesFilter(const FContentBrowserItem& InItem) const override
	{
		FAssetData AssetData;
		if (InItem.Legacy_TryGetAssetData(AssetData))
		{
			FString AssetCategory = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGGraph, Category));

			if (!AssetCategory.IsEmpty() && AssetCategory.StartsWith(Category))
			{
				return true;
			}
		}

		return false;
	}

protected:
	FString Category;
	FText DisplayName;
};

class SPCGSimpleOkCancelWindow : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SPCGSimpleOkCancelWindow)
		{
		}
		SLATE_ARGUMENT(TOptional<FText>, Title)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SWindow::Construct(SWindow::FArguments()
			.Title(InArgs._Title.Get(LOCTEXT("PCGSimpleOkCancelWindowTitle", "PCG Confirmation Window")))
			.SizingRule(ESizingRule::UserSized)
			.ClientSize(FVector2D(400.0f, 500.0f))
			.SupportsMaximize(false)
			.SupportsMinimize(false));
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		FReply Reply = SWindow::OnKeyDown(MyGeometry, InKeyEvent);
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			OnCancel();
		}

		return Reply;
	}

	void OnOk()
	{
		bProceedWithAction = true;
		RequestDestroyWindow();
	}
	void OnCancel()
	{
		bProceedWithAction = false;
		RequestDestroyWindow();
	}

	bool ShouldProceedWithAction() const
	{
		return bProceedWithAction;
	}

private:
	bool bProceedWithAction = false;
};

bool PCGEditorUtils::PickGraphTemplate(FAssetData& OutAssetData, const FText& TitleOverride)
{
	auto HandleAssetSelected = [&OutAssetData](const FAssetData& InSelectedAsset)
	{
		OutAssetData = InSelectedAsset;
	};

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda(HandleAssetSelected);
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bForceShowEngineContent = true;
	AssetPickerConfig.bForceShowPluginContent = true;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.InitialThumbnailSize = EThumbnailSize::Small;
	AssetPickerConfig.Filter.ClassPaths.Add(UPCGGraph::StaticClass()->GetClassPathName());
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([](const FAssetData& AssetData)
	{
		return !AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UPCGGraph, bIsTemplate));
	});

	TSet<FString> CategoryList;
	ForEachPCGGraphAssetData([&CategoryList](const FAssetData& Asset)
	{
		if (Asset.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UPCGGraph, bIsTemplate)))
		{
			FString Category = Asset.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGGraph, Category));
			if (!Category.IsEmpty())
			{
				CategoryList.Add(MoveTemp(Category));
			}
		}
		return true;
	});

	TSharedPtr<FFrontendFilterCategory> FilterCategory = MakeShared<FFrontendFilterCategory>(
		LOCTEXT("GraphTemplateCategoryName", "PCG Graph Template Categories"),
		LOCTEXT("GraphTemplateCategoryName_Tooltip", "Filter templates by categories.")
	);

	for (const FString& Category : CategoryList)
	{
		AssetPickerConfig.ExtraFrontendFilters.Add(MakeShared<FFrontendFilter_PCGGraphTemplate>(FilterCategory, Category));
	}

	// This is so that we can remove the "Other Filters" section easily
	AssetPickerConfig.bUseSectionsForCustomFilterCategories = true;
	// Make sure we only show PCG filters to avoid confusion 
	AssetPickerConfig.OnExtendAddFilterMenu = FOnExtendAddFilterMenu::CreateLambda([](UToolMenu* InToolMenu)
	{
		// "AssetFilterBarFilterAdvancedAsset" taken from SAssetFilterBar.h PopulateAddFilterMenu()
		InToolMenu->RemoveSection("AssetFilterBarFilterAdvancedAsset");
		InToolMenu->RemoveSection("Other Filters");
	});
	
	AssetPickerConfig.bAddFilterUI = !CategoryList.IsEmpty();

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	FText Title = (TitleOverride.IsEmpty() ? LOCTEXT("SelectTemplateDialogTitle", "Initialize from Graph Template...") : TitleOverride);

	TSharedPtr<SPCGSimpleOkCancelWindow> Dialog = SNew(SPCGSimpleOkCancelWindow)
		.Title(Title);

	// Finally, bind double-click & enter to the asset picker
	auto HandleAssetActivated = [HandleAssetSelected, WeakDialog = Dialog.ToWeakPtr()](const TArray<FAssetData>& ActivatedAssets, EAssetTypeActivationMethod::Type ActivationMethod)
	{
		if ((ActivationMethod == EAssetTypeActivationMethod::DoubleClicked || ActivationMethod == EAssetTypeActivationMethod::Opened) && ActivatedAssets.Num() == 1 && ActivatedAssets[0].IsValid())
		{
			HandleAssetSelected(ActivatedAssets[0]);

			if (TSharedPtr<SPCGSimpleOkCancelWindow> PinnedDialog = WeakDialog.Pin())
			{
				PinnedDialog->OnOk();
			}
		}
	};

	AssetPickerConfig.OnAssetsActivated = FOnAssetsActivated::CreateLambda(HandleAssetActivated);

	Dialog->SetContent(
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(5.0f)
		[
			SNew(SBox)
			.WidthOverride(300.0f)
			.HeightOverride(400.0f)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.OnClicked_Lambda([&OutAssetData, WeakDialog = Dialog.ToWeakPtr()]()
				{
					if (TSharedPtr<SPCGSimpleOkCancelWindow> PinnedDialog = WeakDialog.Pin())
					{
						PinnedDialog->OnOk();
					}

					OutAssetData = FAssetData();
					return FReply::Handled();
				})
				.Text(LOCTEXT("InitializeFromEmptyTemplateButton", "Create empty graph"))
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SPrimaryButton)
				.OnClicked_Lambda([WeakDialog = Dialog.ToWeakPtr()]()
				{
					if (TSharedPtr<SPCGSimpleOkCancelWindow> PinnedDialog = WeakDialog.Pin())
					{
						PinnedDialog->OnOk();
					}

					return FReply::Handled();
				})
				.IsEnabled_Lambda([&OutAssetData]()
				{
					return OutAssetData.IsValid();
				})
				.Text(LOCTEXT("InitializeFromTemplateButton", "Initialize From Template"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked_Lambda([WeakDialog = Dialog.ToWeakPtr()]()
				{
					if (TSharedPtr<SPCGSimpleOkCancelWindow> PinnedDialog = WeakDialog.Pin())
					{
						PinnedDialog->OnCancel();
					}

					return FReply::Handled();
				})
				.Text(LOCTEXT("CancelButton", "Cancel"))
			]
		]);

	FSlateApplication::Get().AddModalWindow(Dialog.ToSharedRef(), FGlobalTabmanager::Get()->GetRootWindow());
	return Dialog->ShouldProceedWithAction();
}

void PCGEditorUtils::OpenAssetOrMoveToActorOrComponent(const FSoftObjectPath& InPath)
{
	if (!GEditor || !GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		return;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	// We need to know which class the object is. If it is not an asset, we can't jump to it.
	const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(InPath);
	if (!AssetData.IsValid() || AssetData.IsRedirector())
	{
		return;
	}

	const UClass* AssetClass = AssetData.GetClass(EResolveClass::Yes);
	if (!AssetClass)
	{
		return;
	}

	// Don't jump to a world/level, would be pretty destructive to change levels
	if (AssetClass->IsChildOf<UWorld>() || AssetClass->IsChildOf<ULevel>())
	{
		return;
	}

	// If it is not an actor or an actor component, we can try to open an editor for it.
	if (!AssetClass->IsChildOf<AActor>() && !AssetClass->IsChildOf<UActorComponent>())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InPath);
		return;
	}
	
	// Otherwise, try to resolve the actor/actor component to be able to jump to it. Never try to load it.
	// Encapsulate the move in a transaction to be able to undo the selection.
	if (const UObject* Object = InPath.ResolveObject())
	{	
		FScopedTransaction Transaction(LOCTEXT("PCGHyperLinkSoftObjectPath", "[PCG] Jump to Actor/Component"));
		bool bSuccess = false;
		
		if (const USceneComponent* SceneComponent = Cast<USceneComponent>(Object))
		{
			GEditor->MoveViewportCamerasToComponent(SceneComponent, /*bActiveViewportOnly=*/true);
			GEditor->SelectNone(/*bNoteSelectionChange=*/ false, /*bDeselectBSPSurfsfalse=*/ true);
			GEditor->SelectComponent(const_cast<USceneComponent*>(SceneComponent), /*bInSelected=*/ true, /*bNotify=*/ true);
			bSuccess = true;
		}
		else if (const AActor* Actor = Cast<AActor>(Object))
		{
			GEditor->MoveViewportCamerasToActor(const_cast<AActor&>(*Actor), /*bActiveViewportOnly=*/true);
			GEditor->SelectNone(/*bNoteSelectionChange=*/ false, /*bDeselectBSPSurfsfalse=*/ true);
			GEditor->SelectActor(const_cast<AActor*>(Actor), /*bInSelected=*/ true, /*bNotify=*/ true);
			bSuccess = true;
		}
		else if (const UActorComponent* ActorComponent = Cast<UActorComponent>(Object))
		{
			if (AActor* OwnerActor = ActorComponent->GetOwner())
			{
				GEditor->MoveViewportCamerasToActor(*OwnerActor, /*bActiveViewportOnly=*/true);
				GEditor->SelectNone(/*bNoteSelectionChange=*/ false, /*bDeselectBSPSurfsfalse=*/ true);
				GEditor->SelectComponent(const_cast<UActorComponent*>(ActorComponent), /*bInSelected=*/ true, /*bNotify=*/ true);
				bSuccess = true;
			}
		}

		if (!bSuccess)
		{
			Transaction.Cancel();
		}
	}
}

class SPCGSimpleConfirmationDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPCGSimpleConfirmationDialog) {}
		SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(TAttribute<FText>, Message)

		SLATE_DEFAULT_SLOT(FArguments, AdditionalContent)

		SLATE_EVENT(FSimpleDelegate, OnOkEvent)
		SLATE_EVENT(FSimpleDelegate, OnCancelEvent)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ParentWindow = InArgs._ParentWindow;
		OnConfirmEvent = InArgs._OnOkEvent;
		OnCancelEvent = InArgs._OnCancelEvent;

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoHeight()
			.Padding(10, 16, 10, 0)
			[
				SNew(STextBlock)
				.Text(InArgs._Message)
				.AutoWrapText(true)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(6)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(20, 16, 12, 0)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.FillWidth(1)
				[
					SNew(SBox)
					.MinDesiredWidth(50)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SAssignNew(ConfirmButton, SButton)
						.Text(LOCTEXT("SPCGSimpleConfirmationDialogConfirmButton", "Confirm"))
						.OnClicked(this, &SPCGSimpleConfirmationDialog::OnConfirm)
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(12, 16, 20, 0)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.FillWidth(1)
				[
					SNew(SBox)
					.MinDesiredWidth(50)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SAssignNew(CancelButton, SButton)
						.Text(LOCTEXT("SPCGSimpleConfirmationDialogCancelButton", "Cancel"))
						.ButtonColorAndOpacity(FSlateColor::UseSubduedForeground())
						.OnClicked(this, &SPCGSimpleConfirmationDialog::OnCancel)
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				InArgs._AdditionalContent.Widget
			]
		];
	}

	bool ShouldProceed() const { return bShouldProceed; }

	virtual bool SupportsKeyboardFocus() const override { return true; }

	virtual FReply OnKeyDown(const FGeometry&, const FKeyEvent& InKeyEvent) override
	{
		const FKey Key = InKeyEvent.GetKey();

		if (Key == EKeys::Escape)
		{
			return OnCancel();
		}

		return SCompoundWidget::OnKeyDown(FGeometry(), InKeyEvent);
	}

	TSharedPtr<SButton> GetConfirmButton() const { return ConfirmButton; }
	TSharedPtr<SButton> GetCancelButton() const { return CancelButton; }

private:
	void RequestDestroyWindow() const
	{
		if (TSharedPtr<SWindow> Window = ParentWindow.Pin())
		{
			Window->RequestDestroyWindow();
		}
	}

	FReply OnConfirm()
	{
		bShouldProceed = true;
		OnConfirmEvent.ExecuteIfBound();
		RequestDestroyWindow();
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		bShouldProceed = false;
		OnCancelEvent.ExecuteIfBound();
		RequestDestroyWindow();
		return FReply::Handled();
	}

	bool bShouldProceed = false;
	TWeakPtr<SWindow> ParentWindow = nullptr;
	TSharedPtr<SButton> ConfirmButton = nullptr;
	TSharedPtr<SButton> CancelButton = nullptr;

	FSimpleDelegate OnConfirmEvent;
	FSimpleDelegate OnCancelEvent;
};

void PCGEditorUtils::OpenConfirmationDialog(FText InMessage, TFunction<void()> OnOk, TFunction<void()> OnCancel, TSharedPtr<SWidget> InAdditionalContent)
{
	TSharedRef<SWindow> Window = SNew(SWindow)
		.CreateTitleBar(false)
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.ActivationPolicy(EWindowActivationPolicy::Always)
		.FocusWhenFirstShown(true);

	TSharedPtr<SPCGSimpleConfirmationDialog> Dialog;
	Window->SetContent(
		SAssignNew(Dialog, SPCGSimpleConfirmationDialog)
		.ParentWindow(Window)
		.Message(MoveTemp(InMessage))
		.AdditionalContent()
		[
			InAdditionalContent.ToSharedRef()
		]
		.OnOkEvent(FSimpleDelegate::CreateLambda(OnOk))
		.OnCancelEvent(FSimpleDelegate::CreateLambda(OnCancel)));

	if (Dialog.IsValid() && Dialog->GetConfirmButton().IsValid())
	{
		Window->SetWidgetToFocusOnActivate(Dialog->GetConfirmButton());
	}

	FSlateApplication::Get().AddModalWindow(Window, FSlateApplication::Get().GetActiveTopLevelWindow());
}

#undef LOCTEXT_NAMESPACE
