// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Widgets/SNullWidget.h"

template <typename FuncType> class TFunctionRef;
struct FAssetData;
struct FARFilter;
class FString;
class UObject;
class SWidget;

namespace PCGEditorUtils
{
	bool IsAssetPCGBlueprint(const FAssetData& InAssetData);

	/** 
	* From an object, get its parent package and a unique name 
	* For example, if you want to create a new asset next to the original object, it will return the parent package of the original package
	* and a unique name for the new asset.
	*/
	void GetParentPackagePathAndUniqueName(const UObject* OriginalObject, const FString& NewAssetTentativeName, FString& OutPackagePath, FString& OutUniqueName);

	/** Methods related to visiting asset data. Ends early if InFunc returns false */
	void ForEachAssetData(const FARFilter& Filter, TFunctionRef<bool(const FAssetData&)> InFunc);

	void ForEachPCGBlueprintAssetData(TFunctionRef<bool(const FAssetData&)> InFunc);
	void ForEachPCGSettingsAssetData(TFunctionRef<bool(const FAssetData&)> InFunc);
	void ForEachPCGGraphAssetData(TFunctionRef<bool(const FAssetData&)> InFunc);
	void ForEachPCGAssetData(TFunctionRef<bool(const FAssetData&)> InFunc);

	/** Asset deprecation methods */
	void ForcePCGBlueprintVariableVisibility();

	/** Opens a template picker dialog. Returns true if user selected 'ok'. */
	bool PickGraphTemplate(FAssetData& OutData, const FText& InTitleOverride = FText());

	/** Open an asset or jumps to an actor/component and select it. Will load it, and won't load a new level*/
	void OpenAssetOrMoveToActorOrComponent(const FSoftObjectPath& InPath);

	void OpenConfirmationDialog(FText InMessage, TFunction<void()> OnOk, TFunction<void()> OnCancel = [] {}, TSharedPtr<SWidget> InAdditionalContent = SNullWidget::NullWidget);
} // namespace PCGEditorUtils
