// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGGraphFactory.h"

#include "PCGEditorSettings.h"
#include "PCGEditorUtils.h"
#include "PCGGraph.h"

#include "AssetRegistry/AssetData.h"

/////////////////////////
// PCGGraph
/////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGraphFactory)

#define LOCTEXT_NAMESPACE "PCGGraphFactory"

UPCGGraphFactory::UPCGGraphFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UPCGGraph::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UPCGGraphFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UPCGGraph* NewGraph = nullptr;
	
	if (TemplateGraph)
	{
		NewGraph = Cast<UPCGGraph>(StaticDuplicateObject(TemplateGraph, InParent, InName, Flags, InClass));
		if (NewGraph)
		{
			NewGraph->bIsTemplate = false;
		}
	}
	else
	{
		NewGraph = NewObject<UPCGGraph>(InParent, InClass, InName, Flags);
	}

	return NewGraph;
}

bool UPCGGraphFactory::ShouldShowInNewMenu() const
{
	return true;
}

bool UPCGGraphFactory::ConfigureProperties()
{
	// Early out if this is a standalone factory that does not allow templates.
	if (bSkipTemplateSelection)
	{
		return true;
	}

	// Early out if the editor config says we don't want to have a dialog for it.
	const UPCGEditorSettings* EditorSettings = GetDefault<UPCGEditorSettings>();
	if (EditorSettings && !EditorSettings->bShowTemplatePickerOnNewGraph)
	{
		return true;
	}

	// Check if templates actually exist
	bool bTemplatesExist = false;
	PCGEditorUtils::ForEachPCGGraphAssetData([&bTemplatesExist](const FAssetData& AssetData)
	{
		if (AssetData.IsInstanceOf<UPCGGraph>())
		{
			const bool bIsTemplate = AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UPCGGraph, bIsTemplate));
			if (bIsTemplate)
			{
				bTemplatesExist = true;
				return false;
			}
		}

		return true;
	});

	if (!bTemplatesExist)
	{
		return true;
	}

	// Select template
	FAssetData SelectedTemplate;
	const bool bPicked = PCGEditorUtils::PickGraphTemplate(SelectedTemplate, LOCTEXT("TemplatePickerTitle", "Create Graph From Template..."));

	if (bPicked && SelectedTemplate.IsValid())
	{
		TemplateGraph = Cast<UPCGGraph>(SelectedTemplate.GetAsset());
	}

	return bPicked;
}

/////////////////////////
// PCGGraphInstance
/////////////////////////

UPCGGraphInstanceFactory::UPCGGraphInstanceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UPCGGraphInstance::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UPCGGraphInstanceFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UPCGGraphInstance* PCGGraphInstance = NewObject<UPCGGraphInstance>(InParent, InClass, InName, Flags);

	if (ParentGraph)
	{
		PCGGraphInstance->SetGraph(ParentGraph);
	}
	
	return PCGGraphInstance;
}

bool UPCGGraphInstanceFactory::ShouldShowInNewMenu() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
