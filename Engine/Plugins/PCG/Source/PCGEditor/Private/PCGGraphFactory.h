// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "PCGGraphFactory.generated.h"

class UPCGGraph;
class UPCGGraphInterface;

UCLASS(hidecategories=Object)
class UPCGGraphFactory : public UFactory
{
	GENERATED_BODY()

public:
	UPCGGraphFactory(const FObjectInitializer& ObjectInitializer);

	UPROPERTY()
	TObjectPtr<UPCGGraph> TemplateGraph;

	/** Disables template selection, even if some exist. Useful when creating a graph from a standalone factory. */
	UPROPERTY()
	bool bSkipTemplateSelection = false;

	//~UFactory interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override;
};

UCLASS(hidecategories = Object)
class UPCGGraphInstanceFactory : public UFactory
{
	GENERATED_BODY()

public:
	UPCGGraphInstanceFactory(const FObjectInitializer& ObjectInitializer);

	UPROPERTY()
	TObjectPtr<UPCGGraphInterface> ParentGraph;
	
	//~UFactory interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
};
