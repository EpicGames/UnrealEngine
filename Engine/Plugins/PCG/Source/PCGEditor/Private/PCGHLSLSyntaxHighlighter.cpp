// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGHLSLSyntaxHighlighter.h"

#include "Compute/PCGHLSLSyntaxTokenizer.h"

#include "PCGEditorSettings.h"
#include "PCGEditorStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/SlateTextUnderlineLineHighlighter.h"

namespace PCGHLSLSyntaxHighlighterHelpers
{
	FHLSLSyntaxHighlighterMarshaller::FSyntaxTextStyle BuildSyntaxTextStyle(int32 InFontSize)
	{
		auto GetStyle = [InFontSize](const TCHAR* StyleName)
		{
			FTextBlockStyle Style(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>(StyleName));
			Style.Font.Size = InFontSize;
			return Style;
		};

		return FHLSLSyntaxHighlighterMarshaller::FSyntaxTextStyle(
			GetStyle(TEXT("SyntaxHighlight.SourceCode.Normal")),
			GetStyle(TEXT("SyntaxHighlight.SourceCode.Operator")),
			GetStyle(TEXT("SyntaxHighlight.SourceCode.Keyword")),
			GetStyle(TEXT("SyntaxHighlight.SourceCode.String")),
			GetStyle(TEXT("SyntaxHighlight.SourceCode.Number")),
			GetStyle(TEXT("SyntaxHighlight.SourceCode.Comment")),
			GetStyle(TEXT("SyntaxHighlight.SourceCode.PreProcessorKeyword")),
			GetStyle(TEXT("SyntaxHighlight.SourceCode.Error")));
	}
}

TSharedRef<FPCGHLSLSyntaxHighlighter> FPCGHLSLSyntaxHighlighter::Create(const FSyntaxTextStyle& InSyntaxTextStyle)
{
	return MakeShareable(new FPCGHLSLSyntaxHighlighter(MakeShareable(new FPCGHLSLSyntaxTokenizer()), InSyntaxTextStyle));
}

void FPCGHLSLSyntaxHighlighter::SetCompilerMessages(const FPCGCompilerDiagnostics& InCompilerMessages)
{
	CompilerMessages.Reset();

	for (const FPCGCompilerDiagnostic& Message : InCompilerMessages.Diagnostics)
	{
		// Line adjustment required to work correctly in text box.
		CompilerMessages.Add(Message.Line - 1, Message);
	}

	MakeDirty();
}

void FPCGHLSLSyntaxHighlighter::ClearCompilerMessages()
{
	if (!CompilerMessages.IsEmpty())
	{
		CompilerMessages.Reset();
		MakeDirty();
	}
}

FPCGHLSLSyntaxHighlighter::FPCGHLSLSyntaxHighlighter(TSharedPtr<ISyntaxTokenizer> InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle) :
	FHLSLSyntaxHighlighterMarshaller(MoveTemp(InTokenizer), InSyntaxTextStyle)
{
}

void FPCGHLSLSyntaxHighlighter::Tick(float DeltaTime)
{
	const int32 FontSize = GetDefault<UPCGEditorSettings>()->CodeEditorFontSize;
	if (FontSize != CachedFontSize)
	{
		CachedFontSize = FontSize;
		SyntaxTextStyle = PCGHLSLSyntaxHighlighterHelpers::BuildSyntaxTextStyle(FontSize);
		MakeDirty();
	}
}

TStatId FPCGHLSLSyntaxHighlighter::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FPCGHLSLSyntaxHighlighter, STATGROUP_Tickables);
}

void FPCGHLSLSyntaxHighlighter::ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout,	TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines)
{
	LineHighlightsToAdd.Empty();
	FHLSLSyntaxHighlighterMarshaller::ParseTokens(SourceString, TargetTextLayout, TokenizedLines);

	TargetTextLayout.SetLineHighlights(LineHighlightsToAdd);
}

FTextLayout::FNewLineData FPCGHLSLSyntaxHighlighter::ProcessTokenizedLine(const ISyntaxTokenizer::FTokenizedLine& TokenizedLine, const int32& LineNumber, const FString& SourceString, EParseState& CurrentParseState)
{
	FTextLayout::FNewLineData LineData = FHLSLSyntaxHighlighterMarshaller::ProcessTokenizedLine(TokenizedLine, LineNumber, SourceString, CurrentParseState);

	FTextBlockStyle ErrorTextStyle = SyntaxTextStyle.ErrorTextStyle;
	TSharedPtr<FSlateTextUnderlineLineHighlighter> UnderlineLineHighlighter = FSlateTextUnderlineLineHighlighter::Create(ErrorTextStyle.UnderlineBrush, ErrorTextStyle.Font, ErrorTextStyle.ColorAndOpacity, ErrorTextStyle.ShadowOffset, ErrorTextStyle.ShadowColorAndOpacity);
	
	TArray<const FPCGCompilerDiagnostic*> Diagnostics;
	CompilerMessages.MultiFindPointer(LineNumber, Diagnostics);
		
	for (const FPCGCompilerDiagnostic* Diagnostic : Diagnostics)
	{
		// If no column specified, underline the whole source line.
		const bool bColumnsSpecified = Diagnostic->ColumnStart != -1 && Diagnostic->ColumnEnd != -1;

		// ColumnStart/ColumnEnd are 1-based, closed interval. FTextRange is 0 based, half-closed interval. 
		FTextRange UnderlineRange(Diagnostic->ColumnStart - 1, Diagnostic->ColumnEnd);

		// The highlighting lines have to match the runs and not exceed their bounds, so we chop them up. 
		for (TSharedRef<IRun> Run : LineData.Runs)
		{
			if (!bColumnsSpecified || !Run->GetTextRange().Intersect(UnderlineRange).IsEmpty())
			{
				LineHighlightsToAdd.Add(FTextLineHighlight(
					LineNumber,
					bColumnsSpecified ? UnderlineRange : Run->GetTextRange(),
					FSlateTextUnderlineLineHighlighter::DefaultZIndex,
					UnderlineLineHighlighter.ToSharedRef()));
			}
		}
	}

	return LineData;
}

FPCGHLSLSyntaxHighlighter::FSyntaxTextStyle FPCGHLSLSyntaxHighlighter::GetSyntaxTextStyle()
{
	return PCGHLSLSyntaxHighlighterHelpers::BuildSyntaxTextStyle(GetDefault<UPCGEditorSettings>()->CodeEditorFontSize);
}
