// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Compute/PCGCompilerDiagnostic.h"

#include "Tickable.h"
#include "TickableEditorObject.h"
#include "Framework/Text/SyntaxTokenizer.h"
#include "Text/HLSLSyntaxHighlighterMarshaller.h"

class FTextLayout;
struct FPCGCompilerDiagnostic;

/**
 * Get/set the raw text to/from a text layout, and also inject syntax highlighting for our rich-text markup
 */
class FPCGHLSLSyntaxHighlighter : public FHLSLSyntaxHighlighterMarshaller, public FTickableEditorObject
{
public:
	static TSharedRef<FPCGHLSLSyntaxHighlighter> Create(const FSyntaxTextStyle& InSyntaxTextStyle = GetSyntaxTextStyle());
	static FSyntaxTextStyle GetSyntaxTextStyle();

	void SetCompilerMessages(const FPCGCompilerDiagnostics& InCompilerMessages);
	void ClearCompilerMessages();

	// @todo_pcg: Replace tick with an OnSettingChanged delegate on UPCGEditorSettings.
	//~ Begin FTickableEditorObject interface
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject interface

protected:
	FPCGHLSLSyntaxHighlighter(TSharedPtr<ISyntaxTokenizer> InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle);

	virtual void ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines) override;
	virtual FTextLayout::FNewLineData ProcessTokenizedLine(const ISyntaxTokenizer::FTokenizedLine& TokenizedLine, const int32& LineNumber, const FString& SourceString, EParseState& CurrentParseState) override;

	TMultiMap<int32 /*Line*/, FPCGCompilerDiagnostic> CompilerMessages;

private:
	int32 CachedFontSize = 0;
	TArray<FTextLineHighlight> LineHighlightsToAdd;
};
