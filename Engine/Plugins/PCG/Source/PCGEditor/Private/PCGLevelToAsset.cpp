// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGLevelToAsset.h"

#include "PCGEditorModule.h"
#include "PCGEditorSettings.h"
#include "PCGEditorUtils.h"
#include "PCGAssetExporterUtils.h"

#include "Data/PCGBasePointData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPointArrayData.h"
#include "Data/Registry/PCGDataFunctionRegistryHelpers.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadata.h"

#include "CoreGlobals.h"
#include "ContentBrowserModule.h"
#include "FileHelpers.h"
#include "IContentBrowserSingleton.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/LevelStreaming.h"
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGLevelToAsset)

extern PCG_API TAutoConsoleVariable<bool> CVarPCGEnablePointArrayData;

void UPCGLevelToAsset::CreateOrUpdatePCGAssets(const TArray<FAssetData>& WorldAssets, const FPCGAssetExporterParameters& InParameters, TSubclassOf<UPCGLevelToAsset> ExporterSubclass)
{
	TArray<UPackage*> PackagesToSave;
	FPCGAssetExporterParameters Parameters = InParameters;

	if (WorldAssets.Num() > 1)
	{
		Parameters.bOpenSaveDialog = false;
	}

	for (const FAssetData& WorldAsset : WorldAssets)
	{
		if (UPackage* Package = CreateOrUpdatePCGAsset(TSoftObjectPtr<UWorld>(WorldAsset.GetSoftObjectPath()), Parameters, ExporterSubclass))
		{
			PackagesToSave.Add(Package);
		}
	}

	// Save the file(s)
	if (!PackagesToSave.IsEmpty() && Parameters.bSaveOnExportEnded)
	{
		// Set GIsRunningUnattendedScript to 'true' if it isn't already set while running commandlets
		TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, GIsRunningUnattendedScript || IsRunningCommandlet());
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false);
	}
}

UPackage* UPCGLevelToAsset::CreateOrUpdatePCGAsset(TSoftObjectPtr<UWorld> WorldPath, const FPCGAssetExporterParameters& Parameters, TSubclassOf<UPCGLevelToAsset> ExporterSubclass)
{
	return CreateOrUpdatePCGAsset(WorldPath.LoadSynchronous(), Parameters, ExporterSubclass);
}

UPackage* UPCGLevelToAsset::CreateOrUpdatePCGAsset(UWorld* World, const FPCGAssetExporterParameters& InParameters, TSubclassOf<UPCGLevelToAsset> ExporterSubclass)
{
	if (!World)
	{
		return nullptr;
	}

	if (ULevelStreaming::FindStreamingLevel(World->PersistentLevel) != nullptr)
	{
		UE_LOGF(LogPCGEditor, Error, "Unable to create asset from sub-level. If level is currently loaded as a sub-level, unload it first and try exporting again.");
		return nullptr;
	}

	UPCGLevelToAsset* Exporter = nullptr;
	if (ExporterSubclass)
	{
		Exporter = NewObject<UPCGLevelToAsset>(GetTransientPackage(), ExporterSubclass);
	}
	else
	{
		Exporter = NewObject<UPCGLevelToAsset>(GetTransientPackage());
	}

	if (!Exporter)
	{
		UE_LOGF(LogPCGEditor, Error, "Unable to create Level to Settings exporter.");
		return nullptr;
	}

	Exporter->WorldToExport = World;

	FPCGAssetExporterParameters Parameters = InParameters;
	Parameters.AssetName = World->GetName() + TEXT("_PCG");

	if (InParameters.AssetPath.IsEmpty() && World->GetPackage())
	{
		Parameters.AssetPath = FPackageName::GetLongPackagePath(World->GetPackage()->GetName());
	}

	return UPCGAssetExporterUtils::CreateAsset(Exporter, Parameters);
}

UPackage* UPCGLevelToAsset::UpdateAsset(const FAssetData& PCGAsset)
{
	UPCGDataAsset* Asset = Cast<UPCGDataAsset>(PCGAsset.GetAsset());
	if (!Asset)
	{
		UE_LOGF(LogPCGEditor, Error, "Asset '%ls' isn't a PCG data asset or could not be properly loaded.", *PCGAsset.GetObjectPathString());
		return nullptr;
	}

	UPackage* Package = Asset->GetPackage();
	if (!Package)
	{
		UE_LOGF(LogPCGEditor, Error, "Unable to retrieve package from Asset '%ls'.", *PCGAsset.GetObjectPathString());
		return nullptr;
	}

	TSoftObjectPtr<UWorld> WorldPtr(Asset->ObjectPath);
	UWorld* World = WorldPtr.LoadSynchronous();

	if (!World)
	{
		UE_LOGF(LogPCGEditor, Error, "PCG asset was unable to load world '%ls'.", *Asset->ObjectPath.ToString());
		return nullptr;
	}

	WorldToExport = World;

	if (ExportAsset(Package->GetPathName(), Asset))
	{
		FCoreUObjectDelegates::BroadcastOnObjectModified(Asset);
		return Package;
	}
	else
	{
		return nullptr;
	}
}

bool UPCGLevelToAsset::ExportAsset(const FString& PackageName, UPCGDataAsset* Asset)
{
	return BP_ExportWorld(WorldToExport, PackageName, Asset);
}

bool UPCGLevelToAsset::BP_ExportWorld_Implementation(UWorld* World, const FString& PackageName, UPCGDataAsset* Asset)
{
	check(World && Asset);
	Asset->ObjectPath = FSoftObjectPath(World);
	Asset->Description = FText::Format(NSLOCTEXT("PCGLevelToAsset", "DefaultDescriptionOnExportedLevel", "Generated from world: {0}"), FText::FromString(World->GetName()));
	Asset->ExporterClass = GetClass();

	FPCGDataCollection& DataCollection = Asset->Data;
	DataCollection.TaggedData.Reset();

	// Select proper point data class
	UClass* PointDataClass = CVarPCGEnablePointArrayData.GetValueOnAnyThread() ? UPCGPointArrayData::StaticClass() : UPCGPointData::StaticClass();

	// Create Root Data
	UPCGBasePointData* RootPointData = NewObject<UPCGBasePointData>(Asset, PointDataClass);
	UPCGMetadata* RootMetadata = RootPointData->MutableMetadata();

	RootMetadata->CreateAttribute<FString>(TEXT("Name"), World->GetName(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);
	RootMetadata->CreateAttribute<FSoftObjectPath>(TEXT("Source"), FSoftObjectPath(World), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);

	// Add to data collection
	{
		FPCGTaggedData& RootsTaggedData = DataCollection.TaggedData.Emplace_GetRef();
		RootsTaggedData.Data = RootPointData;
		RootsTaggedData.Pin = TEXT("Root");
	}

	// Create PCGData
	UPCGBasePointData* PointData = NewObject<UPCGBasePointData>(Asset, PointDataClass);
	UPCGMetadata* PointMetadata = PointData->MutableMetadata();

	// Add to data collection
	{
		FPCGTaggedData& PointsTaggedData = DataCollection.TaggedData.Emplace_GetRef();
		PointsTaggedData.Data = PointData;
		PointsTaggedData.Pin = TEXT("Points");
	}

	// Common data shared across steps
	FBox AllActorBounds(EForceInit::ForceInit);

	// Hardcoded attributes
	const FName& MaterialAttributeName = PCGLevelToAssetConstants::MaterialAttributeName;
	const FName& MeshAttributeName = PCGLevelToAssetConstants::MeshAttributeName;
	const FName& SkeletalMeshAttributeName = PCGLevelToAssetConstants::SkeletalMeshAttributeName;
	const FName& HierarchyDepthAttributeName = PCGLevelToAssetConstants::HierarchyDepthAttributeName;
	const FName& ActorIndexAttributeName = PCGLevelToAssetConstants::ActorIndexAttributeName;
	const FName& ParentIndexAttributeName = PCGLevelToAssetConstants::ParentIndexAttributeName;
	const FName& RelativeTransformAttributeName = PCGLevelToAssetConstants::RelativeTransformAttributeName;

	// Attribute setup on the points
	FPCGMetadataAttribute<FSoftObjectPath>* MaterialAttribute = PointMetadata->CreateAttribute<FSoftObjectPath>(MaterialAttributeName, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);
	FPCGMetadataAttribute<FSoftObjectPath>* MeshAttribute = PointMetadata->CreateAttribute<FSoftObjectPath>(MeshAttributeName, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);
	FPCGMetadataAttribute<FSoftObjectPath>* SkeletalMeshAttribute = PointMetadata->CreateAttribute<FSoftObjectPath>(SkeletalMeshAttributeName, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);
	FPCGMetadataAttribute<int64>* HierarchyDepthAttribute = PointMetadata->CreateAttribute<int64>(HierarchyDepthAttributeName, 0, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);
	FPCGMetadataAttribute<int64>* ActorIndexAttribute = PointMetadata->CreateAttribute<int64>(ActorIndexAttributeName, -1, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);
	FPCGMetadataAttribute<int64>* ParentIndexAttribute = PointMetadata->CreateAttribute<int64>(ParentIndexAttributeName, -1, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);
	FPCGMetadataAttribute<FTransform>* RelativeTransformAttribute = PointMetadata->CreateAttribute<FTransform>(RelativeTransformAttributeName, FTransform::Identity, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);

	// Map from raw/unsanitized tag name to corresponding attribute - shared across all actor iterations.
	TMap<FName, FPCGMetadataAttributeBase*> TagToAttributeMap;

	// Relationship Tag:SanitizedName is many:1, so keep track of which sanitized names are created so we don't attempt to create the same one multiple times.
	TSet<FName> SanitizedAttributeNames;

	// Hierarchy root point
	{
		PointData->SetNumPoints(1);
		PointData->SetTransform(FTransform::Identity);
		PointData->SetDensity(1.0f);
		PointData->SetBoundsMin(FVector::Zero());
		PointData->SetBoundsMax(FVector::Zero());
		PointData->SetSteepness(1.0f);
		PointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::MetadataEntry | EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax | EPCGPointNativeProperties::Seed);

		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		PointMetadata->InitializeOnSet(MetadataEntryRange[0]);
		ActorIndexAttribute->SetValue(MetadataEntryRange[0], 0);
	}

	// Make sure all actors are loaded
	TMap<UWorldPartition*, TArray<FWorldPartitionReference>> ActorReferencesPerWorldPartition;

	auto LoadAllActors = [&ActorReferencesPerWorldPartition](UWorldPartition* InWorldPartition) -> bool
	{
		if (InWorldPartition)
		{
			TArray<FWorldPartitionReference> ActorReferences;
			InWorldPartition->LoadAllActors(ActorReferences);
			ActorReferencesPerWorldPartition.Emplace(InWorldPartition, MoveTemp(ActorReferences));

			return true;
		}

		return false;
	};

	// Load all actors
	bool bProcessedNewActors = LoadAllActors(World->GetWorldPartition());
		
	// Make sure to call this once as non World Partition levels do not load new actors
	World->BlockTillLevelStreamingCompleted();

	// Load all level instances and their actors recursively
	TSet<ULevelStreaming*> ProcessedLevelStreamings;
	
	while(bProcessedNewActors)
	{
		bProcessedNewActors = false;

		// Make sure to load all Level Instances
		World->BlockTillLevelStreamingCompleted();

		// For each Streaming Level, make sure to load its actors if it is a World Partition
		for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
		{
			bool bAlreadySet = false;
			if (!ProcessedLevelStreamings.Contains(LevelStreaming))
			{
				ProcessedLevelStreamings.Add(LevelStreaming);
				
				if (ULevel* LoadedLevel = LevelStreaming->GetLoadedLevel())
				{
					bProcessedNewActors |= LoadAllActors(FWorldPartitionHelpers::GetWorldPartition(LoadedLevel));
				}
			}
		}
	}
	
	struct FActorEntry
	{
		FActorEntry(int InIndex) : Index(InIndex) {}
		int Index = 0;
		AActor* AttachmentParent = nullptr;
		TArray<FName> Tags;
	};

	ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(World);
	check(LevelInstanceSubsystem);

	TFunction<AActor*(AActor*)> GetAttachParentActor = [&GetAttachParentActor, LevelInstanceSubsystem](AActor* InActor)
	{
		if(InActor->IsInLevelInstance())
		{
			AActor* AttachParentActor = InActor->GetAttachParentActor();
			if (AttachParentActor && AttachParentActor->IsA<ALevelInstanceEditorInstanceActor>())
			{
				return GetAttachParentActor(Cast<AActor>(LevelInstanceSubsystem->GetParentLevelInstance(InActor)));
			}
			else
			{
				return AttachParentActor;
			}
		}
		else
		{
			return InActor->GetAttachParentActor();
		}
	};

	const UPCGEditorProjectSettings* EditorProjectSettings = UPCGEditorProjectSettings::StaticClass()->GetDefaultObject<UPCGEditorProjectSettings>();
	const TArray<TSubclassOf<AActor>>& RootExcludedClasses = EditorProjectSettings->PCGLevelToAssetExcludedActorClasses;

	// Cache of already processed classes
	TSet<TSubclassOf<AActor>> IncludedActorClasses;
	TSet<TSubclassOf<AActor>> ExcludedActorClasses;
	ExcludedActorClasses.Append(RootExcludedClasses);

	TFunction<bool(AActor*)> IsActorExcluded = [LevelInstanceSubsystem, &GetAttachParentActor, &RootExcludedClasses, &ExcludedActorClasses, &IncludedActorClasses, &IsActorExcluded](AActor* InActor)
	{
		if (InActor->ActorHasTag(PCGLevelToAssetConstants::ExcludedActorTag))
		{
			return true;
		}

		// Check for project specific class exclusions
		if (!RootExcludedClasses.IsEmpty())
		{
			TSubclassOf<AActor> ActorClass(InActor->GetClass());
			if (ExcludedActorClasses.Contains(ActorClass))
			{
				return true;
			}
			// If not already processed inclusion then process actor class
			else if (!IncludedActorClasses.Contains(ActorClass))
			{
				for (const TSubclassOf<AActor>& ExcludedActorClass : RootExcludedClasses)
				{
					if (ActorClass->IsChildOf(ExcludedActorClass))
					{
						ExcludedActorClasses.Add(ActorClass);
						return true;
					}
				}

				IncludedActorClasses.Add(ActorClass);
			}
		}

		// Check Level Instance Hierarchy first (calling IsActorExcluded will go up hierarchy)
		if (InActor->IsInLevelInstance())
		{
			if (ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(InActor))
			{
				AActor* AncestorActor = CastChecked<AActor>(ParentLevelInstance);
				if (IsActorExcluded(AncestorActor))
				{
					return true;
				}
			}
		}

		// Check Attachment hierarchy next  (calling IsActorExcluded will go up hierarchy)
		AActor* AttachParent = GetAttachParentActor(InActor);
		if (AttachParent && IsActorExcluded(AttachParent))
		{
			return true;
		}

		return false;
	};

	// Parent tags are meant to be at the end of the array so they are applied last in the MakePoint lambda (we want the parent tag values to override the childrens)
	auto GetActorTags = [LevelInstanceSubsystem](AActor* InActor, TArray<FName>& OutTags)
	{
		OutTags.Append(InActor->Tags);
		
		if (InActor->IsInLevelInstance())
		{
			LevelInstanceSubsystem->ForEachLevelInstanceAncestors(InActor, [&OutTags](ILevelInstanceInterface* Ancestor)
			{
				AActor* AncestorActor = CastChecked<AActor>(Ancestor);
				OutTags.Append(AncestorActor->Tags);
				return true;
			});
		}
	};

	// Build actor-index map
	TMap<AActor*, FActorEntry> ActorIndexMap;
	TSet<AActor*> ExcludedActors;
	
	int LastActorIndex = 1; // Since the root is the "first" point we'll have, we'll have the map start from 1.
	UPCGActorHelpers::ForEachActorInWorld(World, AActor::StaticClass(), [&ActorIndexMap, &ExcludedActors, &LastActorIndex, &GetAttachParentActor, &GetActorTags, &IsActorExcluded](AActor* Actor)
	{
		if(IsActorExcluded(Actor))
		{
			ExcludedActors.Add(Actor);
			return true;
		}

		FActorEntry ActorEntry(LastActorIndex++);
		
		ActorEntry.AttachmentParent = GetAttachParentActor(Actor);

		GetActorTags(Actor, ActorEntry.Tags);

		ActorIndexMap.Add(Actor, MoveTemp(ActorEntry));
		return true;
	});

	// Prepare actor parsing context - we'll set only the non-actor relevant things here.
	FPCGActorMeshParsingContext Context;
	Context.PointData = PointData;
	Context.PointMetadata = PointMetadata;
	Context.MeshAttribute = MeshAttribute;
	Context.SkeletalMeshAttribute = SkeletalMeshAttribute;
	Context.MaterialAttribute = MaterialAttribute;
	Context.ActorIndexAttribute = ActorIndexAttribute;
	Context.ParentIndexAttribute = ParentIndexAttribute;
	Context.RelativeTransformAttribute = RelativeTransformAttribute;
	Context.HierarchyDepthAttribute = HierarchyDepthAttribute;
	Context.TagToAttributeMap = &TagToAttributeMap;
	Context.SanitizedAttributeNames = &SanitizedAttributeNames;
	Context.bOnlyMeshComponents = true;
	Context.bIgnorePCGCreatedComponents = false;

	// Create points
	UPCGActorHelpers::ForEachActorInWorld(World, AActor::StaticClass(), [&ExcludedActors, &AllActorBounds, &ActorIndexMap, &GetAttachParentActor, &Context](AActor* Actor)
	{
		// TODO Actor-level decisions if any; if the actor is "consumed" at this step, make sure to update AllActorBounds as well.

		if (ExcludedActors.Contains(Actor))
		{
			return true;
		}

		// Only process actors with static mesh components/skinned mesh components.
		if (!Actor->GetComponentByClass<UStaticMeshComponent>() && !Actor->GetComponentByClass<USkinnedMeshComponent>())
		{
			return true;
		}

		// implementation note: we're not ignoring pcg generated components here because they'll be parsed regardless, and the PCG component itself would be lost
		AllActorBounds += PCGHelpers::GetActorBounds(Actor, /*bIgnorePCGCreatedComponents=*/false);

		const FActorEntry& ActorEntry = ActorIndexMap[Actor];
		const FTransform& ActorTransform = Actor->GetTransform();
		AActor* ParentActor = ActorEntry.AttachmentParent;
		const int64 ParentActorIndex = ParentActor ? ActorIndexMap[ParentActor].Index : 0;
		const FTransform RelativeTransform = ParentActor ? ActorTransform.GetRelativeTransform(ParentActor->GetTransform()) : ActorTransform;

		// Hierarchy depth, starts at 1 if the actor doesn't have a parent
		int HierarchyDepth = 1;
		while (ParentActor)
		{
			++HierarchyDepth;
			ParentActor = GetAttachParentActor(ParentActor);
		}

		// Setup the per-actor data in the actor parsing context struct
		Context.ActorIndex = ActorEntry.Index;
		Context.ParentIndex = ParentActorIndex;
		Context.RelativeTransform = RelativeTransform;
		Context.HierarchyDepth = HierarchyDepth;
		Context.ActorTags = &ActorEntry.Tags;

		Context.ParseActorComponents(Actor);
		return true;
	});

	// Finally, create root point in the root data
	{
		RootPointData->SetNumPoints(1);
		RootPointData->SetTransform(FTransform::Identity);
		RootPointData->SetDensity(1.0f);
		RootPointData->SetSeed(0);
		RootPointData->SetBoundsMin(AllActorBounds.Min);
		RootPointData->SetBoundsMax(AllActorBounds.Max);
		RootPointData->SetSteepness(1.0f);
	}

	return true;
}

void UPCGLevelToAsset::SetWorld(UWorld* World)
{
	WorldToExport = World;
}

bool UPCGLevelToAsset::SetWorldObject(UObject* WorldObject)
{
	WorldToExport = Cast<UWorld>(WorldObject);
	return WorldToExport != nullptr;
}

UWorld* UPCGLevelToAsset::GetWorld() const
{
	return WorldToExport;
}
