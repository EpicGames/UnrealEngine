// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGManualEditPanelManager.h"

#include "PCGComponent.h"
#include "PCGComponentVisualizer.h"
#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "PCGEditorModule.h"
#include "PCGGraph.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGNode.h"
#include "PCGSettings.h"
#include "SPCGManualEditPanel.h"
#include "Graph/DataOverride/PCGDataOverride.h"
#include "Nodes/PCGEditorGraphNodeBase.h"

#include "Editor.h"
#include "LevelEditor.h"
#include "Selection.h"
#include "SLevelViewport.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Modules/ModuleManager.h"

namespace PCG::ManualEditPanel
{
	FPCGComponentVisualizer* FindPCGComponentVisualizer()
	{
		if (!GUnrealEd)
		{
			return nullptr;
		}

		TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(UPCGComponent::StaticClass()->GetFName());
		return Visualizer.IsValid() ? static_cast<FPCGComponentVisualizer*>(Visualizer.Get()) : nullptr;
	}
}

FPCGManualEditPanelManager::FPCGManualEditPanelManager()
{
	SelectionChangedDelegateHandle = USelection::SelectionChangedEvent.AddRaw(this, &FPCGManualEditPanelManager::OnEditorSelectionChanged);
}

FPCGManualEditPanelManager::~FPCGManualEditPanelManager()
{
	if (ManualEditPanel.IsValid())
	{
		ManualEditPanel->SetMode(EPCGManualEditPanelMode::UserControlled);
	}

	HideManualEditPanel();
	USelection::SelectionChangedEvent.Remove(SelectionChangedDelegateHandle);
}

void FPCGManualEditPanelManager::OnEditorSelectionChanged(UObject* Object)
{
	// While ExternallyControlled, an external owner controls lifetime and tracked component; ignore selection changes.
	if (ManualEditPanel.IsValid() && ManualEditPanel->GetMode() == EPCGManualEditPanelMode::ExternallyControlled)
	{
		return;
	}

	if (!GEditor)
	{
		return;
	}

	USelection* Selection = GEditor->GetSelectedActors();
	if (!Selection)
	{
		HideManualEditPanel();
		return;
	}

	// Find a PCG component on the selected actor
	UPCGComponent* FoundComponent = nullptr;
	for (FSelectionIterator It(*Selection); It; ++It)
	{
		const AActor* Actor = Cast<AActor>(*It);
		if (!Actor)
		{
			continue;
		}

		TArray<UPCGComponent*> PCGComponents;
		Actor->GetComponents<UPCGComponent>(PCGComponents, /*bIncludeFromChildActors=*/true);

		for (UPCGComponent* Component : PCGComponents)
		{
			if (Component && Component->GetGraph())
			{
				FoundComponent = Component;
				break;
			}
		}

		if (FoundComponent)
		{
			break;
		}
	}

	if (!FoundComponent || !FoundComponent->GetGraph())
	{
		HideManualEditPanel();
		return;
	}

	// Update tracked component if it changed
	if (TrackedPCGComponent != FoundComponent)
	{
		HideManualEditPanel();
		TrackedPCGComponent = FoundComponent;
	}

	UpdateManualEditPanelVisibility();
}

void FPCGManualEditPanelManager::UpdateManualEditPanelVisibility()
{
	// While ExternallyControlled, an external owner controls lifetime; do not run any visibility-driven hide/dismiss paths.
	if (ManualEditPanel.IsValid() && ManualEditPanel->GetMode() == EPCGManualEditPanelMode::ExternallyControlled)
	{
		return;
	}

	UPCGComponent* Component = TrackedPCGComponent.Get();
	if (!Component || !IsValid(Component))
	{
		HideManualEditPanel();
		return;
	}

	UPCGGraph* Graph = Component->GetGraph();
	if (!Graph)
	{
		HideManualEditPanel();
		return;
	}

	// Check if any nodes are marked or actively editing, including nodes inside embedded or asset subgraphs.
	bool bPanelShouldBeVisible = false;
	Graph->ForEachNodeRecursively([&bPanelShouldBeVisible](const UPCGNode* Node)
	{
		const UPCGSettingsInterface* Settings = Node ? Node->GetSettingsInterface() : nullptr;
		// Skip disabled nodes so the panel hides when every marked node is disabled in the graph.
		if (Settings && Settings->bEnabled && (Settings->IsMarkedForManualEditing() || Settings->IsTemporaryManualEditingEnabled()))
		{
			bPanelShouldBeVisible = true;
			return false;
		}

		return true;
	});

	if (!bPanelShouldBeVisible)
	{
		// Dismiss the panel but preserve TrackedPCGComponent so the panel can reappear immediately when a new mark is added.
		DismissManualEditPanel();
		return;
	}

	// Ensure the component has inspection data for the visualizer. If the graph was executed before any node was marked
	// for viewport editing, the inspection data may not exist yet.
	FPCGGraphExecutionInspection& Inspection = Component->GetExecutionState().GetInspection();
	if (!Inspection.IsInspecting())
	{
		Inspection.EnableInspection();

		IPCGGraphExecutionState::FGenerateParams GenerateParams;
		GenerateParams.bEvenIfAlreadyGenerated = true;
		Component->GetExecutionState().Generate(GenerateParams);
	}

	if (ManualEditPanel.IsValid())
	{
		ManualEditPanel->RefreshNodeList();
	}
	else
	{
		ShowManualEditPanel(Graph);
	}
}

void FPCGManualEditPanelManager::ShowManualEditPanel(UPCGGraph* InGraph)
{
	if (ManualEditPanel.IsValid())
	{
		return;
	}

	ManualEditPanel = SNew(SPCGManualEditPanel).Graph(InGraph);
	ManualEditOverlayWidget = ManualEditPanel->CreateOverlayWidget();

	// Wire generic actions from the visualizer
	TWeakPtr<SPCGManualEditPanel> WeakPanel = ManualEditPanel;

	FPCGDeltaViewportCallbacks ViewportCallbacks;

	ViewportCallbacks.RevertDelta = [](const FPCGDeltaKey& Key)
	{
		if (FPCGComponentVisualizer* Visualizer = PCG::ManualEditPanel::FindPCGComponentVisualizer())
		{
			Visualizer->RestoreDelta(Key);
		}
	};

	ViewportCallbacks.RevertAllRestorableDeltas = []()
	{
		if (FPCGComponentVisualizer* Visualizer = PCG::ManualEditPanel::FindPCGComponentVisualizer())
		{
			Visualizer->RestoreAllDeltas();
		}
	};

	ViewportCallbacks.ClearSelection = []()
	{
		if (FPCGComponentVisualizer* Visualizer = PCG::ManualEditPanel::FindPCGComponentVisualizer())
		{
			Visualizer->ClearActiveSelection();
		}
	};

	ViewportCallbacks.SelectElement = [](const FPCGDeltaKey& DeltaKey, int32 SubIndex)
	{
		if (FPCGComponentVisualizer* Visualizer = PCG::ManualEditPanel::FindPCGComponentVisualizer())
		{
			Visualizer->SelectElement(DeltaKey, SubIndex);
		}
	};

	ViewportCallbacks.RequestListRefresh = [WeakPanel]()
	{
		if (TSharedPtr<SPCGManualEditPanel> Panel = WeakPanel.Pin())
		{
			Panel->RefreshActiveExtensionLists();
		}
	};

	ManualEditPanel->SetVisualizerActions(ViewportCallbacks);

	// When a node is selected in the panel, 'inspect' it in the graph editor's Attributes tab
	ManualEditPanel->OnNodeSelected.BindLambda([WeakGraph = TWeakObjectPtr<UPCGGraph>(InGraph)](const UPCGNode* SelectedNode)
	{
		const UPCGGraph* Graph = WeakGraph.Get();
		if (!SelectedNode || !Graph || !Graph->GetEditorGraph())
		{
			return;
		}

		const TSharedPtr<FPCGEditor> Editor = Graph->GetEditorGraph()->GetEditor().Pin();
		if (!Editor.IsValid())
		{
			return;
		}

		if (UPCGEditorGraphNodeBase* EditorNode = Graph->GetEditorGraph()->GetEditorNodeFromPCGNode(SelectedNode))
		{
			Editor->SetNodeInspected(EditorNode, /*bValue=*/true);
		}
	});

	// Jump-to-node: focus the graph editor on the selected node
	ManualEditPanel->OnJumpToNode.BindLambda([WeakGraph = TWeakObjectPtr<UPCGGraph>(InGraph)](const UPCGNode* Node)
	{
		const UPCGGraph* Graph = WeakGraph.Get();
		if (!Node || !Graph || !Graph->GetEditorGraph())
		{
			return;
		}

		TSharedPtr<FPCGEditor> Editor = Graph->GetEditorGraph()->GetEditor().Pin();
		if (Editor.IsValid())
		{
			Editor->JumpToNode(Node);
		}
	});

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TSharedPtr<SLevelViewport> ActiveVP = LevelEditor->GetActiveViewportInterface();
		if (ActiveVP.IsValid())
		{
			ActiveVP->AddOverlayWidget(ManualEditOverlayWidget.ToSharedRef());
			OverlayViewport = ActiveVP;
		}
		else
		{
			UE_LOGF(LogPCGEditor, Warning, "[ManualEdit] ShowManualEditPanel: No active viewport");
		}
	}
	else
	{
		UE_LOGF(LogPCGEditor, Warning, "[ManualEdit] ShowManualEditPanel: No LevelEditor found");
	}
}

void FPCGManualEditPanelManager::HideManualEditPanel()
{
	if (!ensure(!ManualEditPanel.IsValid() || ManualEditPanel->GetMode() != EPCGManualEditPanelMode::ExternallyControlled))
	{
		return;
	}

	DismissManualEditPanel();
	TrackedPCGComponent.Reset();
}

void FPCGManualEditPanelManager::DismissManualEditPanel()
{
	if (!ensure(!ManualEditPanel.IsValid() || ManualEditPanel->GetMode() != EPCGManualEditPanelMode::ExternallyControlled))
	{
		return;
	}

	if (ManualEditOverlayWidget.IsValid())
	{
		TSharedPtr<SLevelViewport> Viewport = OverlayViewport.Pin();
		if (Viewport.IsValid())
		{
			Viewport->RemoveOverlayWidget(ManualEditOverlayWidget.ToSharedRef());
		}

		OverlayViewport.Reset();
	}

	const UPCGComponent* Component = TrackedPCGComponent.Get();
	if (const UPCGGraph* PCGGraph = Component ? Component->GetGraph() : nullptr)
	{
		PCGGraph->ForEachNodeRecursively([](UPCGNode* Node)
		{
			if (UPCGSettingsInterface* Settings = Node ? Node->GetSettingsInterface() : nullptr)
			{
				if (Settings->IsTemporaryManualEditingEnabled())
				{
					Settings->SetTemporaryManualEditingEnabled(false);
					// Broadcast through the PCGNode to trigger a full node reconstruct. The editor graph node is not available here.
					Node->OnNodeChangedDelegate.Broadcast(Node, EPCGChangeType::Cosmetic);
				}
			}

			return true;
		});
	}

	if (FPCGComponentVisualizer* Visualizer = PCG::ManualEditPanel::FindPCGComponentVisualizer())
	{
		Visualizer->ClearActiveSelection();
	}

	ManualEditPanel.Reset();
	ManualEditOverlayWidget.Reset();
}
