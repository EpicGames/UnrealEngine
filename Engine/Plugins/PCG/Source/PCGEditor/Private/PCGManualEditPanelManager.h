// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class SLevelViewport;
class SPCGManualEditPanel;
class UPCGComponent;
class UPCGGraph;
class SWidget;
class UObject;

/**
 * Manages the lifecycle of the manual edit panel viewport overlay, including selection tracking, panel
 * creation/destruction, delegates, etc.
 */
class FPCGManualEditPanelManager
{
public:
	FPCGManualEditPanelManager();
	~FPCGManualEditPanelManager();

	FPCGManualEditPanelManager(const FPCGManualEditPanelManager&) = delete;
	FPCGManualEditPanelManager& operator=(const FPCGManualEditPanelManager&) = delete;

	/** Returns the manual edit panel if one is currently active. */
	TSharedPtr<SPCGManualEditPanel> GetManualEditPanel() const { return ManualEditPanel; }

	/** Checks the tracked component and shows/hides the panel as needed. */
	void UpdateManualEditPanelVisibility();

private:
	void OnEditorSelectionChanged(UObject* Object);
	void ShowManualEditPanel(UPCGGraph* InGraph);
	void HideManualEditPanel();

	/** Removes the panel overlay and clears transient editing flags, but preserves TrackedPCGComponent so the panel can reappear if a new node is marked. */
	void DismissManualEditPanel();

	TSharedPtr<SPCGManualEditPanel> ManualEditPanel;
	TSharedPtr<SWidget> ManualEditOverlayWidget;
	TWeakPtr<SLevelViewport> OverlayViewport;
	FDelegateHandle SelectionChangedDelegateHandle;
	TWeakObjectPtr<UPCGComponent> TrackedPCGComponent;
};
