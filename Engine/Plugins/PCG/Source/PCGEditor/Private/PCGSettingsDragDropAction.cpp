// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGSettingsDragDropAction.h"

#include "PCGEditorGraph.h"
#include "Schema/PCGEditorGraphSchema.h"
#include "Schema/PCGEditorGraphSchemaActions.h"

FReply FPCGSettingsDragDropAction::DroppedOnPanel(const TSharedRef<class SWidget>& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	if (!Graph.GetSchema()->IsA<UPCGEditorGraphSchema>())
	{
		return FReply::Unhandled();
	}

	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(&Graph);
	if (!ensure(EditorGraph))
	{
		return FReply::Unhandled();
	}

	FPCGEditorGraphSchemaAction_NewSettingsElement::MakeSettingsNodesOrContextualMenu(Panel, FDeprecateSlateVector2D(ScreenPosition), &Graph, { SettingsObjectPath }, { FDeprecateSlateVector2D(GraphPosition) }, /*bSelectNewNodes=*/true);

	return FReply::Handled();
}