// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GraphEditorDragDropAction.h"

class FReply;

class UPCGEditorGraph;
class UPCGSettings;

class FPCGSettingsDragDropAction : public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FPCGSettingsDragDropAction, FGraphSchemaActionDragDropAction)

	// ~Begin FGraphEditorDragDropAction interface
	virtual FReply DroppedOnPanel(const TSharedRef<class SWidget>& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	// ~End FGraphEditorGraphDropAction interface

	static TSharedRef<FPCGSettingsDragDropAction> New(TSharedPtr<FEdGraphSchemaAction> InAction, const FSoftObjectPath& InSettingsObjectPath)
	{
		TSharedRef<FPCGSettingsDragDropAction> Operation = MakeShareable(new FPCGSettingsDragDropAction);
		Operation->SourceAction = InAction;
		Operation->SettingsObjectPath = InSettingsObjectPath;
		Operation->Construct();
		return Operation;
	}

protected:
	FPCGSettingsDragDropAction() = default;

	FSoftObjectPath SettingsObjectPath;
};
