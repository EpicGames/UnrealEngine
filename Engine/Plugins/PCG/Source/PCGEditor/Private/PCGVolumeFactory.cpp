// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGVolumeFactory.h"

#include "PCGComponent.h"
#include "PCGEngineSettings.h"
#include "PCGGraph.h"
#include "PCGSubsystem.h"
#include "PCGVolume.h"

#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Subsystems/PlacementSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGVolumeFactory)

#define LOCTEXT_NAMESPACE "PCGVolumeFactory"

UPCGVolumeFactory::UPCGVolumeFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("PCGVolumeDisplayName", "PCG Volume");
	NewActorClass = APCGVolume::StaticClass();
	bUseSurfaceOrientation = false;
}

bool UPCGVolumeFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (UActorFactory::CanCreateActorFrom(AssetData, OutErrorMsg))
	{
		return true;
	}

	if ( AssetData.IsValid() && !AssetData.IsInstanceOf<UPCGGraphInterface>())
	{
		OutErrorMsg = LOCTEXT("NoPCGGraph", "A valid PCG graph asset must be specified.");
		return false;
	}

	return true;
}

bool UPCGVolumeFactory::PreSpawnActor(UObject* Asset, FTransform& InOutLocation)
{
	if (Super::PreSpawnActor(Asset, InOutLocation))
	{
		if (const UPCGEngineSettings* Settings = GetDefault<UPCGEngineSettings>())
		{
			InOutLocation.SetScale3D(Settings->VolumeScale);
		}

		return true;
	}

	return false;
}

void UPCGVolumeFactory::PostPlaceAsset(TArrayView<const FTypedElementHandle> InElementHandles, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	Super::PostPlaceAsset(InElementHandles, InPlacementInfo, InPlacementOptions);

	// Preview elements are created while dragging an asset, but before dropping.
	if (InPlacementOptions.bIsCreatingPreviewElements)
	{
		return;
	}

	for (const FTypedElementHandle& PlacedElement : InElementHandles)
	{
		TTypedElement<ITypedElementObjectInterface> ObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(PlacedElement);
		AActor* NewActor = ObjectInterface ? ObjectInterface.GetObjectAs<AActor>() : nullptr;
		if (!NewActor)
		{
			continue;
		}

		UPCGGraphInterface* PCGGraph = Cast<UPCGGraphInterface>(InPlacementInfo.AssetToPlace.GetAsset());
		const UPCGEngineSettings* Settings = GetDefault<UPCGEngineSettings>();
		if (!PCGGraph || !Settings)
		{
			continue;
		}

		const APCGVolume* PCGVolume = CastChecked<APCGVolume>(NewActor);
		UPCGComponent* PCGComponent = CastChecked<UPCGComponent>(PCGVolume->GetComponentByClass(UPCGComponent::StaticClass()));
		PCGComponent->SetGraph(PCGGraph);

		if (Settings->bGenerateOnDrop)
		{
			if (UPCGSubsystem* Subsystem = PCGComponent->GetSubsystem())
			{
				TWeakObjectPtr<UPCGComponent> ComponentPtr(PCGComponent);

				// Schedule a task to generate this component.
				// We cannot generate the component right away because after PostSpawnActor is called, all
				// actor components are unregistered and re-registered, which cancels component generation
				Subsystem->ScheduleGeneric([ComponentPtr]()
				{
					if (UPCGComponent* Component = ComponentPtr.Get())
					{
						// If the component is not valid anymore, just early out.
						if (!IsValid(Component))
						{
							return true;
						}

						Component->Generate();
					}

					return true;
				}, PCGComponent, /*TaskDependencies=*/{});
			}
		}
	}
}

UObject* UPCGVolumeFactory::GetAssetFromActorInstance(AActor* ActorInstance)
{
	const APCGVolume* PCGVolume = CastChecked<APCGVolume>(ActorInstance);
	const UPCGComponent* PCGComponent = PCGVolume->GetComponentByClass<UPCGComponent>();
	return PCGComponent ? PCGComponent->GetGraph() : nullptr;
}

#undef LOCTEXT_NAMESPACE
