// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactoryBoxVolume.h"

#include "PCGVolumeFactory.generated.h"

struct FPlacementOptions;
struct FTypedElementHandle;

UCLASS(MinimalAPI, config=Editor)
class UPCGVolumeFactory : public UActorFactoryBoxVolume
{
	GENERATED_BODY()

public:
	UPCGVolumeFactory(const FObjectInitializer& ObjectInitializer);

	//~ Begin UActorFactory Interface
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual bool PreSpawnActor(UObject* Asset, FTransform& InOutLocation) override;
	virtual void PostPlaceAsset(TArrayView<const FTypedElementHandle> InElementHandles, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	//~ End UActorFactory Interface
};
