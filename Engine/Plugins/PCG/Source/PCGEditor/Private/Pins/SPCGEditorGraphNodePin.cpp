// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphNodePin.h"

#include "PCGEditorStyle.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "Nodes/PCGEditorGraphNodeBase.h"

#include "SLevelOfDetailBranchNode.h"
#include "SPinTypeSelector.h"
#include "ScopedTransaction.h"
#include "Algo/AnyOf.h"
#include "Schema/PCGEditorGraphSchema.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SButton.h"

namespace PCGEditorGraphNodePin::Constants
{
	static constexpr int32 MaxPinLabelLength = 128;
}

void SPCGEditorGraphNodePin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	// IMPLEMENTATION NOTE: this is the code from SGraphPin::Construct with additional padding exposed,
	// with an optional extra icon shown before the pin label, and with a marker icon to show pins
	// that are required for execution.

	bUsePinColorForText = InArgs._UsePinColorForText;
	this->SetCursor(EMouseCursor::Default);

	GraphPinObj = InPin;
	check(GraphPinObj != NULL);

	const UEdGraphSchema* Schema = GraphPinObj->GetSchema();
	checkf(
		Schema,
		TEXT("Missing schema for pin: %s with outer: %s of type %s"),
		*(GraphPinObj->GetName()),
		GraphPinObj->GetOuter() ? *(GraphPinObj->GetOuter()->GetName()) : TEXT("NULL OUTER"),
		GraphPinObj->GetOuter() ? *(GraphPinObj->GetOuter()->GetClass()->GetName()) : TEXT("NULL OUTER")
		);

	const bool bIsInput = (GetDirection() == EGPD_Input);
	const FSlateColor PinColor = GetPinColor();
	const FSlateColor PinTextColor = GetPinTextColor();

	// A small marker to indicate the pin is required for the node to be executed.
	TSharedPtr<SImage> RequiredPinIconWidget;
	float RequiredPinMarkerWidth = 0.0f;
	bool bDisplayPinMarker = false;
	if (bIsInput)
	{
		const FSlateBrush* RequiredPinMarkerIcon = FPCGEditorStyle::Get().GetBrush(PCGEditorStyleConstants::Pin_Required);
		RequiredPinMarkerWidth = RequiredPinMarkerIcon ? RequiredPinMarkerIcon->GetImageSize().X : 8.0f;
		bDisplayPinMarker = ShouldDisplayAsRequiredForExecution();

		RequiredPinIconWidget =
			SNew(SImage)
			.Image(bDisplayPinMarker ? RequiredPinMarkerIcon : FAppStyle::GetNoBrush())
			.ColorAndOpacity(PinColor);
	}

	// Create the pin icon widget
	TSharedRef<SWidget> PinWidgetRef =
		SNew(SLayeredImage, GetSecondaryPinIcon(), GetSecondaryPinColor())
		.Image(MakeAttributeSP(this, &SPCGEditorGraphNodePin::GetPinIcon))
		.ColorAndOpacity(PinColor);
	PinImage = PinWidgetRef;

	PinWidgetRef->SetCursor(
		TAttribute<TOptional<EMouseCursor::Type>>::Create(
			TAttribute<TOptional<EMouseCursor::Type>>::FGetter::CreateRaw(this, &SPCGEditorGraphNodePin::GetPinCursor)
			)
		);

	// Create the pin indicator widget (used for watched values)
	static const FName NAME_NoBorder("NoBorder");
	TSharedRef<SWidget> PinStatusIndicator =
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), NAME_NoBorder)
		.Visibility(this, &SPCGEditorGraphNodePin::GetPinStatusIconVisibility)
		.ContentPadding(0)
		.OnClicked(this, &SPCGEditorGraphNodePin::ClickedOnPinStatusIcon)
		[
			SNew(SImage)
			.Image(this, &SPCGEditorGraphNodePin::GetPinStatusIcon)
		];

	TSharedRef<SWidget> LabelWidget = SNullWidget::NullWidget;

	if (InArgs._IsRenameable)
	{
		SAssignNew(InlineEditableLabel, SInlineEditableTextBlock)
			.Text(GetPinLabel())
			.Style(FPCGEditorStyle::Get(), "PCG.Node.NodeTitleInlineEditableText")
			.Visibility(InArgs._ShowLabel ? EVisibility::Visible : EVisibility::Collapsed)
			.ColorAndOpacity(PinTextColor)
			.OnVerifyTextChanged(this, &SPCGEditorGraphNodePin::OnVerifyPinLabelChanged)
			.OnTextCommitted(this, &SPCGEditorGraphNodePin::OnPinLabelCommitted)
			.IsReadOnly(false)
			.MultiLine(false);

		LabelWidget = InlineEditableLabel.ToSharedRef();
	}
	else
	{
		LabelWidget = SNew(STextBlock)
			.Text(GetPinLabel())
			.TextStyle(FAppStyle::Get(), InArgs._PinLabelStyle)
			.Visibility(InArgs._ShowLabel ? EVisibility::Visible : EVisibility::Collapsed)
			.ColorAndOpacity(PinTextColor);
	}

	// Create the widget used for the pin body (status indicator, label, and value)
	LabelAndValue =
		SNew(SWrapBox)
		.PreferredSize(150.f);

	TSharedPtr<SImage> ExtraPinIconWidget;
	FName ExtraPinIcon;
	FText ExtraPinIconTooltip;
	if (GetExtraIcon(ExtraPinIcon, ExtraPinIconTooltip))
	{
		ExtraPinIconWidget = SNew(SImage)
			.Image(FAppStyle::GetBrush(ExtraPinIcon))
			.ColorAndOpacity(PinTextColor);

		if (!ExtraPinIconTooltip.IsEmpty())
		{
			ExtraPinIconWidget->SetToolTipText(ExtraPinIconTooltip);
		}
	}

	if (!bIsInput)
	{
		LabelAndValue->AddSlot()
			.VAlign(VAlign_Center)
			[
				PinStatusIndicator
			];

		LabelAndValue->AddSlot()
			.VAlign(VAlign_Center)
			[
				LabelWidget
			];

		if (ExtraPinIconWidget.IsValid())
		{
			LabelAndValue->AddSlot()
				.Padding(5, 0, 0, 0)
				.VAlign(VAlign_Center)
				[
					ExtraPinIconWidget.ToSharedRef()
				];
		}
	}
	else
	{
		if (ExtraPinIconWidget.IsValid())
		{
			LabelAndValue->AddSlot()
				.Padding(0, 0, 5, 0)
				.VAlign(VAlign_Center)
				[
					ExtraPinIconWidget.ToSharedRef()
				];
		}

		LabelAndValue->AddSlot()
			.VAlign(VAlign_Center)
			[
				LabelWidget
			];

		ValueWidget = GetDefaultValueWidget();

		if (ValueWidget != SNullWidget::NullWidget)
		{
			TSharedPtr<SBox> ValueBox;
			LabelAndValue->AddSlot()
				.Padding(FMargin(InArgs._SideToSideMargin, 0, 0, 0))
				.VAlign(VAlign_Center)
				[
					SAssignNew(ValueBox, SBox)
					[
						ValueWidget.ToSharedRef()
					]
				];

			if (!DoesWidgetHandleSettingEditingEnabled())
			{
				ValueBox->SetEnabled(IsEditingEnabled());
			}
		}

		LabelAndValue->AddSlot()
			.VAlign(VAlign_Center)
			[
				PinStatusIndicator
			];
	}

	TSharedPtr<SHorizontalBox> PinContent;
	if (bIsInput)
	{
		// Input pin
		FullPinHorizontalRowWidget = PinContent =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				RequiredPinIconWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(bDisplayPinMarker ? 0.0f : RequiredPinMarkerWidth, 0, InArgs._SideToSideMargin, 0)
			[
				PinWidgetRef
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				LabelAndValue.ToSharedRef()
			];
	}
	else
	{
		// Output pin
		FullPinHorizontalRowWidget = PinContent = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				LabelAndValue.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(InArgs._SideToSideMargin, 0, 0, 0)
			[
				PinWidgetRef
			];
	}

	// Set up a hover for pins that is tinted the color of the pin.

	SBorder::Construct(SBorder::FArguments()
		.BorderImage(this, &SPCGEditorGraphNodePin::GetPinBorder) // Requires dynamic lambda
		.BorderBackgroundColor(this, &SPCGEditorGraphNodePin::GetHighlightColor) // Requires dynamic lambda
		.OnMouseButtonDown(this, &SPCGEditorGraphNodePin::OnPinNameMouseDown) // Requires dynamic lambda
		.Padding(0) // NOTE: This is different from base class implementation
		[
			SNew(SBorder)
			.BorderImage(CachedImg_Pin_DiffOutline)
			.BorderBackgroundColor(this, &SPCGEditorGraphNodePin::GetPinDiffColor) // Requires dynamic lambda
			.Padding(0) // NOTE: This is different from base class implementation
			[
				SNew(SLevelOfDetailBranchNode)
				.UseLowDetailSlot(this, &SPCGEditorGraphNodePin::UseLowDetailPinNames) // Requires dynamic lambda
				.LowDetail()
				[
					//@TODO: Try creating a pin-colored line replacement that doesn't measure text / call delegates but still renders
					PinWidgetRef
				]
				.HighDetail()
				[
					PinContent.ToSharedRef()
				]
			]
		]);

	TSharedPtr<IToolTip> TooltipWidget =
		SNew(SToolTip)
		.Text(GetTooltipText());

	SetToolTip(TooltipWidget);
}

void SPCGEditorGraphNodePin::GetPCGNodeAndPin(const UPCGNode*& OutNode, const UPCGPin*& OutPin) const
{
	UEdGraphPin* GraphPin = GetPinObj();
	if (GraphPin && !GraphPin->IsPendingKill())
	{
		const UPCGEditorGraphNodeBase* EditorNode = CastChecked<const UPCGEditorGraphNodeBase>(GraphPinObj->GetOwningNode());

		OutNode = EditorNode ? EditorNode->GetPCGNode() : nullptr;

		OutPin = OutNode ? OutNode->GetInputPin(GraphPin->GetFName()) : nullptr;
		if (!OutPin)
		{
			OutPin = OutNode ? OutNode->GetOutputPin(GraphPin->GetFName()) : nullptr;
		}
	}
	else
	{
		OutNode = nullptr;
		OutPin = nullptr;
	}
}

void SPCGEditorGraphNodePin::ApplyUnusedPinStyle(FSlateColor& InOutColor) const
{
	const UPCGPin* PCGPin = nullptr;
	const UPCGNode* PCGNode = nullptr;
	GetPCGNodeAndPin(PCGNode, PCGPin);

	bool bPinDisabled = false;

	// Check if the pin was deactivated in the previous execution.
	const UEdGraphPin* Pin = GetPinObj();
	if (Pin && Pin->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		if (const UPCGEditorGraphNodeBase* Node = Cast<UPCGEditorGraphNodeBase>(Pin ? Pin->GetOwningNode() : nullptr))
		{
			// If node is already disabled, don't bother disabling pin on top of that, does not look nice to disable both and may
			// not be meaningful to do so in any case.
			if (!Node->IsDisplayAsDisabledForced())
			{
				bPinDisabled = !Node->IsOutputPinActive(Pin);
			}
		}
	}

	// Halve opacity if pin is unused - intended to happen whether disabled or not
	if (bPinDisabled || (PCGPin && PCGNode && !PCGNode->IsPinUsedByNodeExecution(PCGPin)))
	{
		FLinearColor Color = InOutColor.GetSpecifiedColor();
		Color.A *= 0.5;
		InOutColor = Color;
	}
}

FSlateColor SPCGEditorGraphNodePin::GetPinColor() const
{
	FSlateColor Color = SGraphPin::GetPinColor();

	ApplyUnusedPinStyle(Color);

	return Color;
}

FSlateColor SPCGEditorGraphNodePin::GetPinTextColor() const
{
	FSlateColor Color = SGraphPin::GetPinTextColor();

	ApplyUnusedPinStyle(Color);

	return Color;
}

FName SPCGEditorGraphNodePin::GetLabelStyle(FName DefaultLabelStyle) const
{
	const UPCGPin* PCGPin = nullptr;
	const UPCGNode* PCGNode = nullptr;
	FName LabelStyle = NAME_None;

	GetPCGNodeAndPin(PCGNode, PCGPin);

	const UPCGSettings* Settings = PCGNode ? PCGNode->GetSettings() : nullptr;

	if (!PCGPin || !Settings || !Settings->GetPinLabelStyle(PCGPin, LabelStyle))
	{
		LabelStyle = DefaultLabelStyle;
	}

	return LabelStyle;
}

bool SPCGEditorGraphNodePin::GetExtraIcon(FName& OutExtraIcon, FText& OutTooltip) const
{
	const UPCGPin* PCGPin = nullptr;
	const UPCGNode* PCGNode = nullptr;

	GetPCGNodeAndPin(PCGNode, PCGPin);

	const UPCGSettings* Settings = PCGNode ? PCGNode->GetSettings() : nullptr;
	return (PCGPin && Settings) ? Settings->GetPinExtraIcon(PCGPin, OutExtraIcon, OutTooltip) : false;
}

bool SPCGEditorGraphNodePin::ShouldDisplayAsRequiredForExecution() const
{
	const UPCGPin* PCGPin = nullptr;
	const UPCGNode* PCGNode = nullptr;
	GetPCGNodeAndPin(PCGNode, PCGPin);

	// Trivial early out tests, and advanced pins should never display as required.
	if (!PCGPin || !PCGNode || PCGPin->Properties.IsAdvancedPin())
	{
		return false;
	}

	if (PCGNode->IsInputPinRequiredByExecution(PCGPin))
	{
		return true;
	}

	const UPCGSettings* Settings = PCGNode->GetSettings();
	if (Settings && Settings->CanCullTaskIfUnwired())
	{
		// If the node will cull if unwired, and if it only has a single normal pin (and no required pins), then display the pin
		// as required, because it effectively is. So return false if there are other pins which are not advanced.
		return !Algo::AnyOf(PCGNode->GetInputPins(), [PCGPin](UPCGPin* InOtherPin)
		{
			return InOtherPin && InOtherPin != PCGPin && !InOtherPin->Properties.IsAdvancedPin();
		});
	}

	return false;
}

bool SPCGEditorGraphNodePin::SetPinDefaultValue(const FString& NewValue)
{
	if (!GraphPinObj || GraphPinObj->IsPendingKill() || GraphPinObj->DefaultValue == NewValue)
	{
		return false;
	}

	const UEdGraphSchema* Schema = GraphPinObj->GetSchema();
	if (!Schema)
	{
		return false;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("PCGGraphEditor", "ChangePinDefaultValue", "Change Pin Default Value"));
	GraphPinObj->Modify();
	Schema->TrySetDefaultValue(*GraphPinObj, NewValue);
	OnModifyDelegate.ExecuteIfBound();

	return true;
}

void SPCGEditorGraphNodePin::EnterLabelEditMode()
{
	if (InlineEditableLabel.IsValid())
	{
		InlineEditableLabel->EnterEditingMode();
	}
}

bool SPCGEditorGraphNodePin::OnVerifyPinLabelChanged(const FText& InText, FText& OutErrorMessage)
{
	using namespace PCGEditorGraphNodePin;

	const FString NewName = InText.ToString();

	if (NewName.IsEmpty())
	{
		OutErrorMessage = NSLOCTEXT("PCGGraphEditor", "EmptyPinName", "Pin name cannot be empty.");
		return false;
	}

	if (NewName.Len() > Constants::MaxPinLabelLength)
	{
		OutErrorMessage = FText::Format(NSLOCTEXT("PCGGraphEditor", "PinNameTooLong", "Pin name cannot exceed {0} characters."), Constants::MaxPinLabelLength);
		return false;
	}

	// Reject invalid characters using UE's standard name validation
	if (!FName::IsValidXName(NewName, INVALID_OBJECTNAME_CHARACTERS, &OutErrorMessage))
	{
		return false;
	}

	// Reject duplicates among sibling pins in the same direction
	if (const UEdGraphPin* Pin = GetPinObj())
	{
		if (const UEdGraphNode* Node = Pin->GetOwningNode())
		{
			const FName NewLabel(*NewName);
			for (const UEdGraphPin* OtherPin : Node->GetAllPins())
			{
				if (OtherPin != Pin && OtherPin->Direction == Pin->Direction && OtherPin->PinName == NewLabel)
				{
					OutErrorMessage = NSLOCTEXT("PCGGraphEditor", "DuplicatePinName", "A pin with this name already exists.");
					return false;
				}
			}
		}
	}

	return true;
}

void SPCGEditorGraphNodePin::OnPinLabelCommitted(const FText& InText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnCleared)
	{
		return;
	}

	UEdGraphPin* Pin = GetPinObj();
	if (!Pin || InText.IsEmpty())
	{
		return;
	}

	const FName NewLabel(*InText.ToString());
	if (NewLabel == Pin->PinName)
	{
		return;
	}

	if (UPCGEditorGraphNodeBase* EditorNode = Cast<UPCGEditorGraphNodeBase>(Pin->GetOwningNode()))
	{
		EditorNode->OnUserRenameDynamicPin(Pin, NewLabel);
	}
}
