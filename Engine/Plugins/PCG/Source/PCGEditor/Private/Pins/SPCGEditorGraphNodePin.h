// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"

class SInlineEditableTextBlock;
class UPCGPin;
class UPCGNode;

/** PCG pin primarily to give more control over pin coloring. */
class SPCGEditorGraphNodePin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphNodePin)
		: _PinLabelStyle(NAME_DefaultPinLabelStyle)
		, _SideToSideMargin(5.0f)
		, _UsePinColorForText(false)
		, _ShowLabel(true)
		, _IsRenameable(false)
	{}

	SLATE_ARGUMENT(FName, PinLabelStyle)
	SLATE_ARGUMENT(float, SideToSideMargin)
	SLATE_ARGUMENT(bool, UsePinColorForText)
	SLATE_ARGUMENT(bool, ShowLabel)
	SLATE_ARGUMENT(bool, IsRenameable)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

	virtual FSlateColor GetPinColor() const override;
	virtual FSlateColor GetPinTextColor() const override;
	FName GetLabelStyle(FName DefaultLabelStyle) const;
	bool GetExtraIcon(FName& OutExtraIcon, FText& OutTooltip) const;

	/** Whether pin is required to be connected for execution. */
	bool ShouldDisplayAsRequiredForExecution() const;

	/** Enters inline edit mode on the pin label. Only valid for renameable dynamic pins. */
	void EnterLabelEditMode();

protected:
	/** Sets pin default value inside a scoped transaction if the value changed. Returns true if the value was set. */
	bool SetPinDefaultValue(const FString& NewValue);

	FSimpleDelegate OnModifyDelegate;

private:
	void ApplyUnusedPinStyle(FSlateColor& InOutColor) const;
	void GetPCGNodeAndPin(const UPCGNode*& OutNode, const UPCGPin*& OutPin) const;

	bool OnVerifyPinLabelChanged(const FText& InText, FText& OutErrorMessage);
	void OnPinLabelCommitted(const FText& InText, ETextCommit::Type CommitType);

	TSharedPtr<SInlineEditableTextBlock> InlineEditableLabel;
};
