// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPCGEditorGraphNodePin.h"

#include "ScopedTransaction.h"

#include "Widgets/Input/SCheckBox.h"

/** Note: Derived from Engine/Source/Editor/GraphEditor/Public/KismetPins/SGraphPinBool.h */

class SPCGEditorGraphPinBool final : public SPCGEditorGraphNodePin
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphPinBool) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin, TDelegate<void()>&& OnModify)
	{
		OnModifyDelegate = std::move(OnModify);
		SGraphPin::Construct(SGraphPin::FArguments(), InPin);
	}

protected:
	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override
	{
		return SNew(SCheckBox)
			.IsChecked(this, &SPCGEditorGraphPinBool::IsDefaultValueChecked)
			.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
			.OnCheckStateChanged(this, &SPCGEditorGraphPinBool::OnDefaultValueCheckBoxChanged)
			.Visibility(this, &SGraphPin::GetDefaultValueVisibility);
	}

	//~ End SGraphPin Interface

	/** Determine if the checkbox should be checked or not */
	ECheckBoxState IsDefaultValueChecked() const
	{
		FString CurrentValue = GraphPinObj->GetDefaultAsString();
		return CurrentValue.ToBool() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	/** Called when check box is changed */
	void OnDefaultValueCheckBoxChanged(const ECheckBoxState InIsChecked) const
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		const FString BoolString = (InIsChecked == ECheckBoxState::Checked) ? TEXT("true") : TEXT("false");
		if (GraphPinObj->GetDefaultAsString() != BoolString)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("PCGGraphEditor", "ChangeBoolPinValue", "Change Bool Pin Value"));
			GraphPinObj->Modify();
			OnModifyDelegate.ExecuteIfBound();
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, BoolString);
		}
	}

private:
	FSimpleDelegate OnModifyDelegate;
};
