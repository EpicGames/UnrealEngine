// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPCGEditorGraphNodePin.h"

#include "ScopedTransaction.h"
#include "Editor.h"
#include "NumericPropertyParams.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SNumericEntryBox.h"

/** Note: Derived from Engine/Source/Editor/GraphEditor/Public/KismetPins/SGraphPinNumSlider.h */

template <typename NumericType>
class SPCGEditorGraphPinNumSlider final : public SPCGEditorGraphNodePin
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphPinNumSlider)
		: _PinProperty(nullptr)
		, _MinDesiredBoxWidth(60.0f)
		, _MaxDesiredBoxWidth(400.f)
		, _ShouldShowDisabledWhenConnected(true)
	{}

	SLATE_ARGUMENT(FProperty*, PinProperty)
	SLATE_ARGUMENT(float, MinDesiredBoxWidth)
	SLATE_ARGUMENT(float, MaxDesiredBoxWidth)
	SLATE_ARGUMENT(bool, ShouldShowDisabledWhenConnected)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin, TDelegate<void()>&& OnModify)
	{
		MinDesiredBoxWidth = InArgs._MinDesiredBoxWidth;
		MaxDesiredBoxWidth = InArgs._MaxDesiredBoxWidth;
		bShouldHideWhenConnected = InArgs._ShouldShowDisabledWhenConnected;
		OnModifyDelegate = std::move(OnModify);

		SPCGEditorGraphNodePin::Construct(SPCGEditorGraphNodePin::FArguments(), InPin);
	}

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override
	{
		// Use generic defaults for now.
		TNumericPropertyParams<NumericType> NumericPropertyParams(/*Property=*/nullptr, /*MetaDataGetter=*/nullptr);

		// Save last committed value to compare when value changes
		CachedValue = GetNumericValue().GetValue();

		return SNew(SBox)
			.MinDesiredWidth(MinDesiredBoxWidth)
			.MaxDesiredWidth(400)
			[
				SNew(SNumericEntryBox<NumericType>)
				.EditableTextBoxStyle(FAppStyle::Get(), "Graph.EditableTextBox")
				.BorderForegroundColor(FSlateColor::UseForeground())
				.Visibility(this, &SPCGEditorGraphPinNumSlider::GetDefaultValueVisibility)
				.IsEnabled(this, &SPCGEditorGraphPinNumSlider::GetDefaultValueIsEditable)
				.Value(this, &SPCGEditorGraphPinNumSlider::GetNumericValue)
				.MinValue(NumericPropertyParams.MinValue)
				.MaxValue(NumericPropertyParams.MaxValue)
				.MinSliderValue(NumericPropertyParams.MinSliderValue)
				.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
				.SliderExponent(NumericPropertyParams.SliderExponent)
				.Delta(NumericPropertyParams.Delta)
				.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
				.AllowWheel(true)
				.WheelStep(NumericPropertyParams.WheelStep)
				.AllowSpin(true)
				.OnValueCommitted(this, &SPCGEditorGraphPinNumSlider::OnValueCommitted)
				.OnValueChanged(this, &SPCGEditorGraphPinNumSlider::OnValueChanged)
				.OnBeginSliderMovement(this, &SPCGEditorGraphPinNumSlider::OnBeginSliderMovement)
				.OnEndSliderMovement(this, &SPCGEditorGraphPinNumSlider::OnEndSliderMovement)
			];
	}

private:
	//~ End SGraphPin Interface

	void OnValueChanged(NumericType NewValue)
	{
		SliderValue = NewValue;
	}

	void OnValueCommitted(NumericType NewValue, ETextCommit::Type CommitInfo)
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		if (CachedValue != NewValue)
		{
			CachedValue = NewValue;
			const FScopedTransaction Transaction(NSLOCTEXT("PCGGraphEditor", "ChangePinNumberValue", "Change Pin Number Value"));
			GraphPinObj->Modify();
			OnModifyDelegate.ExecuteIfBound();
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, *LexToSanitizedString(NewValue));
		}
	}

	/**
	* Called when the slider begins to move.
	*/
	void OnBeginSliderMovement()
	{
		GEditor->BeginTransaction(NSLOCTEXT("PCGGraphEditor", "ChangeNumberPinValueSlider", "Change Number Pin Value Slider"));
		GraphPinObj->Modify();
		SliderValue = GetNumericValue().GetValue();
		bIsUsingSlider = true;
	}

	/**
	* Called when the slider stops moving.
	*/
	void OnEndSliderMovement(NumericType NewValue)
	{
		bIsUsingSlider = false;
		GEditor->EndTransaction();
	}

	TOptional<NumericType> GetNumericValue() const
	{
		NumericType Num = NumericType();
		LexFromString(Num, *GraphPinObj->GetDefaultAsString());
		return bIsUsingSlider ? SliderValue : Num;
	}

	virtual EVisibility GetDefaultValueVisibility() const override
	{
		// If this is only for showing default value, always show
		if (bOnlyShowDefaultValue)
		{
			return EVisibility::Visible;
		}

		// First ask schema
		UEdGraphPin* GraphPin = GetPinObj();

	    if (!GraphPin)
	    {
	        return EVisibility::Hidden;
	    }

		const UEdGraphSchema* Schema = (!GraphPin->IsPendingKill()) ? GraphPin->GetSchema() : nullptr;

		const bool bIsInputPin = (GraphPin->Direction == EGPD_Input);
		const bool bIsHiddenBySchema = (!Schema || Schema->ShouldHidePinDefaultValue(GraphPin));
		const bool bHiddenWhenConnected = (IsConnected() && bShouldHideWhenConnected);
		const bool bIsNotConnectableOrOrphaned = (GraphPin->bNotConnectable && !GraphPin->bOrphanedPin);

		if ((bIsInputPin && !bIsHiddenBySchema && !bHiddenWhenConnected) || bIsNotConnectableOrOrphaned)
		{
			return EVisibility::Visible;
		}
		else
		{
			return EVisibility::Hidden;
		}
	}

	NumericType CachedValue = 0;
	NumericType SliderValue = 0;
	float MinDesiredBoxWidth = 0;
	float MaxDesiredBoxWidth = 0;
	bool bIsUsingSlider = false;
	bool bShouldHideWhenConnected = false;
	FSimpleDelegate OnModifyDelegate;
};
