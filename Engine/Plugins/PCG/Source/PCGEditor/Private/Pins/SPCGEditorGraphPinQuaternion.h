// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPCGEditorGraphNodePin.h"

#include "KismetPins/SVectorSlider.h"

/**
 * Quaternion pin widget. Displays rotator inputs (Roll, Pitch, Yaw) that are converted to/from FQuat for storage.
 * Note: Derived from SPCGEditorGraphPinVectorSlider pattern.
 */
class SPCGEditorGraphPinQuaternion final : public SPCGEditorGraphNodePin
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphPinQuaternion)
		{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin, TDelegate<void()>&& OnModify)
	{
		OnModifyDelegate = MoveTemp(OnModify);
		SPCGEditorGraphNodePin::Construct(SPCGEditorGraphNodePin::FArguments(), InPin);
	}

protected:
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override
	{
		return SNew(SVectorSlider<double>, /*bIsRotator=*/true, nullptr)
	   .VisibleText_0(this, &SPCGEditorGraphPinQuaternion::GetCurrentStringValue, 0)
	   .VisibleText_1(this, &SPCGEditorGraphPinQuaternion::GetCurrentStringValue, 1)
	   .VisibleText_2(this, &SPCGEditorGraphPinQuaternion::GetCurrentStringValue, 2)
	   .OnNumericCommitted_Box_0(this, &SPCGEditorGraphPinQuaternion::OnValueCommitted, 0)
	   .OnNumericCommitted_Box_1(this, &SPCGEditorGraphPinQuaternion::OnValueCommitted, 1)
	   .OnNumericCommitted_Box_2(this, &SPCGEditorGraphPinQuaternion::OnValueCommitted, 2)
	   .Visibility(this, &SPCGEditorGraphNodePin::GetDefaultValueVisibility)
	   .IsEnabled(this, &SPCGEditorGraphNodePin::GetDefaultValueIsEditable);
	}

private:
	// --- Getter (index: 0=Roll, 1=Pitch, 2=Yaw) ---
	FString GetCurrentStringValue(const int32 Index) const
	{
		if (Index == 0)
		{
			UpdateFromDefaultValue();
		}
		return FString::SanitizeFloat(Index == 0 ? CachedRotator.Roll : Index == 1 ? CachedRotator.Pitch : CachedRotator.Yaw);
	}

	// --- Setter (the component index is passed along with the value) ---
	void OnValueCommitted(const double NewValue, ETextCommit::Type, const int32 Index)
	{
		double& Component = Index == 0 ? CachedRotator.Roll : Index == 1 ? CachedRotator.Pitch : CachedRotator.Yaw;
		SetComponent(Component, NewValue);
	}

	void UpdateFromDefaultValue() const
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		const uint32 Hash = GetTypeHash(GraphPinObj->DefaultValue);
		if (Hash == DefaultValueHash)
		{
			return;
		}

		DefaultValueHash = Hash;

		FQuat Quat = FQuat::Identity;
		const UScriptStruct* QuatStruct = TBaseStructure<FQuat>::Get();
		QuatStruct->ImportText(*GraphPinObj->DefaultValue, &Quat, /*OwnerObject=*/nullptr, PPF_None, /*ErrorText=*/nullptr, QuatStruct->GetName());
		CachedRotator = Quat.Rotator();
	}

	void SetDefaultValue()
	{
		const FQuat Quat = CachedRotator.Quaternion();
		FString Result;
		TBaseStructure<FQuat>::Get()->ExportText(Result, &Quat, /*Defaults=*/nullptr, /*OwnerObject=*/nullptr, PPF_None, /*ExportRootScope=*/nullptr);
		SetPinDefaultValue(Result);
	}

	void SetComponent(double& Component, const double NewValue)
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		if (NewValue == Component)
		{
			return;
		}

		Component = NewValue;
		SetDefaultValue();
	}

	mutable FRotator CachedRotator = FRotator::ZeroRotator;
	mutable uint32 DefaultValueHash = 0;
};
