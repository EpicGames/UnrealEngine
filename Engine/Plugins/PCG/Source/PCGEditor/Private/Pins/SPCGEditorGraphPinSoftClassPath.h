// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPCGEditorGraphNodePin.h"

#include "Editor.h"

#include "ClassViewerModule.h"
#include "Modules/ModuleManager.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

// @todo_pcg: Propagate MetaClass from UPROPERTY metadata to filter the class viewer.
/** Note: Based on KismetPins/SGraphPinClass.cpp */

class SPCGEditorGraphPinSoftClassPath final : public SPCGEditorGraphNodePin
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphPinSoftClassPath)
		{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin, TDelegate<void()>&& OnModify)
	{
		OnModifyDelegate = MoveTemp(OnModify);
		SPCGEditorGraphNodePin::Construct(SPCGEditorGraphNodePin::FArguments(), InPin);
	}

protected:
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override
	{
		return SNew(SHorizontalBox)
			   .Visibility(this, &SPCGEditorGraphNodePin::GetDefaultValueVisibility)
			   .IsEnabled(this, &SPCGEditorGraphNodePin::GetDefaultValueIsEditable)
				// Combo button — shows class name, opens class viewer
				+ SHorizontalBox::Slot()
			   .AutoWidth()
			   .Padding(2.0f, 0.0f)
			   .MaxWidth(200.0f)
				[
					SAssignNew(ClassPickerAnchor, SComboButton)
				   .ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
				   .ContentPadding(FMargin(2.0f, 2.0f, 2.0f, 1.0f))
				   .MenuPlacement(MenuPlacement_BelowAnchor)
				   .ButtonContent()
					[
						SNew(STextBlock)
					   .TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
					   .Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					   .Text(this, &SPCGEditorGraphPinSoftClassPath::GetDisplayText)
					   .ToolTipText(this, &SPCGEditorGraphPinSoftClassPath::GetFullPathText)
					]
				   .OnGetMenuContent(this, &SPCGEditorGraphPinSoftClassPath::GenerateClassPicker)
				]
				// Use button — set from editor's selected class
				+ SHorizontalBox::Slot()
			   .AutoWidth()
			   .Padding(1.0f, 0.0f)
			   .VAlign(VAlign_Center)
				[
					SNew(SButton)
				   .ButtonStyle(FAppStyle::Get(), "NoBorder")
				   .OnClicked(this, &SPCGEditorGraphPinSoftClassPath::OnClickUse)
				   .ContentPadding(1.0f)
				   .ToolTipText(NSLOCTEXT("PCGGraphEditor", "SoftClassPathUseTooltip", "Use selected class"))
					[
						SNew(SImage)
					   .Image(FAppStyle::GetBrush(TEXT("Icons.CircleArrowLeft")))
					]
				]
				// Browse button — sync content browser to class asset
				+ SHorizontalBox::Slot()
			   .AutoWidth()
			   .Padding(1.0f, 0.0f)
			   .VAlign(VAlign_Center)
				[
					SNew(SButton)
				   .ButtonStyle(FAppStyle::Get(), "NoBorder")
				   .OnClicked(this, &SPCGEditorGraphPinSoftClassPath::OnClickBrowse)
				   .ContentPadding(0.0f)
				   .ToolTipText(NSLOCTEXT("PCGGraphEditor", "SoftClassPathBrowseTooltip", "Browse to class"))
					[
						SNew(SImage)
					   .Image(FAppStyle::GetBrush(TEXT("Icons.Search")))
					]
				];
	}

private:
	FText GetDisplayText() const
	{
		const FString& Path = GraphPinObj->DefaultValue;
		if (Path.IsEmpty())
		{
			return NSLOCTEXT("PCGGraphEditor", "SelectClass", "Select Class");
		}

		return FText::FromString(FSoftClassPath(Path).GetAssetName());
	}

	FText GetFullPathText() const
	{
		return FText::FromString(GraphPinObj->DefaultValue);
	}

	TSharedRef<SWidget> GenerateClassPicker()
	{
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

		FClassViewerInitializationOptions Options;
		Options.Mode = EClassViewerMode::ClassPicker;
		Options.bShowNoneOption = true;
		Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;

		return SNew(SBox)
	   .WidthOverride(350.0f)
	   .HeightOverride(400.0f)
		[
			SNew(SBorder)
		   .Padding(4.0f)
		   .BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &SPCGEditorGraphPinSoftClassPath::OnClassPicked))
			]
		];
	}

	void OnClassPicked(UClass* ChosenClass)
	{
		const FString NewPath = ChosenClass ? ChosenClass->GetPathName() : FString();
		if (SetPinDefaultValue(NewPath))
		{
			ClassPickerAnchor->SetIsOpen(false);
		}
	}

	FReply OnClickUse()
	{
		FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

		if (const UClass* SelectedClass = GEditor->GetFirstSelectedClass(UObject::StaticClass()))
		{
			SetPinDefaultValue(SelectedClass->GetPathName());
		}

		return FReply::Handled();
	}

	FReply OnClickBrowse()
	{
		if (UClass* Class = FSoftClassPath(GraphPinObj->DefaultValue).TryLoadClass<UObject>())
		{
			TArray<FAssetData> Objects;
			Objects.Add(FAssetData(Class));
			GEditor->SyncBrowserToObjects(Objects);
		}

		return FReply::Handled();
	}

	TSharedPtr<SComboButton> ClassPickerAnchor;
};
