// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPCGEditorGraphNodePin.h"

#include "Editor.h"
#include "Selection.h"

#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

// @todo_pcg: Propagate AllowedClasses from UPROPERTY metadata to filter the asset picker.
// @todo_pcg: Show the asset thumbnail next to the display name.
/** Note: Based on KismetPins/SGraphPinObject.h */
class SPCGEditorGraphPinSoftObjectPath final : public SPCGEditorGraphNodePin
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphPinSoftObjectPath)
		{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin, TDelegate<void()>&& OnModify)
	{
		OnModifyDelegate = MoveTemp(OnModify);
		SPCGEditorGraphNodePin::Construct(SPCGEditorGraphNodePin::FArguments(), InPin);
	}

protected:
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override
	{
		return SNew(SHorizontalBox)
			   .Visibility(this, &SPCGEditorGraphNodePin::GetDefaultValueVisibility)
			   .IsEnabled(this, &SPCGEditorGraphNodePin::GetDefaultValueIsEditable)
				// Combo button — shows asset name, opens picker
				+ SHorizontalBox::Slot()
			   .AutoWidth()
			   .Padding(2.0f, 0.0f)
			   .MaxWidth(200.0f)
				[
					SAssignNew(AssetPickerAnchor, SComboButton)
				   .ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
				   .ContentPadding(FMargin(2.0f, 2.0f, 2.0f, 1.0f))
				   .MenuPlacement(MenuPlacement_BelowAnchor)
				   .ButtonContent()
					[
						SNew(STextBlock)
					   .TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
					   .Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					   .Text(this, &SPCGEditorGraphPinSoftObjectPath::GetDisplayText)
					   .ToolTipText(this, &SPCGEditorGraphPinSoftObjectPath::GetFullPathText)
					]
				   .OnGetMenuContent(this, &SPCGEditorGraphPinSoftObjectPath::GenerateAssetPicker)
				]
				// Use button — set from content browser selection
				+ SHorizontalBox::Slot()
			   .AutoWidth()
			   .Padding(1.0f, 0.0f)
			   .VAlign(VAlign_Center)
				[
					SNew(SButton)
				   .ButtonStyle(FAppStyle::Get(), "NoBorder")
				   .OnClicked(this, &SPCGEditorGraphPinSoftObjectPath::OnClickUse)
				   .ContentPadding(1.0f)
				   .ToolTipText(NSLOCTEXT("PCGGraphEditor", "SoftObjectPathUseTooltip", "Use asset browser selection"))
					[
						SNew(SImage)
					   .Image(FAppStyle::GetBrush(TEXT("Icons.CircleArrowLeft")))
					]
				]
				// Browse button — sync content browser to value
				+ SHorizontalBox::Slot()
			   .AutoWidth()
			   .Padding(1.0f, 0.0f)
			   .VAlign(VAlign_Center)
				[
					SNew(SButton)
				   .ButtonStyle(FAppStyle::Get(), "NoBorder")
				   .OnClicked(this, &SPCGEditorGraphPinSoftObjectPath::OnClickBrowse)
				   .ContentPadding(0.0f)
				   .ToolTipText(NSLOCTEXT("PCGGraphEditor", "SoftObjectPathBrowseTooltip", "Browse to asset"))
					[
						SNew(SImage)
					   .Image(FAppStyle::GetBrush(TEXT("Icons.Search")))
					]
				];
	}

private:
	FText GetDisplayText() const
	{
		const FString& Path = GraphPinObj->DefaultValue;
		if (Path.IsEmpty())
		{
			return NSLOCTEXT("PCGGraphEditor", "SelectAsset", "Select Asset");
		}

		return FText::FromString(FSoftObjectPath(Path).GetAssetName());
	}

	FText GetFullPathText() const
	{
		return FText::FromString(GraphPinObj->DefaultValue);
	}

	TSharedRef<SWidget> GenerateAssetPicker()
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		FAssetPickerConfig AssetPickerConfig;
		AssetPickerConfig.bAllowNullSelection = true;
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SPCGEditorGraphPinSoftObjectPath::OnAssetSelected);
		AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &SPCGEditorGraphPinSoftObjectPath::OnAssetEnterPressed);
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.bAllowDragging = false;

		return SNew(SBox)
	   .HeightOverride(300.0f)
	   .WidthOverride(300.0f)
		[
			SNew(SBorder)
		   .BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
		];
	}

	void OnAssetSelected(const FAssetData& AssetData)
	{
		const FString NewPath = AssetData.IsValid() ? AssetData.GetObjectPathString() : FString();
		if (SetPinDefaultValue(NewPath))
		{
			AssetPickerAnchor->SetIsOpen(false);
		}
	}

	void OnAssetEnterPressed(const TArray<FAssetData>& InSelectedAssets)
	{
		if (InSelectedAssets.Num() > 0)
		{
			OnAssetSelected(InSelectedAssets[0]);
		}
	}

	FReply OnClickUse()
	{
		FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

		if (const UObject* SelectedObject = GEditor->GetSelectedObjects()->GetTop<UObject>())
		{
			SetPinDefaultValue(SelectedObject->GetPathName());
		}

		return FReply::Handled();
	}

	FReply OnClickBrowse()
	{
		const FString& Path = GraphPinObj->DefaultValue;
		if (!Path.IsEmpty())
		{
			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(Path));
			if (AssetData.IsValid())
			{
				TArray<FAssetData> Objects;
				Objects.Add(AssetData);
				GEditor->SyncBrowserToObjects(Objects);
			}
		}

		return FReply::Handled();
	}

	TSharedPtr<SComboButton> AssetPickerAnchor;
};
