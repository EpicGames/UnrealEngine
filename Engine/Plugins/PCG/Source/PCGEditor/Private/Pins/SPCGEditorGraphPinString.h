// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPCGEditorGraphNodePin.h"

#include "ScopedTransaction.h"

#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"

/** Note: Derived from Engine/Source/Editor/GraphEditor/Public/KismetPins/SGraphPinString.h */

class SPCGEditorGraphPinString final : public SPCGEditorGraphNodePin
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphPinString)
		: _MinDesiredBoxWidth(60.f)
		, _MaxDesiredBoxWidth(300.f)
		, _IsMultiline(false)
		, _OverflowPolicy(ETextOverflowPolicy::Ellipsis) {}

	SLATE_ARGUMENT(float, MinDesiredBoxWidth)
	SLATE_ARGUMENT(float, MaxDesiredBoxWidth)
	SLATE_ARGUMENT(bool, IsMultiline)
	SLATE_ARGUMENT(ETextOverflowPolicy, OverflowPolicy)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin, TDelegate<void()>&& OnModify)
	{
		OnModifyDelegate = std::move(OnModify);
		MinDesiredBoxWidth = InArgs._MinDesiredBoxWidth;
		MaxDesiredBoxWidth = InArgs._MaxDesiredBoxWidth;
		bIsMultiline = InArgs._IsMultiline;
		OverflowPolicy = InArgs._OverflowPolicy;

		SPCGEditorGraphNodePin::Construct(SPCGEditorGraphNodePin::FArguments(), InPin);
	}

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override
	{
		return SNew(SBox)
			.MinDesiredWidth(MinDesiredBoxWidth)
			.MaxDesiredWidth(MaxDesiredBoxWidth)
			[
				SNew(SMultiLineEditableTextBox)
				.Style(FAppStyle::Get(), "Graph.EditableTextBox")
				.Text(this, &SPCGEditorGraphPinString::GetTypeInValue)
				.AllowMultiLine(bIsMultiline)
				.OverflowPolicy(OverflowPolicy)
				.SelectAllTextWhenFocused(true)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.IsReadOnly(this, &SPCGEditorGraphPinString::GetDefaultValueIsReadOnly)
				.OnTextCommitted(this, &SPCGEditorGraphPinString::SetTypeInValue)
				.ForegroundColor(FSlateColor::UseForeground())
			];
	}

	//~ End SGraphPin Interface

	FText GetTypeInValue() const
	{
		return FText::FromString(GraphPinObj->GetDefaultAsString());
	}

	void SetTypeInValue(const FText& NewTypeInValue, ETextCommit::Type CommitInfo) const
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		if (!GraphPinObj->GetDefaultAsString().Equals(NewTypeInValue.ToString()))
		{
			const FScopedTransaction Transaction(NSLOCTEXT("PCGGraphEditor", "ChangeStringPinValue", "Change String Pin Value"));
			GraphPinObj->Modify();
			OnModifyDelegate.ExecuteIfBound();
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, NewTypeInValue.ToString());
		}
	}

	bool GetDefaultValueIsReadOnly() const
	{
		return GraphPinObj->bDefaultValueIsReadOnly;
	}

private:
	TArray<TSharedPtr<FString>> ComboBoxOptions;
	float MinDesiredBoxWidth = 0;
	float MaxDesiredBoxWidth = 0;
	bool bIsMultiline = false;
	ETextOverflowPolicy OverflowPolicy = ETextOverflowPolicy::Clip;
	FSimpleDelegate OnModifyDelegate;
};
