// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPCGEditorGraphNodePin.h"

#include "KismetPins/SVectorSlider.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

/**
 * Transform pin widget. Displays Location (XYZ), Rotation (Roll/Pitch/Yaw), and Scale (XYZ).
 * Note: Derived from SPCGEditorGraphPinVectorSlider pattern.
 */
class SPCGEditorGraphPinTransform final : public SPCGEditorGraphNodePin
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphPinTransform)
		{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin, TDelegate<void()>&& OnModify)
	{
		OnModifyDelegate = MoveTemp(OnModify);
		SPCGEditorGraphNodePin::Construct(SPCGEditorGraphNodePin::FArguments(), InPin);
	}

protected:
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override
	{
		return SNew(SVerticalBox)
	   .Visibility(this, &SPCGEditorGraphNodePin::GetDefaultValueVisibility)
	   .IsEnabled(this, &SPCGEditorGraphNodePin::GetDefaultValueIsEditable)
		// Location
		+ SVerticalBox::Slot()
	   .AutoHeight()
	   .Padding(0, 1)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		   .AutoWidth()
		   .VAlign(VAlign_Center)
		   .Padding(0, 0, 2, 0)
			[
				SNew(STextBlock)
			   .Text(NSLOCTEXT("PCGGraphEditor", "TransformLocation", "L"))
			   .Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
			+ SHorizontalBox::Slot()
		   .FillWidth(1.0f)
			[
				SNew(SVectorSlider<double>, /*bIsRotator=*/false, nullptr)
			   .VisibleText_0(this, &SPCGEditorGraphPinTransform::GetLocation, 0)
			   .VisibleText_1(this, &SPCGEditorGraphPinTransform::GetLocation, 1)
			   .VisibleText_2(this, &SPCGEditorGraphPinTransform::GetLocation, 2)
			   .OnNumericCommitted_Box_0(this, &SPCGEditorGraphPinTransform::OnLocationCommitted, 0)
			   .OnNumericCommitted_Box_1(this, &SPCGEditorGraphPinTransform::OnLocationCommitted, 1)
			   .OnNumericCommitted_Box_2(this, &SPCGEditorGraphPinTransform::OnLocationCommitted, 2)
			]
		]
		// Rotation
		+ SVerticalBox::Slot()
	   .AutoHeight()
	   .Padding(0, 1)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		   .AutoWidth()
		   .VAlign(VAlign_Center)
		   .Padding(0, 0, 2, 0)
			[
				SNew(STextBlock)
			   .Text(NSLOCTEXT("PCGGraphEditor", "TransformRotation", "R"))
			   .Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
			+ SHorizontalBox::Slot()
		   .FillWidth(1.0f)
			[
				SNew(SVectorSlider<double>, /*bIsRotator=*/true, nullptr)
			   .VisibleText_0(this, &SPCGEditorGraphPinTransform::GetRotation, 0)
			   .VisibleText_1(this, &SPCGEditorGraphPinTransform::GetRotation, 1)
			   .VisibleText_2(this, &SPCGEditorGraphPinTransform::GetRotation, 2)
			   .OnNumericCommitted_Box_0(this, &SPCGEditorGraphPinTransform::OnRotationCommitted, 0)
			   .OnNumericCommitted_Box_1(this, &SPCGEditorGraphPinTransform::OnRotationCommitted, 1)
			   .OnNumericCommitted_Box_2(this, &SPCGEditorGraphPinTransform::OnRotationCommitted, 2)
			]
		]
		// Scale
		+ SVerticalBox::Slot()
	   .AutoHeight()
	   .Padding(0, 1)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		   .AutoWidth()
		   .VAlign(VAlign_Center)
		   .Padding(0, 0, 2, 0)
			[
				SNew(STextBlock)
			   .Text(NSLOCTEXT("PCGGraphEditor", "TransformScale", "S"))
			   .Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
			+ SHorizontalBox::Slot()
		   .FillWidth(1.0f)
			[
				SNew(SVectorSlider<double>, /*bIsRotator=*/false, nullptr)
			   .VisibleText_0(this, &SPCGEditorGraphPinTransform::GetScale, 0)
			   .VisibleText_1(this, &SPCGEditorGraphPinTransform::GetScale, 1)
			   .VisibleText_2(this, &SPCGEditorGraphPinTransform::GetScale, 2)
			   .OnNumericCommitted_Box_0(this, &SPCGEditorGraphPinTransform::OnScaleCommitted, 0)
			   .OnNumericCommitted_Box_1(this, &SPCGEditorGraphPinTransform::OnScaleCommitted, 1)
			   .OnNumericCommitted_Box_2(this, &SPCGEditorGraphPinTransform::OnScaleCommitted, 2)
			]
		];
	}

private:
	// --- Getters (index: 0=X/Roll, 1=Y/Pitch, 2=Z/Yaw) ---
	FString GetLocation(const int32 Index) const
	{
		// Update from default value if needed: ex. Undo/Redo.
		// Components are queried in turn sequentially by the widget, so only need to update once (on the first).
		if (Index == 0)
		{
			UpdateFromDefaultValue();
		}

		return FString::SanitizeFloat(CachedLocation[Index]);
	}

	FString GetRotation(const int32 Index) const
	{
		return FString::SanitizeFloat(
			Index == 0 ? CachedRotation.Roll : Index == 1 ? CachedRotation.Pitch : CachedRotation.Yaw);
	}

	FString GetScale(const int32 Index) const { return FString::SanitizeFloat(CachedScale[Index]); }

	// --- Setters (the component index is passed along with the value) ---
	void OnLocationCommitted(const double NewValue, ETextCommit::Type, const int32 Index) { SetComponent(CachedLocation[Index], NewValue); }

	void OnRotationCommitted(const double NewValue, ETextCommit::Type, const int32 Index)
	{
		double& Component =
			Index == 0 ? CachedRotation.Roll : Index == 1 ? CachedRotation.Pitch : CachedRotation.Yaw;
		SetComponent(Component, NewValue);
	}

	void OnScaleCommitted(const double NewValue, ETextCommit::Type, const int32 Index) { SetComponent(CachedScale[Index], NewValue); }

	void SetComponent(double& Component, const double NewValue)
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		if (NewValue == Component)
		{
			return;
		}

		Component = NewValue;
		SetDefaultValue();
	}

	void UpdateFromDefaultValue() const
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		const uint32 Hash = GetTypeHash(GraphPinObj->DefaultValue);
		if (Hash == DefaultValueHash)
		{
			return;
		}

		DefaultValueHash = Hash;

		FTransform Transform = FTransform::Identity;
		const UScriptStruct* TransformStruct = TBaseStructure<FTransform>::Get();
		TransformStruct->ImportText(*GraphPinObj->DefaultValue, &Transform, /*OwnerObject=*/nullptr, PPF_None, /*ErrorText=*/nullptr, TransformStruct->GetName());

		CachedLocation = Transform.GetLocation();
		CachedRotation = Transform.GetRotation().Rotator();
		CachedScale = Transform.GetScale3D();
	}

	void SetDefaultValue()
	{
		const FTransform Transform(CachedRotation.Quaternion(), CachedLocation, CachedScale);
		FString Result;
		TBaseStructure<FTransform>::Get()->ExportText(Result, &Transform, /*Defaults=*/nullptr, /*OwnerObject=*/nullptr, PPF_None, /*ExportRootScope=*/nullptr);
		SetPinDefaultValue(Result);
	}

	mutable FVector CachedLocation = FVector::ZeroVector;
	mutable FRotator CachedRotation = FRotator::ZeroRotator;
	mutable FVector CachedScale = FVector::OneVector;
	mutable uint32 DefaultValueHash = 0;
};
