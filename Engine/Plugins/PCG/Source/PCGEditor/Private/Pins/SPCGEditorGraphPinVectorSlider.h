// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPCGEditorGraphNodePin.h"

#include "Metadata/PCGMetadataAttributeTraits.h"

#include "ScopedTransaction.h"

#include "KismetPins/SVectorSlider.h"
#include "KismetPins/SVector2DSlider.h"
#include "KismetPins/SVector4Slider.h"

/** Note: Derived from Engine/Source/Editor/GraphEditor/Public/KismetPins/SVectorSlider.h */

namespace PCGEditorGraphPinVectorSlider
{
	template <typename T>
	struct TVectorTrait;

	template <typename T>
	struct TVectorTrait<UE::Math::TRotator<T>>;

	template <typename T>
	struct TVectorTrait<UE::Math::TVector<T>>;

	template <typename T>
	struct TVectorTrait<UE::Math::TVector2<T>>;

	template <typename T>
	struct TVectorTrait<UE::Math::TVector4<T>>;
}

template <typename T>
class SPCGEditorGraphPinVectorSlider final : public SPCGEditorGraphNodePin
{
	using TVectorTrait = PCGEditorGraphPinVectorSlider::TVectorTrait<T>;
	static_assert(TVectorTrait::value);
	using DataType = typename TVectorTrait::DataType;

public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphPinVectorSlider)
	{}

		// SLATE_EVENT(FSimpleDelegate, OnModify)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin, TDelegate<void()>&& OnModify)
	{
		OnModifyDelegate = std::move(OnModify);
		SGraphPin::Construct(SGraphPin::FArguments(), InPin);
	}

protected:
	TSharedRef<SWidget> GetDefaultValueWidget() override
	{
		if constexpr (std::is_same_v<T, UE::Math::TVector<DataType>> || std::is_same_v<T, UE::Math::TRotator<DataType>>)
		{
			return SNew(SVectorSlider<DataType>, TVectorTrait::bIsRotator, nullptr)
				.VisibleText_0(this, &SPCGEditorGraphPinVectorSlider::GetCurrentStringValue_0)
				.VisibleText_1(this, &SPCGEditorGraphPinVectorSlider::GetCurrentStringValue_1)
				.VisibleText_2(this, &SPCGEditorGraphPinVectorSlider::GetCurrentStringValue_2)
				.OnNumericCommitted_Box_0(this, &SPCGEditorGraphPinVectorSlider::OnCommittedValueTextBox_0)
				.OnNumericCommitted_Box_1(this, &SPCGEditorGraphPinVectorSlider::OnCommittedValueTextBox_1)
				.OnNumericCommitted_Box_2(this, &SPCGEditorGraphPinVectorSlider::OnCommittedValueTextBox_2)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable);
		}
		else if constexpr (std::is_same_v<T, UE::Math::TVector2<DataType>>)
		{
			return SNew(SVector2DSlider<DataType>, nullptr)
				.VisibleText_X(this, &SPCGEditorGraphPinVectorSlider::GetCurrentStringValue_0)
				.VisibleText_Y(this, &SPCGEditorGraphPinVectorSlider::GetCurrentStringValue_1)
				.OnNumericCommitted_Box_X(this, &SPCGEditorGraphPinVectorSlider::OnCommittedValueTextBox_0)
				.OnNumericCommitted_Box_Y(this, &SPCGEditorGraphPinVectorSlider::OnCommittedValueTextBox_1)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable);
		}
		else if constexpr (std::is_same_v<T, UE::Math::TVector4<DataType>>)
		{
			return SNew(SVector4Slider<DataType>, nullptr)
				.VisibleText_0(this, &SPCGEditorGraphPinVectorSlider::GetCurrentStringValue_0)
				.VisibleText_1(this, &SPCGEditorGraphPinVectorSlider::GetCurrentStringValue_1)
				.VisibleText_2(this, &SPCGEditorGraphPinVectorSlider::GetCurrentStringValue_2)
				.VisibleText_3(this, &SPCGEditorGraphPinVectorSlider::GetCurrentStringValue_3)
				.OnNumericCommitted_Box_0(this, &SPCGEditorGraphPinVectorSlider::OnCommittedValueTextBox_0)
				.OnNumericCommitted_Box_1(this, &SPCGEditorGraphPinVectorSlider::OnCommittedValueTextBox_1)
				.OnNumericCommitted_Box_2(this, &SPCGEditorGraphPinVectorSlider::OnCommittedValueTextBox_2)
				.OnNumericCommitted_Box_3(this, &SPCGEditorGraphPinVectorSlider::OnCommittedValueTextBox_3)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable);
		}
		else
		{
			checkNoEntry();
			return SNullWidget::NullWidget;
		}
	}

private:
	enum EVectorAxis
	{
		Axis_0,
		Axis_1,
		Axis_2,
		Axis_3
	};

	// Text Box 0: Rotator->Roll, Vector->X
	FString GetCurrentStringValue_0() const
	{
		// Update from default value if needed: ex. Undo/Redo.
		UpdateFromDefaultValue();
		return GetComponentStringAlongAxis(Axis_0);
	}

	// Text Box 1: Rotator->Pitch, Vector->Y
	FString GetCurrentStringValue_1() const
	{
		return GetComponentStringAlongAxis(Axis_1);
	}

	// Text Box 2: Rotator->Yaw, Vector->Z
	FString GetCurrentStringValue_2() const
	{
		return GetComponentStringAlongAxis(Axis_2);
	}

	// Text Box 3: FVector4 only
	FString GetCurrentStringValue_3() const
	{
		static_assert(std::is_same_v<FVector4, T>);
		return GetComponentStringAlongAxis(Axis_3);
	}

	void OnCommittedValueTextBox_0(DataType NewValue, const ETextCommit::Type CommitInfo)
	{
		// TODO: Should work in tandem with OnLoseFocus and (CommitInfo != ETextCommit::OnUserMovedFocus)
		SetNewComponentValue(Axis_0, NewValue, true);
	}

	void OnCommittedValueTextBox_1(DataType NewValue, const ETextCommit::Type CommitInfo)
	{
		SetNewComponentValue(Axis_1, NewValue, true);
	}

	void OnCommittedValueTextBox_2(DataType NewValue, const ETextCommit::Type CommitInfo)
	{
		SetNewComponentValue(Axis_2, NewValue, true);
	}

	void OnCommittedValueTextBox_3(DataType NewValue, const ETextCommit::Type CommitInfo)
	{
		static_assert(std::is_same_v<FVector4, T>);
		SetNewComponentValue(Axis_3, NewValue, true);
	}

	void UpdateFromDefaultValue() const
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		const uint32 Hash = GetTypeHash(GraphPinObj->DefaultValue);
		if (Hash == DefaultValueHash)
		{
			return;
		}

		DefaultValueHash = Hash;
		if constexpr (TVectorTrait::bIsRotator)
		{
			CachedValue = ConvertDefaultValueStringToRotator(GetPinObj()->DefaultValue);
		}
		else
		{
			CachedValue = ConvertDefaultValueStringToVector(GetPinObj()->DefaultValue);
		}
	}

	void SetDefaultValue() const
	{
		const FScopedTransaction Transaction(NSLOCTEXT("PCGGraphEditor", "ChangeVectorPinValue", "Change Vector Pin Value"));
		GraphPinObj->Modify();
		OnModifyDelegate.ExecuteIfBound();

		//Set new default value
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, GetSerializedString());
	}

	void SetNewComponentValue(const EVectorAxis Axis, const DataType NewComponentValue, const bool bUpdate)
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		DataType& CurrentComponentValue = GetComponentRefAlongAxis(CachedValue, Axis);
		if (NewComponentValue == CurrentComponentValue)
		{
			return;
		}

		CurrentComponentValue = NewComponentValue;

		if (bUpdate)
		{
			SetDefaultValue();
		}
	}

	static double& GetComponentRefAlongAxis(T& Value, const EVectorAxis Axis)
	{
		if constexpr (TVectorTrait::bIsRotator)
		{
			switch (Axis)
			{
				case Axis_0:
					return Value.Roll;
				case Axis_1:
					return Value.Pitch;
				case Axis_2:
					return Value.Yaw;
				default:
					checkNoEntry();
					return Value.Roll;
			}
		}
		else
		{
			switch (Axis)
			{
				case Axis_0:
					return Value.X;
				case Axis_1:
					return Value.Y;
				case Axis_2: // fall-through
					if constexpr (!std::is_same_v<T, UE::Math::TVector2<DataType>>)
					{
						return Value.Z;
					}
				case Axis_3: // fall-through
					if constexpr (std::is_same_v<T, UE::Math::TVector4<DataType>>)
					{
						return Value.W;
					}
				default:
					checkNoEntry();
					return Value.X;
			}
		}
	}

	static const double& GetComponentRefAlongAxis(const T& Value, const EVectorAxis Axis)
	{
		// No side effects on Value which is returned as a const ref, so safe to const cast.
		return GetComponentRefAlongAxis(const_cast<T&>(Value), Axis);
	}

	FString GetComponentStringAlongAxis(const EVectorAxis Axis) const
	{
		const DataType& Value = GetComponentRefAlongAxis(CachedValue, Axis);
		return FString::SanitizeFloat(Value);
	}

	FString GetSerializedString() const
	{
		FString Result;
		TBaseStructure<T>::Get()->ExportText(Result, &CachedValue, /*Defaults=*/nullptr, /*OwnerObject=*/nullptr, PPF_None, /*ExportRootScope=*/nullptr);
		return Result;
	}

	T ConvertDefaultValueStringToRotator(const FString& InDefaultValueString) const
	{
		static_assert(TVectorTrait::bIsRotator, "Should only be called if T = TRotator");

		FString ModifiedString = InDefaultValueString;
		// TODO: Format required for InitFromString conflicts with PropertyBag/ExportText. Investigate alternatives.
		ModifiedString.ReplaceInline(TEXT("oll"), TEXT(""), ESearchCase::CaseSensitive);
		ModifiedString.ReplaceInline(TEXT("aw"), TEXT(""), ESearchCase::CaseSensitive);
		ModifiedString.ReplaceInline(TEXT("itch"), TEXT(""), ESearchCase::CaseSensitive);

		T Rotator;
		Rotator.InitFromString(ModifiedString);
		return Rotator;
	}

	T ConvertDefaultValueStringToVector(const FString& InDefaultValueString) const
	{
		static_assert(!TVectorTrait::bIsRotator, "Should only be called if T = TVector, TVector2, or TVector4");

		T Vector;
		Vector.InitFromString(InDefaultValueString);
		return Vector;
	}

	mutable T CachedValue = PCG::Private::MetadataTraits<T>::ZeroValue();
	mutable uint32 DefaultValueHash = 0;
	FSimpleDelegate OnModifyDelegate;
};

namespace PCGEditorGraphPinVectorSlider
{
	template <typename T>
	struct TVectorTrait<UE::Math::TVector<T>> : std::true_type
	{
		using DataType = T;
		static constexpr bool bIsRotator = false;
	};

	template <typename T>
	struct TVectorTrait<UE::Math::TVector2<T>> : TVectorTrait<UE::Math::TVector<T>>
	{};

	template <typename T>
	struct TVectorTrait<UE::Math::TVector4<T>> : TVectorTrait<UE::Math::TVector<T>>
	{};

	template <typename T>
	struct TVectorTrait<UE::Math::TRotator<T>> : TVectorTrait<UE::Math::TVector<T>>
	{
		static constexpr bool bIsRotator = true;
	};

	template <typename>
	struct TVectorTrait : std::false_type
	{
		using DataType = void;
		static constexpr bool bIsRotator = false;
	};
}
