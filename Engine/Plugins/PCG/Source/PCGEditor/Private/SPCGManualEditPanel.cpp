// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGManualEditPanel.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGSettings.h"

#include "PCGComponentVisualizer.h"
#include "PCGEditorModule.h"
#include "PCGEditorStyle.h"

#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Overlay/SDraggableBoxOverlay.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBackgroundBlur.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SPCGManualEditPanel"

void SPCGManualEditPanel::Construct(const FArguments& InArgs)
{
	Graph = InArgs._Graph;

	// Initialize delta type options
	DeltaTypeOptions = FPCGEditorModule::GetConstDeltaViewportExtensionRegistry().GetRegisteredDeltaTypes();
	if (DeltaTypeOptions.Num() > 0)
	{
		ActiveDeltaType = DeltaTypeOptions[0];
	}

	auto CullingVisibility = []() { return PCG::ComponentVisualizer::GEnableCulling ? EVisibility::Visible : EVisibility::Hidden;};

	ChildSlot
	[
		SNew(SVerticalBox)
		// Header with graph name
		+ SVerticalBox::Slot()
	   .AutoHeight()
	   .Padding(4.0f, 2.0f, 4.0f, 4.0f)
		[
			SNew(STextBlock)
		   .Text_Lambda([this]() -> FText
			{
				if (UPCGGraph* PCGGraph = Graph.Get())
				{
					return FText::Format(LOCTEXT("ManualEditHeaderFormat", "Viewport Editing ({0})"), FText::FromString(PCGGraph->GetName()));
				}
				else
				{
					return LOCTEXT("ManualEditHeaderNoGraph", "Viewport Editing");
				}
			})
		   .TextStyle(FAppStyle::Get(), "NormalText.Important")
		]
		// Node list
		+ SVerticalBox::Slot()
	   .FillHeight(1.0f)
	   .Padding(2.0f)
		[
			SAssignNew(NodeListView, SListView<TSharedPtr<FPCGManualEditNodeEntry>>)
		   .ListItemsSource(&NodeEntries)
		   .OnGenerateRow(this, &SPCGManualEditPanel::OnGenerateRow)
		   .OnSelectionChanged(this, &SPCGManualEditPanel::OnListSelectionChanged)
		   .SelectionMode(ESelectionMode::SingleToggle)
		   .Visibility_Lambda([this]() { return CurrentMode == EPCGManualEditPanelMode::ExternallyControlled ? EVisibility::HitTestInvisible : EVisibility::Visible; })
		]
		// Separator
		+ SVerticalBox::Slot()
	   .AutoHeight()
	   .Padding(4.0f, 2.0f)
		[
			SNew(SSeparator)
		   .Visibility_Lambda([this]()
			{
				return GetActiveNode() != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
			})
		]
		// Culling configuration
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(2.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([]() { return PCG::ComponentVisualizer::GEnableCulling ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([](ECheckBoxState NewState) { PCG::ComponentVisualizer::GEnableCulling = (NewState == ECheckBoxState::Checked); })
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EnableCulling", "Enable Culling"))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
					.ToolTipText(LOCTEXT("Cullingtooltip", "Enable culling by distance of the visualizer meshes. Should be on for performance reasons."))
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.MaxWidth(2.0f)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
				.Visibility_Lambda(CullingVisibility)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([]() { return PCG::ComponentVisualizer::GFrustumCulling ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([](ECheckBoxState NewState) { PCG::ComponentVisualizer::GFrustumCulling = (NewState == ECheckBoxState::Checked); })
				.Visibility_Lambda(CullingVisibility)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FrustumCulling", "Frustum Culling"))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
					.ToolTipText(LOCTEXT("FrustumCullingtooltip", "Will cull any debug mesh that its center is outside of the frustum of the camera. Can cull too much on the edge of the screen."))
					.Visibility_Lambda(CullingVisibility)
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.MaxWidth(2.0f)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
				.Visibility_Lambda(CullingVisibility)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CullingDistance", "Cull Distance"))
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				.ToolTipText(LOCTEXT("CullingDistanceTooltip", "Max culling distance of the visualizer meshes in centimeters."))
				.Visibility_Lambda(CullingVisibility)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SNumericEntryBox<float>)
				.MinValue(0)
				.Value_Lambda([]() { return PCG::ComponentVisualizer::GCullingDistance2D; })
				.OnValueChanged_Lambda([](float NewValue) { PCG::ComponentVisualizer::GCullingDistance2D = FMath::Max(0.f, NewValue); })
				.Visibility_Lambda(CullingVisibility)
			]
		]
		// Separator
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 2.0f)
		[
			SNew(SSeparator)
		   .Visibility_Lambda([this]()
			{
				return GetActiveNode() != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
			})
		]
		// Configuration section (shown when a node is selected)
		+ SVerticalBox::Slot()
	   .AutoHeight()
	   .Padding(4.0f, 6.0f)
		[
			SAssignNew(ConfigurationSection, SVerticalBox)
		   .Visibility(EVisibility::Collapsed)
			// Delta Type row
			+ SVerticalBox::Slot()
		   .AutoHeight()
		   .Padding(0.0f, 2.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
			   .AutoWidth()
			   .VAlign(VAlign_Center)
			   .Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
				   .Text(LOCTEXT("DeltaTypeLabel", "Delta Type:"))
				   .Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				]
				+ SHorizontalBox::Slot()
			   .FillWidth(1.0f)
			   .VAlign(VAlign_Center)
				[
					SNew(SComboButton)
				   .OnGetMenuContent(this, &SPCGManualEditPanel::GenerateDeltaTypeMenu)
				   .IsEnabled_Lambda([this]() { return GetActiveNode() != nullptr; })
				   .ButtonContent()
					[
						SNew(STextBlock)
					   .Text(this, &SPCGManualEditPanel::GetCurrentDeltaTypeLabel)
					   .Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
					]
				]
			]
			// Dynamic extension widget slot
			+ SVerticalBox::Slot()
		   .AutoHeight()
			[
				SAssignNew(DeltaTypeWidgetSlot, SVerticalBox)
			]
		]
	];

	RefreshNodeList();
}

void SPCGManualEditPanel::RefreshNodeList()
{
	// Preserve current selection
	const UPCGNode* PreviouslySelectedNode = GetActiveNode();

	NodeEntries.Empty();

	const UPCGGraph* PCGGraph = Graph.Get();
	if (!PCGGraph)
	{
		if (NodeListView.IsValid())
		{
			NodeListView->RequestListRefresh();
		}
		return;
	}

	PCGGraph->ForEachNodeRecursively([this, PCGGraph](UPCGNode* Node) -> bool
	{
		const UPCGSettingsInterface* Settings = Node ? Node->GetSettingsInterface() : nullptr;
		if (!Settings)
		{
			return true;
		}

		const bool bIsMarked = Settings->IsMarkedForManualEditing();
		const bool bIsTemporarilyMarked = Settings->IsTemporaryManualEditingEnabled();

		// Only show nodes that are marked for editing or currently V-key activated
		if (!bIsMarked && !bIsTemporarilyMarked)
		{
			return true;
		}

		// Do not surface disabled nodes for manual editing even if they're marked.
		if (!Settings->bEnabled)
		{
			return true;
		}

		TSharedPtr<FPCGManualEditNodeEntry> Entry = MakeShared<FPCGManualEditNodeEntry>();
		Entry->Node = Node;
		Entry->bIsMarked = bIsMarked;
		Entry->bIsTemporary = bIsTemporarilyMarked && !bIsMarked;
		Entry->bIsActive = bIsTemporarilyMarked;

		// Prefix the display name with the containing subgraph name so users can tell apart inner nodes from top-level ones.
		const UPCGGraph* OuterGraph = Node->GetTypedOuter<UPCGGraph>();
		Entry->DisplayName = (OuterGraph && OuterGraph != PCGGraph)
			? FText::Format(LOCTEXT("InnerNodeDisplayName", "{0} / {1}"), OuterGraph->GetDisplayName(), Node->GetNodeTitle(EPCGNodeTitleType::ListView))
			: Node->GetNodeTitle(EPCGNodeTitleType::ListView);

		NodeEntries.Add(Entry);

		return true;
	});

	if (NodeListView.IsValid())
	{
		NodeListView->RequestListRefresh();

		// Temporarily enabled nodes take priority for selection. Fall back to restoring the previous selection otherwise.
		TSharedPtr<FPCGManualEditNodeEntry> EntryToSelect;
		for (const TSharedPtr<FPCGManualEditNodeEntry>& Entry : NodeEntries)
		{
			if (Entry->bIsActive)
			{
				EntryToSelect = Entry;
				break;
			}

			if (!EntryToSelect && PreviouslySelectedNode && Entry->Node == PreviouslySelectedNode)
			{
				EntryToSelect = Entry;
			}
		}

		if (EntryToSelect)
		{
			EntryToSelect->bIsActive = true;
			ActiveNode = EntryToSelect->Node;
			NodeListView->SetSelection(EntryToSelect, ESelectInfo::Direct);
		}
		else
		{
			ActiveNode = nullptr;
		}
	}

	UpdateConfigurationSection();
}

void SPCGManualEditPanel::SelectNodeInList(const UPCGNode* InNode)
{
	if (!InNode || !NodeListView.IsValid())
	{
		return;
	}

	for (const TSharedPtr<FPCGManualEditNodeEntry>& Entry : NodeEntries)
	{
		if (Entry.IsValid() && Entry->Node.Get() == InNode)
		{
			NodeListView->SetSelection(Entry);
			NodeListView->RequestScrollIntoView(Entry);

			return;
		}
	}
}

void SPCGManualEditPanel::OnListSelectionChanged(TSharedPtr<FPCGManualEditNodeEntry> SelectedEntry, const ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	const UPCGGraph* PCGGraph = Graph.Get();
	if (!PCGGraph)
	{
		return;
	}

	// SingleToggle fires with an invalid entry when the user clicks an already-selected node to deselect it.
	const bool bIsDeselection = !SelectedEntry.IsValid() || !SelectedEntry->Node.IsValid();
	ActiveNode = bIsDeselection ? nullptr : SelectedEntry->Node;

	// Clear any active visualizer selection before switching nodes, otherwise the proxy/gizmo from the old node
	// persists and attaches to the newly selected node's data.
	if (GUnrealEd)
	{
		if (TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(UPCGComponent::StaticClass()->GetFName()))
		{
			if (FPCGComponentVisualizer* PCGVisualizer = static_cast<FPCGComponentVisualizer*>(Visualizer.Get()))
			{
				PCGVisualizer->ClearActiveSelection();
			}
		}
	}

	// Activate the selected node
	if (!bIsDeselection)
	{
		OnNodeSelected.ExecuteIfBound(SelectedEntry->Node.Get());
	}

	// In ExternallyControlled mode, the external system will control the flags.
	if (CurrentMode == EPCGManualEditPanelMode::UserControlled)
	{
		// The user either clicked the temp node again, toggling it back off, or a different node, which will supercede it.
		auto RemoveTempFlag = [](const UPCGNode* Node)
		{
			if (UPCGSettingsInterface* Settings = Node ? Node->GetSettingsInterface() : nullptr)
			{
				Settings->SetTemporaryManualEditingEnabled(false);
			}

			return true;
		};

		PCGGraph->ForEachNodeRecursively(MoveTemp(RemoveTempFlag));
	}

	// Refresh entries to update active state
	RefreshNodeList();

	// Redraw viewports to trigger visualizer update
	if (GEditor)
	{
		GEditor->RedrawAllViewports();
	}
}

TSharedRef<ITableRow> SPCGManualEditPanel::OnGenerateRow(TSharedPtr<FPCGManualEditNodeEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable)
{
	// Active node: green, Marked but not active: light blue, Unmarked: default
	FLinearColor TextColor = FLinearColor::White;
	if (Entry->bIsActive)
	{
		TextColor = FLinearColor::Green;
	}
	else if (Entry->bIsMarked)
	{
		TextColor = FLinearColor(0.4f, 0.7f, 1.0f); // Light blue to match badge
	}

	TSharedPtr<STableRow<TSharedPtr<FPCGManualEditNodeEntry>>> Row;

	SAssignNew(Row, STableRow<TSharedPtr<FPCGManualEditNodeEntry>>, OwnerTable)
	[
		SNew(SHorizontalBox)
		// Node name
		+ SHorizontalBox::Slot()
	   .FillWidth(1.0f)
	   .VAlign(VAlign_Center)
	   .Padding(4.0f, 2.0f)
		[
			SNew(STextBlock)
		   .Text(Entry->DisplayName)
		   .ColorAndOpacity(TextColor)
		]
		// Jump to node in graph editor
		+ SHorizontalBox::Slot()
	   .AutoWidth()
	   .VAlign(VAlign_Center)
	   .Padding(2.0f, 0.0f)
		[
			SNew(SButton)
		   .ButtonStyle(FAppStyle::Get(), "SimpleButton")
		   .ContentPadding(FMargin(1, 0))
		   .ToolTipText(LOCTEXT("JumpToNodeTooltip", "Focus on this node in the graph editor"))
		   .Visibility_Lambda([WeakRow = TWeakPtr<SWidget>(Row)]()
			{
				TSharedPtr<SWidget> PinnedRow = WeakRow.Pin();
				return (PinnedRow.IsValid() && PinnedRow->IsHovered()) ? EVisibility::Visible : EVisibility::Hidden;
			})
		   .OnClicked_Lambda([this, WeakNode = Entry->Node]() -> FReply
			{
				if (const UPCGNode* Node = WeakNode.Get())
				{
					OnJumpToNode.ExecuteIfBound(Node);
				}
				return FReply::Handled();
			})
			[
				SNew(SImage)
			   .Image(FPCGEditorStyle::Get().GetBrush("PCG.Editor.JumpTo"))
			   .ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
		// Active indicator
		+ SHorizontalBox::Slot()
	   .AutoWidth()
	   .VAlign(VAlign_Center)
	   .Padding(2.0f, 0.0f)
		[
			SNew(STextBlock)
		   .Text(Entry->bIsActive ? LOCTEXT("ActiveIndicator", "LIVE") : FText::GetEmpty())
		   .ColorAndOpacity(FLinearColor::Green)
		   .Font(FCoreStyle::GetDefaultFontStyle("Bold", 8))
		]
	];

	return Row.ToSharedRef();
}

TSharedRef<SWidget> SPCGManualEditPanel::CreateOverlayWidget()
{
	TSharedRef<SPCGManualEditPanel> Self = SharedThis(this);

	SAssignNew(OverlayWidget, UE::ToolWidgets::SDraggableBoxOverlay)
   .HAlign(HAlign_Right)
   .VAlign(VAlign_Top)
   .InitialAlignmentOffset(FVector2f(20.0f, 60.0f))
   .Draggable(true)
	[
		SNew(SBackgroundBlur)
	   .BlurStrength(8.0f)
	   .CornerRadius(FVector4(8.0f, 8.0f, 8.0f, 8.0f))
		[
			SNew(SBorder)
		   .BorderImage(FPCGEditorStyle::Get().GetBrush("PCG.OverlayBackgroundBrush"))
		   .Padding(4.0f)
			[
				SNew(SBorder)
			   .BorderImage(FPCGEditorStyle::Get().GetBrush("PCG.OverlayInnerPanelBrush"))
			   .Padding(6.0f)
				[
					SNew(SBox)
				   .MinDesiredWidth(250.0f)
				   .MinDesiredHeight(120.0f)
				   .MaxDesiredWidth(350.0f)
				   .MaxDesiredHeight(500.0f)
					[
						Self
					]
				]
			]
		]
	];

	return StaticCastSharedRef<SWidget>(OverlayWidget.ToSharedRef());
}

TSharedPtr<SWidget> SPCGManualEditPanel::GetOverlayWidget() const
{
	return OverlayWidget;
}

TSharedPtr<const FPCGManualEditNodeConfiguration> SPCGManualEditPanel::GetConfigurationForActiveNode() const
{
	const UPCGNode* CurrentActiveNode = GetActiveNode();
	if (!CurrentActiveNode)
	{
		return nullptr;
	}

	const TSharedPtr<FPCGManualEditNodeConfiguration>* Found = NodeConfigurationMap.Find(TWeakObjectPtr<const UPCGNode>(CurrentActiveNode));
	return Found ? *Found : nullptr;
}

TSharedPtr<FPCGManualEditNodeConfiguration> SPCGManualEditPanel::GetOrCreateConfigurationForNode(const UPCGNode* InNode)
{
	if (!InNode)
	{
		return nullptr;
	}

	TSharedPtr<FPCGManualEditNodeConfiguration>& Configuration = NodeConfigurationMap.FindOrAdd(InNode);
	if (!Configuration)
	{
		Configuration = MakeShared<FPCGManualEditNodeConfiguration>();
	}

	return Configuration;
}

const UPCGNode* SPCGManualEditPanel::GetActiveNode() const
{
	return ActiveNode.Get();
}

void SPCGManualEditPanel::SelectNode(const UPCGNode* InNode)
{
	if (!InNode || !NodeListView.IsValid())
	{
		return;
	}

	ActiveNode = InNode;

	for (const TSharedPtr<FPCGManualEditNodeEntry>& Entry : NodeEntries)
	{
		if (Entry.IsValid() && Entry->Node == InNode)
		{
			NodeListView->SetSelection(Entry, ESelectInfo::Direct);
			NodeListView->RequestListRefresh();
			break;
		}
	}
}

void SPCGManualEditPanel::SetDeltaContext(UPCGComponent* InComponent, const FPCGSourceDataStorageKey& InStorageKey)
{
	ActivePCGComponent = InComponent;
	ActiveStorageKey = InStorageKey;

	if (ActiveExtension)
	{
		FPCGDeltaViewportContext Context = BuildExtensionContext();
		ActiveExtension->RefreshLists(Context);
	}
}

void SPCGManualEditPanel::SetSelectionState(
	const bool bHasSelection,
	const FPCGDeltaKey& SelectedKey,
	const FTransform& InSelectionTransform,
	const int32 InSelectedElementIndex,
	const int32 InOriginalElementIndex)
{
	bSelectionActive = bHasSelection;
	SelectionDeltaKey = SelectedKey;
	SelectionTransform = InSelectionTransform;
	SelectedElementIndex = InSelectedElementIndex;
	OriginalElementIndex = InOriginalElementIndex;

	if (ActiveExtension)
	{
		ActiveExtension->UpdateContext(BuildExtensionContext());
	}
}

void SPCGManualEditPanel::RefreshActiveExtensionLists()
{
	if (ActiveExtension)
	{
		ActiveExtension->RefreshLists(BuildExtensionContext());
	}
}

void SPCGManualEditPanel::SetVisualizerActions(const FPCGDeltaViewportCallbacks& InActions)
{
	VisualizerCallbacks = InActions;
}

FPCGDeltaViewportCallbacks SPCGManualEditPanel::CreateForwardingCallbacks()
{
	// SharedThis() requires shared ownership. Validate the panel.
	if (!ensure(DoesSharedInstanceExist()))
	{
		return {};
	}

	TWeakPtr<SPCGManualEditPanel> WeakSelf = SharedThis(this);

	// Forwards a single FPCGDeltaViewportCallbacks member to the live VisualizerCallbacks, if bound.
	auto ForwardLambda = [WeakSelf](auto Member)
	{
		return [WeakSelf, Member]<typename... T>(T&&... Args)
		{
			TSharedPtr<SPCGManualEditPanel> Self = WeakSelf.Pin();
			if (Self && (Self->VisualizerCallbacks.*Member))
			{
				Invoke(Self->VisualizerCallbacks.*Member, Forward<T>(Args)...);
			}
		};
	};

	// Construct forwardable callbacks fed to an extension.
	FPCGDeltaViewportCallbacks Callbacks;
	Callbacks.RevertDelta = ForwardLambda(&FPCGDeltaViewportCallbacks::RevertDelta);
	Callbacks.RevertAllRestorableDeltas = ForwardLambda(&FPCGDeltaViewportCallbacks::RevertAllRestorableDeltas);
	Callbacks.RequestListRefresh = ForwardLambda(&FPCGDeltaViewportCallbacks::RequestListRefresh);
	Callbacks.ClearSelection = ForwardLambda(&FPCGDeltaViewportCallbacks::ClearSelection);
	Callbacks.SelectElement = ForwardLambda(&FPCGDeltaViewportCallbacks::SelectElement);
	return Callbacks;
}

void SPCGManualEditPanel::UpdateConfigurationSection()
{
	if (!ConfigurationSection.IsValid())
	{
		return;
	}

	const UPCGNode* CurrentActiveNode = GetActiveNode();

	ConfigurationSection->SetVisibility(EVisibility::SelfHitTestInvisible);

	if (CurrentActiveNode)
	{
		GetOrCreateConfigurationForNode(CurrentActiveNode);
	}

	SwapActiveExtensionIfChanged();
	UpdateAndRefreshActiveExtension();
}

void SPCGManualEditPanel::SwapActiveExtensionIfChanged()
{
	IPCGDeltaViewportExtension* Extension = ActiveDeltaType ? FPCGEditorModule::GetMutableDeltaViewportExtensionRegistry().GetExtension(ActiveDeltaType) : nullptr;

	if (Extension == ActiveExtension)
	{
		return;
	}

	ActiveExtension = Extension;

	if (!DeltaTypeWidgetSlot.IsValid())
	{
		return;
	}

	DeltaTypeWidgetSlot->ClearChildren();

	if (ActiveExtension)
	{
		DeltaTypeWidgetSlot->AddSlot()
		   .AutoHeight()
			[
				ActiveExtension->CreateWidget(CreateForwardingCallbacks())
			];
	}
}

void SPCGManualEditPanel::UpdateAndRefreshActiveExtension()
{
	if (!ActiveExtension)
	{
		return;
	}

	const FPCGDeltaViewportContext Context = BuildExtensionContext();
	ActiveExtension->UpdateContext(Context);
	ActiveExtension->RefreshLists(Context);
}

void SPCGManualEditPanel::OnDeltaTypeChanged(const UScriptStruct* InType, ESelectInfo::Type SelectInfo)
{
	ActiveDeltaType = InType;

	if (VisualizerCallbacks.ClearSelection)
	{
		VisualizerCallbacks.ClearSelection();
	}

	UpdateConfigurationSection();
}

TSharedRef<SWidget> SPCGManualEditPanel::GenerateDeltaTypeMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	for (const UScriptStruct* Type : DeltaTypeOptions)
	{
		FText Label;
		if (const IPCGDeltaViewportExtension* Extension = FPCGEditorModule::GetConstDeltaViewportExtensionRegistry().GetExtension(Type))
		{
			Label = Extension->GetDisplayName();
		}

		MenuBuilder.AddMenuEntry(
			Label,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SPCGManualEditPanel::OnDeltaTypeChanged, Type, ESelectInfo::OnMouseClick))
			);
	}

	return MenuBuilder.MakeWidget();
}

FText SPCGManualEditPanel::GetCurrentDeltaTypeLabel() const
{
	if (ActiveDeltaType)
	{
		if (const IPCGDeltaViewportExtension* Extension = FPCGEditorModule::GetConstDeltaViewportExtensionRegistry().GetExtension(ActiveDeltaType))
		{
			return Extension->GetDisplayName();
		}
	}

	return FText::GetEmpty();
}

FPCGDeltaViewportContext SPCGManualEditPanel::BuildExtensionContext()
{
	FPCGDeltaViewportContext Context;
	Context.bSelectionActive = bSelectionActive;
	Context.SelectionDeltaKey = SelectionDeltaKey;
	Context.SelectionTransform = SelectionTransform;
	Context.SelectedElementIndex = SelectedElementIndex;
	Context.OriginalElementIndex = OriginalElementIndex;
	Context.ActivePCGComponent = ActivePCGComponent;
	Context.ActiveStorageKey = ActiveStorageKey;
	Context.NodeConfiguration = GetConfigurationForActiveNode();

	return Context;
}

#undef LOCTEXT_NAMESPACE
