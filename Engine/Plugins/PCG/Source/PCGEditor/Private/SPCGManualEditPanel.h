// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeltaViewportExtensions/PCGDeltaViewportExtension.h"
#include "Graph/PCGSourceDataContainer.h"
#include "Graph/DataOverride/PCGDataOverride.h"
#include "Graph/DataOverride/PCGDataOverrideHelpers.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class UPCGComponent;
class UPCGGraph;
class UPCGNode;
class UPCGSettingsInterface;

namespace UE::ToolWidgets
{
	class SDraggableBoxOverlay;
}

DECLARE_DELEGATE_OneParam(FOnPCGManualEditPanelNodeSelected, const UPCGNode* /*SelectedNode*/);
DECLARE_DELEGATE_OneParam(FOnPCGJumpToNode, const UPCGNode* /*Node*/);

/** Controls whether the panel node selection is via the user or an external system. */
enum class EPCGManualEditPanelMode : uint8
{
	UserControlled,
	ExternallyControlled,
};

struct FPCGManualEditNodeEntry
{
	TWeakObjectPtr<const UPCGNode> Node;
	FText DisplayName;
	bool bIsMarked = false;
	bool bIsTemporary = false;
	bool bIsActive = false;
};

/** Per-node transient configuration for the manual edit panel. Controls delta creation behavior in the viewport. */
struct FPCGManualEditNodeConfiguration
{
	/** Signature controls. I.e. transform components that are used to match source points. */
	bool bSignaturePosition = true;
	bool bSignatureRotation = false;
	bool bSignatureScale = false;
	double SignatureTolerance = PCG::DataOverride::Constants::SpatialToleranceDefault;
};

class SPCGManualEditPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGManualEditPanel) {}
		SLATE_ARGUMENT(TWeakObjectPtr<UPCGGraph>, Graph)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Refresh the node list from the graph. */
	void RefreshNodeList();

	/**Selects the given node in the panel's list. */
	void SelectNodeInList(const UPCGNode* InNode);

	/** Get the overlay wrapper widget. */
	TSharedPtr<SWidget> GetOverlayWidget() const;

	/** Create the overlay wrapper around this panel for viewport integration. */
	TSharedRef<SWidget> CreateOverlayWidget();

	/** Returns the configuration for the currently selected node, or nullptr if none. */
	TSharedPtr<const FPCGManualEditNodeConfiguration> GetConfigurationForActiveNode() const;

	/** Returns the configuration for the specified node, creating it on demand. */
	TSharedPtr<FPCGManualEditNodeConfiguration> GetOrCreateConfigurationForNode(const UPCGNode* InNode);

	/** Sets the active delta context for querying the collection. */
	void SetDeltaContext(UPCGComponent* InComponent, const FPCGSourceDataStorageKey& InStorageKey);

	/** Update the panel and visualizer's current selection. */
	void SetSelectionState(
		bool bHasSelection,
		const FPCGDeltaKey& SelectedKey,
		const FTransform& InSelectionTransform = FTransform::Identity,
		int32 InSelectedElementIndex = INDEX_NONE,
		int32 InOriginalElementIndex = INDEX_NONE);

	/** Refresh the active extension's lists from the current delta context. */
	void RefreshActiveExtensionLists();

	/** Provide the visualizer callbacks for extensions to invoke. */
	void SetVisualizerActions(const FPCGDeltaViewportCallbacks& InActions);

	/** Delegate fired when a node is selected in the panel. */
	FOnPCGManualEditPanelNodeSelected OnNodeSelected;

	/** Jump to node in the graph editor. */
	FOnPCGJumpToNode OnJumpToNode;

	/** Returns the currently selected node or nullptr if none. */
	const UPCGNode* GetActiveNode() const;

	/** Drives the node list selection. */
	void SelectNode(const UPCGNode* InNode);

	/** Returns the current panel mode. */
	EPCGManualEditPanelMode GetMode() const { return CurrentMode; }

	/** Switches the panel's mode. While ExternallyControlled, external owners control lifetime and temp-flag state. */
	void SetMode(const EPCGManualEditPanelMode InMode) { CurrentMode = InMode; }

	/** Returns the delta type. Persists across node selection changes. */
	const UScriptStruct* GetActiveDeltaType() const { return ActiveDeltaType; }

private:
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FPCGManualEditNodeEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable);
	void OnListSelectionChanged(TSharedPtr<FPCGManualEditNodeEntry> SelectedEntry, ESelectInfo::Type SelectInfo);
	void OnDeltaTypeChanged(const UScriptStruct* InType, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> GenerateDeltaTypeMenu();
	FText GetCurrentDeltaTypeLabel() const;

	/** Rebuilds the configuration section visibility and content. */
	void UpdateConfigurationSection();

	/** Swap the active extension (and its widget) when the active delta type changes. */
	void SwapActiveExtensionIfChanged();

	/** Push the current panel state into the active extension and refresh its lists. */
	void UpdateAndRefreshActiveExtension();

	/** Builds an FPCGDeltaViewportContext from current panel state. */
	FPCGDeltaViewportContext BuildExtensionContext();

	/** Builds the callbacks to be forwarded to the extensions. */
	FPCGDeltaViewportCallbacks CreateForwardingCallbacks();

	/** Graph owning the nodes shown in the panel. Used to discover overridable nodes and observe structural changes. */
	TWeakObjectPtr<UPCGGraph> Graph;

	/** Cached list of nodes eligible for manual editing, rebuilt by RefreshNodeList. */
	TArray<TSharedPtr<FPCGManualEditNodeEntry>> NodeEntries;

	/** Per-node transient configuration map. */
	TMap<TWeakObjectPtr<const UPCGNode>, TSharedPtr<FPCGManualEditNodeConfiguration>> NodeConfigurationMap;

	/** Available delta types for the combo box. */
	TArray<const UScriptStruct*> DeltaTypeOptions;

	/** Node list widget for selecting the active editing node. */
	TSharedPtr<SListView<TSharedPtr<FPCGManualEditNodeEntry>>> NodeListView;

	/** Draggable overlay for viewport handles. */
	TSharedPtr<UE::ToolWidgets::SDraggableBoxOverlay> OverlayWidget;

	/** Collapsible section containing node specific configuration widgets. */
	TSharedPtr<SVerticalBox> ConfigurationSection;

	/** Container for the active delta type extension's widget. Swapped when the delta type combo changes. */
	TSharedPtr<SVerticalBox> DeltaTypeWidgetSlot;

	/** The currently active viewport extension (owned by the registry). */
	IPCGDeltaViewportExtension* ActiveExtension = nullptr;

	/** Callbacks provided by the visualizer for extensions to invoke. */
	FPCGDeltaViewportCallbacks VisualizerCallbacks;

	/** The PCG component currently being edited. */
	TWeakObjectPtr<UPCGComponent> ActivePCGComponent;

	/** Storage key identifying the delta collection on the active component. */
	FPCGSourceDataStorageKey ActiveStorageKey;

	/** True when the visualizer has an active element selection. */
	bool bSelectionActive = false;

	/** Index of the selected element within an insertion delta, or INDEX_NONE. */
	int32 SelectedElementIndex = INDEX_NONE;

	/** Index of the selected point in the original source data, or INDEX_NONE. */
	int32 OriginalElementIndex = INDEX_NONE;

	/** The delta key of the currently selected element. */
	FPCGDeltaKey SelectionDeltaKey;

	/** World-space transform of the currently selected element. */
	FTransform SelectionTransform = FTransform::Identity;

	EPCGManualEditPanelMode CurrentMode = EPCGManualEditPanelMode::UserControlled;

	TWeakObjectPtr<const UPCGNode> ActiveNode;

	const UScriptStruct* ActiveDeltaType = nullptr;
};
