// Copyright Epic Games, Inc. All Rights Reserved.

#include "Schema/PCGEditorGraphSchema.h"

#include "PCGComponent.h"
#include "PCGDataAsset.h"
#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGPin.h"
#include "PCGSubgraph.h"
#include "Elements/PCGCollapseElement.h"
#include "Elements/PCGCreateSurfaceFromSpline.h"
#include "Elements/PCGExecuteBlueprint.h"
#include "Elements/PCGFilterByType.h"
#include "Elements/PCGMakeConcreteElement.h"
#include "Elements/PCGReroute.h"
#include "Elements/PCGUserParameterGet.h"
#include "Helpers/PCGAssetHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Helpers/PCGSubgraphHelpers.h"

#include "PCGEditor.h"
#include "PCGEditorCommon.h"
#include "PCGEditorGraph.h"
#include "PCGEditorSettings.h"
#include "PCGEditorUtils.h"
#include "Nodes/PCGEditorGraphNodeBase.h"
#include "Nodes/PCGEditorGraphNodeReroute.h"
#include "Schema/PCGEditorGraphSchemaActions.h"

#include "Algo/AnyOf.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/BlueprintSupport.h"
#include "Editor.h"
#include "Editor/AssetReferenceFilter.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "UObject/UObjectIterator.h"

#include "SGraphPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorGraphSchema)

#define LOCTEXT_NAMESPACE "PCGEditorGraphSchema"

namespace PCGActionsHelpers
{
	bool IsValid(const EPCGDataTypeCompatibilityResult Result, const ECompatibilityMask Mask)
	{
		const bool bIsValid = PCGDataTypeCompatibilityResult::IsValid(Result);

		if (!bIsValid || Mask == ECompatibilityMask::NoMask)
		{
			return bIsValid;
		}
		else
		{
			return !!(Mask & static_cast<ECompatibilityMask>(1 << std::underlying_type_t<EPCGDataTypeCompatibilityResult>(Result)));
		}
	}

	ECompatibilityMask GetPinAndCompatibilityMask(const UEdGraphPin* Pin, const UPCGEditorGraph* PCGEdGraph, const UPCGPin*& OutPin)
	{
		const UEdGraphNode* Node = Pin ? Pin->GetOwningNode() : nullptr;
		const UPCGEditorGraphNodeBase* EditorNode = Cast<UPCGEditorGraphNodeBase>(Node);
		const UPCGNode* PCGNode = EditorNode ? EditorNode->GetPCGNode() : nullptr;
		OutPin = (Pin && PCGNode) ? (Pin->Direction == EGPD_Output ? PCGNode->GetOutputPin(Pin->PinName) : PCGNode->GetInputPin(Pin->PinName)) : nullptr;

		const FPCGEditor* PCGEditor = PCGEdGraph ? PCGEdGraph->GetEditor().Pin().Get() : nullptr;
		return PCGEditor ? PCGEditor->GetContextualCompatibilityMask() : ECompatibilityMask::NoMask;
	}

	bool HasCompatiblePins(const UPCGPin* Pin, const TArray<FPCGPinProperties>& PinProps, ECompatibilityMask CompatibilityMask)
	{
		if (!Pin || CompatibilityMask == ECompatibilityMask::NoMask)
		{
			// Not super relevant here because we don't have enough information to filter against.
			return true;
		}

		if (Pin->IsOutputPin())
		{
			const bool bHasRequiredPins = Algo::AnyOf(PinProps, [](const FPCGPinProperties& Props) { return Props.IsRequiredPin(); });
			const bool bHasNormalPins = Algo::AnyOf(PinProps, [](const FPCGPinProperties& Props) { return Props.IsNormalPin(); });

			return Algo::AnyOf(PinProps, [Pin, bHasRequiredPins, bHasNormalPins, CompatibilityMask](const FPCGPinProperties& Props)
				{
					return (Props.IsRequiredPin() ||
						(!bHasRequiredPins && Props.IsNormalPin()) ||
						(!bHasRequiredPins && !bHasNormalPins && Props.IsAdvancedPin())) &&
						IsValid(FPCGModule::GetConstDataTypeRegistry().IsCompatible(Pin->GetCurrentTypesID(), Props.AllowedTypes), CompatibilityMask);
				});
		}
		else
		{
			// While it is tempting to test in the expected order, since we're looking to add a new node, we can accept anything that we can narrow from the current type - otherwise we might be throwing away all 'any' nodes.
			return Algo::AnyOf(PinProps, [Pin, CompatibilityMask](const FPCGPinProperties& Props) { return IsValid(FPCGModule::GetConstDataTypeRegistry().IsCompatible(Pin->GetCurrentTypesID(), Props.AllowedTypes), CompatibilityMask); });
		}
	}

	bool IsCompatibleSettings(const UPCGSettings* Settings, const UPCGPin* Pin, ECompatibilityMask CompatibilityMask)
	{
		if (!Settings || !Pin || CompatibilityMask == ECompatibilityMask::NoMask)
		{
			// Not super relevant here because we either don't have settings (so there's nothing to be compatible with) or don't have enough information to filter against.
			return true;
		}

		return HasCompatiblePins(Pin, Pin->IsOutputPin() ? Settings->InputPinProperties() : Settings->OutputPinProperties(), CompatibilityMask);
	}

	bool IsCompatibleType(const UPCGPin* Pin, FPCGDataTypeIdentifier AllowedTypes, ECompatibilityMask CompatibilityMask)
	{
		if (!Pin)
		{
			return true;
		}

		if (Pin->IsOutputPin())
		{
			return IsValid(FPCGModule::GetConstDataTypeRegistry().IsCompatible(Pin->GetCurrentTypesID(), { AllowedTypes }), CompatibilityMask);
		}
		else
		{
			return IsValid(FPCGModule::GetConstDataTypeRegistry().IsCompatible({ AllowedTypes }, Pin->GetCurrentTypesID()), CompatibilityMask);
		}
	}
}

FPCGActionsFilter::FPCGActionsFilter(const UEdGraph* InEdGraph, EPCGElementType InElementFilterType, FPCGGraphEditorCustomization InCustomization)
	: FilterType(InElementFilterType)
	, Graph(Cast<UPCGEditorGraph>(InEdGraph))
	, Customization(MoveTemp(InCustomization))
{
}

bool FPCGActionsFilter::Accepts(const FText& InCategory) const
{
	return Customization.Accepts(InCategory);
}

void UPCGEditorGraphSchema::GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const
{
	GetGraphActions(ActionMenuBuilder, ActionFilter, /*bIsContextual=*/false);
}

TSubclassOf<UPCGBaseSubgraphSettings> UPCGEditorGraphSchema::GetSubgraphSettingsClass() const
{
	return UPCGSubgraphSettings::StaticClass();
}

const FPCGGraphEditorCustomization& UPCGEditorGraphSchema::GetGraphEditorCustomization(const UEdGraph* InEdGraph) const
{
	static const FPCGGraphEditorCustomization Default{};
	const UPCGEditorGraph* PCGGraph = Cast<UPCGEditorGraph>(InEdGraph);
	return (PCGGraph && PCGGraph->GetPCGGraph()) ? PCGGraph->GetPCGGraph()->GraphCustomization : Default;
}

void UPCGEditorGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	Super::GetGraphContextActions(ContextMenuBuilder);

	FPCGActionsFilter ActionFilter(ContextMenuBuilder.CurrentGraph, GetElementTypeFiltering(), GetGraphEditorCustomization(ContextMenuBuilder.CurrentGraph));

	GetGraphActions(ContextMenuBuilder, ActionFilter, /*bIsContextual=*/true);
}

void UPCGEditorGraphSchema::GetGraphActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter, bool bIsContextual) const
{
	EPCGElementType ElementTypeFilter = ActionFilter.FilterType;

	if (!!(ElementTypeFilter & EPCGElementType::Native))
	{
		GetNativeElementActions(ActionMenuBuilder, ActionFilter);
	}
	if (!!(ElementTypeFilter & EPCGElementType::Subgraph))
	{
		GetSubgraphElementActions(ActionMenuBuilder, ActionFilter);
	}
	if (!!(ElementTypeFilter & EPCGElementType::Blueprint))
	{
		GetBlueprintElementActions(ActionMenuBuilder, ActionFilter);
	}
	if (!!(ElementTypeFilter & EPCGElementType::Settings))
	{
		GetSettingsElementActions(ActionMenuBuilder, ActionFilter, bIsContextual);
	}
	if (!!(ElementTypeFilter & EPCGElementType::Asset))
	{
		GetDataAssetActions(ActionMenuBuilder, ActionFilter);
	}
	if (!!(ElementTypeFilter & EPCGElementType::Other))
	{
		GetNamedRerouteUsageActions(ActionMenuBuilder, ActionFilter);
		GetExtraElementActions(ActionMenuBuilder, ActionFilter);
	}
}

FConnectionDrawingPolicy* UPCGEditorGraphSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new FPCGEditorConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj);
}

const FPinConnectionResponse UPCGEditorGraphSchema::CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const
{
	check(A && B);
	const UEdGraphNode* NodeA = A->GetOwningNode();
	const UEdGraphNode* NodeB = B->GetOwningNode();

	if (NodeA == NodeB)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionSameNode", "Both pins are on same node"));
	}

	if (A->Direction == B->Direction)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionSameDirection", "Both pins are the same direction"));
	}

	const UPCGEditorGraphNodeBase* EditorNodeA = CastChecked<const UPCGEditorGraphNodeBase>(NodeA);
	const UPCGEditorGraphNodeBase* EditorNodeB = CastChecked<const UPCGEditorGraphNodeBase>(NodeB);
	const UPCGEditorGraphNodeBase* EditorNodeWithInput = nullptr;

	// Check type compatibility & whether we can connect more pins
	const UPCGPin* InputPin = nullptr;
	const UPCGPin* OutputPin = nullptr;

	if (A->Direction == EGPD_Output)
	{
		OutputPin = EditorNodeA->GetPCGNode()->GetOutputPin(A->PinName);
		InputPin = EditorNodeB->GetPCGNode()->GetInputPin(B->PinName);
		EditorNodeWithInput = EditorNodeB;
	}
	else
	{
		OutputPin = EditorNodeB->GetPCGNode()->GetOutputPin(B->PinName);
		InputPin = EditorNodeA->GetPCGNode()->GetInputPin(A->PinName);
		EditorNodeWithInput = EditorNodeA;
	}

	if (!InputPin || !OutputPin)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionFailed", "Unable to verify pins"));
	}

	FText CustomCompatiblityMessage{};
	const EPCGDataTypeCompatibilityResult Compatibility = InputPin->GetCompatibilityWithOtherPin(OutputPin, &CustomCompatiblityMessage);

	switch (Compatibility)
	{
	case EPCGDataTypeCompatibilityResult::RequireFilter:
		return FPinConnectionResponse(CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE, CustomCompatiblityMessage.IsEmpty() ? LOCTEXT("ConnectionUsingFilter", "Filter data to match type") : CustomCompatiblityMessage);
	case EPCGDataTypeCompatibilityResult::RequireConversion:
		return FPinConnectionResponse(CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE, CustomCompatiblityMessage.IsEmpty() ? LOCTEXT("ConnectionConversion", "Convert data to match type") : CustomCompatiblityMessage);
	case EPCGDataTypeCompatibilityResult::TypeCompatibleSubtypeNotCompatible: // fall-through
	case EPCGDataTypeCompatibilityResult::NotCompatible: // fall-through
	case EPCGDataTypeCompatibilityResult::UnknownType:
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, CustomCompatiblityMessage.IsEmpty() ? LOCTEXT("ConnectionTypesIncompatible", "Pins are incompatible") : CustomCompatiblityMessage);
	case EPCGDataTypeCompatibilityResult::Compatible:// fall-through
	default:
		break;
	}

	if (!InputPin->AllowsMultipleConnections() && InputPin->EdgeCount() > 0)
	{
		return FPinConnectionResponse((A->Direction == EGPD_Output) ? CONNECT_RESPONSE_BREAK_OTHERS_B : CONNECT_RESPONSE_BREAK_OTHERS_A, LOCTEXT("ConnectionBreakExisting", "Break existing connection?"));
	}

	FText Reason;
	if (!EditorNodeWithInput->IsCompatible(InputPin, OutputPin, Reason))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, Reason);
	}

	return FPinConnectionResponse();
}

bool UPCGEditorGraphSchema::TryCreateConnection(UEdGraphPin* InA, UEdGraphPin* InB) const
{
	return TryCreateConnectionInternal(InA, InB, /*bAddConversionNodeIfNeeded=*/true);
}

bool UPCGEditorGraphSchema::TryCreateConnectionInternal(UEdGraphPin* InA, UEdGraphPin* InB, bool bAddConversionNodeIfNeeded) const
{
	check(InA && InB);
	if (InA->Direction == InB->Direction)
	{
		// Don't connect same polarity
		return false;
	}

	UEdGraphPin* A = (InA->Direction == EGPD_Output) ? InA : InB;
	UEdGraphPin* B = (InA->Direction == EGPD_Input) ? InA : InB;
	check(A->Direction == EGPD_Output && B->Direction == EGPD_Input);

	UEdGraphNode* NodeA = A->GetOwningNodeUnchecked();
	UEdGraphNode* NodeB = B->GetOwningNodeUnchecked();
	if (!ensure(NodeA && NodeB))
	{
		// TODO: We've had crashes where one of these nodes was nullptr, we need to figure out why this can happen.
		return false;
	}

	UPCGEditorGraphNodeBase* PCGEdGraphNodeA = CastChecked<UPCGEditorGraphNodeBase>(NodeA);
	UPCGEditorGraphNodeBase* PCGEdGraphNodeB = CastChecked<UPCGEditorGraphNodeBase>(NodeB);

	UPCGNode* PCGNodeA = PCGEdGraphNodeA->GetPCGNode();
	UPCGNode* PCGNodeB = PCGEdGraphNodeB->GetPCGNode();
	check(PCGNodeA && PCGNodeB);

	UPCGPin* PCGPinA = PCGNodeA->GetOutputPin(A->PinName);
	UPCGPin* PCGPinB = PCGNodeB->GetInputPin(B->PinName);
	check(PCGPinA && PCGPinB);

	const EPCGDataTypeCompatibilityResult Compatibility = PCGPinA->GetCompatibilityWithOtherPin(PCGPinB);
	
	if (!PCGDataTypeCompatibilityResult::IsValid(Compatibility))
	{
		return false;
	}

	UPCGGraph* PCGGraph = PCGNodeA->GetGraph();
	if (!ensure(PCGGraph))
	{
		return false;
	}

	// UPCGEditorGraphSchema::TryCreateConnectionInternal is called directly by FDragConnection::DroppedOnPin
	PCGGraph->PrimeGraphCompilationCache();

	bool bShouldReconstruct = false;
	
	if (Compatibility == EPCGDataTypeCompatibilityResult::RequireFilter && bAddConversionNodeIfNeeded)
	{
		UPCGNode* ConversionPCGNode = FPCGSubgraphHelpers::SpawnNodeAndConnect(PCGGraph, PCGPinA, PCGPinB, UPCGFilterByTypeSettings::StaticClass(), [PCGPinB](UPCGSettings* NodeSettings)
		{
			UPCGFilterByTypeSettings* Settings = CastChecked<UPCGFilterByTypeSettings>(NodeSettings);
			Settings->TargetType = PCGPinB->Properties.AllowedTypes;
			return true;
		});

		if (ConversionPCGNode)
		{
			bShouldReconstruct = true;
		}
	}
	else if (Compatibility == EPCGDataTypeCompatibilityResult::RequireConversion && bAddConversionNodeIfNeeded)
	{
		const FPCGDataTypeRegistry& Registry = FPCGModule::GetConstDataTypeRegistry();

		const FPCGDataTypeIdentifier PinATypes = PCGPinA->GetCurrentTypesID();
		const FPCGDataTypeIdentifier PinBTypes = PCGPinB->GetCurrentTypesID();

		auto AddConversionNodes = [&Registry, PCGPinA, PCGPinB, &PinATypes, &PinBTypes, PCGGraph, &bShouldReconstruct](const FPCGDataTypeBaseId& Id, bool bIsInput) -> bool
		{
			const FPCGDataTypeInfo* TypeInfo = Registry.GetTypeInfo(Id);

			if (bIsInput && TypeInfo && TypeInfo->AddConversionNodesTo(PinATypes, PinBTypes, PCGGraph, PCGPinA, PCGPinB).IsSet())
			{
				bShouldReconstruct = true;
				return true;
			}
			else if (!bIsInput && TypeInfo && TypeInfo->AddConversionNodesFrom(PinATypes, PinBTypes, PCGGraph, PCGPinA, PCGPinB).IsSet())
			{
				bShouldReconstruct = true;
				return true;
			}

			return false;
		};

		// Try for both pins
		for (const FPCGDataTypeBaseId& Id : PinATypes.GetIds())
		{
			if (AddConversionNodes(Id, /*bIsInput=*/ true))
			{
				break;
			}
		}

		if (!bShouldReconstruct)
		{
			for (const FPCGDataTypeBaseId& Id : PinBTypes.GetIds())
			{
				if (AddConversionNodes(Id, /*bIsInput=*/ false))
				{
					break;
				}
			}
		}
	}
	else
	{
		const bool bModified = Super::TryCreateConnection(InA, InB);
		if (bModified)
		{
			PCGGraph->AddLabeledEdge(PCGNodeA, A->PinName, PCGNodeB, B->PinName);
		}

		return bModified;
	}

	if (bShouldReconstruct)
	{
		if (UPCGEditorGraph* Graph = Cast<UPCGEditorGraph>(NodeA->GetGraph()); ensure(Graph))
		{
			Graph->ReconstructGraph();
		}
	}

	return bShouldReconstruct;
}

void UPCGEditorGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const
{
	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorBreakPinLinks", "Break Pin Links"), nullptr);

	Super::BreakPinLinks(TargetPin, bSendsNodeNotification);

	UEdGraphNode* GraphNode = TargetPin.GetOwningNode();

	UPCGEditorGraphNodeBase* PCGGraphNode = CastChecked<UPCGEditorGraphNodeBase>(GraphNode);

	UPCGNode* PCGNode = PCGGraphNode->GetPCGNode();
	check(PCGNode);

	UPCGGraph* PCGGraph = PCGNode->GetGraph();
	if (!ensure(PCGGraph))
	{
		return;
	}

	if (TargetPin.Direction == EEdGraphPinDirection::EGPD_Input)
	{
		PCGGraph->RemoveInboundEdges(PCGNode, TargetPin.PinName);
	}
	else if (TargetPin.Direction == EEdGraphPinDirection::EGPD_Output)
	{
		PCGGraph->RemoveOutboundEdges(PCGNode, TargetPin.PinName);
	}
}

void UPCGEditorGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorBreakSinglePinLink", "Break Single Pin Link"), nullptr);

	Super::BreakSinglePinLink(SourcePin, TargetPin);

	UEdGraphNode* SourceGraphNode = SourcePin->GetOwningNode();
	UEdGraphNode* TargetGraphNode = TargetPin->GetOwningNode();

	UPCGEditorGraphNodeBase* SourcePCGGraphNode = CastChecked<UPCGEditorGraphNodeBase>(SourceGraphNode);
	UPCGEditorGraphNodeBase* TargetPCGGraphNode = CastChecked<UPCGEditorGraphNodeBase>(TargetGraphNode);

	UPCGNode* SourcePCGNode = SourcePCGGraphNode->GetPCGNode();
	UPCGNode* TargetPCGNode = TargetPCGGraphNode->GetPCGNode();
	check(SourcePCGNode && TargetPCGNode);

	UPCGGraph* PCGGraph = SourcePCGNode->GetGraph();
	if (!ensure(PCGGraph))
	{
		return;
	}

	PCGGraph->RemoveEdge(SourcePCGNode, SourcePin->PinName, TargetPCGNode, TargetPin->PinName);
}

void UPCGEditorGraphSchema::GetNativeElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const
{
	// TO BE REMOVED IN 5.8
	// @todo_pcg: alogut
	// We'll design a extensible way to specify which settings is compatible to which editor in 5.8.
	// In the meantime, hide all Procedural Vegetation node in other editors than the PVEditor.
	const UPackage* PVPackage = FindPackage(nullptr, TEXT("/Script/ProceduralVegetation"));
	const UPackage* PVEPackage = FindPackage(nullptr, TEXT("/Script/ProceduralVegetationEditor"));
	const bool bIsPVEditor = GetClass() && GetClass()->GetOuterUPackage() == PVEPackage;

	// Returns true if it is a PV node in a non-PV editor.
	auto ExtraPVEFilter = [PVPackage, bIsPVEditor](TSubclassOf<UPCGSettings> InSettingsClass)
	{
		return !bIsPVEditor && PVPackage && InSettingsClass && InSettingsClass->GetOuterUPackage() == PVPackage;
	};
	
	TArray<UClass*> SettingsClasses;
	for (TObjectIterator<UClass> It; It; ++It)
	{
		TSubclassOf<UPCGSettings> Class(*It);

		// TO BE REMOVED IN 5.8
		// @todo_pcg: alogut
		// We'll design a extensible way to specify which settings is compatible to which editor in 5.8.
		// In the meantime, hide all Procedural Vegetation node in other editors than the PVEditor.
		if (ExtraPVEFilter(Class))
		{
			continue;
		}

		if (*Class &&
			!Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_Hidden) &&
			(!ActionFilter.Customization.FiltersSettings() || !ActionFilter.Customization.FilterSettings(Class)))
		{
			SettingsClasses.Add(Class);
		}
	}

	const UEdGraphPin* Pin = ActionMenuBuilder.FromPin;
	const UPCGPin* PCGPin = nullptr;
	PCGActionsHelpers::ECompatibilityMask CompatibilityMask = PCGActionsHelpers::GetPinAndCompatibilityMask(Pin, ActionFilter.Graph, PCGPin);

	for (UClass* SettingsClass : SettingsClasses)
	{
		if (const UPCGSettings* PCGSettings = SettingsClass->GetDefaultObject<UPCGSettings>())
		{
			if (PCGSettings->bExposeToLibrary)
			{
				const FText MenuDesc = PCGSettings->GetDefaultNodeTitle();
				const FText Category = StaticEnum<EPCGSettingsType>()->GetDisplayNameTextByValue(static_cast<__underlying_type(EPCGSettingsType)>(PCGSettings->GetType()));
				const FText Description = PCGSettings->GetNodeTooltipText();

				if (!ActionFilter.Accepts(Category))
				{
					continue;
				}

				TArray<FPCGPreConfiguredSettingsInfo> AllPreconfiguredInfo = PCGSettings->GetPreconfiguredInfo();

				const FText Keywords = FText::FromString(SettingsClass->GetMetaData(TEXT("Keywords")));

				if (PCGActionsHelpers::IsCompatibleSettings(PCGSettings, PCGPin, CompatibilityMask) && 
					(AllPreconfiguredInfo.IsEmpty() || !PCGSettings->OnlyExposePreconfiguredSettings()))
				{
					TSharedPtr<FPCGEditorGraphSchemaAction_NewNativeElement> NewAction(new FPCGEditorGraphSchemaAction_NewNativeElement(Category, MenuDesc, Description, 0, Keywords));
					NewAction->SettingsClass = SettingsClass;
					ActionMenuBuilder.AddAction(NewAction);

					// Also add all aliases
					for (const FText& Alias : PCGSettings->GetNodeTitleAliases())
					{
						TSharedPtr<FPCGEditorGraphSchemaAction_NewNativeElement> NewAliasAction(new FPCGEditorGraphSchemaAction_NewNativeElement(Category, Alias, Description, 0, Keywords));
						NewAliasAction->SettingsClass = SettingsClass;
						ActionMenuBuilder.AddAction(NewAliasAction);
					}
				}

				if (AllPreconfiguredInfo.IsEmpty())
				{
					continue;
				}

				// Also add preconfigured settings
				const FText NewCategory = PCGSettings->GroupPreconfiguredSettings() ? FText::Format(LOCTEXT("PreconfiguredSettingsCategory", "{0}|{1}"), Category, MenuDesc) : Category;

				if (!ActionFilter.Accepts(NewCategory))
				{
					continue;
				}

				// Create example setting so we can apply preconfigured info on it, and do proper validation
				UPCGSettings* TemporarySettings = NewObject<UPCGSettings>(GetTransientPackage(), SettingsClass, NAME_None);

				for (FPCGPreConfiguredSettingsInfo PreconfiguredInfo : AllPreconfiguredInfo)
				{
					TemporarySettings->ApplyPreconfiguredSettings(PreconfiguredInfo);
					if (!PCGActionsHelpers::IsCompatibleSettings(TemporarySettings, PCGPin, CompatibilityMask))
					{
						continue;
					}

					TSharedPtr<FPCGEditorGraphSchemaAction_NewNativeElement> NewPreconfiguredAction(new FPCGEditorGraphSchemaAction_NewNativeElement(NewCategory, PreconfiguredInfo.Label, PreconfiguredInfo.Tooltip.IsEmpty() ? Description : PreconfiguredInfo.Tooltip,0, PreconfiguredInfo.SearchHints));
					NewPreconfiguredAction->SettingsClass = SettingsClass;
					NewPreconfiguredAction->PreconfiguredInfo = std::move(PreconfiguredInfo);
					ActionMenuBuilder.AddAction(NewPreconfiguredAction);
				}

				TemporarySettings->MarkAsGarbage();
			}
		}
	}

	const FText UserParameterCategory = LOCTEXT("UserParametersCategoryName", "Graph Parameters");

	// @todo_pcg review usage of the compatible call also for specific subtypes, but might need more info from the user parameters
	// Add getters only when it's an input pin (or no pin), and if we have filtering, don't show them if the pin isn't compatible with attribute sets.
	if ((!Pin || Pin->Direction == EGPD_Input) && 
		PCGActionsHelpers::IsCompatibleType(PCGPin, EPCGDataType::Param, CompatibilityMask) &&
		ActionFilter.Accepts(UserParameterCategory))
	{
		const UPCGEditorGraph* Graph = ActionFilter.Graph;
		const UPCGGraph* PCGGraph = Graph ? const_cast<UPCGEditorGraph*>(Graph)->GetPCGGraph() : nullptr;
		const FInstancedPropertyBag* UserParameters = PCGGraph ? PCGGraph->GetUserParametersStruct() : nullptr;
		if (const UPropertyBag* BagStruct = UserParameters ? UserParameters->GetPropertyBagStruct() : nullptr)
		{
			for (const FPropertyBagPropertyDesc& PropertyDesc : BagStruct->GetPropertyDescs())
			{
				const FText MenuDesc = FText::Format(LOCTEXT("GetterNodeName", "Get {0}"), FText::FromName(PropertyDesc.Name));
				const FText Description = FText::Format(LOCTEXT("NodeTooltip", "Get the value from '{0}' parameter, can be overridden by the graph instance."), FText::FromName(PropertyDesc.Name));

				TSharedPtr<FPCGEditorGraphSchemaAction_NewGetParameterElement> NewAction(new FPCGEditorGraphSchemaAction_NewGetParameterElement(UserParameterCategory, MenuDesc, Description, 0));
				NewAction->SettingsClass = UPCGUserParameterGetSettings::StaticClass();
				NewAction->PropertyDesc = PropertyDesc;
				ActionMenuBuilder.AddAction(NewAction);
			}
		}
	}
}

void UPCGEditorGraphSchema::GetBlueprintElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const
{
	const UEdGraphPin* Pin = ActionMenuBuilder.FromPin;
	const UPCGPin* PCGPin = nullptr;
	PCGActionsHelpers::ECompatibilityMask CompatibilityMask = PCGActionsHelpers::GetPinAndCompatibilityMask(Pin, ActionFilter.Graph, PCGPin);

	PCGEditorUtils::ForEachPCGBlueprintAssetData([&ActionMenuBuilder, &ActionFilter, PCGPin, CompatibilityMask](const FAssetData& AssetData)
	{
		const bool bExposeToLibrary = AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UPCGBlueprintBaseElement, bExposeToLibrary));
		if (bExposeToLibrary)
		{
			FPCGAssetHelpers::FBlueprintAssetOutput AssetOutput;
			if (!FPCGAssetHelpers::GetBlueprintAssetRegistryData(AssetData, AssetOutput))
			{
				return true;
			}

			const FText& Category = AssetOutput.Category;
			const FText& Description = AssetOutput.Description;

			if (!ActionFilter.Accepts(Category))
			{
				return true;
			}

			// @todo_pcg: Consider preconfigured info for pins?
			if (AssetOutput.CachedPins.bHasCachedPins && PCGPin)
			{
				const TArray<FPCGPinProperties>& BlueprintPins = PCGPin->IsOutputPin() ? AssetOutput.CachedPins.InputPins : AssetOutput.CachedPins.OutputPins;
				if (!PCGActionsHelpers::HasCompatiblePins(PCGPin, BlueprintPins, CompatibilityMask))
				{
					return true;
				}
			}

			// Only load the class if we have enabled preconfigured settings.
			TArray<FPCGPreConfiguredSettingsInfo> AllPreconfiguredInfo;
			if(AssetOutput.bEnabledPreconfiguredSettings)
			{
				TSubclassOf<UPCGBlueprintBaseElement> BlueprintClass = AssetOutput.GeneratedClass.TryLoadClass<UPCGBlueprintBaseElement>();
				const UPCGBlueprintBaseElement* BlueprintElement = BlueprintClass ? Cast<UPCGBlueprintBaseElement>(BlueprintClass->GetDefaultObject()) : nullptr;

				AllPreconfiguredInfo = BlueprintElement ? BlueprintElement->PreconfiguredInfo : TArray<FPCGPreConfiguredSettingsInfo>{};
			}

			const FText MenuDesc = FText::FromString(FName::NameToDisplayString(AssetData.AssetName.ToString(), false));

			if (AllPreconfiguredInfo.IsEmpty() || !AssetOutput.bOnlyExposePreconfiguredSettings)
			{
				TSharedPtr<FPCGEditorGraphSchemaAction_NewBlueprintElement> NewBlueprintAction(new FPCGEditorGraphSchemaAction_NewBlueprintElement(Category, MenuDesc, Description, 0));
				NewBlueprintAction->BlueprintClassPath = AssetOutput.GeneratedClass;
				ActionMenuBuilder.AddAction(NewBlueprintAction);
			}

			// Also add preconfigured settings
			const FText NewCategory = FText::Format(LOCTEXT("PreconfiguredSettingsCategory", "{0}|{1}"), Category, MenuDesc);

			if (!ActionFilter.Accepts(NewCategory))
			{
				return true;
			}

			for (FPCGPreConfiguredSettingsInfo PreconfiguredInfo : AllPreconfiguredInfo)
			{
				TSharedPtr<FPCGEditorGraphSchemaAction_NewBlueprintElement> NewPreconfiguredAction(new FPCGEditorGraphSchemaAction_NewBlueprintElement(NewCategory, PreconfiguredInfo.Label, PreconfiguredInfo.Tooltip.IsEmpty() ? Description : PreconfiguredInfo.Tooltip, 0));
				NewPreconfiguredAction->BlueprintClassPath = AssetOutput.GeneratedClass;
				NewPreconfiguredAction->PreconfiguredInfo = std::move(PreconfiguredInfo);
				ActionMenuBuilder.AddAction(NewPreconfiguredAction);
			}
		}

		return true;
	});
}

void UPCGEditorGraphSchema::GetSettingsElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter, bool bIsContextual) const
{
	const FText UncategorizedAssetsCategory = LOCTEXT("AssetMenuDescriptionUncategorized", "Uncategorized Assets");
	const UEdGraphPin* Pin = ActionMenuBuilder.FromPin;
	const UPCGPin* PCGPin = nullptr;
	PCGActionsHelpers::ECompatibilityMask CompatibilityMask = PCGActionsHelpers::GetPinAndCompatibilityMask(Pin, ActionFilter.Graph, PCGPin);

	//@todo_pcg: should add settings instances too if they are assets
	PCGEditorUtils::ForEachPCGSettingsAssetData([&ActionMenuBuilder, &ActionFilter, bIsContextual, &UncategorizedAssetsCategory, PCGPin, CompatibilityMask](const FAssetData& AssetData)
	{
		const bool bExposeToLibrary = AssetData.GetTagValueRef<bool>(TEXT("bExposeToLibrary"));
		if (bExposeToLibrary)
		{
			FPCGAssetHelpers::FSettingsAssetOutput AssetOutput;
			if (!FPCGAssetHelpers::GetSettingsAssetRegistryData(AssetData, AssetOutput))
			{
				return true;
			}

			const FText& Category = AssetOutput.Category.IsEmptyOrWhitespace() ? UncategorizedAssetsCategory : AssetOutput.Category;

			if (!ActionFilter.Accepts(Category))
			{
				return true;
			}

			if (AssetOutput.CachedPins.bHasCachedPins && PCGPin)
			{
				const TArray<FPCGPinProperties>& SettingsPins = PCGPin->IsOutputPin() ? AssetOutput.CachedPins.InputPins : AssetOutput.CachedPins.OutputPins;
				if (!PCGActionsHelpers::HasCompatiblePins(PCGPin, SettingsPins, CompatibilityMask))
				{
					return true;
				}
			}

			const FText Description = FText::Format(INVTEXT("{0}\n{1}"), FText::FromString(AssetData.GetObjectPathString()), AssetOutput.Description);
			const FText MenuDesc = FText::FromString(FName::NameToDisplayString(AssetData.AssetName.ToString(), false));

			if (!bIsContextual)
			{
				TSharedPtr<FPCGEditorGraphSchemaAction_NewSettingsElement> NewSettingsAction(new FPCGEditorGraphSchemaAction_NewSettingsElement(Category, MenuDesc, Description, 0));
				NewSettingsAction->SettingsObjectPath = AssetData.GetSoftObjectPath();
				ActionMenuBuilder.AddAction(NewSettingsAction);
			}
			else
			{
				TSharedPtr<FPCGEditorGraphSchemaAction_NewSettingsElement> NewSettingsActionCopy(new FPCGEditorGraphSchemaAction_NewSettingsElement(Category, MenuDesc, Description, 0));
				NewSettingsActionCopy->SettingsObjectPath = AssetData.GetSoftObjectPath();
				NewSettingsActionCopy->Behavior = EPCGEditorNewSettingsBehavior::ForceCopy;
				ActionMenuBuilder.AddAction(NewSettingsActionCopy);

				TSharedPtr<FPCGEditorGraphSchemaAction_NewSettingsElement> NewSettingsActionInstance(new FPCGEditorGraphSchemaAction_NewSettingsElement(Category, MenuDesc, Description, 0));
				NewSettingsActionInstance->SettingsObjectPath = AssetData.GetSoftObjectPath();
				NewSettingsActionInstance->Behavior = EPCGEditorNewSettingsBehavior::ForceInstance;
				ActionMenuBuilder.AddAction(NewSettingsActionInstance);
			}
		}

		return true;
	});
}

void UPCGEditorGraphSchema::GetSubgraphElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const
{
	const UEdGraphPin* Pin = ActionMenuBuilder.FromPin;
	const UPCGPin* PCGPin = nullptr;
	PCGActionsHelpers::ECompatibilityMask CompatibilityMask = PCGActionsHelpers::GetPinAndCompatibilityMask(Pin, ActionFilter.Graph, PCGPin);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	PCGEditorUtils::ForEachPCGGraphAssetData([&ActionMenuBuilder, &ActionFilter, &AssetRegistryModule, PCGPin, CompatibilityMask](const FAssetData& AssetData) -> bool
	{
		const bool bExposeToLibrary = AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UPCGGraphInterface, bExposeToLibrary));
		if (bExposeToLibrary)
		{
			FPCGAssetHelpers::FGraphAssetOutput AssetOutput;
			if (!FPCGAssetHelpers::GetGraphAssetRegistryData(AssetData, AssetOutput))
			{
				return true;
			}

			if (!ActionFilter.Accepts(AssetOutput.Category))
			{
				return true;
			}

			if (AssetOutput.CachedPins.bHasCachedPins && PCGPin)
			{
				const TArray<FPCGPinProperties>& SubgraphPins = PCGPin->IsOutputPin() ? AssetOutput.CachedPins.InputPins : AssetOutput.CachedPins.OutputPins;
				if (!PCGActionsHelpers::HasCompatiblePins(PCGPin, SubgraphPins, CompatibilityMask))
				{
					return true;
				}
			}

			// As stated above, we either have an override and we take it, or we use the asset name, to differentiate all possible instances of the same graph.
			const FText MenuDesc = AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UPCGGraphInterface, bOverrideTitle))
				? AssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGGraphInterface, Title))
				: FText::FromString(FName::NameToDisplayString(AssetData.AssetName.ToString(), false));

			TSharedPtr<FPCGEditorGraphSchemaAction_NewSubgraphElement> NewSubgraphAction(new FPCGEditorGraphSchemaAction_NewSubgraphElement(AssetOutput.Category, MenuDesc, AssetOutput.Description, 0));
			NewSubgraphAction->SubgraphObjectPath = AssetData.GetSoftObjectPath();
			ActionMenuBuilder.AddAction(NewSubgraphAction);
		}

		return true;
	});

	if (ActionFilter.Graph && ActionFilter.Graph->GetPCGGraph())
	{
		for (UPCGGraph* EmbeddedSubgraph : ActionFilter.Graph->GetPCGGraph()->GetEmbeddedSubgraphs())
		{
			if (PCGPin && !PCGActionsHelpers::HasCompatiblePins(PCGPin, PCGPin->IsOutputPin() ? EmbeddedSubgraph->DefaultInputPinProperties() : EmbeddedSubgraph->DefaultOutputPinProperties(), CompatibilityMask))
			{
				continue;
			}

			TSharedPtr<FPCGEditorGraphSchemaAction_NewSubgraphElement> NewSubgraphAction(new FPCGEditorGraphSchemaAction_NewSubgraphElement(EmbeddedSubgraph->Category, EmbeddedSubgraph->Title, EmbeddedSubgraph->Description, 0));
			NewSubgraphAction->SubgraphObjectPath = EmbeddedSubgraph;
			ActionMenuBuilder.AddAction(NewSubgraphAction);
		}
	}
}

void UPCGEditorGraphSchema::GetExtraElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const
{
	const FText NoCategory;

	// Comment action (only if not contextual)
	if (!ActionMenuBuilder.FromPin)
	{
		const FText CommentMenuDesc = LOCTEXT("PCGAddComment", "Add Comment...");
		const FText CommentDescription = LOCTEXT("PCGAddCommentTooltip", "Create a resizable comment box.");

		const TSharedPtr<FPCGEditorGraphSchemaAction_NewComment> NewCommentAction(new FPCGEditorGraphSchemaAction_NewComment(NoCategory, CommentMenuDesc, CommentDescription, 0));
		ActionMenuBuilder.AddAction(NewCommentAction);
	}

	// Reroute action
	const FText RerouteMenuDesc = LOCTEXT("PCGAddRerouteNode", "Add Reroute Node");
	const FText RerouteDescription = LOCTEXT("PCGAddRerouteNodeTooltip", "Add a reroute node, aka knot.");

	const TSharedPtr<FPCGEditorGraphSchemaAction_NewReroute> NewRerouteAction(new FPCGEditorGraphSchemaAction_NewReroute(NoCategory, RerouteMenuDesc, RerouteDescription, 0));
	ActionMenuBuilder.AddAction(NewRerouteAction);

	// Named reroute declaration action
	const FText NamedRerouteMenuDesc = LOCTEXT("PCGAddNamedRerouteDeclarationNode", "Add Named Reroute Declaration Node...");
	const FText NamedRerouteDescription = LOCTEXT("PCGAddNamedRerouteDeclarationNodeTooltip", "Creates a new Named Reroute Declaration from the input.");

	const TSharedPtr<FPCGEditorGraphSchemaAction_NewNamedRerouteDeclaration> NewNamedRerouteDeclarationAction(new FPCGEditorGraphSchemaAction_NewNamedRerouteDeclaration(NoCategory, NamedRerouteMenuDesc, NamedRerouteDescription, 0));
	ActionMenuBuilder.AddAction(NewNamedRerouteDeclarationAction);
}

void UPCGEditorGraphSchema::GetNamedRerouteUsageActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const
{
	// Usage by definition implies that a pin, if one is provided, has to be an input pin.
	if (ActionMenuBuilder.FromPin && ActionMenuBuilder.FromPin->Direction == EGPD_Output)
	{
		return;
	}

	const UEdGraph* CurrentGraph = ActionFilter.Graph;
	const UPCGEditorGraph* Graph = Cast<UPCGEditorGraph>(CurrentGraph);

	if (!Graph)
	{
		return;
	}

	const UPCGGraph* PCGGraph = const_cast<UPCGEditorGraph*>(Graph)->GetPCGGraph();

	if (!PCGGraph)
	{
		return;
	}

	const UEdGraphPin* Pin = ActionMenuBuilder.FromPin;
	const UPCGPin* PCGPin = nullptr;
	PCGActionsHelpers::ECompatibilityMask CompatibilityMask = PCGActionsHelpers::GetPinAndCompatibilityMask(Pin, ActionFilter.Graph, PCGPin);

	for (const UPCGNode* Node : PCGGraph->GetNodes())
	{
		if (const UPCGNamedRerouteDeclarationSettings* RerouteDeclaration = Cast<UPCGNamedRerouteDeclarationSettings>(Node->GetSettings()))
		{
			// Skip incompatible (typewise) reroute nodes. This isn't direct in this case because we need to resolve the type first.
			if (Node->GetOutputPins().IsEmpty())
			{
				continue;
			}
			
			FPCGDataTypeIdentifier ResolvedType = RerouteDeclaration->GetCurrentPinTypesID(Node->GetOutputPins()[0]);
			if (!PCGActionsHelpers::IsCompatibleType(PCGPin, ResolvedType, CompatibilityMask))
			{
				continue;
			}

			static const FText Category = LOCTEXT("NamedRerouteCategory", "Named Reroutes");
			const FText Name = FText::FromName(Node->NodeTitle);
			const FText Tooltip = FText::Format(LOCTEXT("NamedRerouteTooltip", "Add a usage of '{0}' here."), Name);
			TSharedPtr<FPCGEditorGraphSchemaAction_NewNamedRerouteUsage> NewRerouteAction(new FPCGEditorGraphSchemaAction_NewNamedRerouteUsage(Category, Name, Tooltip, 1 /* We want named reroutes to be on top */));
			NewRerouteAction->DeclarationNode = Node;
			ActionMenuBuilder.AddAction(NewRerouteAction);
		}
	}
}

void UPCGEditorGraphSchema::GetDataAssetActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const
{
	// Early out if the 'from pin' is an output pin - it's not common to pass data to a load asset node.
	// @todo_pcg: it is possible to have the input be a param data for the 'from input' mode so we probably should consider this
	if (ActionMenuBuilder.FromPin && ActionMenuBuilder.FromPin->Direction == EGPD_Output)
	{
		return;
	}

	const UPCGPin* PCGPin = nullptr;
	PCGActionsHelpers::ECompatibilityMask CompatibilityMask = PCGActionsHelpers::GetPinAndCompatibilityMask(ActionMenuBuilder.FromPin, ActionFilter.Graph, PCGPin);

	PCGEditorUtils::ForEachPCGAssetData([&ActionMenuBuilder, &ActionFilter, PCGPin, CompatibilityMask](const FAssetData& AssetData)
	{
		const bool bExposeToLibrary = AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, bExposeToLibrary));
		if (bExposeToLibrary)
		{
			const FText MenuDesc = FText::FromString(FName::NameToDisplayString(AssetData.AssetName.ToString(), false));
			const FText Category = AssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, Category));
			const FText Description = AssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, Description));
			const FString AssetName = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, Name));

			if (!ActionFilter.Accepts(Category))
			{
				return true;
			}

			// @todo_pcg : If there is a pin, check against the cached pins, if they were properly set
			if (CompatibilityMask != PCGActionsHelpers::ECompatibilityMask::NoMask && PCGPin)
			{
				PCGDataAsset::FGetAssetDataOutput AssetDataOutput = UPCGDataAsset::GetAssetRegistryData(AssetData);
				if (AssetDataOutput.CachedPins.bHasCachedPins && !PCGActionsHelpers::HasCompatiblePins(PCGPin, AssetDataOutput.CachedPins.CachedPins, CompatibilityMask))
				{
					return true;
				}
			}

			const FSoftClassPath SettingsClassPath = FSoftClassPath(AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, SettingsClass)));
			TSubclassOf<UPCGSettings> SettingsClass = SettingsClassPath.TryLoadClass<UPCGSettings>();

			TSharedPtr<FPCGEditorGraphSchemaAction_NewLoadAssetElement> NewLoadDataAssetAction(new FPCGEditorGraphSchemaAction_NewLoadAssetElement(Category, AssetName.IsEmpty() ? MenuDesc : FText::FromString(AssetName), Description, 0));
			NewLoadDataAssetAction->Asset = AssetData;
			NewLoadDataAssetAction->SettingsClass = SettingsClass;

			ActionMenuBuilder.AddAction(NewLoadDataAssetAction);
		}

		return true;
	});
}

bool UPCGEditorGraphSchema::DroppedAssetsOnNodeInternal(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, const UEdGraphNode* Node, bool bValidate, FString& OutTooltipText) const
{
	OutTooltipText.Reset();

	if (Assets.Num() != 1)
	{
		return false;
	}

	const UPCGEditorGraphNodeBase* EditorNode = Cast<const UPCGEditorGraphNodeBase>(Node);
	if (!EditorNode)
	{
		return false;
	}

	const UPCGNode* PCGNode = EditorNode->GetPCGNode();
	if (!PCGNode)
	{
		return false;
	}

	UPCGSubgraphSettings* SubgraphSettings = Cast<UPCGSubgraphSettings>(PCGNode->GetSettings());
	if (!SubgraphSettings)
	{
		return false;
	}

	const FAssetData& AssetData = Assets[0];
	const UClass* AssetClass = AssetData.GetClass();

	if (!AssetClass || !AssetClass->IsChildOf(UPCGGraphInterface::StaticClass()))
	{
		return false;
	}

	if (SubgraphSettings->SubgraphAssetFilter(AssetData))
	{
		return false;
	}

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Node->GetGraph());
	if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData))
	{
		return false;
	}

	// Execute action if we are not validating
	if (!bValidate)
	{
		if (UPCGGraphInterface* Graph = Cast<UPCGGraphInterface>(AssetData.GetAsset()))
		{
			FScopedTransaction SetGraphTransaction(LOCTEXT("ReplaceSubgraphTransaction", "Replace Subgraph"));
			SubgraphSettings->SetSubgraph(Graph);
		}
	}

	return true;
}

void UPCGEditorGraphSchema::DroppedAssetsOnNode(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const
{
	FString TooltipText;
	DroppedAssetsOnNodeInternal(Assets, GraphPosition, Node, /*bValidate=*/false, TooltipText);
}

void UPCGEditorGraphSchema::GetAssetsNodeHoverMessage(const TArray<struct FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutOkIcon = DroppedAssetsOnNodeInternal(Assets, HoverNode->GetPosition(), HoverNode, /*bValidate=*/true, OutTooltipText);
}

void UPCGEditorGraphSchema::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Graph);

	FVector2f GraphPositionOffset = GraphPosition;
	constexpr float PositionOffsetIncrementY = 50.f;
	UEdGraphPin* NullFromPin = nullptr;

	TArray<FSoftObjectPath> SubgraphPaths;
	TArray<FVector2D> SubgraphPositions;

	TArray<FSoftObjectPath> SettingsPaths;
	TArray<FVector2D> SettingsGraphPositions;

	for (const FAssetData& AssetData : Assets)
	{
		if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData))
		{
			continue;
		}

		if(const UClass* AssetClass = AssetData.GetClass())
		{
			if(AssetClass->IsChildOf(UPCGGraphInterface::StaticClass()))
			{
				SubgraphPaths.Add(AssetData.GetSoftObjectPath());
				SubgraphPositions.Add(FDeprecateSlateVector2D(GraphPositionOffset));
				GraphPositionOffset.Y += PositionOffsetIncrementY;
			}
			else if (PCGEditorUtils::IsAssetPCGBlueprint(AssetData))
			{
				const FString GeneratedClass = AssetData.GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath);

				FPCGEditorGraphSchemaAction_NewBlueprintElement NewBlueprintAction;
				NewBlueprintAction.BlueprintClassPath = FSoftClassPath(GeneratedClass);
				NewBlueprintAction.PerformAction(Graph, NullFromPin, GraphPositionOffset);
				GraphPositionOffset.Y += PositionOffsetIncrementY;
			}
			else if(AssetClass->IsChildOf(UPCGDataAsset::StaticClass()))
			{
				FPCGEditorGraphSchemaAction_NewLoadAssetElement NewLoadAssetAction;
				NewLoadAssetAction.Asset = AssetData;
				NewLoadAssetAction.SettingsClass = FSoftClassPath(AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, SettingsClass))).TryLoadClass<UPCGSettings>();
				NewLoadAssetAction.PerformAction(Graph, NullFromPin, GraphPositionOffset);
				GraphPositionOffset.Y += PositionOffsetIncrementY;
			}
			else if(AssetClass->IsChildOf(UPCGSettings::StaticClass()))
			{
				// Delay creation so we can open a menu, once, if needed.
				SettingsPaths.Add(AssetData.GetSoftObjectPath());
				SettingsGraphPositions.Add(FDeprecateSlateVector2D(GraphPositionOffset));
				GraphPositionOffset.Y += PositionOffsetIncrementY;
			}
		}
	}

	UPCGEditorGraph* EditorGraph = CastChecked<UPCGEditorGraph>(Graph);

	TSharedPtr<SGraphEditor> GraphEditor = SGraphEditor::FindGraphEditorForGraph(EditorGraph);
	const FDeprecateSlateVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

	// If we've dragged settings assets or a graph, we might want to open a menu (ergo this call)
	if (!SettingsPaths.IsEmpty())
	{
		check(SettingsPaths.Num() == SettingsGraphPositions.Num());
		FPCGEditorGraphSchemaAction_NewSettingsElement::MakeSettingsNodesOrContextualMenu(GraphEditor->GetGraphPanel()->AsShared(), MouseCursorLocation, Graph, SettingsPaths, SettingsGraphPositions, /*bSelectNewNodes=*/true);
	}

	if (!SubgraphPaths.IsEmpty())
	{
		check(SubgraphPaths.Num() == SubgraphPositions.Num());
		FPCGEditorGraphSchemaAction_NewSubgraphElement::MakeGraphNodesOrContextualMenu(GraphEditor->GetGraphPanel()->AsShared(), MouseCursorLocation, Graph, SubgraphPaths, SubgraphPositions, /*bSelectNewNodes=*/true);
	}
}

void UPCGEditorGraphSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText.Reset();
	OutOkIcon = false;

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverGraph);

	for (const FAssetData& AssetData : Assets)
	{
		// TODO: get class from asset data, shouldn't require loading
		if(const UClass* AssetClass = AssetData.GetClass())
		{
			if(AssetClass->IsChildOf(UPCGGraphInterface::StaticClass()) || AssetClass->IsChildOf(UPCGSettings::StaticClass()) || AssetClass->IsChildOf(UPCGDataAsset::StaticClass()) || PCGEditorUtils::IsAssetPCGBlueprint(AssetData))
			{
				if (AssetReferenceFilter)
				{
					FText FailureReason;
					if (!AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
					{
						if (OutTooltipText.IsEmpty())
						{
							OutTooltipText = FailureReason.ToString();
						}
						continue;
					}
				}

				OutTooltipText.Reset();
				OutOkIcon = true;
				return;
			}
			else if(AssetClass->IsChildOf(UBlueprint::StaticClass()))
			{
				if (OutTooltipText.IsEmpty())
				{
					OutTooltipText = LOCTEXT("PCGEditorDropAssetInvalidBP", "Blueprint does not derive from UPCGBlueprintBaseElement").ToString();
				}
			}
		}
	}

	if (OutTooltipText.IsEmpty())
	{
		OutTooltipText = LOCTEXT("PCGEditorDropAssetInvalid", "Can't create a node for this asset").ToString();
	}
}

void UPCGEditorGraphSchema::OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const
{
	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGCreateRerouteNodeOnWire", "Create Reroute Node"), nullptr);

	const FVector2f NodeSpacerSize(42.0f, 24.0f);
	const FVector2f KnotTopLeft = GraphPosition - (NodeSpacerSize * 0.5f);

	UEdGraph* EditorGraph = PinA->GetOwningNode()->GetGraph();
	EditorGraph->Modify();

	FPCGEditorGraphSchemaAction_NewReroute Action;

	UEdGraphNode* SourceGraphNode = PinA->GetOwningNode();
	UEdGraphNode* TargetGraphNode = PinB->GetOwningNode();

	UPCGEditorGraphNodeBase* SourcePCGGraphNode = CastChecked<UPCGEditorGraphNodeBase>(SourceGraphNode);
	UPCGEditorGraphNodeBase* TargetPCGGraphNode = CastChecked<UPCGEditorGraphNodeBase>(TargetGraphNode);

	// We need to disable full node reconstruction to make sure the pins are valid when creating the connections.
	SourcePCGGraphNode->EnableDeferredReconstruct();
	TargetPCGGraphNode->EnableDeferredReconstruct();

	if (UPCGEditorGraphNodeReroute* RerouteNode = Cast<UPCGEditorGraphNodeReroute>(Action.PerformAction(EditorGraph, nullptr, KnotTopLeft, /*bSelectNewNode=*/true)))
	{
		BreakSinglePinLink(PinA, PinB);
		TryCreateConnection(PinA, (PinA->Direction == EGPD_Output) ? RerouteNode->GetInputPin() : RerouteNode->GetOutputPin());
		TryCreateConnection(PinB, (PinB->Direction == EGPD_Output) ? RerouteNode->GetInputPin() : RerouteNode->GetOutputPin());
	}

	SourcePCGGraphNode->DisableDeferredReconstruct();
	TargetPCGGraphNode->DisableDeferredReconstruct();
}

void UPCGEditorGraphSchema::GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const
{
	if (const UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(&Graph))
	{
		if (const UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph())
		{
			DisplayInfo.DisplayName = PCGGraph->IsEmbeddedSubgraph() ? PCGGraph->GetDisplayName() : LOCTEXT("DefaultGraphDisplayInformation", "Graph");
			DisplayInfo.PlainName = FText::FromString(PCGGraph->GetName());
			return;
		}
	}

	Super::GetGraphDisplayInformation(Graph, DisplayInfo);
}

const FSlateBrush* UPCGEditorGraphSchema::GetMetadataTypeSlateBrush(const EPCGContainerType ContainerType) const
{
	switch (ContainerType)
	{
		case EPCGContainerType::Array:
			return FAppStyle::GetBrush(TEXT("Kismet.VariableList.ArrayTypeIcon"));
		case EPCGContainerType::Set:
			return FAppStyle::GetBrush(TEXT("Kismet.VariableList.SetTypeIcon"));
		// @todo_pcg: Enable after Map support finalizes
		// case EPCGContainerType::Map:
		//	return FAppStyle::GetBrush(TEXT("Kismet.VariableList.MapKeyTypeIcon"));
		default: // fall-through
		case EPCGContainerType::Element:
			return FAppStyle::GetBrush(TEXT("Kismet.VariableList.TypeIcon"));
	}
}

FLinearColor UPCGEditorGraphSchema::GetMetadataTypeColor(const EPCGMetadataTypes Type) const
{
	const UGraphEditorSettings* GraphSettings = GetDefault<UGraphEditorSettings>();
	check(GraphSettings)

	switch (Type)
	{
		case EPCGMetadataTypes::Float:
			return GraphSettings->FloatPinTypeColor;
		case EPCGMetadataTypes::Double:
			return GraphSettings->DoublePinTypeColor;
		case EPCGMetadataTypes::Integer32:
			return GraphSettings->IntPinTypeColor;
		case EPCGMetadataTypes::Integer64:
			return GraphSettings->Int64PinTypeColor;
		case EPCGMetadataTypes::Vector2: // fall-through
		case EPCGMetadataTypes::Vector:  // fall-through
		case EPCGMetadataTypes::Vector4: // fall-through
			return GraphSettings->VectorPinTypeColor;
		case EPCGMetadataTypes::Quaternion:
			return GraphSettings->RotatorPinTypeColor;
		case EPCGMetadataTypes::Transform:
			return GraphSettings->TransformPinTypeColor;
		case EPCGMetadataTypes::String:
			return GraphSettings->StringPinTypeColor;
		case EPCGMetadataTypes::Boolean:
			return GraphSettings->BooleanPinTypeColor;
		case EPCGMetadataTypes::Rotator:
			return GraphSettings->RotatorPinTypeColor;
		case EPCGMetadataTypes::Name:
			return GraphSettings->NamePinTypeColor;
		case EPCGMetadataTypes::SoftObjectPath:
			return GraphSettings->SoftObjectPinTypeColor;
		case EPCGMetadataTypes::SoftClassPath:
			return GraphSettings->SoftClassPinTypeColor;
		case EPCGMetadataTypes::Text:
			return GraphSettings->TextPinTypeColor;
		case EPCGMetadataTypes::Struct:
			return GraphSettings->StructPinTypeColor;
		case EPCGMetadataTypes::Object:
			return GraphSettings->ObjectPinTypeColor;
		case EPCGMetadataTypes::SoftObject:
			return GraphSettings->SoftObjectPinTypeColor;
		case EPCGMetadataTypes::Class:
			return GraphSettings->ClassPinTypeColor;
		case EPCGMetadataTypes::SoftClass:
			return GraphSettings->SoftClassPinTypeColor;
		case EPCGMetadataTypes::Byte: // fall-through
		case EPCGMetadataTypes::Enum:
			return GraphSettings->BytePinTypeColor;
		default: // fall-through
		case EPCGMetadataTypes::Unknown:
			return FLinearColor::White;
	}
}

const FSlateBrush* UPCGEditorGraphSchema::GetPropertyBagTypeSlateBrush(EPropertyBagContainerType ContainerType) const
{
	switch (ContainerType)
	{
	case EPropertyBagContainerType::Array:
		return FAppStyle::GetBrush(TEXT("Kismet.VariableList.ArrayTypeIcon"));
	case EPropertyBagContainerType::Set:
		return FAppStyle::GetBrush(TEXT("Kismet.VariableList.SetTypeIcon"));
		// @todo_pcg: Enable after Map support finalizes
		// case EPropertyBagContainerType::Map:
		//	return FAppStyle::GetBrush(TEXT("Kismet.VariableList.MapKeyTypeIcon"));
	default: // fall-through
		return FAppStyle::GetBrush(TEXT("Kismet.VariableList.TypeIcon"));
	}
}

FLinearColor UPCGEditorGraphSchema::GetPropertyBagTypeColor(const FPropertyBagPropertyDesc& Desc) const
{
	const UGraphEditorSettings* GraphSettings = GetDefault<UGraphEditorSettings>();
	check(GraphSettings)

	// For struct, extract the common structs
	if (Desc.ValueType == EPropertyBagPropertyType::Struct && Desc.ValueTypeObject)
	{
		if (Desc.ValueTypeObject == TBaseStructure<FVector>::Get()
			|| Desc.ValueTypeObject == TBaseStructure<FVector2D>::Get()
			|| Desc.ValueTypeObject == TBaseStructure<FVector4>::Get())
		{
			return GraphSettings->VectorPinTypeColor;
		}
		else if (Desc.ValueTypeObject == TBaseStructure<FRotator>::Get()
			|| Desc.ValueTypeObject == TBaseStructure<FQuat>::Get())
		{
			return GraphSettings->RotatorPinTypeColor;
		}
		else if (Desc.ValueTypeObject == TBaseStructure<FTransform>::Get())
		{
			return GraphSettings->TransformPinTypeColor;
		}
	}

	switch (Desc.ValueType)
	{
	case EPropertyBagPropertyType::Bool:
		return GraphSettings->BooleanPinTypeColor;
	case EPropertyBagPropertyType::Byte:
		return GraphSettings->BytePinTypeColor;
	case EPropertyBagPropertyType::Int32:
		return GraphSettings->IntPinTypeColor;
	case EPropertyBagPropertyType::Int64:
		return GraphSettings->Int64PinTypeColor;
	case EPropertyBagPropertyType::Float:
		return GraphSettings->FloatPinTypeColor;
	case EPropertyBagPropertyType::Double:
		return GraphSettings->DoublePinTypeColor;
	case EPropertyBagPropertyType::Name:
		return GraphSettings->NamePinTypeColor;
	case EPropertyBagPropertyType::String:
		return GraphSettings->StringPinTypeColor;
	case EPropertyBagPropertyType::Text:
		return GraphSettings->TextPinTypeColor;
	case EPropertyBagPropertyType::Enum:
		return GraphSettings->ObjectPinTypeColor;
	case EPropertyBagPropertyType::Struct:
		return GraphSettings->StructPinTypeColor;
	case EPropertyBagPropertyType::Object:
		return GraphSettings->ObjectPinTypeColor;
	case EPropertyBagPropertyType::SoftObject:
		return GraphSettings->SoftObjectPinTypeColor;
	case EPropertyBagPropertyType::Class:
		return GraphSettings->ClassPinTypeColor;
	case EPropertyBagPropertyType::SoftClass:
		return GraphSettings->SoftClassPinTypeColor;
	default:
		break;
	}

	return FLinearColor::White;
}

FLinearColor UPCGEditorGraphSchema::GetPinColor(const UEdGraphPin* InPin) const
{
	if (!InPin)
	{
		return GetDefault<UPCGEditorSettings>()->DefaultPinColor;
	}
	
	const UPCGEditorGraphNodeBase* EditorNode = CastChecked<const UPCGEditorGraphNodeBase>(InPin->GetOwningNode());
	const UPCGNode* PCGNode = EditorNode ? EditorNode->GetPCGNode() : nullptr;
	const UPCGPin* PCGPin = nullptr;
	
	if (InPin->Direction == EGPD_Input)
	{
		PCGPin = PCGNode ? PCGNode->GetInputPin(InPin->GetFName()) : nullptr;
	}
	else
	{
		PCGPin = PCGNode ? PCGNode->GetOutputPin(InPin->GetFName()) : nullptr;
	}
	
	return EditorNode ? EditorNode->GetCachedPinColor(PCGPin) : GetDefault<UPCGEditorSettings>()->DefaultPinColor;
}

TSharedPtr<IAssetReferenceFilter> UPCGEditorGraphSchema::MakeAssetReferenceFilter(const UEdGraph* Graph)
{
	if (const UPCGEditorGraph* EditorGraph = Cast<const UPCGEditorGraph>(Graph))
	{
		if (const UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph())
		{
			if (GEditor)
			{
				FAssetReferenceFilterContext AssetReferenceFilterContext;
				AssetReferenceFilterContext.AddReferencingAsset(PCGGraph);
				return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
			}
		}
	}

	return {};
}

FPCGEditorConnectionDrawingPolicy::FPCGEditorConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraph)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements)
	, Graph(CastChecked<UPCGEditorGraph>(InGraph))
{
	ArrowImage = nullptr;
	ArrowRadius = FVector2D::ZeroVector;
}

bool FPCGEditorConnectionDrawingPolicy::UpdateParamsIfDebugging(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, FConnectionParams& Params)
{
	check(OutputPin && InputPin);

	// Early validation
	const UPCGEditorGraphNodeBase* UpstreamEditorNode = CastChecked<const UPCGEditorGraphNodeBase>(OutputPin->GetOwningNode());
	if (!UpstreamEditorNode || !UpstreamEditorNode->GetPCGNode())
	{
		return false;
	}

	const UPCGNode* NodeToInspect = nullptr;
	const UPCGPin* PinToInspect = nullptr;

	// Walk up the graph if the current node is a reroute node because there is no associated inspection data.
	PCGEditorGraphUtils::GetInspectablePin(UpstreamEditorNode->GetPCGNode(), UpstreamEditorNode->GetPCGNode()->GetOutputPin(OutputPin->GetFName()), NodeToInspect, PinToInspect);

	if (!NodeToInspect || !PinToInspect)
	{
		return false;
	}

	// Early out if we aren't in debug mode
	if (!Graph || !Graph->GetEditor().IsValid())
	{
		return false;
	}

	// Use defaults if data is emitted by GPU node as we don't have entries in the inspection cache to drive this info.
	if (const UPCGSettings* NodeSettings = NodeToInspect->GetSettings(); NodeSettings && NodeSettings->bEnabled && NodeSettings->ShouldExecuteOnGPU())
	{
		return false;
	}

	const FPCGEditor* Editor = Graph->GetEditor().Pin().Get();
	const FPCGStack* PCGStack = Editor ? Editor->GetStackBeingInspected() : nullptr;
	IPCGGraphExecutionSource* PCGSource = Editor ? Editor->GetPCGSourceBeingInspected() : nullptr;

	if (!Editor || !PCGStack || !PCGSource || !PCGSource->GetExecutionState().GetInspection().IsInspecting())
	{
		return false;
	}

	FPCGStack Stack = *PCGStack;
	TArray<FPCGStackFrame>& StackFrames = Stack.GetStackFramesMutable();
	StackFrames.Reserve(StackFrames.Num() + 2);
	StackFrames.Emplace(NodeToInspect);
	StackFrames.Emplace(PinToInspect);

	const bool bFoundData = PCGSource->GetExecutionState().GetInspection().InspectData(Stack, [&Params](const FPCGDataCollection& InspectionData)
	{
		if(InspectionData.TaggedData.Num() > 1)
		{
			Params.WireThickness *= GetDefault<UPCGEditorSettings>()->MultiDataEdgeDebugEmphasis;
		}
	});

	if(!bFoundData)
	{
		Params.WireColor = Params.WireColor.Desaturate(GetDefault<UPCGEditorSettings>()->EmptyEdgeDebugDesaturateFactor);
	}

	return true;
}

void FPCGEditorConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	FConnectionDrawingPolicy::DetermineWiringStyle(OutputPin, InputPin, Params);

	Params.WireThickness = GetDefault<UPCGEditorSettings>()->DefaultWireThickness;

	// Emphasize wire thickness on hovered pins
	if (HoveredPins.Contains(InputPin) && HoveredPins.Contains(OutputPin))
	{
		Params.WireThickness *= GetDefault<UPCGEditorSettings>()->HoverEdgeEmphasis;
	}

	// Base the color of the wire on the color of the output pin
	if (OutputPin && Graph && Graph->GetSchema())
	{
		if (const UPCGEditorGraphSchema* Schema = Cast<UPCGEditorGraphSchema>(Graph->GetSchema()))
		{
			Params.WireColor = Schema->GetPinColor(OutputPin);
		}
	}

	// Desaturate connection if downstream node is disabled or if the data on this wire won't be used
	if (InputPin && OutputPin)
	{
		// Try to apply debugging/dynamic visualization - if it fails, fall back to static visualization
		if (!UpdateParamsIfDebugging(OutputPin, InputPin, Params))
		{
			const UPCGEditorGraphNodeBase* EditorNode = CastChecked<const UPCGEditorGraphNodeBase>(InputPin->GetOwningNode());
			const UPCGNode* PCGNode = EditorNode ? EditorNode->GetPCGNode() : nullptr;
			const UPCGPin* PCGPin = PCGNode ? PCGNode->GetInputPin(InputPin->GetFName()) : nullptr;
			const UPCGEditorGraphNodeBase* UpstreamEditorNode = CastChecked<const UPCGEditorGraphNodeBase>(OutputPin->GetOwningNode());
			const UPCGEditorGraphNodeBase* DownstreamEditorNode = CastChecked<const UPCGEditorGraphNodeBase>(InputPin->GetOwningNode());

			if (PCGPin && UpstreamEditorNode && DownstreamEditorNode)
			{
				const bool bDownstreamNodeForceDisabled = DownstreamEditorNode->IsDisplayAsDisabledForced();

				// Look for the PCG edge that correlates with passed in (OutputPin, InputPin) edge
				const TObjectPtr<UPCGEdge>* PCGEdge = PCGPin->Edges.FindByPredicate([UpstreamEditorNode, OutputPin](const UPCGEdge* ConnectedPCGEdge)
				{
					return UpstreamEditorNode->GetPCGNode() == ConnectedPCGEdge->InputPin->Node && ConnectedPCGEdge->InputPin->Properties.Label == OutputPin->GetFName();
				});

				const bool bDownstreamNodeDoesNotUseData = PCGEdge && Graph && !Graph->IsEdgeUsedByExecution(*PCGEdge);

				// If edge found and is not used, gray it out
				if (bDownstreamNodeForceDisabled || bDownstreamNodeDoesNotUseData)
				{
					Params.WireColor = Params.WireColor.Desaturate(GetDefault<UPCGEditorSettings>()->EmptyEdgeDebugDesaturateFactor);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE