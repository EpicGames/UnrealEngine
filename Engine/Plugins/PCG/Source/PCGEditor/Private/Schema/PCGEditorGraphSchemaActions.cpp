// Copyright Epic Games, Inc. All Rights Reserved.

#include "Schema/PCGEditorGraphSchemaActions.h"

#include "PCGEditorCommon.h"
#include "PCGEditorGraph.h"
#include "PCGEditorModule.h"
#include "Nodes/PCGEditorGraphNode.h"
#include "Nodes/PCGEditorGraphNodeComment.h"
#include "Nodes/PCGEditorGraphNodeReroute.h"
#include "Schema/PCGEditorGraphSchema.h"

#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGSubgraph.h"
#include "Elements/IO/PCGLoadAssetElement.h"
#include "Elements/PCGExecuteBlueprint.h"
#include "Elements/PCGLoopElement.h"
#include "Elements/PCGReroute.h"
#include "Elements/PCGUserParameterGet.h"

#include "UObject/UObjectGlobals.h"

#include "GraphEditor.h"
#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/WidgetPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditorGraphSchemaActions)

#define LOCTEXT_NAMESPACE "PCGEditorGraphSchemaActions"

FText FPCGEditorGraphSchemaActionBase::GetMenuDescriptionOverride() const
{
	return GetMenuDescription();
}

FSlateBrush const* FPCGEditorGraphSchemaActionBase::GetPaletteIcon() const
{
	return PaletteIcon.Get();
}

void FPCGEditorGraphSchemaActionBase::SetPaletteIcon(TSharedPtr<FSlateBrush>& InPaletteIcon)
{
	PaletteIcon = InPaletteIcon;
}

UEdGraphNode* FPCGEditorGraphSchemaAction_NewNativeElement::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(ParentGraph);
	if (!EditorGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid EditorGraph");
		return nullptr;
	}

	UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph();
	if (!PCGGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid PCGGraph");
		return nullptr;
	}

	// Important - do not reconstruct the editor graph node/pins midway through this function as this will invalidate FromPin.
	const FPCGDeferNodeReconstructScope DisableReconstruct(FromPin);

	// Ensure compilation cache is populated before changing graph, so we can compare before/after compiled tasks later to detect change.
	PCGGraph->PrimeGraphCompilationCache();

	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorNewNativeElement", "PCG Editor: New Native Element"), nullptr);
	EditorGraph->Modify();

	PCGGraph->DisableNotificationsForEditor();

	ON_SCOPE_EXIT
	{
		PCGGraph->EnableNotificationsForEditor();
	};

	UPCGSettings* DefaultNodeSettings = nullptr;
	UPCGNode* NewPCGNode = PCGGraph->AddNodeOfType(SettingsClass, DefaultNodeSettings);

	if (!NewPCGNode)
	{
		UE_LOGF(LogPCGEditor, Error, "Failed to add a node of type %ls", *SettingsClass->GetName());
		return nullptr;
	}

	if (DefaultNodeSettings)
	{
		DefaultNodeSettings->ApplyPreconfiguredSettings(PreconfiguredInfo);
		NewPCGNode->UpdateAfterSettingsChangeDuringCreation();
	}

	PostCreation(NewPCGNode);

	FGraphNodeCreator<UPCGEditorGraphNode> NodeCreator(*EditorGraph);
	const TSubclassOf<UPCGEditorGraphNode> GraphNodeClass = TSubclassOf<UPCGEditorGraphNode>(UPCGEditorGraph::GetGraphNodeClassFromPCGSettings(DefaultNodeSettings));
	UPCGEditorGraphNode* NewNode = NodeCreator.CreateUserInvokedNode(bSelectNewNode, GraphNodeClass);
	NewNode->Construct(NewPCGNode);
	NewNode->NodePosX = Location.X;
	NewNode->NodePosY = Location.Y;
	NodeCreator.Finalize();

	NewPCGNode->PositionX = Location.X;
	NewPCGNode->PositionY = Location.Y;

	if (FromPin && ensure(FromPin->GetOwningNode()))
	{
		NewNode->AutowireNewNode(FromPin);
	}

	return NewNode;
}

FSlateBrush const* FPCGEditorGraphSchemaAction_NewNativeElement::GetPaletteIcon() const
{
	// Get the icon from the descriptor if it exists. Should only need to do this once.
	if (!PaletteIcon && PreconfiguredInfo.ActionIconDescriptor)
	{
		const IPCGActionIconDescriptorBase* IconDescriptor = PreconfiguredInfo.ActionIconDescriptor.Get();
		const FSlateBrush* BaseBrush = nullptr;
		FLinearColor TintColor = FLinearColor::White;
		switch (IconDescriptor->GetActionIconDescriptorType())
		{
			case EPCGActionIconDescriptorType::Label:
			{
				const FPCGActionIconByLabelDescriptor* LabelDescriptor = static_cast<const FPCGActionIconByLabelDescriptor*>(IconDescriptor);
				BaseBrush = FAppStyle::GetBrush(LabelDescriptor->BrushLabel);
				TintColor = LabelDescriptor->Tint;
			}
			break;
			case EPCGActionIconDescriptorType::Metadata:
			{
				const UPCGEditorGraphSchema* Schema = GetDefault<UPCGEditorGraphSchema>();
				check(Schema);

				const FPCGActionIconByMetadataDescriptor* MetadataDescriptor = static_cast<const FPCGActionIconByMetadataDescriptor*>(IconDescriptor);
				BaseBrush = Schema->GetMetadataTypeSlateBrush(MetadataDescriptor->GetContainerType());
				TintColor = Schema->GetMetadataTypeColor(MetadataDescriptor->GetMetadataType());
			}
			break;
			default:
				// Invalid descriptor type
				checkNoEntry();
		}

		// Brush is expected to be set and have a valid resource.
		if (BaseBrush && ensure(BaseBrush->IsSet() && BaseBrush->GetResourceName() != NAME_None))
		{
			PaletteIcon = MakeShared<FSlateBrush>(*BaseBrush);
			PaletteIcon->TintColor = TintColor;
		}
	}

	return PaletteIcon ? PaletteIcon.Get() : nullptr;
}

FSlateBrush const* FPCGEditorGraphSchemaAction_NewGetParameterElement::GetPaletteIcon() const
{
	if (!PaletteIcon)
	{
		const UPCGEditorGraphSchema* Schema = GetDefault<UPCGEditorGraphSchema>();
		check(Schema);
		
		if (const FSlateBrush* BaseBrush = Schema->GetPropertyBagTypeSlateBrush(!PropertyDesc.ContainerTypes.IsEmpty() ? PropertyDesc.ContainerTypes[0] : EPropertyBagContainerType::None))
		{
			PaletteIcon = MakeShared<FSlateBrush>(*BaseBrush);
			PaletteIcon->TintColor = Schema->GetPropertyBagTypeColor(PropertyDesc);
		}
	}

	return PaletteIcon ? PaletteIcon.Get() : nullptr;
}

void FPCGEditorGraphSchemaAction_NewGetParameterElement::PostCreation(UPCGNode* NewNode)
{
	check(NewNode);
	UPCGUserParameterGetSettings* Settings = CastChecked<UPCGUserParameterGetSettings>(NewNode->GetSettings());

	Settings->PropertyGuid = PropertyDesc.ID;
	Settings->PropertyName = PropertyDesc.Name;

	NewNode->SetNodeTitle(Settings->PropertyName, /*bApplySanitization=*/false);

	// We need to set the settings to update the pins.
	NewNode->SetSettingsInterface(Settings);
}

FText FPCGEditorGraphSchemaAction_NewSettingsElement::GetMenuDescriptionOverride() const
{
	switch (Behavior)
	{
		case EPCGEditorNewSettingsBehavior::ForceCopy:
			return FText::Format(LOCTEXT("AssetMenuDescriptionCopyQualifierFormat", "{0} (copy)"), GetMenuDescription());
		case EPCGEditorNewSettingsBehavior::ForceInstance:
			return FText::Format(LOCTEXT("AssetMenuDescriptionInstanceQualifierFormat", "{0} (instance)"), GetMenuDescription());
		case EPCGEditorNewSettingsBehavior::Normal:
			break;
	}

	return Super::GetMenuDescriptionOverride();
}

UEdGraphNode* FPCGEditorGraphSchemaAction_NewSettingsElement::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(ParentGraph);
	if (!EditorGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid EditorGraph");
		return nullptr;
	}

	UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph();
	if (!PCGGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid PCGGraph");
		return nullptr;
	}

	UPCGSettings* Settings = CastChecked<UPCGSettings>(SettingsObjectPath.TryLoad());
	if (!Settings)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid settings");
		return nullptr;
	}

	// Important - do not reconstruct the editor graph node/pins midway through this function as this will invalidate FromPin.
	const FPCGDeferNodeReconstructScope DisableReconstruct(FromPin);

	// Ensure compilation cache is populated before changing graph, so we can compare before/after compiled tasks later to detect change.
	PCGGraph->PrimeGraphCompilationCache();

	bool bCreateInstance = false;

	if(Behavior != EPCGEditorNewSettingsBehavior::Normal)
	{
		bCreateInstance = (Behavior == EPCGEditorNewSettingsBehavior::ForceInstance);
	}
	else
	{
		FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
		bCreateInstance = ModifierKeys.IsAltDown();
	}

	return MakeSettingsNode(EditorGraph, FromPin, Settings, FDeprecateSlateVector2D(Location), bSelectNewNode, bCreateInstance);
}

FSlateBrush const* FPCGEditorGraphSchemaAction_NewSettingsElement::GetPaletteIcon() const
{
	// @todo_pcg: Enable when icons exist for these types.
	// switch (Behavior)
	// {
	// 	case EPCGEditorNewSettingsBehavior::ForceCopy:
	// 		return FAppStyle::GetBrush("");
	// 	case EPCGEditorNewSettingsBehavior::ForceInstance:
	// 		return FAppStyle::GetBrush("");
	// 	case EPCGEditorNewSettingsBehavior::Normal:
	// 		break;
	// }

	return nullptr;
}

void FPCGEditorGraphSchemaAction_NewSettingsElement::MakeSettingsNodesOrContextualMenu(const TSharedRef<class SWidget>& InPanel, const FVector2D& InScreenPosition, UEdGraph* InGraph, const TArray<FSoftObjectPath>& InSettingsPaths, const TArray<FVector2D>& InLocations, bool bInSelectNewNodes)
{
	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(InGraph);

	if (!EditorGraph)
	{
		return;
	}

	TArray<UPCGSettings*> Settings;
	TArray<FVector2D> SettingsLocations;
	check(InSettingsPaths.Num() == InLocations.Num());

	for(int32 PathIndex = 0; PathIndex < InSettingsPaths.Num(); ++PathIndex)
	{
		const FSoftObjectPath& SettingsPath = InSettingsPaths[PathIndex];
		if (UPCGSettings* LoadedSettings = Cast<UPCGSettings>(SettingsPath.TryLoad()))
		{
			Settings.Add(LoadedSettings);
			SettingsLocations.Add(InLocations[PathIndex]);
		}
	}

	FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	const bool bModifiedKeysActive = ModifierKeys.IsControlDown() || ModifierKeys.IsAltDown();

	if (bModifiedKeysActive)
	{
		MakeSettingsNodes(EditorGraph, Settings, SettingsLocations, bInSelectNewNodes, ModifierKeys.IsAltDown());
	}
	else if(!Settings.IsEmpty())
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		const FText SettingsTextName = ((Settings.Num() == 1) ? FText::FromName(Settings[0]->GetFName()) : LOCTEXT("ManySettings", "Settings"));

		MenuBuilder.BeginSection("SettingsDroppedOn", SettingsTextName);

		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("CopySettings", "Copy {0}"), SettingsTextName),
			FText::Format(LOCTEXT("CopySettingsToolTip", "Copy the settings asset {0}, and keeps no reference to the original\n(Ctrl-drop to automatically create a copy)"), SettingsTextName),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FPCGEditorGraphSchemaAction_NewSettingsElement::MakeSettingsNodes, EditorGraph, Settings, SettingsLocations, bInSelectNewNodes, false), 
				FCanExecuteAction()
			)
		);

		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("InstanceSettings", "Instance {0}"), SettingsTextName),
			FText::Format(LOCTEXT("InstanceSettingsToolTip", "Instance the settings asset {0}\n(Alt-drop to automatically create an instance)"), SettingsTextName),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FPCGEditorGraphSchemaAction_NewSettingsElement::MakeSettingsNodes, EditorGraph, Settings, SettingsLocations, bInSelectNewNodes, true), 
				FCanExecuteAction()
			)
		);

		TSharedRef<SWidget> PanelWidget = InPanel;
		// Show dialog to choose getter vs setter
		FSlateApplication::Get().PushMenu(
			PanelWidget,
			FWidgetPath(),
			MenuBuilder.MakeWidget(),
			InScreenPosition,
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);

		MenuBuilder.EndSection();
	}
}

void FPCGEditorGraphSchemaAction_NewSettingsElement::MakeSettingsNodes(UPCGEditorGraph* InEditorGraph, TArray<UPCGSettings*> InSettings, TArray<FVector2D> InSettingsLocations, bool bInSelectNewNodes, bool bInCreateInstances)
{
	check(InSettings.Num() == InSettingsLocations.Num());
	for (int32 SettingsIndex = 0; SettingsIndex < InSettings.Num(); ++SettingsIndex)
	{
		FPCGEditorGraphSchemaAction_NewSettingsElement::MakeSettingsNode(InEditorGraph, nullptr, InSettings[SettingsIndex], InSettingsLocations[SettingsIndex], bInSelectNewNodes, bInCreateInstances);
	}
}

UEdGraphNode* FPCGEditorGraphSchemaAction_NewSettingsElement::MakeSettingsNode(UPCGEditorGraph* InEditorGraph, UEdGraphPin* InFromPin, UPCGSettings* InSettings, FVector2D InLocation, bool bInSelectNewNode, bool bInCreateInstance)
{
	if (!InEditorGraph || !InSettings)
	{
		return nullptr;
	}

	UPCGGraph* PCGGraph = InEditorGraph->GetPCGGraph();
	if (!PCGGraph)
	{
		return nullptr;
	}

	// Important - do not reconstruct the editor graph node/pins midway through this function as this will invalidate InFromPin.
	const FPCGDeferNodeReconstructScope DisableReconstruct(InFromPin);

	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorNewSettingsElement", "PCG Editor: New Settings Element"), nullptr);
	InEditorGraph->Modify();

	UPCGNode* NewPCGNode = nullptr;

	if (bInCreateInstance)
	{
		NewPCGNode = PCGGraph->AddNodeInstance(InSettings);
	}
	else
	{
		UPCGSettings* NewSettings = nullptr;
		NewPCGNode = PCGGraph->AddNodeCopy(InSettings, NewSettings);
	}

	if (!NewPCGNode)
	{
		UE_LOGF(LogPCGEditor, Error, "Unable to create node");
		return nullptr;
	}

	FGraphNodeCreator<UPCGEditorGraphNode> NodeCreator(*InEditorGraph);
	UPCGEditorGraphNode* NewNode = NodeCreator.CreateUserInvokedNode(bInSelectNewNode);
	NewNode->Construct(NewPCGNode);
	NewNode->NodePosX = InLocation.X;
	NewNode->NodePosY = InLocation.Y;
	NodeCreator.Finalize();

	NewPCGNode->PositionX = InLocation.X;
	NewPCGNode->PositionY = InLocation.Y;

	if (InFromPin)
	{
		NewNode->AutowireNewNode(InFromPin);
	}

	return NewNode;
}

UEdGraphNode* FPCGEditorGraphSchemaAction_NewLoadAssetElement::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(ParentGraph);
	if (!EditorGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid EditorGraph");
		return nullptr;
	}

	UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph();
	if (!PCGGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid PCGGraph");
		return nullptr;
	}

	if(!Asset.GetSoftObjectPath().IsValid())
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid asset path.");
		return nullptr;
	}

	// Important - do not reconstruct the editor graph node/pins midway through this function as this will invalidate FromPin.
	const FPCGDeferNodeReconstructScope DisableReconstruct(FromPin);

	// Ensure compilation cache is populated before changing graph, so we can compare before/after compiled tasks later to detect change.
	PCGGraph->PrimeGraphCompilationCache();

	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorNewBlueprintELement", "PCG Editor: New Blueprint Element"), nullptr);
	EditorGraph->Modify();

	UPCGSettings* DefaultNodeSettings = nullptr;
	UPCGNode* NewPCGNode = PCGGraph->AddNodeOfType(SettingsClass ? SettingsClass.Get() : UPCGLoadDataAssetSettings::StaticClass(), DefaultNodeSettings);
	UPCGLoadDataAssetSettings* DefaultLoadAssetSettings = CastChecked<UPCGLoadDataAssetSettings>(DefaultNodeSettings);

	DefaultLoadAssetSettings->SetFromAsset(Asset);

	NewPCGNode->UpdateAfterSettingsChangeDuringCreation();

	FGraphNodeCreator<UPCGEditorGraphNode> NodeCreator(*EditorGraph);
	UPCGEditorGraphNode* NewNode = NodeCreator.CreateUserInvokedNode(bSelectNewNode);
	NewNode->Construct(NewPCGNode);
	NewNode->NodePosX = Location.X;
	NewNode->NodePosY = Location.Y;
	NodeCreator.Finalize();

	NewPCGNode->PositionX = Location.X;
	NewPCGNode->PositionY = Location.Y;

	return NewNode;
}

UEdGraphNode* FPCGEditorGraphSchemaAction_NewBlueprintElement::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(ParentGraph);
	if (!EditorGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid EditorGraph");
		return nullptr;
	}

	UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph();
	if (!PCGGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid PCGGraph");
		return nullptr;
	}

	// Important - do not reconstruct the editor graph node/pins midway through this function as this will invalidate FromPin.
	const FPCGDeferNodeReconstructScope DisableReconstruct(FromPin);

	// Ensure compilation cache is populated before changing graph, so we can compare before/after compiled tasks later to detect change.
	PCGGraph->PrimeGraphCompilationCache();

	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorNewBlueprintELement", "PCG Editor: New Blueprint Element"), nullptr);
	EditorGraph->Modify();

	UPCGSettings* DefaultNodeSettings = nullptr;
	UPCGNode* NewPCGNode = PCGGraph->AddNodeOfType(UPCGBlueprintSettings::StaticClass(), DefaultNodeSettings);
	UPCGBlueprintSettings* DefaultBlueprintSettings = CastChecked<UPCGBlueprintSettings>(DefaultNodeSettings);

	UPCGBlueprintBaseElement* ElementInstance = nullptr;
	TSubclassOf<UPCGBlueprintBaseElement> BlueprintClass = BlueprintClassPath.TryLoadClass<UPCGBlueprintBaseElement>();
	DefaultBlueprintSettings->SetBlueprintElementType(BlueprintClass, ElementInstance);
	DefaultBlueprintSettings->ApplyPreconfiguredSettings(PreconfiguredInfo);

	NewPCGNode->UpdateAfterSettingsChangeDuringCreation();

	FGraphNodeCreator<UPCGEditorGraphNode> NodeCreator(*EditorGraph);
	UPCGEditorGraphNode* NewNode = NodeCreator.CreateUserInvokedNode(bSelectNewNode);
	NewNode->Construct(NewPCGNode);
	NewNode->NodePosX = Location.X;
	NewNode->NodePosY = Location.Y;
	NodeCreator.Finalize();

	NewPCGNode->PositionX = Location.X;
	NewPCGNode->PositionY = Location.Y;

	if (FromPin)
	{
		NewNode->AutowireNewNode(FromPin);
	}

	return NewNode;
}

FSlateBrush const* FPCGEditorGraphSchemaAction_NewBlueprintElement::GetPaletteIcon() const
{
	return FAppStyle::GetBrush("Icons.Blueprint");
}

UEdGraphNode* FPCGEditorGraphSchemaAction_NewSubgraphElement::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(ParentGraph);
	if (!EditorGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid EditorGraph");
		return nullptr;
	}

	UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph();
	if (!PCGGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid PCGGraph");
		return nullptr;
	}

	UPCGGraphInterface* Graph = CastChecked<UPCGGraphInterface>(SubgraphObjectPath.TryLoad());
	if (!Graph)
	{
		UE_LOGF(LogPCGEditor, Error, "Could not load Graph");
		return nullptr;
	}

	// Important - do not reconstruct the editor graph node/pins midway through this function as this will invalidate FromPin.
	const FPCGDeferNodeReconstructScope DisableReconstruct(FromPin);

	// Ensure compilation cache is populated before changing graph, so we can compare before/after compiled tasks later to detect change.
	PCGGraph->PrimeGraphCompilationCache();
	bool bCreateLoop = false;

	if (Behavior != EPCGEditorNewPCGGraphBehavior::Normal)
	{
		bCreateLoop = (Behavior == EPCGEditorNewPCGGraphBehavior::LoopNode);
	}
	else
	{
		FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
		bCreateLoop = ModifierKeys.IsAltDown();
	}

	return MakeGraphNode(EditorGraph, FromPin, Graph, FDeprecateSlateVector2D(Location), bSelectNewNode, bCreateLoop);
}

void FPCGEditorGraphSchemaAction_NewSubgraphElement::MakeGraphNodesOrContextualMenu(const TSharedRef<class SWidget>& InPanel, const FVector2D& InScreenPosition, UEdGraph* InGraph, const TArray<FSoftObjectPath>& InGraphPaths, const TArray<FVector2D>& InLocations, bool bInSelectNewNodes)
{
	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(InGraph);

	if (!EditorGraph)
	{
		return;
	}

	TArray<UPCGGraphInterface*> Graph;
	TArray<FVector2D> GraphLocations;
	check(InGraphPaths.Num() == InLocations.Num());

	for (int32 PathIndex = 0; PathIndex < InGraphPaths.Num(); ++PathIndex)
	{
		const FSoftObjectPath& GraphPath = InGraphPaths[PathIndex];
		if (UPCGGraphInterface* LoadedGraph = Cast<UPCGGraphInterface>(GraphPath.TryLoad()))
		{
			Graph.Add(LoadedGraph);
			GraphLocations.Add(InLocations[PathIndex]);
		}
	}

	FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	const bool bModifiedKeysActive = ModifierKeys.IsControlDown() || ModifierKeys.IsAltDown();

	if (bModifiedKeysActive)
	{
		MakeGraphNodes(EditorGraph, Graph, GraphLocations, bInSelectNewNodes, ModifierKeys.IsAltDown());
	}
	else if (!Graph.IsEmpty())
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		const FText GraphTextName = ((Graph.Num() == 1) ? Graph[0]->GetDisplayName() : LOCTEXT("MultipleGraphs", "Multiple Graphs"));

		MenuBuilder.BeginSection("GraphDroppedOn", GraphTextName);

		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("CreateSubgraphNode", "Create {0} Subgraph Node"), GraphTextName),
			FText::Format(LOCTEXT("CreateSubgraphToolTip", "Creates a subgraph for Graph asset {0} \n(Ctrl-drop to automatically create a subgraph node)"), GraphTextName),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FPCGEditorGraphSchemaAction_NewSubgraphElement::MakeGraphNodes, EditorGraph, Graph, GraphLocations, bInSelectNewNodes, false)
			)
		);

		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("CreateLoopNode", "Create {0} Loop Node"), GraphTextName),
			FText::Format(LOCTEXT("CreateLoopNodeToolTip", "Creates a loop node for Graph asset {0}\n(Alt-drop to automatically create a loop node)"), GraphTextName),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FPCGEditorGraphSchemaAction_NewSubgraphElement::MakeGraphNodes, EditorGraph, Graph, GraphLocations, bInSelectNewNodes, true)
			)
		);

		TSharedRef<SWidget> PanelWidget = InPanel;
		// Show dialog to choose getter vs setter
		FSlateApplication::Get().PushMenu(
			PanelWidget,
			FWidgetPath(),
			MenuBuilder.MakeWidget(),
			InScreenPosition,
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);

		MenuBuilder.EndSection();
	}
}

void FPCGEditorGraphSchemaAction_NewSubgraphElement::MakeGraphNodes(UPCGEditorGraph* InEditorGraph, TArray<UPCGGraphInterface*> InGraph, TArray<FVector2D> InGraphLocations, bool bInSelectNewNodes, bool bInCreateLoop)
{
	check(InGraph.Num() == InGraphLocations.Num());
	for (int32 GraphIndex = 0; GraphIndex < InGraph.Num(); ++GraphIndex)
	{
		FPCGEditorGraphSchemaAction_NewSubgraphElement::MakeGraphNode(InEditorGraph, nullptr, InGraph[GraphIndex], InGraphLocations[GraphIndex], bInSelectNewNodes, bInCreateLoop);
	}
}

UEdGraphNode* FPCGEditorGraphSchemaAction_NewSubgraphElement::MakeGraphNode(UPCGEditorGraph* InEditorGraph, UEdGraphPin* InFromPin, UPCGGraphInterface* InGraph, const FVector2D& InLocation, bool bInSelectNewNode, bool bInCreateLoop)
{
	if (!InEditorGraph || !InGraph)
	{
		return nullptr;
	}

	UPCGGraph* PCGGraph = InEditorGraph->GetPCGGraph();
	if (!PCGGraph)
	{
		return nullptr;
	}

	// Important - do not reconstruct the editor graph node/pins midway through this function as this will invalidate InFromPin.
	const FPCGDeferNodeReconstructScope DisableReconstruct(InFromPin);

	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorNewSubgraphElement", "PCG Editor: New Subgraph Element"), nullptr);
	InEditorGraph->Modify();

	UPCGSettings* DefaultNodeSettings = nullptr;
	UPCGNode* NewPCGNode = nullptr;
	
	if (!bInCreateLoop)
	{
		const UPCGEditorGraphSchema* PCGSchema = CastChecked<UPCGEditorGraphSchema>(InEditorGraph->GetSchema());
		
		NewPCGNode = PCGGraph->AddNodeOfType(PCGSchema->GetSubgraphSettingsClass(), DefaultNodeSettings);
		UPCGBaseSubgraphSettings* DefaultSubgraphSettings = CastChecked<UPCGBaseSubgraphSettings>(DefaultNodeSettings);
		DefaultSubgraphSettings->SetSubgraph(InGraph);
	}
	else
	{
		NewPCGNode = PCGGraph->AddNodeOfType(UPCGLoopSettings::StaticClass(), DefaultNodeSettings);
		UPCGLoopSettings* DefaultLoopSettings = CastChecked<UPCGLoopSettings>(DefaultNodeSettings);
		DefaultLoopSettings->SetSubgraph(InGraph);
	}

	if (!NewPCGNode)
	{
		UE_LOGF(LogPCGEditor, Error, "Unable to create node");
		return nullptr;
	}

	NewPCGNode->UpdateAfterSettingsChangeDuringCreation();

	FGraphNodeCreator<UPCGEditorGraphNode> NodeCreator(*InEditorGraph);
	UPCGEditorGraphNode* NewNode = NodeCreator.CreateUserInvokedNode(bInSelectNewNode);
	NewNode->Construct(NewPCGNode);
	NewNode->NodePosX = InLocation.X;
	NewNode->NodePosY = InLocation.Y;
	NodeCreator.Finalize();

	NewPCGNode->PositionX = InLocation.X;
	NewPCGNode->PositionY = InLocation.Y;

	if (InFromPin)
	{
		NewNode->AutowireNewNode(InFromPin);
	}

	return NewNode;
}

UEdGraphNode* FPCGEditorGraphSchemaAction_NewComment::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode /*= true*/)
{
	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(ParentGraph);
	if (!EditorGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid EditorGraph");
		return nullptr;
	}

	UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph();
	if (!PCGGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid PCGGraph");
		return nullptr;
	}

	UPCGEditorGraphNodeComment* CommentTemplate = NewObject<UPCGEditorGraphNodeComment>();

	TSharedPtr<SGraphEditor> GraphEditorPtr = SGraphEditor::FindGraphEditorForGraph(ParentGraph);

	FVector2f SpawnLocation = Location;
	FSlateRect Bounds;
	if (GraphEditorPtr && GraphEditorPtr->GetBoundsForSelectedNodes(Bounds, 50.0f))
	{
		CommentTemplate->SetBounds(Bounds);
		SpawnLocation.X = CommentTemplate->NodePosX;
		SpawnLocation.Y = CommentTemplate->NodePosY;
	}
	
	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorGraphSchemaAction_NewComment", "PCG Editor: New Comment"), nullptr);
	EditorGraph->Modify();
	PCGGraph->Modify();
	
	UPCGEditorGraphNodeComment* NewNode = FEdGraphSchemaAction_NewNode::SpawnNodeFromTemplate<UPCGEditorGraphNodeComment>(ParentGraph, CommentTemplate, SpawnLocation, bSelectNewNode);
	
	return NewNode;
}

UEdGraphNode* FPCGEditorGraphSchemaAction_NewReroute::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(ParentGraph);
	if (!EditorGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid EditorGraph");
		return nullptr;
	}

	UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph();
	if (!PCGGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid PCGGraph");
		return nullptr;
	}

	// Important - do not reconstruct the editor graph node/pins midway through this function as this will invalidate FromPin.
	const FPCGDeferNodeReconstructScope DisableReconstruct(FromPin);

	// Ensure compilation cache is populated before changing graph, so we can compare before/after compiled tasks later to detect change.
	PCGGraph->PrimeGraphCompilationCache();

	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorNewReroute", "PCG Editor: New Reroute Node"), nullptr);
	EditorGraph->Modify();

	UPCGSettings* DefaultNodeSettings = nullptr;
	UPCGNode* NewPCGNode = PCGGraph->AddNodeOfType(UPCGRerouteSettings::StaticClass(), DefaultNodeSettings);

	FGraphNodeCreator<UPCGEditorGraphNodeReroute> NodeCreator(*EditorGraph);
	UPCGEditorGraphNodeReroute* NewNode = NodeCreator.CreateUserInvokedNode(bSelectNewNode);
	NewNode->Construct(NewPCGNode);
	NewNode->NodePosX = Location.X;
	NewNode->NodePosY = Location.Y;
	NodeCreator.Finalize();

	NewPCGNode->PositionX = Location.X;
	NewPCGNode->PositionY = Location.Y;

	if (FromPin)
	{
		NewNode->AutowireNewNode(FromPin);
	}

	return NewNode;
}

UEdGraphNode* FPCGEditorGraphSchemaAction_NewNamedRerouteUsage::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	check(DeclarationNode);

	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(ParentGraph);
	if (!EditorGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid EditorGraph");
		return nullptr;
	}

	UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph();
	if (!PCGGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid PCGGraph");
		return nullptr;
	}

	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorNewNamedRerouteUsage", "PCG Editor: New Named Reroute Node Usage"), nullptr);
	EditorGraph->Modify();

	// Ensure compilation cache is populated before changing graph, so we can compare before/after compiled tasks later to detect change.
	PCGGraph->PrimeGraphCompilationCache();

	PCGGraph->DisableNotificationsForEditor();

	ON_SCOPE_EXIT
	{
		PCGGraph->EnableNotificationsForEditor();
	};

	UPCGNamedRerouteDeclarationSettings* Declaration = CastChecked<UPCGNamedRerouteDeclarationSettings>(DeclarationNode->GetSettings());

	UPCGSettings* DefaultNodeSettings = nullptr;
	UPCGNode* NewPCGNode = PCGGraph->AddNodeOfType(UPCGNamedRerouteUsageSettings::StaticClass(), DefaultNodeSettings);

	if (!NewPCGNode || !DefaultNodeSettings)
	{
		UE_LOGF(LogPCGEditor, Error, "Failed creating named reroute node.");
		return nullptr;
	}

	NewPCGNode->UpdateAfterSettingsChangeDuringCreation();

	CastChecked<UPCGNamedRerouteUsageSettings>(DefaultNodeSettings)->Declaration = Declaration;
	NewPCGNode->SetNodeTitle(DeclarationNode->NodeTitle, /*bApplySanitization=*/false);
	NewPCGNode->NodeTitleColor = DeclarationNode->NodeTitleColor;

	// Create edge from the declaration to the new node
	PCGGraph->AddEdge(const_cast<UPCGNode*>(DeclarationNode.Get()), PCGNamedRerouteConstants::InvisiblePinLabel, NewPCGNode, PCGPinConstants::DefaultInputLabel);
	
	FGraphNodeCreator<UPCGEditorGraphNodeNamedRerouteUsage> NodeCreator(*EditorGraph);
	UPCGEditorGraphNodeNamedRerouteUsage* NewNode = NodeCreator.CreateUserInvokedNode(bSelectNewNode);
	NewNode->Construct(NewPCGNode);
	NewNode->NodePosX = Location.X;
	NewNode->NodePosY = Location.Y;
	NodeCreator.Finalize();

	NewPCGNode->PositionX = Location.X;
	NewPCGNode->PositionY = Location.Y;

	// Finally, is this was dragged in from a pin, connect it.
	if (FromPin)
	{
		NewNode->AutowireNewNode(FromPin);
	}

	return NewNode;
}

UEdGraphNode* FPCGEditorGraphSchemaAction_NewNamedRerouteDeclaration::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(ParentGraph);
	if (!EditorGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid EditorGraph");
		return nullptr;
	}

	UPCGGraph* PCGGraph = EditorGraph->GetPCGGraph();
	if (!PCGGraph)
	{
		UE_LOGF(LogPCGEditor, Error, "Invalid PCGGraph");
		return nullptr;
	}

	// Important - do not reconstruct the editor graph node/pins midway through this function as this will invalidate FromPin.
	const FPCGDeferNodeReconstructScope DisableReconstruct(FromPin);

	// Ensure compilation cache is populated before changing graph, so we can compare before/after compiled tasks later to detect change.
	PCGGraph->PrimeGraphCompilationCache();

	const FScopedTransaction Transaction(*FPCGEditorCommon::ContextIdentifier, LOCTEXT("PCGEditorNewNamedRerouteDeclaration", "PCG Editor: New Named Reroute Node Declaration"), nullptr);
	EditorGraph->Modify();

	UPCGSettings* DefaultNodeSettings = nullptr;
	UPCGNode* NewPCGNode = PCGGraph->AddNodeOfType(UPCGNamedRerouteDeclarationSettings::StaticClass(), DefaultNodeSettings);

	if (!NewPCGNode || !DefaultNodeSettings)
	{
		UE_LOGF(LogPCGEditor, Error, "Failed creating named reroute node.");
		return nullptr;
	}

	NewPCGNode->UpdateAfterSettingsChangeDuringCreation();

	FGraphNodeCreator<UPCGEditorGraphNodeNamedRerouteDeclaration> NodeCreator(*EditorGraph);
	UPCGEditorGraphNodeNamedRerouteDeclaration* NewNode = NodeCreator.CreateUserInvokedNode(bSelectNewNode);
	NewNode->Construct(NewPCGNode);
	NewNode->NodePosX = Location.X;
	NewNode->NodePosY = Location.Y;
	NodeCreator.Finalize();

	NewPCGNode->PositionX = Location.X;
	NewPCGNode->PositionY = Location.Y;

	UPCGNode* FromNode = nullptr;

	if (FromPin)
	{
		FromNode = CastChecked<UPCGEditorGraphNodeBase>(FromPin->GetOwningNode())->GetPCGNode();
		NewNode->AutowireNewNode(FromPin);
	}

	const FString NewName = NewNode->GenerateNodeName(FromNode, FromPin ? FromPin->PinName : NAME_None);
	NewNode->OnRenameNode(NewName);

	return NewNode;
}

#undef LOCTEXT_NAMESPACE
