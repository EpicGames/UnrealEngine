// Copyright Epic Games, Inc. All Rights Reserved.

#include "Schema/PCGGraphParametersSchema.h"

#include "PCGGraph.h"
#include "HierarchyEditor/PropertyBagHierarchyViewModel.h"
#include "StructUtilsMetadata.h"

bool UPCGGraphParametersSchema::SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const
{
	return ContainerType == EPinContainerType::None || ContainerType == EPinContainerType::Array || ContainerType == EPinContainerType::Set;
}

UPropertyBagHierarchyRoot* UPCGGraphParametersSchema::GetHierarchyRoot(const TArray<UObject*>& ObjectsWithProperty) const
{
	if (ObjectsWithProperty.Num() != 1)
	{
		return nullptr;
	}
	
	if (UPCGGraphInterface* PCGGraphInterface = Cast<UPCGGraphInterface>(ObjectsWithProperty[0]))
	{
		return Cast<UPropertyBagHierarchyRoot>(PCGGraphInterface->GetUserParameterHierarchyRoot());
	}
	
	return nullptr;
}

TArray<UScriptStruct*> UPCGGraphParametersSchema::GetHierarchyPropertyMetaDataTypes(const FPropertyBagPropertyDesc& Desc) const
{
	return Super::GetHierarchyPropertyMetaDataTypes(Desc);
}

TSubclassOf<UPropertyBagHierarchyCategory> UPCGGraphParametersSchema::GetHierarchyCategoryType() const
{
	return UPCGPropertyBagHierarchyCategory::StaticClass();
}

TSubclassOf<UPropertyBagHierarchySection> UPCGGraphParametersSchema::GetHierarchySectionType() const
{
	return UPCGPropertyBagHierarchySection::StaticClass();
}
