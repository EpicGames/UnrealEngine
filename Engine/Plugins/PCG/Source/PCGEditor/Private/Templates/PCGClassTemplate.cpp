// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGClassTemplate.h"

#include "PCGNode.h"
#include "PCGSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGClassTemplate)

UPCGNativeSettingsClassTemplate::UPCGNativeSettingsClassTemplate(const FObjectInitializer& ObjectInitializer)
	: UPluginClassTemplate(ObjectInitializer)
{
	PluginName = TEXT("PCG");
	SetGeneratedBaseClass(UPCGSettings::StaticClass());
}

FString UPCGNativeSettingsClassTemplate::GetFilename() const
{
	return TEXT("PCGTemplateSettings");
}

UPCGNativeNodeClassTemplate::UPCGNativeNodeClassTemplate(const FObjectInitializer& ObjectInitializer)
	: UPluginClassTemplate(ObjectInitializer)
{
	PluginName = TEXT("PCG");
	SetGeneratedBaseClass(UPCGNode::StaticClass());
}

FString UPCGNativeNodeClassTemplate::GetFilename() const
{
	return TEXT("PCGTemplateSettings");
}