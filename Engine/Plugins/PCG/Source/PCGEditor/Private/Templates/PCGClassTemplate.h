// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ClassTemplateEditorSubsystem.h"

#include "PCGClassTemplate.generated.h"


UCLASS(Hidden)
class UPCGNativeSettingsClassTemplate : public UPluginClassTemplate
{
	GENERATED_UCLASS_BODY()

protected:
	virtual FString GetFilename() const override;
};

UCLASS(Hidden)
class UPCGNativeNodeClassTemplate : public UPluginClassTemplate
{
	GENERATED_UCLASS_BODY()
	
protected:
	virtual FString GetFilename() const override;
};