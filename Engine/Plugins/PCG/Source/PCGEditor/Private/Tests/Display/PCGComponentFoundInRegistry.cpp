// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentTypeRegistry.h"
#include "SComponentClassCombo.h"
#include "Tests/PCGTestsCommon.h"
#include "PCGComponent.h"

#if WITH_AUTOMATION_TESTS

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(PCGComponentFoundInRegistry, FPCGTestBaseClass, "Plugins.PCG.Display.PCGComponentFoundInRegistry", PCGTestsCommon::TestFlags)

bool PCGComponentFoundInRegistry::RunTest(const FString& Parameters)
{
	//Add component button gets list from ComponentList
	FComponentTypeRegistry& Registry = FComponentTypeRegistry::Get();

	TArray<FComponentClassComboEntryPtr>* ComponentClassList = nullptr;

	Registry.SubscribeToComponentList(ComponentClassList);

	if (ComponentClassList != nullptr) {

		const bool bFoundPCGComponent = ComponentClassList->ContainsByPredicate([](const FComponentClassComboEntryPtr& Entry) -> bool {
			
			return Entry->GetComponentClass() == UPCGComponent::StaticClass();
			
			});

		TestTrue(TEXT("UPCGComponent not found in component list"), bFoundPCGComponent);
	}

	return true;
}
#endif
