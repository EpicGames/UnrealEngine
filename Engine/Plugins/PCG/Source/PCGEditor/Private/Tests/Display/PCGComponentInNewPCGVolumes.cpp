// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineUtils.h"
#include "Engine/World.h"

#include "Editor.h"

#include "PCGVolume.h"
#include "PCGComponent.h"
#include "Tests/PCGTestsCommon.h"


#if WITH_AUTOMATION_TESTS

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGComponentInNewPCGVolumes, FPCGTestBaseClass, "Plugins.PCG.Display.PCGComponentInNewPCGVolumes", PCGTestsCommon::TestFlags)

bool FPCGComponentInNewPCGVolumes::RunTest(const FString& Parameters)
{
	UTEST_NOT_NULL(TEXT("Failed to get editor!"), GEditor);
	// Condition for static analysis.
	check(GEditor);

	// Get current world
	UWorld* World = GEditor->GetEditorWorldContext().World();
	UTEST_NOT_NULL(TEXT("Failed to get editor world context!"), World);

	// Condition for static analysis.
	check(World);

	TSubclassOf<APCGVolume> PCGVolumeClass = APCGVolume::StaticClass();
	UActorFactory* PCGVolumeFactory = GEditor->FindActorFactoryForActorClass(PCGVolumeClass);

	UTEST_NOT_NULL(TEXT("Failed to find PCGVolume actor factory."), PCGVolumeFactory);
	check(PCGVolumeFactory);

	APCGVolume* VolumeActor = nullptr;

	if (GCurrentLevelEditingViewportClient)
	{
		FTransform ActorTransform;
		VolumeActor = Cast<APCGVolume>(GEditor->UseActorFactory(PCGVolumeFactory, FAssetData(PCGVolumeClass), &ActorTransform));
	}

	UTEST_NOT_NULL(TEXT("Failed to add PCGVolume actor."), VolumeActor);
	// Condition for static analysis.
	check(VolumeActor);

	VolumeActor->SetFlags(RF_Transient);

	UPCGComponent* PCGComponent = VolumeActor->FindComponentByClass<UPCGComponent>();
	UTEST_NOT_NULL(TEXT("PCGVolume actor does not contain a PCGComponent!"), PCGComponent);
	// Condition for static analysis.
	check(PCGComponent);

	PCGComponent->ClearPCGLink(APCGVolume::StaticClass());

	// Deselect before destroying - UseActorFactory auto-selects the spawned actor, and DestroyActor
	// does not clear editor selection. After GC, the stale selected actor gains RF_BeginDestroyed,
	// causing a selection warning on the next test run.
	GEditor->SelectNone(/*bNoteSelectionChange=*/false, /*bDeselectBSPSurfs=*/true);

	// Cleanup the VolumeActor.
	const bool bSuccessful = World->DestroyActor(VolumeActor);
	TestTrue(TEXT("References to PCGVolume component could not be cleaned"), bSuccessful);

	return true;
};

#endif