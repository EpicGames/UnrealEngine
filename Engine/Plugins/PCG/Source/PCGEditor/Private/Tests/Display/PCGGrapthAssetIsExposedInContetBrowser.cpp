// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include  "Factories/Factory.h"

#if WITH_AUTOMATION_TESTS

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphExposedInContentMenuTest, FPCGTestBaseClass, "Plugins.PCG.Display.PCGGraphExposedInContentMenuTest", PCGTestsCommon::TestFlags)

bool FPCGGraphExposedInContentMenuTest::RunTest(const FString& Parameters)
{
	const FName NAME_AssetTools = "AssetTools";
	bool pcgGraphFound = false;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(NAME_AssetTools);

	// We check if the PCG category factory's display name contains "PCG Graph"
	TArray<UFactory*> Factories = AssetToolsModule.Get().GetNewAssetFactories();
	for (UFactory* Factory : Factories)
	{
		if (Factory->GetDisplayName().ToString().Contains("PCG Graph"))
		{
			pcgGraphFound = true;
			break;
		}
	}

	TestTrue(TEXT("PCG Graph not found in content browser"), pcgGraphFound);

	return true;
}

#endif