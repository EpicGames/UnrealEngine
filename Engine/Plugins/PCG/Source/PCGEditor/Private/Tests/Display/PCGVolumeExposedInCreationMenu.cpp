// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"
#include "IPlacementModeModule.h"

#if WITH_AUTOMATION_TESTS

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGVolumeExposedInCreationMenu, FPCGTestBaseClass, "Plugins.PCG.Display.PCGVolumeExposedInCreationMenu", PCGTestsCommon::TestFlags)

bool FPCGVolumeExposedInCreationMenu::RunTest(const FString& Parameters)
{
	// create menu queries place mode categories to display
	const IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	// PCG volume is located in Volumes category
	const FName PlacementModeCategoryHandle = TEXT("Volumes");
	FString PCGDisplayVolumeName = TEXT("PCG Volume");
	TArray<TSharedPtr<FPlaceableItem>> OutItems;


	PlacementModeModule.GetItemsForCategory(PlacementModeCategoryHandle, OutItems);
	bool VolumeCategoryFound = OutItems.ContainsByPredicate([&PCGDisplayVolumeName](const TSharedPtr<FPlaceableItem>& Item) {
		return Item->DisplayName.ToString().Contains(PCGDisplayVolumeName);
		});

	TestTrue(TEXT("PCGVolume is not found in Create Menu"), VolumeCategoryFound);
	return true;
}

#endif 
