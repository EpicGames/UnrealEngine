// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/AssetEditorViewport/PCGEditorViewportToolbarSections.h"

#include "PCGEditorCommands.h"
#include "PCGEditorStyle.h"

#include "ToolMenu.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "PCGEditorViewportToolbarSections"

namespace UE::PCGEditor
{
	FToolMenuEntry CreatePCGViewportSubmenu()
	{
		return FToolMenuEntry::InitDynamicEntry(TEXT("PCGMenu"), FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
		{
			FToolMenuEntry& Entry = Section.AddSubMenu(
				/*Name=*/TEXT("PCGSubmenu"),
				/*Label=*/FText::GetEmpty(),
				/*Tooltip=*/FText::GetEmpty(),
				FNewToolMenuDelegate::CreateLambda([](UToolMenu* Submenu)
				{
					// Add action buttons to this section.
					//FToolMenuSection& ActionsSection = Submenu->FindOrAddSection(TEXT("PCGActionsSection"), LOCTEXT("PCGActionsSectionLabel", "Actions"));

					// Add preference toggles to this section.
					FToolMenuSection& PreferencesSection = Submenu->FindOrAddSection(TEXT("PCGPreferencesSection"), LOCTEXT("PCGPreferencesSectionLabel", "Preferences"));
					PreferencesSection.AddMenuEntry(FPCGEditorCommands::Get().AutoFocusViewport);
				}),
				/*bInOpenSubMenuOnClick=*/false,
				FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.EditorIcon"));
		}));
	}
}

#undef LOCTEXT_NAMESPACE
