// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/AssetEditorViewport/SPCGEditorViewport.h"

#include "PCGEditorCommands.h"
#include "PCGEditorSettings.h"
#include "Widgets/AssetEditorViewport/PCGEditorViewportToolbarSections.h"

#include "AdvancedPreviewScene.h"
#include "AdvancedPreviewSceneMenus.h"
#include "AssetViewerSettings.h"
#include "EditorViewportClient.h"
#include "PreviewProfileController.h"
#include "ToolMenus.h"
#include "Engine/World.h"
#include "SceneInterface.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"

namespace PCGEditorViewportConstants
{
	static const FVector DefaultViewLocation = FVector::ZeroVector;
	static const FRotator DefaultViewRotation = FRotator(-25.0f, -135.0f, 0.0f);
	static const float DefaultOrbitDistance = 500.0f;
}

class FPCGEditorViewportClient : public FEditorViewportClient
{
public:
	FPCGEditorViewportClient(FEditorModeTools* InModeTools, const TSharedRef<SPCGEditorViewport>& InAssetEditorViewport);

	//~ Begin FViewportClient Interface
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual bool InputAxis(const FInputKeyEventArgs& Args) override;
	//~ End FViewportClient Interface

	//~ Begin FEditorViewportClient Interface
	virtual FLinearColor GetBackgroundColor() const override;
	virtual void Tick(float DeltaSeconds) override;
	//~ End FEditorViewportClient Interface

	void Reset();
};

FPCGEditorViewportClient::FPCGEditorViewportClient(FEditorModeTools* InModeTools, const TSharedRef<SPCGEditorViewport>& InAssetEditorViewport)
	: FEditorViewportClient(InModeTools, InAssetEditorViewport->GetAdvancedPreviewScene(), StaticCastSharedRef<SEditorViewport>(InAssetEditorViewport))
{
	check(PreviewScene);

	if (UWorld* World = PreviewScene->GetWorld(); World != nullptr && GWorld != nullptr && GWorld->Scene != nullptr)
	{
		World->ShaderPlatformChanged(GWorld->Scene->GetShaderPlatform());
	}

	bUsesDrawHelper = true;

	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = true;
	DrawHelper.GridColorAxis = FColor(80, 80, 80);
	DrawHelper.GridColorMajor = FColor(72, 72, 72);
	DrawHelper.GridColorMinor = FColor(64, 64, 64);
	DrawHelper.PerspectiveGridSize = static_cast<float>(HALF_WORLD_MAX1);

	EngineShowFlags.DisableAdvancedFeatures();
	EngineShowFlags.SetSnap(0);
	EngineShowFlags.SetSeparateTranslucency(true);

	FAdvancedPreviewScene* AdvancedPreviewScene = static_cast<FAdvancedPreviewScene*>(PreviewScene);
	check(AdvancedPreviewScene);

	AdvancedPreviewScene->SetFloorVisibility(false);
	
	Reset();
}

bool FPCGEditorViewportClient::InputKey(const FInputKeyEventArgs& Args)
{
	FAdvancedPreviewScene* AdvancedPreviewScene = static_cast<FAdvancedPreviewScene*>(PreviewScene);
	check(AdvancedPreviewScene);

	bool bHandled = FEditorViewportClient::InputKey(Args);
	bHandled |= InputTakeScreenshot(Args.Viewport, Args.Key, Args.Event);
	bHandled |= AdvancedPreviewScene->HandleInputKey(Args);

	return bHandled;
}

bool FPCGEditorViewportClient::InputAxis(const FInputKeyEventArgs& Args)
{
	FAdvancedPreviewScene* AdvancedPreviewScene = static_cast<FAdvancedPreviewScene*>(PreviewScene);
	check(AdvancedPreviewScene);

	bool bResult = true;

	if (!bDisableInput)
	{
		bResult = AdvancedPreviewScene->HandleViewportInput(Args.Viewport, Args.InputDevice, Args.Key, Args.AmountDepressed, Args.DeltaTime, Args.NumSamples, Args.IsGamepad());

		if (bResult)
		{
			Invalidate();
		}
		else
		{
			bResult = FEditorViewportClient::InputAxis(Args);
		}
	}

	return bResult;
}

FLinearColor FPCGEditorViewportClient::GetBackgroundColor() const
{
	return ensure(PreviewScene) ? PreviewScene->GetBackgroundColor() : FColor(64, 64, 64);
}

void FPCGEditorViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	if (!GIntraFrameDebuggingGameThread && ensure(PreviewScene && PreviewScene->GetWorld()))
	{
		PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
	}
}

void FPCGEditorViewportClient::Reset()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGEditorViewportClient::Reset);

	SetViewportType(ELevelViewportType::LVT_Perspective);

	Invalidate();
}

SPCGEditorViewport::SPCGEditorViewport()
	: AdvancedPreviewScene(MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues())))
{
}

SPCGEditorViewport::~SPCGEditorViewport()
{
	ReleaseManagedResources();
}

void SPCGEditorViewport::Construct(const FArguments& InArgs)
{
	ModeTools = InArgs._ModeTools;
	
	SEditorViewport::Construct(SEditorViewport::FArguments());

	UE::AdvancedPreviewScene::BindDefaultOnSettingsChangedHandler(AdvancedPreviewScene, EditorViewportClient);
}

TSharedRef<class SEditorViewport> SPCGEditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SPCGEditorViewport::GetExtenders() const
{
	return MakeShareable(new FExtender);
}

void SPCGEditorViewport::SetupScene(const TArray<UObject*>& InResources, const FPCGSetupSceneFunc& SetupFunc)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorViewport::SetupScene);

	check(EditorViewportClient.IsValid());
	check(AdvancedPreviewScene.IsValid());

	ResetScene();

	if (SetupFunc.IsSet())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorViewport::SetupSceneCallback);

		FPCGSceneSetupParams SceneSetupParams;
		SceneSetupParams.Scene = AdvancedPreviewScene.Get();
		SceneSetupParams.EditorViewportClient = EditorViewportClient.Get();
		SceneSetupParams.Resources = InResources;

		SetupFunc(SceneSetupParams);

		ManagedResources = MoveTemp(SceneSetupParams.ManagedResources);
		FocusBounds = MoveTemp(SceneSetupParams.FocusBounds);
		FocusOrthoZoom = SceneSetupParams.FocusOrthoZoom;

		if (DynamicToolbarSlot.IsValid() && SceneSetupParams.ViewportToolbarWidget.IsValid())
		{
			DynamicToolbarSlot->SetContent(SceneSetupParams.ViewportToolbarWidget.ToSharedRef());
		}

		if (bPendingViewportReset)
		{
			if (SceneSetupParams.PreferredViewportType.IsSet())
			{
				EditorViewportClient->SetViewportType(SceneSetupParams.PreferredViewportType.GetValue());
			}
			if (IsAutoFocusViewportChecked())
			{
				OnFocusViewportToSelection();
			}
			
			bPendingViewportReset = false;
		}

		EditorViewportClient->Invalidate();
	}

	OnSetupScene();
}

void SPCGEditorViewport::OnNewNodeInspected()
{
	bPendingViewportReset = true;
}

void SPCGEditorViewport::ResetScene()
{
	check(EditorViewportClient.IsValid());
	check(AdvancedPreviewScene.IsValid());

	EditorViewportClient->Invalidate();

	if (FPreviewSceneProfile* Profile = AdvancedPreviewScene->GetCurrentProfile())
	{
		Profile->SetShowFlags(EditorViewportClient->EngineShowFlags);
	}

	for (TObjectPtr<UObject> ManagedResource : ManagedResources)
	{
		if (UActorComponent* ActorComponent = Cast<UActorComponent>(ManagedResource))
		{
			AdvancedPreviewScene->RemoveComponent(ActorComponent);
		}
	}

	ReleaseManagedResources();

	if (DynamicToolbarSlot.IsValid())
	{
		DynamicToolbarSlot->SetContent(SNullWidget::NullWidget);
	}

	FocusBounds = FBoxSphereBounds(EForceInit::ForceInit);
	FocusOrthoZoom.Reset();

	if (IsAutoFocusViewportChecked())
	{
		OnFocusViewportToSelection();
	}

	OnResetScene();
}

TSharedRef<FEditorViewportClient> SPCGEditorViewport::MakeEditorViewportClient()
{
	EditorViewportClient = MakeShareable(new FPCGEditorViewportClient(ModeTools, SharedThis(this)));
	EditorViewportClient->SetViewLocation(PCGEditorViewportConstants::DefaultViewLocation);
	EditorViewportClient->SetViewRotation(PCGEditorViewportConstants::DefaultViewRotation);
	EditorViewportClient->SetViewLocationForOrbiting(PCGEditorViewportConstants::DefaultViewLocation, PCGEditorViewportConstants::DefaultOrbitDistance);
	EditorViewportClient->bSetListenerPosition = false;
	EditorViewportClient->EngineShowFlags.EnableAdvancedFeatures();
	EditorViewportClient->EngineShowFlags.SetGrid(false);
	EditorViewportClient->EngineShowFlags.SetLighting(true);
	EditorViewportClient->EngineShowFlags.SetIndirectLightingCache(true);
	EditorViewportClient->EngineShowFlags.SetPostProcessing(true);
	EditorViewportClient->Invalidate();

	return EditorViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SPCGEditorViewport::BuildViewportToolbar()
{
	check(AdvancedPreviewScene.IsValid());

	const FName ToolbarName = "PCG.ViewportToolbar";
	
	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(ToolbarName, /*Parent=*/NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		Menu->StyleName = "ViewportToolbar";
		Menu->AddSection("Left");
		
		FToolMenuSection& RightSection = Menu->AddSection("Right");
		RightSection.Alignment = EToolMenuSectionAlign::Last;
		RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowAll()));
		RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
		RightSection.AddEntry(UE::UnrealEd::CreateDefaultShowSubmenu());
		RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
		UE::AdvancedPreviewScene::Menus::ExtendAdvancedPreviewSceneSettings(
			"PCG.ViewportToolbar.AssetViewerProfile",
			UE::AdvancedPreviewScene::Menus::FSettingsOptions()
		);

		RightSection.AddEntry(UE::PCGEditor::CreatePCGViewportSubmenu());
	}

	FToolMenuContext Context;
	{
		Context.AppendCommandList(AdvancedPreviewScene->GetCommandList());
		Context.AppendCommandList(GetCommandList());
		Context.AddExtender(GetExtenders());
		Context.AddObject(UE::UnrealEd::CreateViewportToolbarDefaultContext(GetViewportWidget()));
	}

	const TSharedRef<SWidget> GeneratedMenu = UToolMenus::Get()->GenerateWidget(ToolbarName, Context);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SAssignNew(DynamicToolbarSlot, SBox)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			GeneratedMenu
		];
}

TSharedPtr<IPreviewProfileController> SPCGEditorViewport::CreatePreviewProfileController()
{
	return MakeShared<FPreviewProfileController>();
}

void SPCGEditorViewport::BindCommands()
{
	SAssetEditorViewport::BindCommands();

	const FPCGEditorCommands& Commands = FPCGEditorCommands::Get();

	CommandList->MapAction(
		Commands.AutoFocusViewport,
		FExecuteAction::CreateSP(this, &SPCGEditorViewport::ToggleAutoFocusViewport),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SPCGEditorViewport::IsAutoFocusViewportChecked));
}

void SPCGEditorViewport::OnFocusViewportToSelection()
{
	check(EditorViewportClient.IsValid());

	if (FocusBounds != FBoxSphereBounds(EForceInit::ForceInit))
	{
		// In some cases the focus is a little bit too close, so expanding the bounds slightly is helpful.
		const FBoxSphereBounds ExpandedFocusBounds = FocusBounds.ExpandBy(FocusBounds.SphereRadius * 0.1f);
		EditorViewportClient->FocusViewportOnBox(ExpandedFocusBounds.GetBox());

		if (FocusOrthoZoom.IsSet() && EditorViewportClient->IsOrtho())
		{
			EditorViewportClient->SetOrthoZoom(FocusOrthoZoom.GetValue());
		}
	}
}

void SPCGEditorViewport::ReleaseManagedResources()
{
	ManagedResources.Empty();
}

void SPCGEditorViewport::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ManagedResources);
}

void SPCGEditorViewport::ToggleAutoFocusViewport()
{
	UPCGEditorSettings* PCGEditorSettings = GetMutableDefault<UPCGEditorSettings>();
	PCGEditorSettings->bAutoFocusViewport = !PCGEditorSettings->bAutoFocusViewport;
}

bool SPCGEditorViewport::IsAutoFocusViewportChecked() const
{
	return GetDefault<UPCGEditorSettings>()->bAutoFocusViewport;
}
