// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGCodeEditor.h"

#include "PCGEditorSettings.h"
#include "PCGSettings.h"
#include "Compute/IPCGCodeEditorTextProvider.h"
#include "Helpers/PCGHelpers.h"

#include "PCGEditor.h"
#include "PCGHLSLSyntaxHighlighter.h"
#include "SPCGCodeEditorTextBox.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SPCGCodeEditor"

namespace PCGCodeEditorHelpers
{
	// Mirrors UPCGEditorSettings::CodeEditorFontSize ClampMin/ClampMax meta values.
	constexpr int32 MinFontSize = 6;
	constexpr int32 MaxFontSize = 32;

	int32 GetFontSize()
	{
		return GetDefault<UPCGEditorSettings>()->CodeEditorFontSize;
	}

	void SetFontSize(int32 InValue, bool bSaveConfig)
	{
		UPCGEditorSettings* Settings = GetMutableDefault<UPCGEditorSettings>();
		const int32 Clamped = FMath::Clamp(InValue, MinFontSize, MaxFontSize);
		if (Settings->CodeEditorFontSize != Clamped)
		{
			Settings->CodeEditorFontSize = Clamped;
			if (bSaveConfig)
			{
				Settings->SaveConfig();
			}
		}
	}

	TSharedRef<SWidget> BuildFontSizeRow()
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PCGCodeEditor_FontSize_Label", "Font Size:"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				SNew(SBox)
				.WidthOverride(60.0f)
				[
					SNew(SSpinBox<int32>)
					.MinValue(MinFontSize)
					.MaxValue(MaxFontSize)
					.MinSliderValue(MinFontSize)
					.MaxSliderValue(MaxFontSize)
					.Value_Lambda(&GetFontSize)
					.OnValueChanged_Lambda([](int32 NewValue) { SetFontSize(NewValue, /*bSaveConfig=*/false); })
					.OnValueCommitted_Lambda([](int32 NewValue, ETextCommit::Type) { SetFontSize(NewValue, /*bSaveConfig=*/true); })
					.ToolTipText(LOCTEXT("PCGCodeEditor_FontSize_SpinnerTooltip", "Adjust the code editor font size. Persisted to PCG editor preferences on commit."))
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([]() { SetFontSize(GetFontSize() - 1, /*bSaveConfig=*/true); return FReply::Handled(); })
				.ToolTipText(LOCTEXT("PCGCodeEditor_ZoomOut_Tooltip", "Decrease code editor font size"))
				.ContentPadding(2.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Minus"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([]() { SetFontSize(GetFontSize() + 1, /*bSaveConfig=*/true); return FReply::Handled(); })
				.ToolTipText(LOCTEXT("PCGCodeEditor_ZoomIn_Tooltip", "Increase code editor font size"))
				.ContentPadding(2.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	TSharedRef<SWidget> BuildMenuContent()
	{
		FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/false, /*InCommandList=*/nullptr);
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("PCGCodeEditor_Menu_Section", "Code Editor"));
		MenuBuilder.AddWidget(BuildFontSizeRow(), FText::GetEmpty(), /*bNoIndent=*/true);
		MenuBuilder.EndSection();
		return MenuBuilder.MakeWidget();
	}
}

SPCGCodeEditor::SPCGCodeEditor()
	: SyntaxHighlighterDeclarations(FPCGHLSLSyntaxHighlighter::Create())
	, SyntaxHighlighterText(FPCGHLSLSyntaxHighlighter::Create())
{
}

void SPCGCodeEditor::Construct(const FArguments& InArgs)
{
	SExpandableArea::FArguments ExpandableAreaArgs;
	ExpandableAreaArgs.AreaTitleFont(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"));
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(2.0f)
		[
			MakeMenuButton()
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			.PhysicalSplitterHandleSize(4.0f)
			.HitDetectionSplitterHandleSize(6.0f)
			+SSplitter::Slot()
			.Value(1.0)
			.SizeRule(this, &SPCGCodeEditor::GetDeclarationsSectionSizeRule)
			[
				SAssignNew(DeclarationsExpandableArea, SExpandableArea) = ExpandableAreaArgs
				.AreaTitle(LOCTEXT("PCGCodeEditor_Declarations_Title", "Declarations (Read-Only)"))
				.InitiallyCollapsed(true)
				.AllowAnimatedTransition(false)
				.BodyContent()
				[
					SAssignNew(DeclarationsTextBox, SPCGCodeEditorTextBox)
					.Text(this, &SPCGCodeEditor::GetDeclarationsText)
					.IsReadOnly(true)
					.Marshaller(SyntaxHighlighterDeclarations)
				]
			]
			+SSplitter::Slot()
			.Value(1.0)
			.SizeRule(this, &SPCGCodeEditor::GetFunctionsSectionSizeRule)
			[
				SAssignNew(FunctionsExpandableArea, SExpandableArea) = ExpandableAreaArgs
				.AreaTitle(LOCTEXT("PCGCodeEditor_Functions_Title", "Functions"))
				.InitiallyCollapsed(true)
				.AllowAnimatedTransition(false)
				.BodyContent()
				[
					SAssignNew(FunctionsTextBox, SPCGCodeEditorTextBox)
					.Text(this, &SPCGCodeEditor::GetFunctionsText)
					.IsReadOnly(this, &SPCGCodeEditor::IsReadOnly)
					.Marshaller(SyntaxHighlighterDeclarations)
					.OnTextChanged(this, &SPCGCodeEditor::OnFunctionsTextChanged)
					.OnTextCommitted(this, &SPCGCodeEditor::OnFunctionsTextCommitted)
					.OnTextChangesApplied(this, &SPCGCodeEditor::OnFunctionsTextChangesApplied)
				]
			]
			+SSplitter::Slot()
			.Value(1.0)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					ConstructNonExpandableHeaderWidget(ExpandableAreaArgs.AreaTitle(LOCTEXT("PCGCodeEditor_Text_Title", "Source")))
				]
				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SAssignNew(SourceTextBox, SPCGCodeEditorTextBox)
					.Text(this, &SPCGCodeEditor::GetSourceText)
					.IsReadOnly(this, &SPCGCodeEditor::IsReadOnly)
					.Marshaller(SyntaxHighlighterText)
					.OnTextChanged(this, &SPCGCodeEditor::OnSourceTextChanged)
					.OnTextCommitted(this, &SPCGCodeEditor::OnSourceTextCommitted)
					.OnTextChangesApplied(this, &SPCGCodeEditor::OnSourceTextChangesApplied)
				]
			]
		]
	];
}

void SPCGCodeEditor::SetTextProviderObject(UObject* InProviderObject)
{
	TextProviderObject = InProviderObject;

	FunctionsTextBox->SetTextProviderObject(InProviderObject);
	SourceTextBox->SetTextProviderObject(InProviderObject);

	FunctionsText = GetFunctionsText();
	SourceText = GetSourceText();

	SyntaxHighlighterText->ClearCompilerMessages();

	Refresh();
}

void SPCGCodeEditor::Refresh()
{
	DeclarationsTextBox->Refresh();
	SourceTextBox->Refresh();
}

SSplitter::ESizeRule SPCGCodeEditor::GetDeclarationsSectionSizeRule() const
{
	SSplitter::ESizeRule SizeRule = SSplitter::ESizeRule::FractionOfParent;

	if (ensure(DeclarationsExpandableArea.IsValid()))
	{
		SizeRule = DeclarationsExpandableArea->IsExpanded() ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent;
	}

	return SizeRule;
}

SSplitter::ESizeRule SPCGCodeEditor::GetFunctionsSectionSizeRule() const
{
	SSplitter::ESizeRule SizeRule = SSplitter::ESizeRule::FractionOfParent;

	if (ensure(FunctionsExpandableArea.IsValid()))
	{
		SizeRule = FunctionsExpandableArea->IsExpanded() ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent;
	}

	return SizeRule;
}

IPCGCodeEditorTextProvider* SPCGCodeEditor::GetTextProviderInterface() const
{
	return Cast<IPCGCodeEditorTextProvider>(TextProviderObject);
}

FText SPCGCodeEditor::GetDeclarationsText() const
{
	if (GetTextProviderInterface())
	{
		return FText::FromString(GetTextProviderInterface()->GetDeclarationsText());
	}

	return FText::GetEmpty();
}

FText SPCGCodeEditor::GetFunctionsText() const
{
	if (GetTextProviderInterface())
	{
		return FText::FromString(GetTextProviderInterface()->GetFunctionsText());
	}

	return FText::GetEmpty();
}

FText SPCGCodeEditor::GetSourceText() const
{
	if (GetTextProviderInterface())
	{
		return FText::FromString(GetTextProviderInterface()->GetSourceText());
	}

	return FText::GetEmpty();
}

bool SPCGCodeEditor::IsReadOnly() const
{
	if (IPCGCodeEditorTextProvider* Provider = Cast<IPCGCodeEditorTextProvider>(TextProviderObject))
	{
		return Provider->IsReadOnly();
	}

	return false;
}

void SPCGCodeEditor::OnFunctionsTextChanged(const FText& InText)
{
	// Always clear compiler messages when text is edited so that markup doesn't lurk in random places.
	SyntaxHighlighterText->ClearCompilerMessages();

	FunctionsText = InText;
}

void SPCGCodeEditor::OnSourceTextChanged(const FText& InText)
{
	// Always clear compiler messages when text is edited so that markup doesn't lurk in random places.
	SyntaxHighlighterText->ClearCompilerMessages();

	SourceText = InText;
}

void SPCGCodeEditor::OnFunctionsTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo)
{
	FunctionsText = InText;
	SetFunctionsText();
}

void SPCGCodeEditor::OnSourceTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo)
{
	SourceText = InText;
	SetSourceText();
}

void SPCGCodeEditor::OnFunctionsTextChangesApplied() const
{
	SetFunctionsText();
}

void SPCGCodeEditor::OnSourceTextChangesApplied() const
{
	SetSourceText();
}

void SPCGCodeEditor::SetFunctionsText() const
{
	if (IPCGCodeEditorTextProvider* Provider = GetTextProviderInterface())
	{
		const FString FunctionsString = FunctionsText.ToString();

		if (!FunctionsString.Equals(Provider->GetFunctionsText(), ESearchCase::CaseSensitive))
		{
			Provider->SetFunctionsText(FunctionsString);
		}
	}
}

void SPCGCodeEditor::SetSourceText() const
{
	if (IPCGCodeEditorTextProvider* Provider = GetTextProviderInterface())
	{
		const FString SourceString = SourceText.ToString();

		if (!SourceString.Equals(Provider->GetSourceText(), ESearchCase::CaseSensitive))
		{
			Provider->SetSourceText(SourceString);
		}
	}
}

void SPCGCodeEditor::OnDiagnosticsUpdated(const FPCGCompilerDiagnostics& InDiagnostics) const
{
	SyntaxHighlighterText->SetCompilerMessages(InDiagnostics);
}

TSharedRef<SWidget> SPCGCodeEditor::MakeMenuButton()
{
	return SNew(SComboButton)
		.HasDownArrow(false)
		.ForegroundColor(FSlateColor::UseForeground())
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ContentPadding(0.0f)
		.ToolTipText(LOCTEXT("PCGCodeEditor_MenuButton_Tooltip", "Code editor display options."))
		.OnGetMenuContent_Static(&PCGCodeEditorHelpers::BuildMenuContent)
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("DetailsView.ViewOptions"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
}

TSharedRef<SWidget> SPCGCodeEditor::ConstructNonExpandableHeaderWidget(const SExpandableArea::FArguments& InArgs) const
{
	return SNew(SBorder)
		.BorderImage(FStyleDefaults::GetNoBrush())
		.BorderBackgroundColor(FLinearColor::Transparent)
		.Padding(0.0f)
		[
			SNew(SButton)
			.ButtonStyle(FCoreStyle::Get(), "NoBorder")
			.ContentPadding(InArgs._HeaderPadding)
			.ForegroundColor(FSlateColor::UseForeground())
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(InArgs._AreaTitlePadding)
				.VAlign(VAlign_Center)
				[
					SNew(SSpacer)
					.Size(InArgs._Style->CollapsedImage.GetImageSize())
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(InArgs._AreaTitle)
					.Font(InArgs._AreaTitleFont)
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
