// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FPCGEditor;
class FPCGHLSLSyntaxHighlighter;
class IPCGCodeEditorTextProvider;
class SPCGCodeEditorTextBox;
struct FPCGCompilerDiagnostics;

/** Window for editing source code. */
class SPCGCodeEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGCodeEditor) { }
	SLATE_END_ARGS()

	SPCGCodeEditor();

	void Construct(const FArguments& InArgs);

	void SetTextProviderObject(UObject* InProviderObject);
	const UObject* GetTextProviderObject() const { return TextProviderObject.IsValid() ? TextProviderObject.Get() : nullptr; }

	void OnDiagnosticsUpdated(const FPCGCompilerDiagnostics& InDiagnostics) const;

	static TSharedRef<SWidget> MakeMenuButton();

private:
	void Refresh();

	SSplitter::ESizeRule GetDeclarationsSectionSizeRule() const;
	SSplitter::ESizeRule GetFunctionsSectionSizeRule() const;
	IPCGCodeEditorTextProvider* GetTextProviderInterface() const;

	FText GetDeclarationsText() const;
	FText GetFunctionsText() const;
	FText GetSourceText() const;

	bool IsReadOnly() const;

	void OnFunctionsTextChanged(const FText& InText);
	void OnSourceTextChanged(const FText& InText);

	void OnFunctionsTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo);
	void OnSourceTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo);

	void OnFunctionsTextChangesApplied() const;
	void OnSourceTextChangesApplied() const;

	void SetFunctionsText() const;
	void SetSourceText() const;

	TSharedRef<SWidget> ConstructNonExpandableHeaderWidget(const SExpandableArea::FArguments& InArgs) const;

	TSharedRef<FPCGHLSLSyntaxHighlighter> SyntaxHighlighterDeclarations;
	TSharedRef<FPCGHLSLSyntaxHighlighter> SyntaxHighlighterText;

	TWeakObjectPtr<UObject> TextProviderObject;

	TSharedPtr<SExpandableArea> DeclarationsExpandableArea;
	TSharedPtr<SExpandableArea> FunctionsExpandableArea;

	TSharedPtr<SPCGCodeEditorTextBox> DeclarationsTextBox;
	TSharedPtr<SPCGCodeEditorTextBox> FunctionsTextBox;
	TSharedPtr<SPCGCodeEditorTextBox> SourceTextBox;

	FText FunctionsText;
	FText SourceText;
};
