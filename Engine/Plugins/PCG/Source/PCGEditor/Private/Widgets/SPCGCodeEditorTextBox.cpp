// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGCodeEditorTextBox.h"

#include "Compute/IPCGCodeEditorTextProvider.h"

#include "PCGEditorStyle.h"
#include "SPCGShaderTextSearchWidget.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h" 
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Text/SMultiLineEditableText.h"

#define LOCTEXT_NAMESPACE "PCGShaderTextDocumentTextBox"

static int32 GetNumSpacesAtStartOfLine(const FString& InLine)
{
	int32 NumSpaces = 0;
	for (const TCHAR Char : InLine)
	{
		if ((Char != TEXT(' ')))
		{
			break;
		}
				
		NumSpaces++;
	}

	return NumSpaces;
}

static bool IsOpenBrace(const TCHAR& InCharacter)
{
	return (InCharacter == TEXT('{') || InCharacter == TEXT('[') || InCharacter == TEXT('('));
}

static bool IsCloseBrace(const TCHAR& InCharacter)
{
	return (InCharacter == TEXT('}') || InCharacter == TEXT(']') || InCharacter == TEXT(')'));
}

static TCHAR GetMatchedCloseBrace(const TCHAR& InCharacter)
{
	return InCharacter == TEXT('{') ? TEXT('}') :
		(InCharacter == TEXT('[') ? TEXT(']') : TEXT(')'));
}

static bool IsWhiteSpace(const TCHAR& InCharacter)
{
	return InCharacter == TEXT(' ') || InCharacter == TEXT('\r') || InCharacter == TEXT('\n');
}

FPCGCodeEditorTextBoxCommands::FPCGCodeEditorTextBoxCommands() 
	: TCommands<FPCGCodeEditorTextBoxCommands>(
		"PCGShaderTextEditorDocumentTextBox", // Context name for fast lookup
		NSLOCTEXT("Contexts", "PCGShaderTextEditorDocumentTextBox", "PCG HLSL Source Editor Document TextBox"), // Localized context name for display
		NAME_None,
		FAppStyle::GetAppStyleSetName()
	)
{
}

void FPCGCodeEditorTextBoxCommands::RegisterCommands()
{
	UI_COMMAND(ApplyChanges, "Apply Changes", "Apply Changes", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter, EModifierKey::Alt));

	UI_COMMAND(Search, "Search", "Search for a String", EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Control));
	UI_COMMAND(NextOccurrence, "Next Occurrence", "Go to Next Occurrence", EUserInterfaceActionType::Button, FInputChord(EKeys::F3, EModifierKey::None));
	UI_COMMAND(PreviousOccurrence, "Previous Occurrence", "Go to Previous Occurrence", EUserInterfaceActionType::Button, FInputChord(EKeys::F3, EModifierKey::Shift));

	UI_COMMAND(ToggleComment, "Toggle Comment", "Comment/Uncomment selected lines", EUserInterfaceActionType::Button, FInputChord(EKeys::Slash, EModifierKey::Control));
}

SPCGCodeEditorTextBox::SPCGCodeEditorTextBox()
	: bIsSearchBarHidden(true)
	, TopLevelCommandList(MakeShared<FUICommandList>())
	, TextCommandList(MakeShared<FUICommandList>())
{
}

SPCGCodeEditorTextBox::~SPCGCodeEditorTextBox()
{
}

void SPCGCodeEditorTextBox::Construct(const FArguments& InArgs)
{
	RegisterCommands();
	
	OnTextChangesAppliedCallback = InArgs._OnTextChangesApplied;

	const TSharedPtr<SScrollBar> HScrollBar = SNew(SScrollBar)
		.Orientation(EOrientation::Orient_Horizontal);
	
	const TSharedPtr<SScrollBar> VScrollBar = SNew(SScrollBar)
		.Orientation(EOrientation::Orient_Vertical);
	
	const FTextBlockStyle& TextStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("MessageLog");
	const FSlateFontInfo &Font = TextStyle.Font;

	const bool bReadOnly = InArgs._IsReadOnly.Get();
	const FPCGCodeEditorTextBoxCommands& Commands = FPCGCodeEditorTextBoxCommands::Get();
	
	Text = SNew(SMultiLineEditableText)
			.Font(Font)
			.TextStyle(&TextStyle)
			.Text(InArgs._Text)
			.OnTextChanged(InArgs._OnTextChanged)
			.OnTextCommitted(InArgs._OnTextCommitted)
			.OnKeyCharHandler(this, &SPCGCodeEditorTextBox::OnTextKeyChar)
			.OnKeyDownHandler(this, &SPCGCodeEditorTextBox::OnTextKeyDown)
			// By default, the Tab key gets routed to "next widget". We want to disable that behaviour.
			.OnIsTypedCharValid_Lambda([](const TCHAR InChar) { return true; })
			.Marshaller(InArgs._Marshaller)
			.AutoWrapText(false)
			.ClearTextSelectionOnFocusLoss(false)
			.AllowContextMenu(true)
			.ContextMenuExtender(
				FMenuExtensionDelegate::CreateLambda(
					[this, bReadOnly, Commands](FMenuBuilder& InBuilder)
					{
						if (!bReadOnly)
						{
							InBuilder.PushCommandList(TextCommandList);
							InBuilder.AddMenuEntry(Commands.ApplyChanges);
							InBuilder.AddMenuEntry(Commands.ToggleComment);
							InBuilder.PopCommandList();
						}
						
					}))
			.IsReadOnly(this, &SPCGCodeEditorTextBox::IsReadOnly)
			.HScrollBar(HScrollBar)
			.VScrollBar(VScrollBar);

	SearchBar = SNew(SPCGShaderTextSearchWidget)
		.OnTextChanged(this, &SPCGCodeEditorTextBox::OnSearchTextChanged)
		.OnTextCommitted(this, &SPCGCodeEditorTextBox::OnSearchTextCommitted)
		.SearchResultData(this, &SPCGCodeEditorTextBox::GetSearchResultData)
		.OnResultNavigationButtonClicked(this, &SPCGCodeEditorTextBox::OnSearchResultNavigationButtonClicked);

	ChildSlot
	[
		SAssignNew(TabBody, SVerticalBox)
		+SVerticalBox::Slot()
		[
			SNew(SBorder)
			.BorderImage(FPCGEditorStyle::Get().GetBrush("TextEditor.Border"))
			.BorderBackgroundColor(this, &SPCGCodeEditorTextBox::GetBorderBackgroundColor)
			[
				SNew(SGridPanel)
				.FillColumn(0,1.0f)
				.FillRow(0,1.0f)
				+SGridPanel::Slot(0,0)
				[
					Text.ToSharedRef()
				]
				+SGridPanel::Slot(1,0)
				[
					VScrollBar.ToSharedRef()
				]
				+SGridPanel::Slot(0,1)
				[
					HScrollBar.ToSharedRef()
				]
			]
		]
	];
}

void SPCGCodeEditorTextBox::SetTextProviderObject(UObject* InProviderObject)
{
	ShaderTextProviderObject = InProviderObject;

	Refresh();
}

void SPCGCodeEditorTextBox::RegisterCommands()
{
	const FPCGCodeEditorTextBoxCommands& Commands = FPCGCodeEditorTextBoxCommands::Get();
	
	TopLevelCommandList->MapAction(Commands.Search, FExecuteAction::CreateSP(this, &SPCGCodeEditorTextBox::OnTriggerSearch));
	TopLevelCommandList->MapAction(Commands.NextOccurrence, FExecuteAction::CreateSP(this, &SPCGCodeEditorTextBox::OnGoToNextOccurrence));
	TopLevelCommandList->MapAction(Commands.PreviousOccurrence, FExecuteAction::CreateSP(this, &SPCGCodeEditorTextBox::OnGoToPreviousOccurrence));
	TextCommandList->MapAction(Commands.ApplyChanges, FExecuteAction::CreateSP(this, &SPCGCodeEditorTextBox::OnApplyChanges));
	TextCommandList->MapAction(Commands.ToggleComment, FExecuteAction::CreateSP(this, &SPCGCodeEditorTextBox::OnToggleComment));
}

FReply SPCGCodeEditorTextBox::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const FKey Key = InKeyEvent.GetKey();
	
	if (Key == EKeys::Escape)
	{
		if (HandleEscape())
		{
			return FReply::Handled();
		}
	}

	if (TopLevelCommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	
	return SCompoundWidget::OnPreviewKeyDown(MyGeometry, InKeyEvent);
}

bool SPCGCodeEditorTextBox::HandleEscape()
{
	if (HideSearchBar())
	{
		return true;
	}

	return false;
}

void SPCGCodeEditorTextBox::ShowSearchBar()
{
	if (bIsSearchBarHidden)
	{
		bIsSearchBarHidden = false;
		
		TabBody->InsertSlot(0)
		.AutoHeight()
		[
			SearchBar.ToSharedRef()
		];
	}
}

bool SPCGCodeEditorTextBox::HideSearchBar() 
{
	if (!bIsSearchBarHidden)
	{
		bIsSearchBarHidden = true;
		SearchBar->ClearSearchText();
		TabBody->RemoveSlot(SearchBar.ToSharedRef());
			
		FSlateApplication::Get().ForEachUser([&](FSlateUser& User) {
			User.SetFocus(Text.ToSharedRef(), EFocusCause::SetDirectly);
		});

		return true;
	}
	
	return false;
}

void SPCGCodeEditorTextBox::OnTriggerSearch()
{
	ShowSearchBar();

	FText SelectedText = Text->GetSelectedText();
	
	SearchBar->TriggerSearch(SelectedText);
}

void SPCGCodeEditorTextBox::Refresh() const
{
	check(IsInGameThread());
	Text->Refresh();
}

void SPCGCodeEditorTextBox::OnSearchTextChanged(const FText& InTextToSearch)
{
	// we start the search from the beginning of current selection.
	// goto clears the selection, but it will be restored by the first search
	Text->GoTo(Text->GetSelection().GetBeginning());

	Text->SetSearchText(InTextToSearch);
}

void SPCGCodeEditorTextBox::OnSearchTextCommitted(const FText& InTextToSearch, ETextCommit::Type InCommitType)
{
	if (!InTextToSearch.EqualTo(Text->GetSearchText()))
	{
		Text->SetSearchText(InTextToSearch);
	}
	else
	{
		if (InCommitType == ETextCommit::Type::OnEnter)
		{
			OnSearchResultNavigationButtonClicked(SSearchBox::SearchDirection::Next);
		}
	}
}

TOptional<SSearchBox::FSearchResultData> SPCGCodeEditorTextBox::GetSearchResultData() const
{
	FText SearchText = Text->GetSearchText();
	
	if (!SearchText.IsEmpty())
	{
		SSearchBox::FSearchResultData Result;
		Result.CurrentSearchResultIndex = Text->GetSearchResultIndex();
		Result.NumSearchResults = Text->GetNumSearchResults();
		
		return Result;
	}
		
	return TOptional<SSearchBox::FSearchResultData>();
}

void SPCGCodeEditorTextBox::OnSearchResultNavigationButtonClicked(SSearchBox::SearchDirection InDirection)
{
	Text->AdvanceSearch(InDirection == SSearchBox::SearchDirection::Previous);
}

void SPCGCodeEditorTextBox::OnGoToNextOccurrence()
{
	OnSearchResultNavigationButtonClicked(SSearchBox::SearchDirection::Next);
}

void SPCGCodeEditorTextBox::OnGoToPreviousOccurrence()
{
	OnSearchResultNavigationButtonClicked(SSearchBox::SearchDirection::Previous);
}

FReply SPCGCodeEditorTextBox::OnTextKeyDown(
	const FGeometry& MyGeometry,
	const FKeyEvent& InKeyEvent) const
{
	if (TextCommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	// Let SMultiLineEditableText::OnKeyDown handle it.
	return FReply::Unhandled();
}

void SPCGCodeEditorTextBox::OnApplyChanges() const
{
	OnTextChangesAppliedCallback.ExecuteIfBound();
}

void SPCGCodeEditorTextBox::OnToggleComment() const
{
	SMultiLineEditableText::FScopedEditableTextTransaction Transaction(Text);
	
	const FTextLocation CursorLocation = Text->GetCursorLocation();
	
	const FTextSelection Selection = Text->GetSelection();
	const FTextLocation SelectionStart =
		CursorLocation == Selection.GetBeginning() ? Selection.GetEnd() : Selection.GetBeginning();
	
	// Need to shift the selection according to the new indentation
	FTextLocation NewCursorLocation = CursorLocation;
	FTextLocation NewSelectionStart = SelectionStart;
		
	const int32 StartLine = Selection.GetBeginning().GetLineIndex();
	const int32 EndLine = Selection.GetEnd().GetLineIndex();

	bool bShouldComment = false;
	bool bAreAllLinesEmpty = true;
	int32 MinNumLeadingSpaces = INDEX_NONE;
	for (int32 Index = StartLine; Index <= EndLine; Index++)
	{
		FString Line;
		Text->GetTextLine(Index, Line);
		// Empty lines are not considered when deciding whether to comment/uncomment
		const int32 NumSpaces = GetNumSpacesAtStartOfLine(Line);
		if (NumSpaces >= Line.Len())	
		{
			continue;
		}

		bAreAllLinesEmpty = false;
		
		FStringView LineView(&Line[NumSpaces]);
		if (!LineView.StartsWith(TEXT("//")))
		{
			bShouldComment = true;
		}

		MinNumLeadingSpaces =
			MinNumLeadingSpaces == INDEX_NONE ?
			NumSpaces : FMath::Min(MinNumLeadingSpaces, NumSpaces);
	}

	// Find nearest tab
	MinNumLeadingSpaces = MinNumLeadingSpaces / 4 * 4;

	// Single line comment or single line no-op should move the cursor down
	const bool bShouldGoToNewLine = !Text->AnyTextSelected() && (bShouldComment || bAreAllLinesEmpty);
	
	for (int32 Index = StartLine; Index <= EndLine; Index++)
	{
		FString Line;
		Text->GetTextLine(Index, Line);

		if (!Line.IsEmpty())
		{
			const int32 NumSpaces = GetNumSpacesAtStartOfLine(Line);
			if (NumSpaces < Line.Len())
			{
				int32 LineShift = 0;
				if (bShouldComment)
				{
					Text->GoTo(FTextLocation(Index, MinNumLeadingSpaces));
					Text->InsertTextAtCursor(TEXT("// "));
					LineShift = 3;
				}
				else
				{
					int32 CommentOffset = Line.Find("//");
					if (ensure(CommentOffset != INDEX_NONE))
					{
						int32 ContentOffset =
							Line.IsValidIndex(CommentOffset+2) && Line[CommentOffset+2] == TEXT(' ') ?
								3 : 2;
						Text->SelectText({Index, CommentOffset}, {Index, CommentOffset+ContentOffset});
						Text->DeleteSelectedText();
						LineShift = -ContentOffset;
					}
				}

				if (Index == CursorLocation.GetLineIndex())
				{
					NewCursorLocation = FTextLocation(CursorLocation, LineShift);
				}
					
				if (Index == SelectionStart.GetLineIndex())
				{
					NewSelectionStart = FTextLocation(SelectionStart, LineShift);
				}	
			}
		}
	}

	if (bShouldGoToNewLine)
	{
		// Automatically go to the next line if there is no selection
		int32 LineIndex = CursorLocation.GetLineIndex() + 1;
		if (LineIndex < const_cast<const SMultiLineEditableText*>(Text.Get())->GetTextLineCount())
		{
			FString Line;
			Text->GetTextLine(LineIndex, Line);
			
			int32 Offset = FMath::Min(CursorLocation.GetOffset(), Line.Len());
			
			Text->GoTo({LineIndex, Offset});
		}
		else
		{
			Text->GoTo(NewCursorLocation);
		}
	}
	else
	{
		Text->SelectText(NewSelectionStart, NewCursorLocation);
	}
}

FReply SPCGCodeEditorTextBox::OnTextKeyChar(
	const FGeometry& MyGeometry,
	const FCharacterEvent& InCharacterEvent) const
{
	if (Text->IsTextReadOnly())
	{
		return FReply::Unhandled();
	}

	const TCHAR Character = InCharacterEvent.GetCharacter();

	if (Character == TEXT('\b'))
	{
		if (!Text->AnyTextSelected())
		{
			// if we are deleting a single open brace
			// look for a matching close brace following it and delete the close brace as well
			const FTextLocation CursorLocation = Text->GetCursorLocation();
			const int Offset = CursorLocation.GetOffset();
			FString Line;
			Text->GetTextLine(CursorLocation.GetLineIndex(), Line);

			if (Line.IsValidIndex(Offset) && Line.IsValidIndex(Offset-1))
			{
				if (IsOpenBrace(Line[Offset-1]))
				{
					if (Line[Offset] == GetMatchedCloseBrace(Line[Offset-1]))
					{
						SMultiLineEditableText::FScopedEditableTextTransaction ScopedTransaction(Text);
						Text->SelectText(FTextLocation(CursorLocation, -1),FTextLocation(CursorLocation, 1));
						Text->DeleteSelectedText();
						Text->GoTo(FTextLocation(CursorLocation, -1));
						return FReply::Handled();
					}
				}
			}
		}

		return FReply::Unhandled();
	}
	else if (Character == TEXT('\t'))
	{
		SMultiLineEditableText::FScopedEditableTextTransaction Transaction(Text);

		const FTextLocation CursorLocation = Text->GetCursorLocation();

		const bool bShouldIncreaseIndentation = InCharacterEvent.GetModifierKeys().IsShiftDown() ? false : true;
		
		// When there is no text selected, shift tab should also decrease line indentation
		const bool bShouldIndentLine = Text->AnyTextSelected() || (!Text->AnyTextSelected() && !bShouldIncreaseIndentation);
		
		if (bShouldIndentLine)
		{
			// Indent the whole line if there is a text selection
			const FTextSelection Selection = Text->GetSelection();
			const FTextLocation SelectionStart =
				CursorLocation == Selection.GetBeginning() ? Selection.GetEnd() : Selection.GetBeginning();

			// Shift the selection according to the new indentation
			FTextLocation NewCursorLocation;
			FTextLocation NewSelectionStart;
		
			const int32 StartLine = Selection.GetBeginning().GetLineIndex();
			const int32 EndLine = Selection.GetEnd().GetLineIndex();
		
			for (int32 Index = StartLine; Index <= EndLine; Index++)
			{
				const FTextLocation LineStart(Index, 0);
				Text->GoTo(LineStart);
				
				FString Line;
				Text->GetTextLine(Index, Line);
				const int32 NumSpaces = GetNumSpacesAtStartOfLine(Line);
				const int32 NumExtraSpaces = NumSpaces % 4;

				// Tab to nearest 4.
				int32 NumSpacesForIndentation;
				if (bShouldIncreaseIndentation)
				{
					NumSpacesForIndentation = NumExtraSpaces == 0 ? 4 : 4 - NumExtraSpaces ;
					Text->InsertTextAtCursor(FString::ChrN(NumSpacesForIndentation, TEXT(' ')));
				}
				else
				{
					NumSpacesForIndentation = NumExtraSpaces == 0 ? FMath::Min(4, NumSpaces) : NumExtraSpaces;
					Text->SelectText(LineStart,FTextLocation(LineStart, NumSpacesForIndentation));
					Text->DeleteSelectedText();
				}

				const int32 CursorShiftDirection = bShouldIncreaseIndentation ? 1 : -1;
				const int32 CursorShift = NumSpacesForIndentation * CursorShiftDirection;
				
				if (Index == CursorLocation.GetLineIndex())
				{
					NewCursorLocation = FTextLocation(CursorLocation, CursorShift);
				}
				
				if (Index == SelectionStart.GetLineIndex())
				{
					NewSelectionStart = FTextLocation(SelectionStart, CursorShift);
				}
			}
			
			Text->SelectText(NewSelectionStart, NewCursorLocation);
		}
		else
		{
			FString Line;
			Text->GetCurrentTextLine(Line);

			const int32 Offset = CursorLocation.GetOffset();

			// Tab to nearest 4.
			if (ensure(bShouldIncreaseIndentation))
			{
				const int32 NumSpacesForIndentation = 4 - Offset % 4;
				Text->InsertTextAtCursor(FString::ChrN(NumSpacesForIndentation, TEXT(' ')));
			}
		}

		return FReply::Handled();
	}
	else if (Character == TEXT('\n') || Character == TEXT('\r'))
	{
		SMultiLineEditableText::FScopedEditableTextTransaction Transaction(Text);
		
		// at this point, the text after the text cursor is already in a new line
		HandleAutoIndent();
		
		return FReply::Handled();
	}
	else if (IsOpenBrace(Character))
	{
		const TCHAR CloseBrace = GetMatchedCloseBrace(Character);
		const FTextLocation CursorLocation = Text->GetCursorLocation();
		FString Line;
		Text->GetCurrentTextLine(Line);
		
		bool bShouldAutoInsertBraces = false;

		if (CursorLocation.GetOffset() < Line.Len())
		{
			const TCHAR NextChar = Text->GetCharacterAt(CursorLocation);

			if (IsWhiteSpace(NextChar))
			{
				bShouldAutoInsertBraces = true;
			}
			else if (IsCloseBrace(NextChar))
			{
				int32 BraceBalancePrior = 0;
				for (int32 Index = 0; Index < CursorLocation.GetOffset() && Index < Line.Len(); Index++)
				{
					BraceBalancePrior += (Line[Index] == Character);
					BraceBalancePrior -= (Line[Index] == CloseBrace);
				}
				
				int32 BraceBalanceLater = 0;
				for (int32 Index = CursorLocation.GetOffset(); Index < Line.Len(); Index++)
				{
					BraceBalanceLater += (Line[Index] == Character);
					BraceBalanceLater -= (Line[Index] == CloseBrace);
				}

				if (BraceBalancePrior >= -BraceBalanceLater)
				{
					bShouldAutoInsertBraces = true;
				}
			}
		}
		else
		{
			bShouldAutoInsertBraces = true;
		}
		
		// auto insert if we have more open braces
		// on the left side than close braces on the right side
		if (bShouldAutoInsertBraces)
		{
			// auto insert the matched close brace
			SMultiLineEditableText::FScopedEditableTextTransaction Transaction(Text);
			Text->InsertTextAtCursor(FString::Chr(Character));
			Text->InsertTextAtCursor(FString::Chr(CloseBrace));
			const FTextLocation NewCursorLocation(Text->GetCursorLocation(), -1);
			Text->GoTo(NewCursorLocation);
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}
	else if (IsCloseBrace(Character))
	{
		if (!Text->AnyTextSelected())
		{
			const FTextLocation CursorLocation = Text->GetCursorLocation();	
			FString Line;
			Text->GetTextLine(CursorLocation.GetLineIndex(), Line);
			
			const int32 Offset =CursorLocation.GetOffset();
			
			if (Line.IsValidIndex(Offset))
			{
				if (Line[Offset] == Character)
				{
					// avoid creating a duplicated close brace and simply
					// advance the cursor
					Text->GoTo(FTextLocation(CursorLocation, 1));
					return FReply::Handled();
				}
			}
		}
		
		return FReply::Unhandled();
	}
	else
	{
		// Let SMultiLineEditableText::OnKeyChar handle it.
		return FReply::Unhandled();
	}
}

void SPCGCodeEditorTextBox::HandleAutoIndent() const
{
	const FTextLocation CursorLocation = Text->GetCursorLocation();
	const int32 CurLineIndex = CursorLocation.GetLineIndex();
	const int32 LastLineIndex = CurLineIndex - 1;

	if (LastLineIndex > 0)
	{
		FString LastLine;
		Text->GetTextLine(LastLineIndex, LastLine);

		const int32 NumSpaces = GetNumSpacesAtStartOfLine(LastLine);

		const int32 NumSpacesForCurrentIndentation = NumSpaces/4*4;
		const FString CurrentIndentation = FString::ChrN(NumSpacesForCurrentIndentation, TEXT(' '));
		const FString NextIndentation = FString::ChrN(NumSpacesForCurrentIndentation + 4, TEXT(' '));
	
		// See what the open/close curly brace balance is.
		int32 BraceBalance = 0;
		for (const TCHAR Char : LastLine)
		{
			BraceBalance += (Char == TEXT('{'));
			BraceBalance -= (Char == TEXT('}'));
		}

		if (BraceBalance <= 0)
		{
			Text->InsertTextAtCursor(CurrentIndentation);
		}
		else
		{
			Text->InsertTextAtCursor(NextIndentation);
		
			// Look for an extra close curly brace and auto-indent it as well
			FString CurLine;
			Text->GetTextLine(CurLineIndex, CurLine);

			BraceBalance = 0;
			int32 CloseBraceOffset = 0;
			for (const TCHAR Char : CurLine)
			{
				BraceBalance += (Char == TEXT('{'));
				BraceBalance -= (Char == TEXT('}'));

				// Found the first extra '}'
				if (BraceBalance < 0)
				{
					break;
				}

				CloseBraceOffset++;
			}

			if (BraceBalance < 0)
			{
				const FTextLocation SavedCursorLocation = Text->GetCursorLocation();
				const FTextLocation CloseBraceLocation(CurLineIndex, CloseBraceOffset);
			
				// Create a new line and apply indentation for the close curly brace
				FString NewLineAndIndent(TEXT("\n"));
				NewLineAndIndent.Append(CurrentIndentation);
			
				Text->GoTo(CloseBraceLocation);
				Text->InsertTextAtCursor(NewLineAndIndent);
				// Recover cursor location
				Text->GoTo(SavedCursorLocation);
			}
		}
	}
}

bool SPCGCodeEditorTextBox::IsReadOnly() const
{
	if (IPCGCodeEditorTextProvider* Provider = Cast<IPCGCodeEditorTextProvider>(ShaderTextProviderObject))
	{
		return Provider->IsReadOnly();
	}

	return true;
}

FSlateColor SPCGCodeEditorTextBox::GetBorderBackgroundColor() const
{
	// Hand tweaked values.
	return IsReadOnly() ? FLinearColor(0.72f, 0.72f, 0.72f, 1.0f) : FLinearColor(0.4f, 0.4f, 0.4f, 1.0f);
}

#undef LOCTEXT_NAMESPACE
