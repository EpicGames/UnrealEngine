// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Framework/Commands/Commands.h"

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Input/SSearchBox.h"

class ITextLayoutMarshaller;
class SMultiLineEditableText;
class SPCGShaderTextSearchWidget;
class SVerticalBox;

class FPCGCodeEditorTextBoxCommands :
	public TCommands<FPCGCodeEditorTextBoxCommands>
{
public:
	FPCGCodeEditorTextBoxCommands();
	
	// TCommands<> overrides
	virtual void RegisterCommands() override;
	
	TSharedPtr<FUICommandInfo> Search;
	TSharedPtr<FUICommandInfo> NextOccurrence;
	TSharedPtr<FUICommandInfo> PreviousOccurrence;
	
	TSharedPtr<FUICommandInfo> ApplyChanges;
	TSharedPtr<FUICommandInfo> ToggleComment;
};

/** Text box for editing code. */
class SPCGCodeEditorTextBox : public SCompoundWidget
{
public:
	SPCGCodeEditorTextBox();
	virtual ~SPCGCodeEditorTextBox() override;

	DECLARE_DELEGATE(FOnTextChangesApplied);

	SLATE_BEGIN_ARGS(SPCGCodeEditorTextBox) {};
		/** The initial text that will appear in the widget. */
		SLATE_ATTRIBUTE(FText, Text)

		/** Text to search for (a new search is triggered whenever this text changes) */
		SLATE_ATTRIBUTE(FText, SearchText)

		/** The marshaller used to get/set the raw text to/from the text layout. */
		SLATE_ARGUMENT(TSharedPtr<ITextLayoutMarshaller>, Marshaller)

		/** Sets whether this text box can actually be modified interactively by the user */
		SLATE_ATTRIBUTE(bool, IsReadOnly)

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT(FOnTextChanged, OnTextChanged)

		/** Called whenever the text is committed. This happens when the text box loses focus. */
		SLATE_EVENT(FOnTextCommitted, OnTextCommitted)

		/** Called when user applies the changes */
		SLATE_EVENT(FOnTextChangesApplied, OnTextChangesApplied)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void Refresh() const;

	void SetTextProviderObject(UObject* InProviderObject);

private:
	//~ Begin SWidget interface
	// Use Preview Key Down so that the commands related to this tab is given the priority
	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget interface

	void RegisterCommands();
	
	bool HandleEscape();
	
	void ShowSearchBar();
	bool HideSearchBar();
	
	void OnTriggerSearch();
	void OnSearchTextChanged(const FText& InTextToSearch);
	void OnSearchTextCommitted(const FText& InTextToSearch, ETextCommit::Type InCommitType);
	TOptional<SSearchBox::FSearchResultData> GetSearchResultData() const;
	void OnSearchResultNavigationButtonClicked(SSearchBox::SearchDirection InDirection);
	void OnGoToNextOccurrence();
	void OnGoToPreviousOccurrence();

	FReply OnTextKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InCharacterEvent) const;

	void OnApplyChanges() const;

	void OnToggleComment() const;
	
	FReply OnTextKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent) const;
	
	// Figure out if we need to auto-indent.
	void HandleAutoIndent() const;

	bool IsReadOnly() const;
	FSlateColor GetBorderBackgroundColor() const;

private:
	TSharedPtr<SVerticalBox> TabBody;
	
	TSharedPtr<SMultiLineEditableText> Text;

	TSharedPtr<SPCGShaderTextSearchWidget> SearchBar;
	bool bIsSearchBarHidden = true;

	// Top level command list is processed during preview key down
	// such that its commands are given the priority
	TSharedRef<FUICommandList> TopLevelCommandList;
	
	TSharedRef<FUICommandList> TextCommandList;
	
	FOnTextChangesApplied OnTextChangesAppliedCallback;

	TWeakObjectPtr<UObject> ShaderTextProviderObject;
};
