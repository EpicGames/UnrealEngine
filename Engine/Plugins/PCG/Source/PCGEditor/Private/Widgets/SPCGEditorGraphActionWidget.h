// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Schema/PCGEditorGraphSchemaActions.h"

#include "PCGEditorStyle.h"

#include "SGraphActionMenu.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "EdGraph/EdGraphSchema.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SLayeredImage.h"

class SPCGGraphActionWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPCGGraphActionWidget) {}
		SLATE_ARGUMENT(TSharedPtr<SWidget>, NameWidget)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FCreateWidgetForActionData* const InCreateData)
	{
		check(InCreateData->Action.IsValid());

		TSharedRef<SWidget> IconWidget = SNullWidget::NullWidget;
		ActionPtr = InCreateData->Action;
		MouseButtonDownDelegate = InCreateData->MouseButtonDownDelegate;

		if (const FSlateBrush* const PrimaryIcon = InCreateData->Action->GetPaletteIcon())
		{
			// @todo_pcg: Support secondary icon/colors for more complex type icons (like Maps)
			const FSlateBrush* const SecondaryIcon = nullptr;
			const FLinearColor SecondaryColor = FLinearColor::White;

			SAssignNew(IconWidget, SLayeredImage, SecondaryIcon, SecondaryColor)
			.DesiredSizeOverride(FVector2D(16))
			.Image(PrimaryIcon);
		}

		const TSharedPtr<FPCGEditorGraphSchemaActionBase> PCGAction = StaticCastSharedPtr<FPCGEditorGraphSchemaActionBase>(InCreateData->Action);
		TSharedRef<SWidget> NameSlotWidget =
			InArgs._NameWidget
				? InArgs._NameWidget.ToSharedRef()
				: SNew(STextBlock)
				.Text(PCGAction->GetMenuDescriptionOverride());

		// Show a "GPU compatible" badge for nodes whose UPCGSettings has a GPU execution backend.
		TSharedRef<SWidget> BadgeWidget = SNullWidget::NullWidget;
		if (InCreateData->Action->GetTypeId() == FPCGEditorGraphSchemaAction_NewNativeElement::StaticGetTypeId())
		{
			const FPCGEditorGraphSchemaAction_NewNativeElement* NativeAction = static_cast<const FPCGEditorGraphSchemaAction_NewNativeElement*>(InCreateData->Action.Get());
			if (NativeAction->SettingsClass && NativeAction->SettingsClass.GetDefaultObject()->IsGPUFriendly(&NativeAction->PreconfiguredInfo))
			{
				const FText TooltipText = NSLOCTEXT("SPCGGraphActionWidget", "GPUCompatibleTooltip", "This node is GPU friendly; it either executes on the GPU or it executes on the CPU but does not force data uploads or readbacks.");
				// Hand tweaked to be close to the badge placed on nodes while also not being too prominent next to node names.
				static const FSlateRoundedBoxBrush BadgeBorderBrush(FLinearColor::Transparent, /*Radius=*/7.0f, FLinearColor(0.45f, 0.45f, 0.45f, 0.5f), /*Stroke=*/1.0f);
				static const FLinearColor BadgeTextColor(0.45f, 0.45f, 0.45f, 0.8f);

				BadgeWidget = SNew(SBorder)
					.BorderImage(&BadgeBorderBrush)
					.Padding(FMargin(4, 1))
					.ToolTipText(TooltipText)
					[
						SNew(STextBlock)
						.TextStyle(FPCGEditorStyle::Get(), "PCG.Node.AdditionalOverlayWidgetText")
						.Text(NSLOCTEXT("SPCGGraphActionWidget", "GPULabel", "GPU"))
						.Justification(ETextJustify::Center)
						.ColorAndOpacity(BadgeTextColor)
					];
			}
		}

		this->ChildSlot
		[
			SNew(SHorizontalBox)
			.ToolTipText(InCreateData->Action->GetTooltipDescription())
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				IconWidget
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4, 0, 0, 0)
			[
				MoveTemp(NameSlotWidget)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(8, 0, 4, 0)
			[
				MoveTemp(BadgeWidget)
			]
		];
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseButtonDownDelegate.Execute(ActionPtr))
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	/** The action that we want to display with this widget */
	TWeakPtr<FEdGraphSchemaAction> ActionPtr;
	/** Delegate executed when mouse button goes down */
	FCreateWidgetMouseButtonDown MouseButtonDownDelegate;
};
