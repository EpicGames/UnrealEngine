// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphAttributeListView.h"

#include "PCGAssetExporterUtils.h"
#include "PCGComponent.h"
#include "PCGData.h"
#include "PCGDataVisualization.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSubsystem.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Data/PCGTextureData.h"
#include "Elements/IO/PCGSaveAssetElement.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "PCGEditorSettings.h"
#include "PCGEditorStyle.h"
#include "PCGEditorUtils.h"
#include "SPCGMetadataAttributeGenericViewer.h"
#include "DataVisualizations/PCGDataVisualizationHelpers.h"
#include "Managers/PCGEditorInspectionDataManager.h"
#include "Nodes/PCGEditorGraphNodeBase.h"
#include "Widgets/AssetEditorViewport/SPCGEditorViewport.h"

#include "Engine/StreamableManager.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "SPCGEditorGraphAttributeListView"

namespace PCGEditorGraphAttributeListView
{
	const FText NoPinAvailableText = LOCTEXT("NoPinAvailableText", "No pins");
	const FText CrcLabelFormat = LOCTEXT("InfoTextBlockWithCrcFmt", "{0} | CRC: {1}");
	const FText LastLabelFormat = LOCTEXT("InfoTextBlockWithLastAttributeFmt", "{0} | Last attribute: {1}");
	const FText NoDataAvailableText = LOCTEXT("NoDataAvailableText", "No data available");
	const FText NoNodeInspectedText = LOCTEXT("NoNodeInspectedText", "No node being inspected");
	const FText NoNodeInspectedToolTip = LOCTEXT("NoNodeInspectedToolTip", "Inspect a node using the right click menu");
	const FText TEXT_IndexLabel = LOCTEXT("IndexLabel", "Index");

	bool IsGraphCacheDebuggingEnabled()
	{
		UWorld* World = GEditor ? (GEditor->PlayWorld ? GEditor->PlayWorld.Get() : GEditor->GetEditorWorldContext().World()) : nullptr;
		UPCGSubsystem* Subsystem = UPCGSubsystem::GetInstance(World);
		return Subsystem && Subsystem->IsGraphCacheDebuggingEnabled();
	}

	float CalculateColumnWidth(const FText& InText, bool bClampToMaxColumnWidth = true)
	{
		check(FSlateApplication::Get().GetRenderer());
		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FSlateFontInfo FontInfo = FAppStyle::GetFontStyle(TEXT("NormalText"));

		const float TextWidth = FontMeasure->Measure(InText, FontInfo).X;
		constexpr float ColumnPadding = 22.0f; // TODO: Grab padding from header style
		const float ColumnWidth = TextWidth + ColumnPadding;
		return bClampToMaxColumnWidth ? FMath::Min(ColumnWidth, MaxColumnWidth) : ColumnWidth;
	}
	
	struct FPCGListViewItemRowTextInfo
	{
		FText RowText = LOCTEXT("ColumnError", "Unrecognized Column");

		// We will only compute cell width for types are that not numbers.
		bool bShouldComputeCellWidth = false;

		// Make the text clickable for soft object paths
		bool bIsClickable = false;

		ETextOverflowPolicy OverflowPolicy = GetDefault<UPCGEditorSettings>()->AttributeListViewSettings.GeneralOverflowPolicy;
		
		bool bShowText = false;
	};
	
	FPCGListViewItemRowTextInfo ExtractText(FPCGColumnData* PCGColumnData, int32 Index, const FName& ColumnId)
	{
		FPCGListViewItemRowTextInfo Result{};
		
		if (Index == -1 && !PCGColumnData->DataAccessor->IsAttribute())
		{
			Result.RowText = ColumnId == PCGDataVisualizationConstants::NAME_Index ? LOCTEXT("DefaultIndex", "Default") : FText{};
			return Result;
		}
		
		const FPCGMetadataAttributeDesc& Desc = PCGColumnData->DataAccessor->GetUnderlyingDesc();
		
		// If the descriptor is an array, print the first few values.
		if (Desc.IsArray())
		{
			// Always show some text for the array to know that it is an array.
			Result.bShowText = true;

			auto CallbackArray = [&PCGColumnData, Index, &Result]<typename ValueType>(ValueType)
			{
				PCG::TPCGArrayAccessorWrapper<ValueType> Value{};

				static const FPCGAttributeAccessorKeysEntries DefaultKeys(PCGInvalidEntryKey);
				const IPCGAttributeAccessorKeys& DataKeys = (Index == -1 && PCGColumnData->DataAccessor->IsAttribute()) ? DefaultKeys : *PCGColumnData->DataKeys;

				if (!PCGColumnData->DataAccessor->Get(Value, Index == -1 ? 0 : Index, DataKeys))
				{
					return false;
				}

				if (PCG::Private::IsOfTypes<ValueType, FString, FName, FSoftClassPath, FSoftObjectPath>())
				{
					Result.bShouldComputeCellWidth = true;
				}
				
				Result.OverflowPolicy = GetDefault<UPCGEditorSettings>()->AttributeListViewSettings.StringOverflowPolicy;

				constexpr int32 MaxElements = 3;
				const int32 NumElements = Value.GetView().Num();
				
				auto Projection = [](const ValueType& Item)
				{
					// To be consistent with FText::AsNumber
					if constexpr (FTextAsNumberIsValid<ValueType>::value && !std::is_same_v<ValueType, bool>)
					{
						static const FDecimalNumberFormattingRules DefaultFormattingRules{};
						static const FNumberFormattingOptions DefaultFormattingOptions{};
						return FastDecimalFormat::NumberToString(Item, DefaultFormattingRules, DefaultFormattingOptions);
					}
					else
					{
						return PCG::Private::MetadataTraits<ValueType>::ToString(Item);
					}
				};
				
				TStringBuilder<256> Builder;
				Builder += TEXT("{ ");
				Builder += FString::JoinBy(MakeConstArrayView(Value.GetView().GetData(), FMath::Min(MaxElements, NumElements)), TEXT(", "), Projection);
				if (NumElements > MaxElements)
				{
					Builder += TEXT(", ...");
				}
				Builder += TEXT(" }");

				Result.RowText = FText::FromStringView(Builder.ToView());
				return true;
			};
			
			if (!PCGMetadataAttribute::CallbackWithRightType(static_cast<uint16>(Desc.ValueType), CallbackArray))
			{
				Result.RowText = INVTEXT("{ (...) }");
			}

			return Result;
		}
		
		auto Callback = [&PCGColumnData, Index, &Result]<typename ValueType>(ValueType)
		{
			ValueType Value = PCG::Private::MetadataTraits<ValueType>::ZeroValue();

			static const FPCGAttributeAccessorKeysEntries DefaultKeys(PCGInvalidEntryKey);
			const IPCGAttributeAccessorKeys& DataKeys = (Index == -1 && PCGColumnData->DataAccessor->IsAttribute()) ? DefaultKeys : *PCGColumnData->DataKeys;

			if (PCGColumnData->DataAccessor->Get<ValueType>(Value, Index == -1 ? 0 : Index, DataKeys))
			{
				Result.bShowText = true;

				if constexpr (PCG::Private::IsOfTypes<ValueType, bool>())
				{
					Result.RowText = FText::FromString(LexToString(Value));
				}
				else if constexpr (PCG::Private::IsOfTypes<ValueType, FString>())
				{
					Result.RowText = FText::FromString(Value);
					Result.bShouldComputeCellWidth = true;
					Result.OverflowPolicy = GetDefault<UPCGEditorSettings>()->AttributeListViewSettings.StringOverflowPolicy;
				}
				else if constexpr (PCG::Private::IsOfTypes<ValueType, FName>())
				{
					Result.RowText = FText::FromName(Value);
					Result.bShouldComputeCellWidth = true;
					Result.OverflowPolicy = GetDefault<UPCGEditorSettings>()->AttributeListViewSettings.NameOverflowPolicy;
				}
				else if constexpr (FTextAsNumberIsValid<ValueType>::value)
				{
					Result.RowText = FText::AsNumber(Value);
				}
				else if constexpr (PCG::Private::IsOfTypes<ValueType, FSoftObjectPath, FSoftClassPath>())
				{
					Result.RowText = FText::FromString(Value.ToString());
					Result.bShouldComputeCellWidth = true;
					Result.bIsClickable = true;
					Result.OverflowPolicy = GetDefault<UPCGEditorSettings>()->AttributeListViewSettings.HyperLinkOverflowPolicy;
				}
				else
				{
					ensureMsgf(false, TEXT("Unsupported Data Type"));
					Result.RowText = LOCTEXT("UnsupportedDataTypeError", "Unsupported Data Type");
				}
			}
		};

		PCGMetadataAttribute::CallbackWithRightType(PCGColumnData->DataAccessor->GetUnderlyingType(), Callback);
		
		return Result;
	}
}

bool FPCGListViewUpdater::IsCompleted() const
{
	return UpdateTask.IsCompleted();
}

void FPCGListViewUpdater::Launch()
{
	// Passing a shared pointer to this in order for the task to keep the object alive even if we discard it in the attribute list view
	UpdateTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [SharedContext = SharedThis(this)]()
	{
		SharedContext->AsyncFilter();
		SharedContext->AsyncSort();
	});
}

void FPCGListViewUpdater::AsyncSort()
{
	if (const FPCGColumnData* Data = ColumnData.Find(SortingColumn))
	{
		if (Data->DataAccessor.IsValid() && Data->DataKeys.IsValid() && Data->DataKeys->GetNum() == ListViewItems.Num())
		{
			//lambda used here to get the index value of an item in the array for sorting
			PCGAttributeAccessorHelpers::SortByAttribute(*Data->DataAccessor, *Data->DataKeys, ListViewItems, !(SortMode & EColumnSortMode::Descending), [this](int Index) { return ListViewItems[Index]->Index; });
		}
	}
}

void FPCGListViewUpdater::AsyncFilter()
{
	TArray<PCGListViewItemPtr> FilteredListViewItems;
	FilteredListViewItems.Reserve(ListViewItems.Num());

	for (const PCGListViewItemPtr& ListViewItem : ListViewItems)
	{
		const FPCGPointFilterExpressionContext PointFilterContext(ListViewItem.Get(), &ColumnData);
		if (TextFilter->TestTextFilter(PointFilterContext))
		{
			FilteredListViewItems.Add(ListViewItem);
		}
	}

	ListViewItems = MoveTemp(FilteredListViewItems);
}

void SPCGListViewItemRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	check(InArgs._ListViewItem && InArgs._AttributeListView);
	InternalItem = InArgs._ListViewItem;
	AttributeListView = InArgs._AttributeListView;

	SMultiColumnTableRow<PCGListViewItemPtr>::Construct(
		SMultiColumnTableRow::FArguments()
		.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
		InOwnerTableView);
}

TSharedRef<SWidget> SPCGListViewItemRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	PCGEditorGraphAttributeListView::FPCGListViewItemRowTextInfo RowTextInfo{};

	const TSharedPtr<SPCGEditorGraphAttributeListView> SharedAttributeListView = AttributeListView.Pin();
	check(SharedAttributeListView.IsValid());
	
	TOptional<TTuple<const FPCGMetadataAttributeBase*, PCGMetadataEntryKey>> AttrAndKey{};
	
	const int32 Index = InternalItem->Index;

	if (FPCGColumnData* PCGColumnData = SharedAttributeListView->PCGColumnData.Find(ColumnId))
	{
		if (PCGColumnData->DataKeys.IsValid() && PCGColumnData->UnderlyingAttribute)
		{
			const PCGMetadataEntryKey* Key = nullptr;
			if (PCGColumnData->DataKeys->GetKey<PCGMetadataEntryKey>(Index, Key))
			{
				AttrAndKey = MakeTuple(PCGColumnData->UnderlyingAttribute, *Key);
			}
		}

		// We have to make sure the data is still valid otherwise this will crash.
		if (ensure(SharedAttributeListView->DataPtr) && PCGColumnData->DataAccessor.IsValid() && PCGColumnData->DataKeys.IsValid())
		{
			RowTextInfo = PCGEditorGraphAttributeListView::ExtractText(PCGColumnData, Index, ColumnId);
		}
	}

	if (RowTextInfo.bShouldComputeCellWidth)
	{
		float& CurrentColumnWidth = SharedAttributeListView->ColumnsMaxWidthMapping[ColumnId];
		const float TextWidth = PCGEditorGraphAttributeListView::CalculateColumnWidth(RowTextInfo.RowText, /*bClampToMaxColumnWidth=*/false);
		if (TextWidth > CurrentColumnWidth)
		{
			CurrentColumnWidth = TextWidth;
		}
	}

	const FMargin Margin = FMargin(2.0f, 0.0f);

	// TextStyles are pointers so they need to stay alive. Create 2 static copies for each of our style: Normal and Italic.
	static const FTextBlockStyle NormalTextStyle = FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	static const FTextBlockStyle ItalicTextStyle = FTextBlockStyle(NormalTextStyle)
		.SetFont(FCoreStyle::Get().GetFontStyle("NormalFontItalic"));

	// Set the default line (-1) in Italic, Normal for the rest.
	const FTextBlockStyle* TextStyle = InternalItem->Index == -1 ? &ItalicTextStyle : &NormalTextStyle;

	if (RowTextInfo.bIsClickable)
	{
		TSharedPtr<SHyperlink> Hyperlink;
		SAssignNew(Hyperlink, SHyperlink)
			.Text(RowTextInfo.RowText)
			.ToolTipText(LOCTEXT("Hyperlink", "Ctrl + Click to jump to the asset."))
			.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
			.TextStyle(TextStyle)
			.Padding(Margin)
			.OverflowPolicy(RowTextInfo.OverflowPolicy)
			.OnNavigate(FSimpleDelegate::CreateLambda([Text = MoveTemp(RowTextInfo.RowText)](){ SPCGListViewItemRow::OnSoftObjectPathHyperlinkClicked(Text); }));

		return Hyperlink.ToSharedRef();
	}
	else if (!AttrAndKey.IsSet())
	{
		return SNew(STextBlock)
			.Text(MoveTemp(RowTextInfo.RowText))
			.OverflowPolicy(RowTextInfo.OverflowPolicy)
			.Margin(Margin)
			.TextStyle(TextStyle);
	}
	else if (AttrAndKey.IsSet() && AttrAndKey->Get<0>() && SharedAttributeListView->DataPtr)
	{
		auto CreateDetails = [AttrAndKey, WeakData = TWeakObjectPtr<const UPCGData>(SharedAttributeListView->DataPtr)]()
		{
			TSharedRef<SWindow> ModalWindow = SNew(SWindow)
				.Title(INVTEXT("Viewer"))
				.ClientSize(FVector2D(700, 700));

			TSharedRef<SPCGMetadataAttributeGenericViewer> Viewer = SNew(SPCGMetadataAttributeGenericViewer);
			Viewer->Setup(WeakData, AttrAndKey->Get<0>(), AttrAndKey->Get<1>());

			ModalWindow->SetContent(Viewer);

			GEditor->EditorAddModalWindow(ModalWindow);
		};

		TSharedPtr<SHyperlink> Hyperlink;
		SAssignNew(Hyperlink, SHyperlink)
			.Text(RowTextInfo.bShowText ? RowTextInfo.RowText : INVTEXT("(...)"))
			.ToolTipText(LOCTEXT("HyperlinkGeneric", "Click to open a details view for this complex value."))
			.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
			.TextStyle(TextStyle)
			.Padding(Margin)
			.OverflowPolicy(RowTextInfo.OverflowPolicy)
			.OnNavigate(FSimpleDelegate::CreateLambda(CreateDetails));

		return Hyperlink.ToSharedRef();
	}
	else
	{
		return SNew(STextBlock)
			.Text(INVTEXT(""))
			.OverflowPolicy(RowTextInfo.OverflowPolicy)
			.Margin(Margin)
			.TextStyle(TextStyle);
	}
}

void SPCGListViewItemRow::OnSoftObjectPathHyperlinkClicked(const FText& InText)
{
	if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
	{
		PCGEditorUtils::OpenAssetOrMoveToActorOrComponent(FSoftObjectPath(InText.ToString()));
	}
}

FPCGPointFilterExpressionContext::FPCGPointFilterExpressionContext(const FPCGListViewItem* InRowItem, const TMap<FName, FPCGColumnData>* InPCGColumnData)
	: RowItem(InRowItem)
	, PCGColumnData(InPCGColumnData)
{
	check(InRowItem);
}

bool FPCGPointFilterExpressionContext::TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	// Basic string search is disabled as it would require us to search the entire attribute table at once and it's not very useful.
	return false;
}

bool FPCGPointFilterExpressionContext::TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	const int32 Index = RowItem->Index;

	if (PCGEditorGraphAttributeListView::TEXT_IndexLabel.EqualToCaseIgnored(FText::FromName(InKey)))
	{
		const FTextFilterString PointValue(FString::FromInt(Index));
		return TextFilterUtils::TestComplexExpression(PointValue, InValue, InComparisonOperation, InTextComparisonMode);
	}
	else if(const FPCGColumnData* PCGColumnInfo = PCGColumnData->Find(InKey))
	{
		if (PCGColumnInfo->DataAccessor.IsValid() && PCGColumnInfo->DataKeys.IsValid())
		{
			auto Callback = [&PCGColumnInfo, Index, &InValue, &InComparisonOperation, &InTextComparisonMode] (auto Dummy)
			{
				using ValueType = decltype(Dummy);
				ValueType Value{};
				if (PCGColumnInfo->DataAccessor->Get<ValueType>(Value, Index, *PCGColumnInfo->DataKeys))
				{
					FText TextValue;
					bool bInvalid = false;
					if constexpr (PCG::Private::IsOfTypes<ValueType, bool>())
					{
						TextValue = FText::FromString(LexToString(Value));
					}
					else if constexpr (PCG::Private::IsOfTypes<ValueType, FString>())
					{
						TextValue = FText::FromString(Value);
					}
					else if constexpr (PCG::Private::IsOfTypes<ValueType, FName>())
					{
						TextValue = FText::FromName(Value);
					}
					else if constexpr (FTextAsNumberIsValid<ValueType>::value)
					{
						TextValue = FText::AsNumber(Value, &FNumberFormattingOptions::DefaultNoGrouping());
					}
					else
					{
						ensureMsgf(false, TEXT("Unsupported Data Type"));
						bInvalid = true;
					}

					if (!bInvalid)
					{
						const FTextFilterString PointValue(TextValue.ToString());
						return TextFilterUtils::TestComplexExpression(PointValue, InValue, InComparisonOperation, InTextComparisonMode);
					}
				}

				return false;
			};

			return PCGMetadataAttribute::CallbackWithRightType(PCGColumnInfo->DataAccessor->GetUnderlyingType(), Callback);
		}
	}

	return true;
}

SPCGEditorGraphAttributeListView::~SPCGEditorGraphAttributeListView()
{
	if (PCGEditorPtr.IsValid())
	{
		PCGEditorPtr.Pin()->GetInspectionDataManager().OnInspectedStackChangedDelegate.RemoveAll(this);
	}

	DataPtr = nullptr;
}

void SPCGEditorGraphAttributeListView::Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor)
{
	WidgetEntryNumber = InArgs._WidgetEntryNumber;

	check(InPCGEditor.IsValid());
	
	PCGEditorPtr = InPCGEditor;
	PCGEditorPtr.Pin()->GetInspectionDataManager().OnInspectedStackChangedDelegate.AddSP(this, &SPCGEditorGraphAttributeListView::OnInspectedStackChanged);

	TextFilter = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex));

	ColumnWidthVisibilityCache.Empty(PCGEditorGraphAttributeListView::MaxNodeColumnWidthCachedItems);

	ListViewHeader = CreateHeaderRowWidget();

	ListViewCommands = MakeShareable(new FUICommandList);
	ListViewCommands->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::CopySelectionToClipboard),
		FCanExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::CanCopySelectionToClipboard));

	const TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal)
		.Thickness(FVector2D(12.0f, 12.0f));

	const TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(12.0f, 12.0f));

	auto VisibilityTest = [this]()
	{
		return (ListViewItems.IsEmpty() && ListViewHeader->GetColumns().IsEmpty()) ? EVisibility::Hidden : EVisibility::Visible;
	};

	SAssignNew(ListView, SListView<PCGListViewItemPtr>)
		.ListItemsSource(&ListViewItems)
		.HeaderRow(ListViewHeader)
		.OnGenerateRow(this, &SPCGEditorGraphAttributeListView::OnGenerateRow)
		.OnMouseButtonDoubleClick(this, &SPCGEditorGraphAttributeListView::OnItemDoubleClicked)
		.OnContextMenuOpening(this, &SPCGEditorGraphAttributeListView::OnItemsContextMenu)
		.AllowOverscroll(EAllowOverscroll::No)
		.ExternalScrollbar(VerticalScrollBar)
		.Visibility_Lambda(VisibilityTest)
		.OnKeyDownHandler(this, &SPCGEditorGraphAttributeListView::OnListViewKeyDown)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always);

	SAssignNew(PinComboBox, SComboBox<TSharedPtr<FPinComboBoxItem>>)
		.OptionsSource(&PinComboBoxItems)
		.OnGenerateWidget(this, &SPCGEditorGraphAttributeListView::OnGeneratePinWidget)
		.OnSelectionChanged(this, &SPCGEditorGraphAttributeListView::OnSelectionChangedPin)
		[
			SNew(STextBlock)
			.Text(this, &SPCGEditorGraphAttributeListView::OnGenerateSelectedPinText)
		];

	SAssignNew(DataComboBox, SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&DataComboBoxItems)
		.OnGenerateWidget(this, &SPCGEditorGraphAttributeListView::OnGenerateDataWidget)
		.OnSelectionChanged(this, &SPCGEditorGraphAttributeListView::OnSelectionChanged)
		[
			SNew(STextBlock)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.Text(this, &SPCGEditorGraphAttributeListView::OnGenerateSelectedDataText)
		];

	SAssignNew(DomainsComboBox, SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&DomainsComboBoxItems)
		.OnGenerateWidget(this, &SPCGEditorGraphAttributeListView::OnGenerateDataWidget)
		.OnSelectionChanged(this, &SPCGEditorGraphAttributeListView::OnSelectionChanged)
		[
			SNew(STextBlock)
			.Text(this, &SPCGEditorGraphAttributeListView::OnGenerateSelectedDomainText)
		];

	TSharedPtr<SLayeredImage> FilterImage = SNew(SLayeredImage)
		.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
		.ColorAndOpacity(FSlateColor::UseForeground());

	FilterImage->AddLayer(TAttribute<const FSlateBrush*>(this, &SPCGEditorGraphAttributeListView::GetFilterBadgeIcon));

	TSharedPtr<SButton> LockButton = SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnClicked(this, &SPCGEditorGraphAttributeListView::OnLockClick)
		.ContentPadding(FMargin(4, 2))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ToolTipText(LOCTEXT("LockSelectionButton_ToolTip", "Locks the current attribute list view to this selection."))
		[
			SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(this, &SPCGEditorGraphAttributeListView::OnGetLockButtonImageResource)
		];

	TSharedPtr<SButton> FrameDataButton = SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnClicked(this, &SPCGEditorGraphAttributeListView::OnFocusOnDataClicked)
		.IsEnabled(this, &SPCGEditorGraphAttributeListView::IsFocusOnDataEnabled)
		.ContentPadding(FMargin(4, 2))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ToolTipText(LOCTEXT("FocusOnDataButton_ToolTip", "Zoom to selected data."))
		[
			SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FPCGEditorStyle::Get().GetBrush("PCG.Editor.ZoomToSelection"))
		];

	TSharedPtr<SButton> OpenViewportButton = SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnClicked_Lambda([this]() -> FReply
			{
				if (const TSharedPtr<const FPCGEditor> SharedPtr = PCGEditorPtr.Pin())
				{
					SharedPtr->BringFocusToPanel(ViewportEditorPanel);
					bViewportNeedsRefresh = true;
				}

				return FReply::Handled();
			})
		.Visibility_Lambda([this]()
		{
			const TSharedPtr<const FPCGEditor> SharedPtr = PCGEditorPtr.Pin();
			return (SharedPtr.IsValid() && SharedPtr->IsPanelCurrentlyOpen(ViewportEditorPanel)) ? EVisibility::Collapsed : EVisibility::Visible;
		})
		.ContentPadding(FMargin(4, 2))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ToolTipText(LOCTEXT("OpenDataViewport_ToolTip", "Opens Data Viewport Panel."))
		[
			SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("ClassIcon.CameraComponent"))
		];

	TSharedPtr<SComboButton> FilterButton = SNew(SComboButton)
		.ForegroundColor(FSlateColor::UseStyle())
		.HasDownArrow(false)
		.OnGetMenuContent(this, &SPCGEditorGraphAttributeListView::OnGenerateFilterMenu)
		.ContentPadding(1)
		.ButtonContent()
		[
			FilterImage.ToSharedRef()
		];

	TSharedPtr<SComboButton> AdditionalOperationsButton = SNew(SComboButton)
		.ForegroundColor(FSlateColor::UseStyle())
		.HasDownArrow(false)
		.OnGetMenuContent(this, &SPCGEditorGraphAttributeListView::OnGenerateAdditionalOperationsMenu)
		.ContentPadding(1)
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("EditorViewportToolBar.OptionsDropdown"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];

	SAssignNew(SearchBoxWidget, SSearchBox)
		.MinDesiredWidth(100.0f)
		.InitialText(ActiveFilterText)
		.OnTextChanged(this, &SPCGEditorGraphAttributeListView::OnFilterTextChanged)
		.OnTextCommitted(this, &SPCGEditorGraphAttributeListView::OnFilterTextCommitted)
		.DelayChangeNotificationsWhileTyping(true)
		.DelayChangeNotificationsWhileTypingSeconds(0.5f);

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(1.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.0f, 0.0f)
			[
				LockButton->AsShared()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.0f, 0.0f)
			[
				FrameDataButton->AsShared()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.0f, 0.0f)
			[
				OpenViewportButton->AsShared()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.0f, 0.0f)
			[
				FilterButton->AsShared()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.0f, 0.0f)
			[
				PinComboBox->AsShared()
			]
			+SHorizontalBox::Slot()
			.MinWidth(100.0)
			.FillWidth(1.0)
			.Padding(1.0f, 0.0f)
			[
				DataComboBox->AsShared()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.0f, 0.0f)
			[
				DomainsComboBox->AsShared()
			]
			+SHorizontalBox::Slot()
			.MinWidth(100.0)
			.MaxWidth(300.0)
			.Padding(1.0f, 0.0f)
			[
				SearchBoxWidget->AsShared()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SPCGEditorGraphAttributeListView::OnNodeNameClicked)
				[
					SAssignNew(NodeNameTextBlock, STextBlock)
					.Text(PCGEditorGraphAttributeListView::NoNodeInspectedText)
					.ToolTipText(PCGEditorGraphAttributeListView::NoNodeInspectedToolTip)
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(1.0f, 0.0f)
			[
				AdditionalOperationsButton->AsShared()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SAssignNew(InfoTextBlock, STextBlock)
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					SNew(SScrollBox)
					.Orientation(Orient_Horizontal)
					.ExternalScrollbar(HorizontalScrollBar)
					+SScrollBox::Slot()
					[
						ListView->AsShared()
					]
				]
				+SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SCircularThrobber)
					.Radius(12.0f)
					.Visibility_Lambda([this](){return CurrentUpdateTask.IsValid() && !CurrentUpdateTask->IsCompleted() ? EVisibility::Visible : EVisibility::Hidden; })
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				VerticalScrollBar
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				HorizontalScrollBar
			]
		]
	];
}

void SPCGEditorGraphAttributeListView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	OnTick();
}

void SPCGEditorGraphAttributeListView::OnTick()
{
	if (bNeedsRefresh)
	{
		bool bKeepSelection = !bPCGEditorGraphNodeChanged;
		bPCGEditorGraphNodeChanged = false;

		bool bSelectionChanged = false;
		RefreshPinComboBox(bKeepSelection, bSelectionChanged);

		// Read back any GPU data after the pin selection is set.
		if (!ReadbackGPUData())
		{
			return;
		}

		bKeepSelection &= !bSelectionChanged;

		RefreshDataComboBox(bKeepSelection || bKeepDataComboBoxSelection);
		RefreshAttributeList();

		bKeepDataComboBoxSelection = false;

		bNeedsRefresh = false;
	}

	if (bViewportNeedsRefresh)
	{
		bViewportNeedsRefresh = false;
		RefreshViewport();
	}

	if (CurrentUpdateTask.IsValid() && CurrentUpdateTask->IsCompleted())
	{
		// In case of default value, we stick the first line (default line) at the top.
		if (bShowDefaultValue && !ListViewItems.IsEmpty())
		{
			FilteredListViewItems = {ListViewItems[0]};
			FilteredListViewItems.Append(MoveTemp(CurrentUpdateTask->ListViewItems));
		}
		else
		{
			FilteredListViewItems = CurrentUpdateTask->ListViewItems;
		}
		
		if (ListView.IsValid())
		{
			ListView->SetItemsSource(&FilteredListViewItems);
			ListView->RequestListRefresh();

			// Don't take into account the default value in the number of entries.
			const int32 FilteredNum = bShowDefaultValue ? FMath::Max(FilteredListViewItems.Num() - 1, 0) : FilteredListViewItems.Num();
			const int32 NonFilteredNum = bShowDefaultValue ? FMath::Max(ListViewItems.Num() - 1, 0) : ListViewItems.Num();

			InfoTextBlock->SetText(FText::Format(LOCTEXT("InfoTextBlockFmt", "Showing {0}/{1} entries"), FilteredNum, NonFilteredNum));

			if (PCGEditorGraphAttributeListView::IsGraphCacheDebuggingEnabled())
			{
				// If cache debugging enabled, write CRC to help diagnose missed-dependency issues
				const FPCGDataCollection* InspectionData = GetInspectionData();
				const int32 DataIndex = GetSelectedDataIndex();
				const FPCGCrc Crc = (InspectionData && InspectionData->DataCrcs.IsValidIndex(DataIndex)) ? InspectionData->DataCrcs[DataIndex] : FPCGCrc(0);
				InfoTextBlock->SetText(FText::Format(PCGEditorGraphAttributeListView::CrcLabelFormat, InfoTextBlock->GetText(), Crc.GetValue()));
			}

			if (DataPtr && DataPtr->HasCachedLastSelector())
			{
				const FText LastSelector = DataPtr->GetCachedLastSelector().GetDisplayText();
				InfoTextBlock->SetText(FText::Format(PCGEditorGraphAttributeListView::LastLabelFormat, InfoTextBlock->GetText(), LastSelector));
			}
		}

		CurrentUpdateTask.Reset();
	}

	bTickForced = false;
}

bool SPCGEditorGraphAttributeListView::ReadbackGPUData() const
{
	bool bAllDataReady = true;

	if (const FPCGDataCollection* InspectionData = GetInspectionData())
	{
		for (int32 DataIndex = 0; DataIndex < InspectionData->TaggedData.Num(); ++DataIndex)
		{
			if (const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(InspectionData->TaggedData[DataIndex].Data))
			{
				// Poll all readbacks to progress them (don't early out if one is not ready).
				bAllDataReady &= Proxy->GetCPUData(/*InContext=*/nullptr).bComplete;
			}
			else if (const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(InspectionData->TaggedData[DataIndex].Data))
			{
				bAllDataReady &= TextureData->RequestCPUReadback();
			}
		}
	}

	return bAllDataReady;
}

void SPCGEditorGraphAttributeListView::RequestRefresh()
{
	bNeedsRefresh = true;
	// When we ask for refresh, we should also release the hold on the visualized data
	DataPtr = nullptr;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!IsOpen() && PCGEditorPtr.IsValid() && PCGEditorPtr.Pin()->bForceRefreshAttributeEvenIfClosed)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		RequestViewportRefresh();

		if (!bTickForced)
		{
			// Force tick on next game thread tick to ensure refresh happens.
			FTSTicker::GetCoreTicker().AddTicker(
				FTickerDelegate::CreateLambda([WeakThis=AsWeak()](float DeltaTime)
				{
					if (SPCGEditorGraphAttributeListView* This = static_cast<SPCGEditorGraphAttributeListView*>(WeakThis.Pin().Get()))
					{
						This->OnTick();
					}
					return false; // Run once
				}),
				/*InDelay=*/0.0f
			);

			bTickForced = true;
		}
	}
}

void SPCGEditorGraphAttributeListView::RequestViewportRefresh()
{
	bViewportNeedsRefresh = true;
	bRefreshLoadHandles = true;
}

TSharedRef<SHeaderRow> SPCGEditorGraphAttributeListView::CreateHeaderRowWidget() const
{
	return SNew(SHeaderRow);
}

TWeakInterfacePtr<IPCGGraphExecutionSource> SPCGEditorGraphAttributeListView::GetExecutionSource() const
{
	return PCGEditorPtr.IsValid() ? PCGEditorPtr.Pin()->GetPCGSourceBeingInspected() : nullptr;
}

void SPCGEditorGraphAttributeListView::OnInspectedStackChanged(const FPCGStack& InPCGStack)
{
	RequestRefresh();
}

UPCGEditorGraphNodeBase* SPCGEditorGraphAttributeListView::GetNodeBeingInspected() const
{
	return PCGEditorGraphNode.Get();
}

void SPCGEditorGraphAttributeListView::SetNodeBeingInspected(UPCGEditorGraphNodeBase* InPCGEditorGraphNode)
{
	if (PCGEditorGraphNode == InPCGEditorGraphNode)
	{
		return;
	}

	CacheColumnWidthVisibility();

	PCGEditorGraphNode = InPCGEditorGraphNode;

	if (PCGEditorGraphNode.IsValid())
	{
		NodeNameTextBlock->SetText(PCGEditorGraphNode->GetNodeTitle(ENodeTitleType::ListView));
		NodeNameTextBlock->SetToolTipText(PCGEditorGraphNode->GetTooltipText());
	}
	else
	{
		NodeNameTextBlock->SetText(PCGEditorGraphAttributeListView::NoNodeInspectedText);
		NodeNameTextBlock->SetToolTipText(PCGEditorGraphAttributeListView::NoNodeInspectedToolTip);
	}

	// Always unlock when changing the node, to make sure we unlock when removing the inspected node
	bIsLocked = false;

	bPCGEditorGraphNodeChanged = true;

	if (ViewportWidget.IsValid())
	{
		ViewportWidget->OnNewNodeInspected();
	}

	RequestRefresh();
}

void SPCGEditorGraphAttributeListView::SetViewportWidget(TSharedPtr<SPCGEditorViewport> InViewportWidget, FName InViewportEditorPanel)
{
	ViewportWidget = InViewportWidget;
	ViewportEditorPanel = InViewportEditorPanel;
}

void SPCGEditorGraphAttributeListView::ResetViewport()
{
	ViewportWidget->ResetScene();
}

void SPCGEditorGraphAttributeListView::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(DataPtr);

	if (ViewportWidget)
	{
		ViewportWidget->AddReferencedObjects(Collector);
	}
}

TSharedPtr<FPCGInspectionData> SPCGEditorGraphAttributeListView::GetInspectionDataPtr(const TSharedPtr<FPinComboBoxItem>& EditorPin) const
{
	if (!EditorPin)
	{
		return nullptr;
	}

	if (TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin())
	{
		const FPCGEditorInspectionDataEntrySetupParams Params{WidgetEntryNumber, PCGEditorGraphNode.Get(), EditorPin->PinIndex, EditorPin->bIsOutputPin};
		return PCGEditor->GetInspectionDataManager().GetInspectionDataPtr(Params);
	}
	else
	{
		return nullptr;
	}
}

bool SPCGEditorGraphAttributeListView::InspectData(const TSharedPtr<FPinComboBoxItem>& EditorPin, TFunctionRef<void(const FPCGDataCollection&)> InspectFunc) const
{
	if (!EditorPin)
	{
		return false;
	}

	if (TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin())
	{
		const FPCGEditorInspectionDataEntrySetupParams Params{ WidgetEntryNumber, PCGEditorGraphNode.Get(), EditorPin->PinIndex, EditorPin->bIsOutputPin };
		return PCGEditor->GetInspectionDataManager().InspectData(Params, InspectFunc);
	}
	else
	{
		return false;
	}
}

const FPCGDataCollection* SPCGEditorGraphAttributeListView::GetInspectionData() const
{
	if (!PCGEditorPtr.IsValid())
	{
		return nullptr;
	}

	const FPCGEditorInspectionDataEntry& Entry = PCGEditorPtr.Pin()->GetInspectionDataManager().GetInspectionEntry(WidgetEntryNumber);
	return &Entry.InspectionData;
}

void SPCGEditorGraphAttributeListView::RefreshAttributeList()
{
	// Don't refresh the list if we are not opened
	if (!IsOpen())
	{
		return;
	}

	CacheColumnWidthVisibility();

	HiddenAttributes = ListViewHeader->GetHiddenColumnIds();

	// Swapping to an empty item list to force a widget clear, otherwise the widgets will try to update during add column and access invalid data
	static const TArray<PCGListViewItemPtr> EmptyList;
	ListView->SetItemsSource(&EmptyList);

	PCGColumnData.Empty();
	ColumnsMaxWidthMapping.Empty();
	ListViewItems.Empty();
	ListViewHeader->ClearColumns();
	InfoTextBlock->SetText(FText::GetEmpty());
	DataPtr = nullptr;
	bViewportNeedsRefresh = true;

	const FPCGDataCollection* InspectionData = GetInspectionData();
	if (!InspectionData)
	{
		return;
	}

	const int32 DataIndex = GetSelectedDataIndex();
	if (!InspectionData->TaggedData.IsValidIndex(DataIndex))
	{
		return;
	}

	const int32 DomainIndex = GetSelectedDomainIndex();
	if (!DomainsComboBoxIds.IsValidIndex(DomainIndex))
	{
		return;
	}

	const FPCGTaggedData& TaggedData = InspectionData->TaggedData[DataIndex];
	const UPCGData* PCGData = TaggedData.Data;
	const FPCGMetadataDomainID& MetadataDomainID = DomainsComboBoxIds[DomainIndex];
	const FPCGCrc Crc = InspectionData->DataCrcs.IsValidIndex(DataIndex) ? InspectionData->DataCrcs[DataIndex] : FPCGCrc(0);

	if (!PCGData)
	{
		return;
	}

	// If we have a proxy for GPU data, read back CPU data for inspection.
	const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(PCGData);
	if (Proxy)
	{
		UPCGProxyForGPUData::FReadbackResult Result = Proxy->GetCPUData(/*InContext=*/nullptr);

		if (!Result.bComplete)
		{
			// Poll next tick.
			bNeedsRefresh = true;
			return;
		}

		if (!Result.TaggedData.Data)
		{
			return;
		}

		PCGData = Result.TaggedData.Data;
	}
	else if (const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(PCGData))
	{
		if (!TextureData->RequestCPUReadback())
		{
			bNeedsRefresh = true;
			return;
		}
	}

	const FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetConstPCGDataVisualizationRegistry();
	const UPCGData* DataToVisualize = PCGData;

	if (const IPCGDataVisualization* DataVisualization = DataVisRegistry.GetDataVisualization(PCGData->GetClass()))
	{
		const FPCGTableVisualizerInfo TableVisualizerInfo = DataVisualization->GetTableVisualizerInfoWithDomain(PCGData, MetadataDomainID);
		DataToVisualize = TableVisualizerInfo.Data;

		int32 NumEntries = 0;

		for (const FPCGTableVisualizerColumnInfo& ColumnInfo : TableVisualizerInfo.ColumnInfos)
		{
			AddColumn(ColumnInfo);
			NumEntries = FMath::Max(NumEntries, ColumnInfo.AccessorKeys.IsValid() ? ColumnInfo.AccessorKeys->GetNum() : 0);
		}

		SortingColumn = TableVisualizerInfo.SortingColumn;
		SortMode = (EColumnSortMode::Type)TableVisualizerInfo.SortingMode;
		FocusOnDataCallback = TableVisualizerInfo.FocusOnDataCallback;

		ListViewItems.Reserve(NumEntries);

		int32 StartIndex = bShowDefaultValue ? -1 : 0;

		for (int32 Index = StartIndex; Index < NumEntries; ++Index)
		{
			PCGListViewItemPtr ListViewItem = MakeShared<FPCGListViewItem>();
			ListViewItem->Index = Index;
			ListViewItems.Add(ListViewItem);
		}
	}
	else // No visualization, just default back the values
	{
		SortingColumn = NAME_None;
		SortMode = EColumnSortMode::Type::Ascending;
		FocusOnDataCallback.Reset();
	}

	if (DataPtr.Get() != DataToVisualize)
	{
		// If the visualized data has changed for whatever reason, we should make sure to be loading the relevant assets.
		bRefreshLoadHandles = true;
	}

	DataPtr = DataToVisualize;

	ListView->SetItemsSource(&ListViewItems);
	ListView->RequestListRefresh();

	RestoreColumnWidthVisibility();

	LaunchUpdateTask();
}

void SPCGEditorGraphAttributeListView::RefreshPinComboBox(bool bKeepSelection, bool& OutSelectionChanged)
{
	OutSelectionChanged = !bKeepSelection;
	const int32 PinComboBoxItemSelectedIndex = bKeepSelection ? PinComboBoxItems.IndexOfByKey(PinComboBox->GetSelectedItem()) : INDEX_NONE;
	PinComboBoxItems.Empty();
	PinComboBox->ClearSelection();
	PinComboBox->RefreshOptions();

	const UPCGNode* PCGNode = PCGEditorGraphNode.IsValid() ? PCGEditorGraphNode->GetPCGNode() : nullptr;
	if (!PCGNode)
	{
		return;
	}

	// Add output and then input pins to list. Optionally output the first connected item - useful for initializing
	// the selected item to the first connected output pin.
	auto PopulatePins = [](
		const TArray<TObjectPtr<UPCGPin>>& InPins,
		const FString& InFormatText,
		TArray<TSharedPtr<FPinComboBoxItem>>& InOutItems,
		int32* OutFirstConnectedItemIndex)
	{
		for (int32 PinIndex = 0; PinIndex < InPins.Num(); ++PinIndex)
		{
			const UPCGPin* PCGPin = InPins[PinIndex];
			const bool bIsOutputPin = PCGPin->IsOutputPin();
			// Pin is included in list if it is connected, or if it is an output pin.
			if ((PCGPin->IsConnected() || bIsOutputPin) && !PCGPin->Properties.bInvisiblePin && !PCGPin->Properties.IsDatalessPin())
			{
				FString ItemName = FString::Format(*InFormatText, { PCGPin->Properties.Label.ToString() });
				InOutItems.Add(MakeShared<FPinComboBoxItem>(FName(ItemName), PinIndex, bIsOutputPin));

				// Look for first connected, null pointer once found so only first is taken.
				if (OutFirstConnectedItemIndex && PCGPin->IsConnected())
				{
					*OutFirstConnectedItemIndex = InOutItems.Num() - 1;
					OutFirstConnectedItemIndex = nullptr;
				}
			}
		}
	};

	// Pick first connected output pin by default if there is one, otherwise default to first output pin.
	int32 FirstConnectedItemIndex = 0;
	PopulatePins(PCGNode->GetOutputPins(), TEXT("Output: {0}"), PinComboBoxItems, &FirstConnectedItemIndex);
	PopulatePins(PCGNode->GetInputPins(), TEXT("Input: {0}"), PinComboBoxItems, nullptr);

	bool bResetDataComboBoxItemsSelectedIndex = true;
	int32 SelectedIndex = PinComboBoxItems.IsValidIndex(PinComboBoxItemSelectedIndex) ? PinComboBoxItemSelectedIndex : FirstConnectedItemIndex;
	if (PinComboBoxItems.Num() > 0 && ensure(PinComboBoxItems.IsValidIndex(SelectedIndex)))
	{
		PinComboBox->SetSelectedItem(PinComboBoxItems[SelectedIndex]);

		OutSelectionChanged = SelectedIndex != PinComboBoxItemSelectedIndex;
	}
}

void SPCGEditorGraphAttributeListView::RefreshDataComboBox(bool bKeepSelection)
{
	// Cache previous selection only if we have items because this code gets called twice.
	// First call early outs on GetInspectionData(), when we get called the second time the combo box is empty but
	// we want to try and restore the cached element.
	if (!DataComboBoxItems.IsEmpty())
	{
		DataComboBoxItemsSelectedIndex = GetSelectedDataIndex();
	}
	
	if (!bKeepSelection)
	{
		DataComboBoxItemsSelectedIndex = INDEX_NONE;
	}

	DataComboBoxItems.Empty();
	DataComboBox->ClearSelection();
	DataComboBox->RefreshOptions();

	const FPCGDataCollection* InspectionData = GetInspectionData();
	if (!InspectionData)
	{
		RefreshDomainComboBox(bKeepSelection);
		return;
	}

	for (int32 TaggedDataIndex = 0; TaggedDataIndex < InspectionData->TaggedData.Num(); ++TaggedDataIndex)
	{
		const FPCGTaggedData& TaggedData = InspectionData->TaggedData[TaggedDataIndex];
		const UPCGData* Data = TaggedData.Data;

		if (const UPCGProxyForGPUData* ProxyData = Cast<UPCGProxyForGPUData>(Data))
		{
			UPCGProxyForGPUData::FReadbackResult Readback = ProxyData->GetCPUData(nullptr);
			if (ensure(Readback.bComplete)) // Readbacks should have been performed at beginning of refresh
			{
				Data = Readback.TaggedData.Data;
			}
		}

		FString ItemName = FString::Format(
			TEXT("[{0}] {1}"),
			{ FText::AsNumber(TaggedDataIndex).ToString(), (Data ? Data->GetClass()->GetDisplayNameText().ToString(): TEXT("No Data")) });

		if (!TaggedData.Tags.IsEmpty())
		{
			ItemName.Append(FString::Format(TEXT(": ({0})"), { FString::Join(TaggedData.Tags, TEXT(", ")) }));
		}

		DataComboBoxItems.Add(MakeShared<FString>(ItemName));
	}

	bool bKeepDomainComboBox = false;

	if (DataComboBoxItems.IsValidIndex(DataComboBoxItemsSelectedIndex))
	{
		DataComboBox->SetSelectedItem(DataComboBoxItems[DataComboBoxItemsSelectedIndex]);
		bKeepDomainComboBox = true;
	}
	else if (DataComboBoxItems.Num() > 0)
	{
		DataComboBox->SetSelectedItem(DataComboBoxItems[0]);
	}

	DataComboBoxItemsSelectedIndex = INDEX_NONE;

	RefreshDomainComboBox(bKeepDomainComboBox);
}

void SPCGEditorGraphAttributeListView::RefreshDomainComboBox(bool bKeepSelection)
{
	// Cache previous selection only if we have items because this code gets called twice.
	// First call early outs on GetInspectionData(), when we get called the second time the combo box is empty but
	// we want to try and restore the cached element.
	if (!DomainsComboBoxItems.IsEmpty())
	{
		const int32 SelectedDomainIndex = GetSelectedDomainIndex();
		DomainsComboBoxItemsSelectedDomain = DomainsComboBoxIds.IsValidIndex(SelectedDomainIndex) ? DomainsComboBoxIds[SelectedDomainIndex] : PCGMetadataDomainID::Invalid;
	}

	if (!bKeepSelection)
	{
		DomainsComboBoxItemsSelectedDomain = PCGMetadataDomainID::Invalid;
	}

	DomainsComboBoxItems.Empty();
	DomainsComboBoxIds.Empty();
	DomainsComboBox->ClearSelection();
	DomainsComboBox->RefreshOptions();

	const FPCGDataCollection* InspectionData = GetInspectionData();
	if (!InspectionData)
	{
		return;
	}
	
	const int32 DataIndex = GetSelectedDataIndex();
	if (!InspectionData->TaggedData.IsValidIndex(DataIndex))
	{
		return;
	}

	const FPCGTaggedData& TaggedData = InspectionData->TaggedData[DataIndex];
	const UPCGData* PCGData = TaggedData.Data;

	if (const UPCGProxyForGPUData* ProxyData = Cast<UPCGProxyForGPUData>(PCGData))
	{
		UPCGProxyForGPUData::FReadbackResult Readback = ProxyData->GetCPUData(nullptr);
		if (ensure(Readback.bComplete)) // Readbacks should have been performed at beginning of refresh
		{
			PCGData = Readback.TaggedData.Data;
		}
	}

	if (!PCGData)
	{
		return;
	}
	
	const FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetConstPCGDataVisualizationRegistry();

	const IPCGDataVisualization* DataVisualization = DataVisRegistry.GetDataVisualization(PCGData->GetClass());
	if (!DataVisualization)
	{
		return;
	}

	const FPCGMetadataDomainID DefaultDomainID = DataVisualization->GetDefaultDomainForInspection(PCGData);
	int32 DefaultIndex = 0;
	int32 PreviousDomainIndex = INDEX_NONE;
	for (const FPCGMetadataDomainID& DomainID : DataVisualization->GetAllSupportedDomainsForInspection(PCGData))
	{
		if (DomainsComboBoxItemsSelectedDomain == DomainID)
		{
			PreviousDomainIndex = DomainsComboBoxItems.Num();
		}
		
		if (DomainID == DefaultDomainID)
		{
			DefaultIndex = DomainsComboBoxItems.Num();
		}
		
		DomainsComboBoxItems.Add(MakeShared<FString>(DataVisualization->GetDomainDisplayNameForInspection(PCGData, DomainID)));
		DomainsComboBoxIds.Add(DomainID);
	}

	if (DomainsComboBoxItems.Num() > 0)
	{
		const int32 Index = PreviousDomainIndex != INDEX_NONE ? PreviousDomainIndex : DefaultIndex; 
		DomainsComboBox->SetSelectedItem(DomainsComboBoxItems[Index]);
	}

	DomainsComboBoxItemsSelectedDomain = PCGMetadataDomainID::Invalid;
}

void SPCGEditorGraphAttributeListView::RefreshViewport()
{
	if (!IsViewportOpen())
	{
		return;
	}

	ResetViewport();

	const FPCGDataCollection* InspectionData = GetInspectionData();
	if (!InspectionData)
	{
		return;
	}

	const int32 DataIndex = GetSelectedDataIndex();
	if (!InspectionData->TaggedData.IsValidIndex(DataIndex))
	{
		return;
	}

	const int32 DomainIndex = GetSelectedDomainIndex();
	if (!DomainsComboBoxIds.IsValidIndex(DomainIndex))
	{
		return;
	}

	const UPCGData* Data = InspectionData->TaggedData[DataIndex].Data;

	if (!Data)
	{
		return;
	}

	// If we have a proxy for GPU data, read back CPU data for inspection.
	if (const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(Data))
	{
		UPCGProxyForGPUData::FReadbackResult Result = Proxy->GetCPUData(/*InContext=*/nullptr);

		if (!Result.bComplete)
		{
			// Poll next tick.
			bViewportNeedsRefresh = true;
			return;
		}

		if (!Result.TaggedData.Data)
		{
			return;
		}

		Data = Result.TaggedData.Data;
	}
	else if (const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(Data))
	{
		if (!TextureData->RequestCPUReadback())
		{
			bViewportNeedsRefresh = true;
			return;
		}
	}

	const FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetConstPCGDataVisualizationRegistry();
	const IPCGDataVisualization* DataVisualization = DataVisRegistry.GetDataVisualization(Data->GetClass());

	if (!DataVisualization)
	{
		return;
	}

	if (bRefreshLoadHandles)
	{
		LoadHandles = DataVisualization->LoadRequiredResources(Data);
	}

	bool bAllResourcesLoaded = true;
	TArray<UObject*> LoadedResources;

	for (TSharedPtr<FStreamableHandle> LoadHandle : LoadHandles)
	{
		if (!LoadHandle.IsValid())
		{
			continue;
		}

		if (LoadHandle->HasLoadCompleted())
		{
			LoadedResources.Add(LoadHandle->GetLoadedAsset());
		}
		else
		{
			bAllResourcesLoaded = false;
		}
	}

	if (!bAllResourcesLoaded)
	{
		// Poll next tick.
		bViewportNeedsRefresh = true;
		return;
	}

	const UPCGNode* PCGNode = PCGEditorGraphNode.IsValid() ? PCGEditorGraphNode->GetPCGNode() : nullptr;
	const UPCGSettingsInterface* Settings = PCGNode ? PCGNode->GetSettingsInterface() : nullptr;

	ViewportWidget->SetupScene(LoadedResources, DataVisualization->GetViewportSetupFunc(Settings, Data));
}

void SPCGEditorGraphAttributeListView::LaunchUpdateTask()
{
	// Discarding any currently running updater, task will still run and keep the old object alive but we wont care about the result.
	// This is done because we cant afford to wait for task completion before starting a new task.
	// Note that in the case we are showing the default value, skip the first element (index -1).
	// We will not use the default value for sorting/filtering.
	CurrentUpdateTask.Reset();
	CurrentUpdateTask = MakeShared<FPCGListViewUpdater>(bShowDefaultValue ? MakeArrayView(ListViewItems).RightChop(1) : MakeArrayView(ListViewItems),
		PCGColumnData,
		SortMode,
		SortingColumn,
		TextFilter);
	CurrentUpdateTask->Launch();
}

const FSlateBrush* SPCGEditorGraphAttributeListView::GetFilterBadgeIcon() const
{
	for (const SHeaderRow::FColumn& Column : ListViewHeader->GetColumns())
	{
		if (!Column.bIsVisible)
		{
			return FAppStyle::Get().GetBrush("Icons.BadgeModified");
		}
	}

	return nullptr;
}

TSharedRef<SWidget> SPCGEditorGraphAttributeListView::OnGenerateFilterMenu()
{
	FMenuBuilder MenuBuilder(false, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToggleAllAttributes", "Toggle All"),
		LOCTEXT("ToggleAllAttributesTooltip", "Toggle visibility for all attributes"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::ToggleAllAttributes),
			FCanExecuteAction(),
			FGetActionCheckState::CreateSP(this, &SPCGEditorGraphAttributeListView::GetAnyAttributeEnabledState)),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	MenuBuilder.AddSeparator();

	const TIndirectArray<SHeaderRow::FColumn>& Columns = ListViewHeader->GetColumns();
	TArray<FName> HiddenColumns = ListViewHeader->GetHiddenColumnIds();

	for (const SHeaderRow::FColumn& Column : Columns)
	{
		MenuBuilder.AddMenuEntry(
			Column.DefaultText,
			Column.DefaultTooltip,
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::ToggleAttribute, Column.ColumnId),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SPCGEditorGraphAttributeListView::IsAttributeEnabled, Column.ColumnId)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SPCGEditorGraphAttributeListView::OnGenerateAdditionalOperationsMenu()
{
	FMenuBuilder MenuBuilder(false, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SaveThisData", "Save this data"),
		LOCTEXT("SaveThisDataTooltip", "Saves this data to a PCG Data Asset."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Download"),
		FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::SaveData, true, true),
			FCanExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::CanSaveData, true, true)),
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SaveThisPinData", "Save pin data"),
		LOCTEXT("SaveThisPinDataTooltip", "Saves all data from the selected pin to a PCG Data Asset."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Download"),
		FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::SaveData, true, false),
			FCanExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::CanSaveData, true, false)),
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SaveAllData", "Save all"),
		LOCTEXT("SaveAllDataTooltip", "Saves all the input or output data to a PCG Data Asset."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Download"),
		FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::SaveData, false, false),
			FCanExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::CanSaveData, false, false)),
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ResetDefaultWidth", "Reset columns to default width"),
		LOCTEXT("ResetDefaultWidthTooltip", "Resets all the columns width to their default."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::ResetColumnsWidthToDefault)),
		NAME_None,
		EUserInterfaceActionType::Button);
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ExpendMaxWidth", "Expend columns to max width"),
		LOCTEXT("ExpendMaxWidthTooltip", "Expends all the columns width to their maximum to see their content."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::ExpendAllColumnToMaxWidth)),
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowDefaultValue", "Show Default Value for Attributes"),
		LOCTEXT("ShowDefaultValueTooltip", "[ADVANCED] For all non-$ attributes, show the underlying default value. For all $ attributes, it will be 0. Stick at the top"),
		FSlateIcon(),
		FUIAction(
	FExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::OnToggleShowDefaultValue),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SPCGEditorGraphAttributeListView::IsShowingDefaultValue)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	return MenuBuilder.MakeWidget();
}

bool SPCGEditorGraphAttributeListView::BuildDataCollectionForSave(bool bUsePinComboIndex, bool bUseDataComboIndex, FPCGDataCollection* OutSaveData) const
{
	// Return if we found any data to save, when OutSaveData is nullptr this method can be used to test without copying any data
	bool bDataToSave = false;

	// Can't save N'th data on all pins, as it makes no sense
	check(!bUseDataComboIndex || bUsePinComboIndex);

	if (!bUsePinComboIndex)
	{
		for (const auto& PinComboBoxItem : PinComboBoxItems)
		{
			InspectData(PinComboBoxItem, [&bDataToSave, OutSaveData](const FPCGDataCollection& InspectionData)
			{
				if(InspectionData.TaggedData.Num() > 0)
				{
					bDataToSave = true;
				}

				if (OutSaveData)
				{
					OutSaveData->TaggedData.Append(InspectionData.TaggedData);
				}
			});
		}
	}
	else
	{
		if (const FPCGDataCollection* PinInspectionData = GetInspectionData())
		{
			if (!bUseDataComboIndex)
			{
				if (PinInspectionData->TaggedData.Num() > 0)
				{
					bDataToSave = true;
				}

				if (OutSaveData)
				{
					*OutSaveData = *PinInspectionData;
				}
			}
			else if(int SelectedIndex = GetSelectedDataIndex(); PinInspectionData->TaggedData.IsValidIndex(SelectedIndex))
			{
				bDataToSave = true;
				
				if (OutSaveData)
				{
					OutSaveData->TaggedData.Add(PinInspectionData->TaggedData[SelectedIndex]);
				}
			}
		}
	}

	return bDataToSave;
}

void SPCGEditorGraphAttributeListView::SaveData(bool bUsePinComboIndex, bool bUseDataComboIndex)
{
	// FPCGInspectionData will make sure data is visible to GC while we save
	FPCGInspectionData SaveData;
	SaveData.Data = MakeShared<FPCGDataCollection>();
	if (!BuildDataCollectionForSave(bUsePinComboIndex, bUseDataComboIndex, SaveData.Data.Get()))
	{
		return;
	}

	UPCGDataCollectionExporter* Exporter = NewObject<UPCGDataCollectionExporter>();
	Exporter->Data = *SaveData.Data;

	FPCGAssetExporterParameters Parameters;
	UPCGAssetExporterUtils::CreateAsset(Exporter, Parameters);
}

bool SPCGEditorGraphAttributeListView::CanSaveData(bool bUsePinComboIndex, bool bUseDataComboIndex) const
{
	// Query only
	return BuildDataCollectionForSave(bUsePinComboIndex, bUseDataComboIndex, nullptr);
}

FText SPCGEditorGraphAttributeListView::OnGenerateSelectedPinText() const
{
	if (const TSharedPtr<FPinComboBoxItem> SelectedPin = PinComboBox->GetSelectedItem())
	{
		return FText::FromName(SelectedPin->Name);
	}
	else
	{
		return PCGEditorGraphAttributeListView::NoPinAvailableText;
	}
}

void SPCGEditorGraphAttributeListView::OnSelectionChangedPin(TSharedPtr<FPinComboBoxItem> InItem, ESelectInfo::Type InSelectInfo)
{
	TSharedPtr<FPinComboBoxItem> SelectedPin = PinComboBox->GetSelectedItem();

	// If the pin is invalid, it will still refresh to clean up.
	const FPCGEditorInspectionDataEntrySetupParams Params{WidgetEntryNumber, PCGEditorGraphNode.Get(), SelectedPin ? SelectedPin->PinIndex : INDEX_NONE, SelectedPin ? SelectedPin->bIsOutputPin : false};

	if (PCGEditorPtr.IsValid())
	{
		PCGEditorPtr.Pin()->GetInspectionDataManager().SetupInspectionEntry(Params);
	}

	bKeepDataComboBoxSelection = InSelectInfo == ESelectInfo::Direct;
	RequestRefresh();
}

TSharedRef<SWidget> SPCGEditorGraphAttributeListView::OnGeneratePinWidget(TSharedPtr<FPinComboBoxItem> InItem) const
{
	return SNew(STextBlock).Text(FText::FromName(InItem.IsValid() ? InItem->Name : NAME_None));
}

TSharedRef<SWidget> SPCGEditorGraphAttributeListView::OnGenerateDataWidget(TSharedPtr<FString> InItem) const
{
	return SNew(STextBlock).Text(InItem.IsValid() ? FText::FromString(*InItem) : FText()).OverflowPolicy(ETextOverflowPolicy::Ellipsis);
}

void SPCGEditorGraphAttributeListView::OnSelectionChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		RefreshAttributeList();
	}
}

FText SPCGEditorGraphAttributeListView::OnGenerateSelectedDataText() const
{
	if (const TSharedPtr<FString> SelectedDataName = DataComboBox->GetSelectedItem())
	{
		return FText::FromString(*SelectedDataName);
	}
	else
	{
		return PCGEditorGraphAttributeListView::NoDataAvailableText;
	}
}

FText SPCGEditorGraphAttributeListView::OnGenerateSelectedDomainText() const
{
	if (const TSharedPtr<FString> SelectedDataName = DomainsComboBox->GetSelectedItem())
	{
		return FText::FromString(*SelectedDataName);
	}
	else
	{
		return PCGEditorGraphAttributeListView::NoDataAvailableText;
	}
}

int32 SPCGEditorGraphAttributeListView::GetSelectedDataIndex() const
{
	int32 Index = INDEX_NONE;
	if (const TSharedPtr<FString> SelectedItem = DataComboBox->GetSelectedItem())
	{
		DataComboBoxItems.Find(SelectedItem, Index);
	}

	return Index;
}

int32 SPCGEditorGraphAttributeListView::GetSelectedDomainIndex() const
{
	int32 Index = INDEX_NONE;
	if (const TSharedPtr<FString> SelectedItem = DomainsComboBox->GetSelectedItem())
	{
		DomainsComboBoxItems.Find(SelectedItem, Index);
	}

	return Index;
}

void SPCGEditorGraphAttributeListView::ToggleAllAttributes()
{
	const TArray<FName> HiddenColumns = ListViewHeader->GetHiddenColumnIds();
	if (HiddenColumns.Num() > 0)
	{
		for (const FName& HiddenColumn : HiddenColumns)
		{
			ListViewHeader->SetShowGeneratedColumn(HiddenColumn, /*InShow=*/true);
		}
	}
	else
	{
		const TIndirectArray<SHeaderRow::FColumn>& Columns = ListViewHeader->GetColumns();
		for (const SHeaderRow::FColumn& Column : Columns)
		{
			ListViewHeader->SetShowGeneratedColumn(Column.ColumnId, /*InShow=*/false);
		}
	}
}

void SPCGEditorGraphAttributeListView::ToggleAttribute(FName InAttributeName)
{
	ListViewHeader->SetShowGeneratedColumn(InAttributeName, !ListViewHeader->IsColumnVisible(InAttributeName));
}

ECheckBoxState SPCGEditorGraphAttributeListView::GetAnyAttributeEnabledState() const
{
	bool bAllEnabled = true;
	bool bAnyEnabled = false;

	for (const SHeaderRow::FColumn& Column : ListViewHeader->GetColumns())
	{
		bAllEnabled &= Column.bIsVisible;
		bAnyEnabled |= Column.bIsVisible;
	}

	if (bAllEnabled)
	{
		return ECheckBoxState::Checked;
	}
	else if (bAnyEnabled)
	{
		return ECheckBoxState::Undetermined;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}

bool SPCGEditorGraphAttributeListView::IsAttributeEnabled(FName InAttributeName) const
{
	return ListViewHeader->IsColumnVisible(InAttributeName);
}

TSharedRef<ITableRow> SPCGEditorGraphAttributeListView::OnGenerateRow(PCGListViewItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SPCGListViewItemRow, OwnerTable)
		.AttributeListView(SharedThis(this))
		.ListViewItem(Item);
}

void SPCGEditorGraphAttributeListView::OnItemDoubleClicked(PCGListViewItemPtr Item) const
{
	check(Item);
	if (FocusOnDataCallback && DataPtr.Get())
	{
		FocusOnDataCallback(DataPtr.Get(), { Item->Index });
	}
}

TSharedPtr<SWidget> SPCGEditorGraphAttributeListView::OnItemsContextMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, /*InCommandList=*/nullptr);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowInViewport", "Zoom to selection"),
		LOCTEXT("ShowInViewport_Tooltip", "Frames the viewport so that all selected entries are visible."),
		FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Editor.ZoomToSelection"),
		FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::FocusOnSelection),
			FCanExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::CanFocusOnSelection)),
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CopyToClipboard", "Copy to clipboard"),
		LOCTEXT("CopyToClipboard_Tooltip", "Copies the contents of the entries to the clipboard."),
		FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Editor.CopyToClipboard"),
		FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::CopySelectionToClipboard),
			FCanExecuteAction::CreateSP(this, &SPCGEditorGraphAttributeListView::CanCopySelectionToClipboard)),
		NAME_None,
		EUserInterfaceActionType::Button);

	return MenuBuilder.MakeWidget();
}

void SPCGEditorGraphAttributeListView::OnColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InNewSortMode)
{
	if (SortingColumn == InColumnId)
	{
		// Cycling
		SortMode = static_cast<EColumnSortMode::Type>((SortMode + 1) % 3);
	}
	else
	{
		SortingColumn = InColumnId;
		SortMode = InNewSortMode;
	}

	LaunchUpdateTask();
}

EColumnSortMode::Type SPCGEditorGraphAttributeListView::GetColumnSortMode(const FName InColumnId) const
{
	if (SortingColumn != InColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

void SPCGEditorGraphAttributeListView::OnFilterTextChanged(const FText& InFilterText)
{
	ActiveFilterText = InFilterText;
	TextFilter->SetFilterText(InFilterText);

	const FText ErrorText = TextFilter->GetFilterErrorText();
	if(ErrorText.IsEmpty())
	{
		LaunchUpdateTask();
	}

	SearchBoxWidget->SetError(ErrorText);
}

void SPCGEditorGraphAttributeListView::OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnCleared)
	{
		SearchBoxWidget->SetText(FText::GetEmpty());
		OnFilterTextChanged(FText::GetEmpty());
	}
}

FReply SPCGEditorGraphAttributeListView::OnListViewKeyDown(const FGeometry& /*InGeometry*/, const FKeyEvent& InKeyEvent) const
{
	if (ListViewCommands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SPCGEditorGraphAttributeListView::AddColumn(const FPCGTableVisualizerColumnInfo& InColumnInfo)
{
	FPCGColumnData& ColumnData = PCGColumnData.Add(InColumnInfo.Id);
	ColumnData.DataAccessor = InColumnInfo.Accessor;
	ColumnData.DataKeys = InColumnInfo.AccessorKeys;
	ColumnData.UnderlyingAttribute = InColumnInfo.UnderlyingAttribute;

	const float ColumnWidth = InColumnInfo.Width < 0.0f ? PCGEditorGraphAttributeListView::CalculateColumnWidth(InColumnInfo.Label) : InColumnInfo.Width;
	ColumnsMaxWidthMapping.FindOrAdd(InColumnInfo.Id) = ColumnWidth;

	SHeaderRow::FColumn::FArguments Arguments;
	Arguments.ColumnId(InColumnInfo.Id);
	Arguments.DefaultLabel(InColumnInfo.Label);
	Arguments.DefaultTooltip(InColumnInfo.Tooltip);
	Arguments.ManualWidth(ColumnWidth);
	Arguments.HAlignHeader(HAlign_Center);
	Arguments.HAlignCell((EHorizontalAlignment)InColumnInfo.CellAlignment);
	Arguments.SortMode(this, &SPCGEditorGraphAttributeListView::GetColumnSortMode, InColumnInfo.Id);
	Arguments.OnSort(this, &SPCGEditorGraphAttributeListView::OnColumnSortModeChanged);
	Arguments.OverflowPolicy(GetDefault<UPCGEditorSettings>()->AttributeListViewSettings.ColumnNameOverflowPolicy);
	Arguments.HeaderComboVisibility(EHeaderComboVisibility::Never);
	Arguments.MenuContent()
	[
		GenerateColumnMenu(InColumnInfo.Id)
	];
	Arguments.OnColumnSplitterDoubleClick_Lambda([WeakThis = AsWeak(), ColumnId = InColumnInfo.Id](const FGeometry&, const FPointerEvent&) -> FReply
	{
		if (SPCGEditorGraphAttributeListView* This = static_cast<SPCGEditorGraphAttributeListView*>(WeakThis.Pin().Get()))
		{
			This->ResizeColumnToMaxWidth(ColumnId);
			return FReply::Handled();
		}
		
		return FReply::Unhandled();
	});

	SHeaderRow::FColumn* NewColumn = new SHeaderRow::FColumn(Arguments);
	NewColumn->bIsVisible = !HiddenAttributes.Contains(InColumnInfo.Id);
	ListViewHeader->AddColumn(*NewColumn);
}

void SPCGEditorGraphAttributeListView::ResizeColumnToMaxWidth(const FName ColumnId)
{
	ListViewHeader->SetColumnWidth(ColumnId, ColumnsMaxWidthMapping[ColumnId]);
}

void SPCGEditorGraphAttributeListView::ResetColumnsWidthToDefault()
{
	if (!ListViewHeader.IsValid())
	{
		return;
	}
	
	for (const SHeaderRow::FColumn& Column : ListViewHeader->GetColumns())
	{
		ListViewHeader->SetColumnWidth(Column.ColumnId, PCGEditorGraphAttributeListView::CalculateColumnWidth(Column.DefaultText.Get()));
	}
}

void SPCGEditorGraphAttributeListView::ExpendAllColumnToMaxWidth()
{
	if (!ListViewHeader.IsValid())
	{
		return;
	}
	
	for (const SHeaderRow::FColumn& Column : ListViewHeader->GetColumns())
	{
		ResizeColumnToMaxWidth(Column.ColumnId);
	}
}

void SPCGEditorGraphAttributeListView::CacheColumnWidthVisibility()
{
	if (!PCGEditorGraphNode.IsValid() || !ListViewHeader.IsValid() || ListViewHeader->GetColumns().IsEmpty())
	{
		return;
	}
	
	const TObjectKey<UPCGEditorGraphNodeBase> NodeKey(PCGEditorGraphNode.Get());
	
	FNodeKeyToColumnWidthVisibilityMap* It = Algo::FindBy(ColumnWidthVisibilityCache, NodeKey, [](const FNodeKeyToColumnWidthVisibilityMap& It)
	{
		return It.Get<0>();
	});

	if (!It)
	{
		FNodeKeyToColumnWidthVisibilityMap KeyAndCache;
		KeyAndCache.Get<0>() = NodeKey;
		It = &ColumnWidthVisibilityCache.Add_GetRef(std::move(KeyAndCache));
	}

	check(It);
	for (const SHeaderRow::FColumn& Column : ListViewHeader->GetColumns())
	{
		TTuple<float, bool>& WidthVisiblity = It->Get<1>().FindOrAdd(Column.ColumnId);
		WidthVisiblity.Get<0>() = Column.GetWidth();
		WidthVisiblity.Get<1>() = Column.bIsVisible;
	}
}

void SPCGEditorGraphAttributeListView::RestoreColumnWidthVisibility()
{
	if (!PCGEditorGraphNode.IsValid() || !ListViewHeader.IsValid() || ListViewHeader->GetColumns().IsEmpty())
	{
		return;
	}
	
	const TObjectKey<UPCGEditorGraphNodeBase> NodeKey(PCGEditorGraphNode.Get());
	
	const FNodeKeyToColumnWidthVisibilityMap* It = Algo::FindBy(ColumnWidthVisibilityCache, NodeKey, [](const FNodeKeyToColumnWidthVisibilityMap& It)
	{
		return It.Get<0>();
	});

	if (!It)
	{
		return;
	}

	for (const SHeaderRow::FColumn& Column : ListViewHeader->GetColumns())
	{
		if (const TTuple<float, bool>* WidthVisibility = It->Get<1>().Find(Column.ColumnId))
		{
			ListViewHeader->SetColumnWidth(Column.ColumnId, WidthVisibility->Get<0>());
			ListViewHeader->SetShowGeneratedColumn(Column.ColumnId, WidthVisibility->Get<1>());
		}
	}
}

TSharedRef<SWidget> SPCGEditorGraphAttributeListView::GenerateColumnMenu(FName ColumnId)
{
	FMenuBuilder MenuBuilder(/*bShouldCloseAfterMenuSelection=*/true, nullptr);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CopyAttributeNameToClipboard", "Copy attribute name"),
		LOCTEXT("CopyAttributeNameToClipboardTooltip", "Copies the attribute name to the clipboard."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([ColumnId]()
		{
			FString ColumnIdString = ColumnId.ToString();

			// TODO[UE-221219]: until we support @None as an actual valid token, we need to replace it with None.
			ColumnIdString = ColumnIdString.Replace(TEXT("@None"), TEXT("None"));

			FPlatformApplicationMisc::ClipboardCopy(*ColumnIdString);
		})),
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ResizeColumnToMaxWidth", "Resize Column width to match content"),
		LOCTEXT("ResizeColumnToMaxWidthTooltip", "Resizes column width to match the content width."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([WeakThis = AsWeak(), ColumnId]()
		{
			if (SPCGEditorGraphAttributeListView* This = static_cast<SPCGEditorGraphAttributeListView*>(WeakThis.Pin().Get()))
			{
				This->ResizeColumnToMaxWidth(ColumnId);
			}
		})),
		NAME_None,
		EUserInterfaceActionType::Button);

	return MenuBuilder.MakeWidget();
}

void SPCGEditorGraphAttributeListView::CopySelectionToClipboard() const
{
	constexpr TCHAR Delimiter = TEXT(',');
	constexpr TCHAR LineEnd = TEXT('\n');

	const TArray<FName> HiddenColumnIds = ListViewHeader->GetHiddenColumnIds();

	TArray<TPair<FName, FPCGColumnData>> FilteredPCGColumnData;
	for (const TPair<FName, FPCGColumnData>& Data : PCGColumnData)
	{
		if (!HiddenColumnIds.Contains(Data.Key))
		{
			FilteredPCGColumnData.Add(Data);
		}
	}

	TStringBuilder<2048> CSVExport;

	// Write column header row
	for (int ColumnIndex = 0; ColumnIndex < FilteredPCGColumnData.Num(); ColumnIndex++)
	{
		const TPair<FName, FPCGColumnData>& Data = FilteredPCGColumnData[ColumnIndex];

		if (ColumnIndex > 0)
		{
			CSVExport += Delimiter;
		}
		CSVExport += Data.Key.ToString();
	}

	// Gather selected rows and sort them to match the displayed order instead of selection order
	TArray<PCGListViewItemPtr> SelectedListViewItems = ListView->GetSelectedItems();
	if (const FPCGColumnData* ColumnData = PCGColumnData.Find(SortingColumn))
	{
		if (ColumnData->DataAccessor.IsValid() && ColumnData->DataKeys.IsValid())
		{
			// Lambda used here to get the index value of an item in the array for sorting
			PCGAttributeAccessorHelpers::SortByAttribute(*ColumnData->DataAccessor, *ColumnData->DataKeys, SelectedListViewItems, !(SortMode & EColumnSortMode::Descending), [&SelectedListViewItems](int Index) { return SelectedListViewItems[Index]->Index; });
		}
	}

	// Write each row
	for (const PCGListViewItemPtr&  ListViewItem : SelectedListViewItems)
	{
		CSVExport += LineEnd;

		for (int ColumnIndex = 0; ColumnIndex < FilteredPCGColumnData.Num(); ColumnIndex++)
		{
			if (ColumnIndex > 0)
			{
				CSVExport += Delimiter;
			}

			const TPair<FName, FPCGColumnData>& Data = FilteredPCGColumnData[ColumnIndex];
			const FPCGColumnData& ColumnData = Data.Value;
			if (ColumnData.DataAccessor.IsValid() && ColumnData.DataKeys.IsValid())
			{
				FString RowString;
				if (ColumnData.DataAccessor->Get<FString>(RowString, ListViewItem->Index, *ColumnData.DataKeys, EPCGAttributeAccessorFlags::AllowBroadcast))
				{
					CSVExport += RowString;
				}
			}
		}
	}

	FPlatformApplicationMisc::ClipboardCopy(*CSVExport);
}

bool SPCGEditorGraphAttributeListView::CanCopySelectionToClipboard() const
{
	return ListView->GetNumItemsSelected() > 0;
}

const FSlateBrush* SPCGEditorGraphAttributeListView::OnGetLockButtonImageResource() const
{
	return FAppStyle::GetBrush(bIsLocked ? TEXT("PropertyWindow.Locked") : TEXT("PropertyWindow.Unlocked"));
}

FReply SPCGEditorGraphAttributeListView::OnLockClick()
{
	bIsLocked = !bIsLocked;
	return FReply::Handled();
}

FReply SPCGEditorGraphAttributeListView::OnNodeNameClicked()
{
	if (PCGEditorGraphNode.Get() && PCGEditorPtr.IsValid())
	{
		PCGEditorPtr.Pin()->JumpToNode(PCGEditorGraphNode.Get());
	}

	return FReply::Handled();
}

FReply SPCGEditorGraphAttributeListView::OnFocusOnDataClicked() const
{
	if (IsFocusOnDataEnabled() && DataPtr.Get())
	{
		FocusOnDataCallback(DataPtr.Get(), {});
	}

	return FReply::Handled();
}

bool SPCGEditorGraphAttributeListView::IsFocusOnDataEnabled() const
{
	return (!ListViewItems.IsEmpty() && FocusOnDataCallback);
}

void SPCGEditorGraphAttributeListView::FocusOnSelection() const
{
	if (!IsFocusOnDataEnabled() || !DataPtr.Get())
	{
		return;
	}

	// Note: this implementation assumes it's the same callback for all entries, which is currently true
	const TArray<PCGListViewItemPtr> SelectedItems = ListView->GetSelectedItems();
	TArray<int> Indices;
	Indices.Reserve(SelectedItems.Num());

	for (PCGListViewItemPtr SelectedItem : SelectedItems)
	{
		if (SelectedItem)
		{
			Indices.Add(SelectedItem->Index);
		}
	}

	FocusOnDataCallback(DataPtr.Get(), Indices);
}

bool SPCGEditorGraphAttributeListView::CanFocusOnSelection() const
{
	return ListView->GetNumItemsSelected() > 0;
}

bool SPCGEditorGraphAttributeListView::IsViewportOpen() const
{
	return PCGEditorPtr.IsValid() && PCGEditorPtr.Pin()->IsPanelCurrentlyOpen(ViewportEditorPanel);
}

bool SPCGEditorGraphAttributeListView::IsOpen() const
{
	return PCGEditorPtr.IsValid() && (WidgetEntryNumber >= 0) && (WidgetEntryNumber < PCGEditorTabs::NumAttributeTabs) && PCGEditorPtr.Pin()->IsPanelCurrentlyOpen(PCGEditorTabs::AttributesID[WidgetEntryNumber]);
}


void SPCGEditorGraphAttributeListView::OnToggleShowDefaultValue()
{
	bShowDefaultValue = !bShowDefaultValue;
	RequestRefresh();
}

bool SPCGEditorGraphAttributeListView::IsShowingDefaultValue() const
{
	return bShowDefaultValue;
}

#undef LOCTEXT_NAMESPACE
