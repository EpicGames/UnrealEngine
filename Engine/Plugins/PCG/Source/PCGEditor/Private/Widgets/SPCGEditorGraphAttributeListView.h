// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGPointData.h"
#include "Graph/PCGStackContext.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "Containers/RingBuffer.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Tasks/Task.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakInterfacePtr.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

enum class EPCGMetadataTypes : uint8;
struct FPCGDataCollection;
struct FPCGInspectionData;
class FPCGEditor;
struct FPCGEditorInspectionDataEntry;
class FPCGEditorInspectionDataManager;
class FPCGMetadataAttributeBase;
struct FPCGPoint;
struct FPCGTableVisualizerColumnInfo;
class FUICommandList;
class SPCGEditorGraphAttributeListView;
class SPCGEditorViewport;
class UPCGData;
class UPCGEditorGraphNodeBase;
class UPCGMetadata;
class UPCGParamData;
class UPCGPointData;

template <typename OptionType> class SComboBox;
struct FSlateBrush;
struct FStreamableHandle;
class SComboButton;
class SHeaderRow;
class SSearchBox;
class STableViewBase;
class STextBlock;

namespace PCGEditorGraphAttributeListView
{
	constexpr float MaxColumnWidth = 200.0f;
	constexpr int32 MaxNodeColumnWidthCachedItems = 256;
}

struct FPCGListViewItem
{
	int32 Index = INDEX_NONE;
};

struct FPCGColumnData
{
	TSharedPtr<const IPCGAttributeAccessor> DataAccessor;
	TSharedPtr<const IPCGAttributeAccessorKeys> DataKeys;
	const FPCGMetadataAttributeBase* UnderlyingAttribute = nullptr;
};

typedef TSharedPtr<FPCGListViewItem> PCGListViewItemPtr;

template <typename T, typename = void>
struct FTextAsNumberIsValid : std::false_type {};

/** Utility to see if a value type is supported by FText::AsNumber */
template <typename T>
struct FTextAsNumberIsValid<T, std::void_t<decltype(FText::AsNumber(std::declval<T>()))>> : std::true_type {};

/** Class used for threaded filtering and sorting of list view items */
class FPCGListViewUpdater : public TSharedFromThis<FPCGListViewUpdater>
{
public:
	FPCGListViewUpdater(
		const TArrayView<const PCGListViewItemPtr>& InListViewItems,
		const TMap<FName, FPCGColumnData>& InColumnData,
		const EColumnSortMode::Type InSortMode,
		const FName InSortingColumn,
		const TSharedPtr<FTextFilterExpressionEvaluator>& InTextFilter)
	: ListViewItems(InListViewItems)
	, ColumnData(InColumnData)
	, SortMode(InSortMode)
	, SortingColumn(InSortingColumn)
	, TextFilter(InTextFilter)
	{}

	bool IsCompleted() const;
	void Launch();

	TArray<PCGListViewItemPtr> ListViewItems;

private:
	void AsyncSort();
	void AsyncFilter();

	TMap<FName, FPCGColumnData> ColumnData;

	EColumnSortMode::Type SortMode = EColumnSortMode::Type::Ascending;
	FName SortingColumn = NAME_None;

	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;

	UE::Tasks::FTask UpdateTask;
};

class SPCGListViewItemRow : public SMultiColumnTableRow<PCGListViewItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SPCGListViewItemRow) {}
	SLATE_ARGUMENT(TSharedPtr<SPCGEditorGraphAttributeListView>, AttributeListView)
	SLATE_ARGUMENT(PCGListViewItemPtr, ListViewItem)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

	static void OnSoftObjectPathHyperlinkClicked(const FText& InText);

private:
	TWeakPtr<SPCGEditorGraphAttributeListView> AttributeListView;
	PCGListViewItemPtr InternalItem;
};

class FPCGPointFilterExpressionContext : public ITextFilterExpressionContext
{
public:
	explicit FPCGPointFilterExpressionContext(const FPCGListViewItem* InRowItem, const TMap<FName, FPCGColumnData>* InPCGColumnData);

	// ~Begin ITextFilterExpressionContext interface
	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override;
	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override;
	// ~End ITextFilterExpressionContext interface

private:
	const FPCGListViewItem* RowItem = nullptr;
	const TMap<FName, FPCGColumnData>* PCGColumnData = nullptr;
};

class SPCGEditorGraphAttributeListView : public SCompoundWidget
{
	friend SPCGListViewItemRow;

public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphAttributeListView) {}
		SLATE_ARGUMENT(int32, WidgetEntryNumber)
	SLATE_END_ARGS()

	virtual ~SPCGEditorGraphAttributeListView();

	void Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void RequestRefresh();
	void RequestViewportRefresh();

	UPCGEditorGraphNodeBase* GetNodeBeingInspected() const;
	void SetNodeBeingInspected(UPCGEditorGraphNodeBase* InNode);

	bool IsLocked() const { return bIsLocked; }
	void SetIsLocked(bool bInIsLocked) { bIsLocked = bInIsLocked; }

	TSharedPtr<SPCGEditorViewport> GetViewportWidget() const { return ViewportWidget; }
	void SetViewportWidget(TSharedPtr<SPCGEditorViewport> InViewportWidget, FName InViewportEditorPanel);

	/** Resets/clears the viewport scene. */
	void ResetViewport();

	/** Add UObject references for GC */
	void AddReferencedObjects(FReferenceCollector& Collector);
private:
	void OnTick();
	TSharedRef<SHeaderRow> CreateHeaderRowWidget() const;

	/** Read back any GPU data in the inspection data. Returns true when read backs complete. */
	bool ReadbackGPUData() const;

	void OnInspectedStackChanged(const FPCGStack& InPCGStack);

	void ResizeColumnToMaxWidth(FName InColumnId);
	void ResetColumnsWidthToDefault();
	void ExpendAllColumnToMaxWidth();
	void CacheColumnWidthVisibility();
	void RestoreColumnWidthVisibility();

	void RefreshAttributeList();
	void RefreshPinComboBox(bool bKeepSelection, bool& bOutSelectionChanged);
	void RefreshDataComboBox(bool bKeepSelection);
	void RefreshDomainComboBox(bool bKeepSelection);
	void RefreshViewport();

	void LaunchUpdateTask();

	/** Only connected input pins are added to combo box, so keep track of the node pin index for each item. */
	struct FPinComboBoxItem
	{
		explicit FPinComboBoxItem(FName InName, int32 InPinIndex, bool bInIsOutputPin)
			: Name(InName)
			, PinIndex(InPinIndex)
			, bIsOutputPin(bInIsOutputPin)
		{}

		FName Name;
		int32 PinIndex = INDEX_NONE;
		bool bIsOutputPin = true;
	};

	TSharedPtr<FPCGInspectionData> GetInspectionDataPtr(const TSharedPtr<FPinComboBoxItem>& EditorPin) const;
	bool InspectData(const TSharedPtr<FPinComboBoxItem>& EditorPin, TFunctionRef<void(const FPCGDataCollection&)> InspectFunc) const;
	const FPCGDataCollection* GetInspectionData() const;

	FText OnGenerateSelectedPinText() const;
	void OnSelectionChangedPin(TSharedPtr<FPinComboBoxItem> InItem, ESelectInfo::Type InSelectInfo);
	TSharedRef<SWidget> OnGeneratePinWidget(TSharedPtr<FPinComboBoxItem> InItem) const;

	const FSlateBrush* GetFilterBadgeIcon() const;
	TSharedRef<SWidget> OnGenerateFilterMenu();
	TSharedRef<SWidget> OnGenerateAdditionalOperationsMenu();
	TSharedRef<SWidget> OnGenerateDataWidget(TSharedPtr<FString> InItem) const;
	void OnSelectionChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo);
	FText OnGenerateSelectedDataText() const;
	FText OnGenerateSelectedDomainText() const;
	int32 GetSelectedDataIndex() const;
	int32 GetSelectedDomainIndex() const;

	void ToggleAllAttributes();
	void ToggleAttribute(FName InAttributeName);
	ECheckBoxState GetAnyAttributeEnabledState() const;
	bool IsAttributeEnabled(FName InAttributeName) const;

	bool BuildDataCollectionForSave(bool bUsePinComboIndex, bool bUseDataComboIndex, FPCGDataCollection* OutData) const;
	void SaveData(bool bUsePinIndex, bool bUseDataIndex);
	bool CanSaveData(bool bUsePinIndex, bool bUseDataIndex) const;

	void OnToggleShowDefaultValue();
	bool IsShowingDefaultValue() const;

	TSharedRef<ITableRow> OnGenerateRow(PCGListViewItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnItemDoubleClicked(PCGListViewItemPtr Item) const;
	TSharedPtr<SWidget> OnItemsContextMenu();

	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);
	EColumnSortMode::Type GetColumnSortMode(const FName InColumnId) const;
	TSharedRef<SWidget> GenerateColumnMenu(FName ColumnId);

	void OnFilterTextChanged(const FText& InFilterText);
	void OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	FReply OnListViewKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) const;

	void AddColumn(const FPCGTableVisualizerColumnInfo& InColumnInfo);

	void CopySelectionToClipboard() const;
	bool CanCopySelectionToClipboard() const;

	/** @return the Slate brush to use for the lock image */
	const FSlateBrush* OnGetLockButtonImageResource() const;

	FReply OnLockClick();
	FReply OnNodeNameClicked();

	FReply OnFocusOnDataClicked() const;
	bool IsFocusOnDataEnabled() const;

	void FocusOnSelection() const;
	bool CanFocusOnSelection() const;

	bool IsViewportOpen() const;
	bool IsOpen() const;

	/** Returns the current PCG component */
	TWeakInterfacePtr<IPCGGraphExecutionSource> GetExecutionSource() const;

	/** Index of the entry of this widget, to know which entry to query. */
	int32 WidgetEntryNumber = 0;

	/** Pointer back to the PCG editor that owns us */
	TWeakPtr<FPCGEditor> PCGEditorPtr;

	/** Data visualization viewport. */
	TSharedPtr<SPCGEditorViewport> ViewportWidget;

	/** Data visualization viewport editor panel id. */
	FName ViewportEditorPanel;

	/** Cached PCGGraphNode being viewed */
	TWeakObjectPtr<UPCGEditorGraphNodeBase> PCGEditorGraphNode;
	bool bPCGEditorGraphNodeChanged = false;

	TSharedPtr<FUICommandList> ListViewCommands;

	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;

	TSharedPtr<SSearchBox> SearchBoxWidget;
	TSharedPtr<SHeaderRow> ListViewHeader;
	TSharedPtr<SListView<PCGListViewItemPtr>> ListView;
	TArray<PCGListViewItemPtr> ListViewItems;
	TArray<PCGListViewItemPtr> FilteredListViewItems;

	TSharedPtr<SComboBox<TSharedPtr<FPinComboBoxItem>>> PinComboBox;
	TArray<TSharedPtr<FPinComboBoxItem>> PinComboBoxItems;

	TSharedPtr<SComboBox<TSharedPtr<FString>>> DataComboBox;
	TArray<TSharedPtr<FString>> DataComboBoxItems;

	/** Cached Selected Data ComboBox Item */
	int32 DataComboBoxItemsSelectedIndex = INDEX_NONE;

	TSharedPtr<SComboBox<TSharedPtr<FString>>> DomainsComboBox;
	TArray<TSharedPtr<FString>> DomainsComboBoxItems;
	TArray<FPCGMetadataDomainID> DomainsComboBoxIds;

	/** Cached Selected Domain */
	FPCGMetadataDomainID DomainsComboBoxItemsSelectedDomain = PCGMetadataDomainID::Invalid;

	TSharedPtr<STextBlock> NodeNameTextBlock;
	TSharedPtr<STextBlock> InfoTextBlock;

	TArray<FName> HiddenAttributes;

	TMap<FName, FPCGColumnData> PCGColumnData;
	
	/** Contains info of the max width to display the full content of the column. Computed only for string types. */
	TMap<FName, float> ColumnsMaxWidthMapping;

	/** Ring buffer to keep the latest inspected nodes column width and visibility. */
	using FNodeKeyToColumnWidthVisibilityMap = TTuple<TObjectKey<UPCGEditorGraphNodeBase>, TMap<FName, TTuple<float, bool>>>;
	TRingBuffer<FNodeKeyToColumnWidthVisibilityMap> ColumnWidthVisibilityCache;

	FText ActiveFilterText;

	FName SortingColumn = NAME_None;
	EColumnSortMode::Type SortMode = EColumnSortMode::Type::Ascending;
	TFunction<void(const UPCGData*, TArrayView<const int>)> FocusOnDataCallback;

	bool bNeedsRefresh : 1 = false;
	bool bViewportNeedsRefresh : 1 = false;
	bool bTickForced : 1 = false;

	bool bKeepDataComboBoxSelection = true;

	/** True if this property view is currently locked (i.e. The objects being observed are not changed automatically due to user selection)*/
	bool bIsLocked : 1 = false;

	bool bShowDefaultValue : 1 = false;

	TSharedPtr<FPCGListViewUpdater> CurrentUpdateTask = nullptr;

	/** In some cases, like when inspecting temporary collapsed point data, there is no owner of the data.
	* Since the ALV needs the data for visualization and double-click functionality,
	* it stores a reference to the data to keep it alive during inspection.
	*/
	TObjectPtr<const UPCGData> DataPtr = nullptr;

	/** Handles for any resources the data needs to load before visualization. */
	TArray<TSharedPtr<FStreamableHandle>> LoadHandles;
	bool bRefreshLoadHandles : 1 = false;
};
