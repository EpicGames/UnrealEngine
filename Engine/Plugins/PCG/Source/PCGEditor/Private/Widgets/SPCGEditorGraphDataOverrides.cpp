// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphDataOverrides.h"

#include "PCGEditor.h"
#include "PCGEditorModule.h"
#include "PCGEditorStyle.h"
#include "PCGElement.h"
#include "PCGGraphExecutionInspection.h"
#include "PCGNode.h"
#include "DeltaVisualizations/PCGDeltaVisualizationRegistry.h"
#include "Graph/PCGSourceDataContainer.h"
#include "Graph/DataOverride/PCGDataOverrideHelpers.h"
#include "Nodes/PCGEditorGraphNodeBase.h"

#include "Editor.h"
#include "Algo/AnyOf.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "PCGDataOverridesListView"

namespace PCGDataOverride::Constants
{
	static constexpr float CellPadding = 8.f;
	static constexpr float ScrollBarThickness = 12.f;

	const FText ClearOverride = LOCTEXT("ClearOverride", "Clear Override");
	const FText ClearOverrides = LOCTEXT("ClearOverrides", "Clear Overrides");
	const FText ClearOrphanedOverrides = LOCTEXT("ClearOrphanedOverrides", "Clear Orphaned Overrides");

	const FName OrphanedOverrideRowStyleName = "PCG.OrphanedOverrideRow";
}

void SPCGDataOverrideRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = InArgs._Item;
	check(Item.IsValid());

	FSuperRowType::FArguments Arguments;
	if (Item->OrphanStatus == EPCGDeltaOrphanStatus::Orphaned)
	{
		Arguments.Style(&FPCGEditorStyle::Get().GetWidgetStyle<FTableRowStyle>(PCGDataOverride::Constants::OrphanedOverrideRowStyleName));
	}
	else
	{
		Arguments.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow");
	}

	SMultiColumnTableRow::Construct(Arguments, InOwnerTableView);
}

TSharedRef<SWidget> SPCGDataOverrideRow::GenerateWidgetForColumn(const FName& InColumnID)
{
	if (!Item.IsValid() || !Item->Visualizer)
	{
		return SNullWidget::NullWidget;
	}

	// Find and check for custom widget first.
	const FConstStructView DeltaView(Item->DeltaStruct.GetScriptStruct(), Item->DeltaStruct.GetMemory());
	const TSharedPtr<SWidget> CustomWidget = Item->Visualizer->CreateCellWidget(InColumnID, Item->DeltaKey, DeltaView);
	if (CustomWidget.IsValid())
	{
		return CustomWidget.ToSharedRef();
	}

	// Fall back to text.
	const FText CellText = Item->Visualizer->GetCellText(InColumnID, Item->DeltaKey, DeltaView);

	return SNew(SBox)
		.Padding(FMargin(PCGDataOverride::Constants::CellPadding, 0))
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(CellText)
		];
}

void SPCGEditorGraphDataOverridesView::Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor)
{
	check(InPCGEditor.IsValid());

	PCGEditorPtr = InPCGEditor;

	InPCGEditor->GetInspectionDataManager().OnInspectedStackChangedDelegate.AddSP(this, &SPCGEditorGraphDataOverridesView::OnInspectedStackChanged);

	namespace Constants = PCGDataOverride::Constants;

	// Start with an empty header row; columns are populated during RefreshDataOverrides.
	HeaderRow = SNew(SHeaderRow);

	const TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal)
		.Thickness(FVector2D(Constants::ScrollBarThickness));

	const TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(Constants::ScrollBarThickness));

	SAssignNew(ListView, SListView<FPCGDeltaOverrideItemPtr>)
		.ListItemsSource(&ListViewItems)
		.OnGenerateRow(this, &SPCGEditorGraphDataOverridesView::OnGenerateRow)
		.OnContextMenuOpening(this, &SPCGEditorGraphDataOverridesView::OnListViewContextMenuOpening)
		.HeaderRow(HeaderRow)
		.SelectionMode(ESelectionMode::Single)
		.AllowOverscroll(EAllowOverscroll::No)
		.ExternalScrollbar(VerticalScrollBar)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
			   .ButtonStyle(FAppStyle::Get(), "SimpleButton")
			   .ToolTipText(LOCTEXT("ClearOrphanedOverridesTooltip", "Clear all orphaned overrides for this node (deltas that failed to resolve in the last execution)"))
			   .ContentPadding(FMargin(2, 2))
			   .IsEnabled(this, &SPCGEditorGraphDataOverridesView::HasOrphanedDeltas)
			   .OnClicked(this, &SPCGEditorGraphDataOverridesView::OnClearOrphanedOverrides)
				[
					SNew(SImage)
				   .Image(this, &SPCGEditorGraphDataOverridesView::GetClearOrphanedOverridesBrush)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("ClearNodeOverridesTooltip", "Clear all overrides for this node"))
				.ContentPadding(FMargin(2, 2))
				.OnClicked(this, &SPCGEditorGraphDataOverridesView::OnClearNodeOverrides)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				.ExternalScrollbar(HorizontalScrollBar)
				+ SScrollBox::Slot()
				[
					ListView.ToSharedRef()
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				VerticalScrollBar
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			HorizontalScrollBar
		]
	];
}

SPCGEditorGraphDataOverridesView::~SPCGEditorGraphDataOverridesView()
{
	if (const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin())
	{
		PCGEditor->GetInspectionDataManager().OnInspectedStackChangedDelegate.RemoveAll(this);
	}
}

void SPCGEditorGraphDataOverridesView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bNeedsRefresh)
	{
		bNeedsRefresh = false;
		RefreshDataOverrides();
	}
}

void SPCGEditorGraphDataOverridesView::SetNodeBeingInspected(UPCGEditorGraphNodeBase* InNode)
{
	if (PCGEditorGraphNode.Get() == InNode)
	{
		return;
	}

	PCGEditorGraphNode = InNode;
	RequestRefresh();
}

UPCGEditorGraphNodeBase* SPCGEditorGraphDataOverridesView::GetNodeBeingInspected() const
{
	return PCGEditorGraphNode.Get();
}

void SPCGEditorGraphDataOverridesView::ClearItems()
{
	ListViewItems.Empty();
	RequestRefresh();
}

void SPCGEditorGraphDataOverridesView::RequestRefresh()
{
	bNeedsRefresh = true;
}

TSharedRef<ITableRow> SPCGEditorGraphDataOverridesView::OnGenerateRow(FPCGDeltaOverrideItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SPCGDataOverrideRow, InOwnerTable).Item(InItem);
}

void SPCGEditorGraphDataOverridesView::OnInspectedStackChanged(const FPCGStack& InPCGStack)
{
	RequestRefresh();
}

void SPCGEditorGraphDataOverridesView::RebuildHeaderRow()
{
	if (!HeaderRow.IsValid())
	{
		return;
	}

	HeaderRow->ClearColumns();

	// Add columns from the merged set.
	for (const FPCGDeltaVisualizerColumnInfo& ColumnInfo : ActiveColumns)
	{
		SHeaderRow::FColumn::FArguments ColumnArgs;
		ColumnArgs.ColumnId(ColumnInfo.Id);
		ColumnArgs.DefaultLabel(ColumnInfo.Label);
		ColumnArgs.DefaultTooltip(ColumnInfo.Tooltip);

		if (ColumnInfo.Width > 0.f)
		{
			ColumnArgs.ManualWidth(ColumnInfo.Width);
		}
		else
		{
			ColumnArgs.FillWidth(1.0f);
		}

		switch (ColumnInfo.CellAlignment)
		{
			case EPCGTableVisualizerCellAlignment::Left:
				ColumnArgs.HAlignCell(HAlign_Left);
				break;
			case EPCGTableVisualizerCellAlignment::Center:
				ColumnArgs.HAlignCell(HAlign_Center);
				ColumnArgs.HAlignHeader(HAlign_Center);
				break;
			case EPCGTableVisualizerCellAlignment::Right:
				ColumnArgs.HAlignCell(HAlign_Right);
				break;
			default:
				break;
		}

		HeaderRow->AddColumn(ColumnArgs);
	}
}

void SPCGEditorGraphDataOverridesView::RefreshDataOverrides()
{
	ListViewItems.Empty();
	ActiveColumns.Empty();

	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();
	if (!PCGEditor.IsValid() || !PCGEditorGraphNode.IsValid())
	{
		RebuildHeaderRow();

		if (ListView.IsValid())
		{
			ListView->RequestListRefresh();
		}

		return;
	}

	const FPCGEditorInspectionDataManager& InspectionDataManager = PCGEditor->GetInspectionDataManager();

	IPCGGraphExecutionSource* ExecutionSource = InspectionDataManager.GetPCGSourceBeingInspected();
	const FPCGSourceDataContainer* DataContainer = ExecutionSource ? ExecutionSource->GetExecutionState().GetSourceDataContainer() : nullptr;

	const UPCGNode* PCGNode = PCGEditorGraphNode->GetPCGNode();

	if (!DataContainer || !PCGNode)
	{
		RebuildHeaderRow();

		if (ListView.IsValid())
		{
			ListView->RequestListRefresh();
		}

		return;
	}

	// Get the execution stacks for this node. These are the actual runtime stacks captured during graph execution,
	// which is the same stack source that the storage side uses. There exists a hash mismatch that occurs when
	// using the inspection stack (GetStackBeingInspected), which may be reconstructed with different frames.
	const FPCGGraphExecutionInspection& Inspection = ExecutionSource->GetExecutionState().GetInspection();
	const TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData> StackSet = Inspection.GetExecutedNodeStacks(PCGNode);
	if (StackSet.IsEmpty())
	{
		RebuildHeaderRow();

		if (ListView.IsValid())
		{
			ListView->RequestListRefresh();
		}

		return;
	}

	const FPCGDeltaVisualizationRegistry& Registry = FPCGEditorModule::GetConstDeltaVisualizationRegistry();

	// Track which column IDs have already been added to avoid duplicates from multiple visualizers.
	TSet<FName> TrackedColumnIds;
	TSet<const IPCGDeltaVisualization*> TrackedVisualizers;

	using namespace PCG::DataOverride;

	// Process a delta collection into list view items and merge columns.
	auto ProcessDeltaCollection = [this, &Registry, &TrackedColumnIds, &TrackedVisualizers](const FPCGDeltaCollection* DeltaCollection, FName PinLabel, const FPCGSourceDataStorageKey& StorageKey)
	{
		DeltaCollection->ForEachDelta([this, &Registry, &TrackedColumnIds, &TrackedVisualizers, PinLabel, &StorageKey](const FPCGDeltaKey& DeltaKey, const TInstancedStruct<FPCGDeltaBase>& DeltaStruct) -> bool
		{
			const UScriptStruct* ScriptStruct = DeltaStruct.GetScriptStruct();
			const IPCGDeltaVisualization* Visualizer = Registry.GetDeltaVisualization(ScriptStruct);

			// Merge visualizer columns into ActiveColumns (deduplicate by Id). Skip if this visualizer is already tracked.
			bool bVisualizerIsAlreadyTracked = false;
			if (Visualizer)
			{
				TrackedVisualizers.Add(Visualizer, &bVisualizerIsAlreadyTracked);
			}

			if (Visualizer && !bVisualizerIsAlreadyTracked)
			{
				for (const FPCGDeltaVisualizerColumnInfo& ColumnInfo : Visualizer->GetColumnInfos())
				{
					bool bColumnAlreadyTracked = false;
					TrackedColumnIds.Add(ColumnInfo.Id, &bColumnAlreadyTracked);
					if (!bColumnAlreadyTracked)
					{
						ActiveColumns.Add(ColumnInfo);
					}
				}
			}

			// Create the item.
			TSharedPtr<FPCGDeltaOverrideItem> NewItem = MakeShared<FPCGDeltaOverrideItem>();
			NewItem->DeltaKey = DeltaKey;
			NewItem->DeltaStruct = DeltaStruct; // Copy
			NewItem->Visualizer = Visualizer;
			NewItem->PinLabel = PinLabel;
			NewItem->OrphanStatus = EPCGDeltaOrphanStatus::Unknown;
			NewItem->StorageKey = StorageKey;

			ListViewItems.Emplace(MoveTemp(NewItem));
			return true;
		});
	};

	const TArray<TObjectPtr<UPCGPin>>* OverridePins = Helpers::GetOverridePins(PCGNode);

	// Iterate each execution stack for this node (there may be multiple in loop/subgraph scenarios).
	for (const FPCGGraphExecutionInspection::FNodeExecutedNotificationData& StackData : StackSet)
	{
		const FPCGStack& ExecutionStack = StackData.Stack;
		const FXxHash64 NodeKey = Helpers::ComputeNodeOverrideKey(ExecutionStack, PCGNode);

		// Try pin-level keys for each override pin.
		bool bFoundAnyPinData = false;
		if (OverridePins)
		{
			for (const UPCGPin* Pin : *OverridePins)
			{
				if (!Pin)
				{
					continue;
				}

				const FName PinLabel = Pin->Properties.Label;
				const FPCGSourceDataStorageKey PinDataKey(Constants::DefaultOverrideLabel, Helpers::ComputePinOverrideKey(NodeKey, PinLabel).Hash);
				FConstSharedStruct SharedStruct = DataContainer->Get<FPCGDeltaCollection>(PinDataKey);
				const FPCGDeltaCollection* DeltaCollection = SharedStruct.IsValid() ? SharedStruct.GetPtr<const FPCGDeltaCollection>() : nullptr;

				if (!DeltaCollection || DeltaCollection->IsEmpty())
				{
					continue;
				}

				bFoundAnyPinData = true;
				ProcessDeltaCollection(DeltaCollection, PinLabel, PinDataKey);
			}
		}

		// Fall back to node-level key if no pin-level data was found.
		if (!bFoundAnyPinData)
		{
			const FPCGSourceDataStorageKey NodeDataKey(Constants::DefaultOverrideLabel, NodeKey.Hash);
			FConstSharedStruct SharedStruct = DataContainer->Get<FPCGDeltaCollection>(NodeDataKey);
			const FPCGDeltaCollection* DeltaCollection = SharedStruct.IsValid() ? SharedStruct.GetPtr<const FPCGDeltaCollection>() : nullptr;

			if (DeltaCollection && !DeltaCollection->IsEmpty())
			{
				ProcessDeltaCollection(DeltaCollection, NAME_None, NodeDataKey);
			}
		}
	}

	// Populate orphan status from the resolution set built during the last execution.
	for (const FPCGDeltaOverrideItemPtr& Item : ListViewItems)
	{
		if (!Item.IsValid())
		{
			continue;
		}

		Item->OrphanStatus = Inspection.IsDeltaResolved(Item->StorageKey, Item->DeltaKey) ? EPCGDeltaOrphanStatus::Resolved : EPCGDeltaOrphanStatus::Orphaned;
	}

	RebuildHeaderRow();

	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}
}

// Removes all delta collections (pin and node level) for the inspected node, then triggers re-execution.
FReply SPCGEditorGraphDataOverridesView::OnClearNodeOverrides()
{
	using namespace PCG::DataOverride;

	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();
	if (!PCGEditor.IsValid() || !PCGEditorGraphNode.IsValid())
	{
		return FReply::Handled();
	}

	IPCGGraphExecutionSource* ExecutionSource = PCGEditor->GetInspectionDataManager().GetPCGSourceBeingInspected();
	FPCGSourceDataContainer* DataContainer = Helpers::GetMutableSourceDataContainer(ExecutionSource);
	const UPCGNode* PCGNode = PCGEditorGraphNode->GetPCGNode();
	if (!DataContainer || !PCGNode)
	{
		return FReply::Handled();
	}

	// Execution sources aren't always UObjects, so wrap in a manual transaction only when possible.
	UObject* SourceObject = Cast<UObject>(ExecutionSource);
	if (SourceObject)
	{
		GEditor->BeginTransaction(LOCTEXT("ClearNodeOverrides", "PCG: Clear All Node Overrides"));
		SourceObject->Modify();
	}

	for (const FPCGSourceDataStorageKey& Key : Helpers::CollectNodeStorageKeys(PCGNode, ExecutionSource))
	{
		DataContainer->Remove(Key);
	}

	DataContainer->MarkDirty();

	if (SourceObject)
	{
		GEditor->EndTransaction();
	}

	Helpers::RefreshExecutionSource(ExecutionSource);

	RequestRefresh();

	return FReply::Handled();
}

const FSlateBrush* SPCGEditorGraphDataOverridesView::GetClearOrphanedOverridesBrush() const
{
	return FAppStyle::Get().GetBrush(HasOrphanedDeltas() ? "Icons.Unlink" : "Icons.Link");
}

bool SPCGEditorGraphDataOverridesView::HasOrphanedDeltas() const
{
	return Algo::AnyOf(ListViewItems, [](const FPCGDeltaOverrideItemPtr& Item)
	{
		return Item.IsValid() && Item->OrphanStatus == EPCGDeltaOrphanStatus::Orphaned;
	});
}

// Removes only delta overrides flagged as orphaned in the last execution.
FReply SPCGEditorGraphDataOverridesView::OnClearOrphanedOverrides()
{
	TArray<FPCGDeltaOverrideItemPtr> OrphanedItems;
	OrphanedItems.Reserve(ListViewItems.Num());
	for (const FPCGDeltaOverrideItemPtr& Item : ListViewItems)
	{
		if (Item.IsValid() && Item->OrphanStatus == EPCGDeltaOrphanStatus::Orphaned)
		{
			OrphanedItems.Add(Item);
		}
	}

	if (!OrphanedItems.IsEmpty())
	{
		RemoveDeltaItems(MoveTemp(OrphanedItems), PCGDataOverride::Constants::ClearOrphanedOverrides);
	}

	return FReply::Handled();
}

TSharedPtr<SWidget> SPCGEditorGraphDataOverridesView::OnListViewContextMenuOpening()
{
	if (!ListView.IsValid())
	{
		return nullptr;
	}

	TArray<FPCGDeltaOverrideItemPtr> SelectedItems = ListView->GetSelectedItems();
	SelectedItems.RemoveAll([](const FPCGDeltaOverrideItemPtr& Item) { return !Item.IsValid(); });
	if (SelectedItems.IsEmpty())
	{
		return nullptr;
	}

	using namespace PCGDataOverride::Constants;
	const FText& Label = (SelectedItems.Num() > 1) ? ClearOverrides : ClearOverride;

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, /*CommandList=*/nullptr);
	MenuBuilder.BeginSection("PCGDataOverridesContextMenu", LOCTEXT("DataOverridesContextMenuHeader", "Data Override"));
	{
		MenuBuilder.AddMenuEntry(
			Label,
			LOCTEXT("ClearOverrideTooltip", "Removes the selected delta overrides from the inspected node and re-executes the graph"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
			FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphDataOverridesView::OnClearSelectedOverrides, MoveTemp(SelectedItems))));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

// Removes a specific set of delta overrides from their owning collections and triggers re-execution.
void SPCGEditorGraphDataOverridesView::OnClearSelectedOverrides(TArray<FPCGDeltaOverrideItemPtr> InItems)
{
	using namespace PCGDataOverride::Constants;
	const FText& TransactionText = (InItems.Num() > 1) ? ClearOverrides : ClearOverride;
	RemoveDeltaItems(MoveTemp(InItems), TransactionText);
}

void SPCGEditorGraphDataOverridesView::RemoveDeltaItems(TArray<FPCGDeltaOverrideItemPtr> InItems, const FText& InTransactionText)
{
	if (InItems.IsEmpty())
	{
		return;
	}

	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();
	IPCGGraphExecutionSource* ExecutionSource = PCGEditor.IsValid() ? PCGEditor->GetInspectionDataManager().GetPCGSourceBeingInspected() : nullptr;
	FPCGSourceDataContainer* DataContainer = ExecutionSource ? ExecutionSource->GetExecutionState().GetSourceDataContainer() : nullptr;
	if (!DataContainer)
	{
		return;
	}

	// Wrap in a manual transaction only when possible.
	UObject* SourceObject = Cast<UObject>(ExecutionSource);
	if (SourceObject)
	{
		GEditor->BeginTransaction(InTransactionText);
		SourceObject->Modify();
	}

	for (const FPCGDeltaOverrideItemPtr& Item : InItems)
	{
		if (!Item.IsValid())
		{
			continue;
		}

		FSharedStruct MutableCollection = DataContainer->GetMutable<FPCGDeltaCollection>(Item->StorageKey);
		FPCGDeltaCollection* DeltaCollection = MutableCollection.GetPtr<FPCGDeltaCollection>();
		if (!DeltaCollection)
		{
			continue;
		}

		DeltaCollection->Remove(Item->DeltaKey);

		// Drop the empty collection so its CRC contribution disappears entirely.
		if (DeltaCollection->IsEmpty())
		{
			DataContainer->Remove(Item->StorageKey);
		}
	}

	DataContainer->MarkDirty();

	if (SourceObject)
	{
		GEditor->EndTransaction();
	}

	// Re-execute only the inspected source instead of every component using this graph.
	PCG::DataOverride::Helpers::RefreshExecutionSource(ExecutionSource);

	RequestRefresh();
}

#undef LOCTEXT_NAMESPACE
