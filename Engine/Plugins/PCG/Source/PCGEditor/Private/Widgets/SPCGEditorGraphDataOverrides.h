// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeltaVisualizations/PCGDeltaVisualization.h"
#include "Graph/PCGSourceDataContainer.h"
#include "Graph/DataOverride/PCGDataOverride.h"

#include "StructUtils/InstancedStruct.h"
#include "Widgets/Views/STableRow.h"

class FPCGEditor;
class IPCGDeltaVisualization;
class UPCGEditorGraphNodeBase;
struct FPCGStack;
struct FSlateBrush;

/** Whether the delta was consumed. Used for override detection. */
enum class EPCGDeltaOrphanStatus : uint8
{
	Unknown,
	Resolved,
	Orphaned
};

/** View model for a single row in the Data Overrides table. Pairs a delta key/struct with its resolved visualizer. */
struct FPCGDeltaOverrideItem
{
	FPCGDeltaKey DeltaKey;
	TInstancedStruct<FPCGDeltaBase> DeltaStruct;
	const IPCGDeltaVisualization* Visualizer = nullptr;
	FName PinLabel;
	EPCGDeltaOrphanStatus OrphanStatus = EPCGDeltaOrphanStatus::Unknown;
	/** Data storage key for the collection that owns this delta. */
	FPCGSourceDataStorageKey StorageKey;
};

typedef TSharedPtr<FPCGDeltaOverrideItem> FPCGDeltaOverrideItemPtr;

/** Table row widget. Populated by a registered IPCGDeltaVisualization. */
class SPCGDataOverrideRow : public SMultiColumnTableRow<FPCGDeltaOverrideItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SPCGDataOverrideRow) {}
		SLATE_ARGUMENT(FPCGDeltaOverrideItemPtr, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnID) override;

private:
	FPCGDeltaOverrideItemPtr Item;
};

/**
 * Data Overrides tab widget. Displays a table of delta overrides (manual edits) for the currently inspected node. Reads
 * delta collections from the execution source's data container using an identical key signature.
 */
class SPCGEditorGraphDataOverridesView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphDataOverridesView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor);

	virtual ~SPCGEditorGraphDataOverridesView() override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// To mirror the ALV
	void SetNodeBeingInspected(UPCGEditorGraphNodeBase* InNode);
	UPCGEditorGraphNodeBase* GetNodeBeingInspected() const;

	void ClearItems();
	void RequestRefresh();

private:
	FReply OnClearNodeOverrides();
	FReply OnClearOrphanedOverrides();
	bool HasOrphanedDeltas() const;
	const FSlateBrush* GetClearOrphanedOverridesBrush() const;
	void OnClearSelectedOverrides(TArray<FPCGDeltaOverrideItemPtr> InItems);
	void RemoveDeltaItems(TArray<FPCGDeltaOverrideItemPtr> InItems, const FText& InTransactionText);

	TSharedRef<ITableRow> OnGenerateRow(FPCGDeltaOverrideItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable);
	TSharedPtr<SWidget> OnListViewContextMenuOpening();

	void OnInspectedStackChanged(const FPCGStack& InPCGStack);
	void RefreshDataOverrides();
	void RebuildHeaderRow();

	TWeakObjectPtr<UPCGEditorGraphNodeBase> PCGEditorGraphNode;
	bool bNeedsRefresh = false;

	TWeakPtr<FPCGEditor> PCGEditorPtr;
	TSharedPtr<SHeaderRow> HeaderRow;
	TSharedPtr<SListView<FPCGDeltaOverrideItemPtr>> ListView;
	TArray<FPCGDeltaOverrideItemPtr> ListViewItems;

	/** Merged column set from all visualizers active in the current view. */
	TArray<FPCGDeltaVisualizerColumnInfo> ActiveColumns;
};
