// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphDebugObjectTree.h"

#include "PCGComponent.h"
#include "PCGDefaultExecutionSource.h"
#include "PCGGraph.h"
#include "PCGGraphExecutionRegistry.h"
#include "PCGModule.h"
#include "PCGSubsystem.h"
#include "Editor/IPCGEditorModule.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "PCGEditorStyle.h"
#include "AssetDefinitions/AssetDefinition_PCGGraphInterface.h"

#include "PropertyCustomizationHelpers.h"
#include "Selection.h"
#include "UnrealEdGlobals.h"
#include "Algo/AnyOf.h"
#include "Editor/UnrealEdEngine.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "SPCGEditorGraphDebugObjectTree"

namespace PCGEditorGraphDebugObjectTree
{
	void GetRowIconState(FPCGEditorGraphDebugObjectItemPtr Item, const FSlateBrush*& OutBrush, FLinearColor& OutColorAndOpacity)
	{
		OutBrush = Item->GetIcon();

		const FLinearColor DefaultGraphColor = CastChecked<UAssetDefinition_PCGGraphInterface>(UAssetDefinition_PCGGraphInterface::StaticClass()->GetDefaultObject(false))->GetAssetColor();

		OutColorAndOpacity = Item->IsDebuggable() ? DefaultGraphColor : FLinearColor(1.0f, 1.0f, 1.0f, 0.5f);
	}

	bool StackContainsErrorOrWarning(const FPCGStack* InStack)
	{
		if (!InStack)
		{
			return false;
		}

		if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
		{
			bool bHasErrorOrWarning = false;
			PCGEditorModule->GetNodeVisualLogs().ForAllMatchingLogs(*InStack, [&bHasErrorOrWarning](const FPCGStack& InStack, const FPCGPerNodeVisualLogs& InLogs)
			{
				for(const FPCGNodeLogEntry& LogEntry : InLogs)
				{
					if (LogEntry.Verbosity > ELogVerbosity::NoLogging && LogEntry.Verbosity <= ELogVerbosity::Warning)
					{
						bHasErrorOrWarning = true;
						return false;
					}
				}

				return true;
			});

			return bHasErrorOrWarning;
		}

		return false;
	}

	void GetErrorInfoInternal(FPCGEditorGraphDebugObjectItemPtr Item, ELogVerbosity::Type& OutMinVerbosity, ELogVerbosity::Type& OutMinLocalVerbosity, FPCGPerNodeVisualLogs& OutLogs)
	{
		if (!Item || !ensure(Item->GetPCGStack()))
		{
			return;
		}

		if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
		{
			PCGEditorModule->GetNodeVisualLogs().ForAllMatchingLogs(*Item->GetPCGStack(), [&Item, &OutLogs, &OutMinVerbosity, &OutMinLocalVerbosity](const FPCGStack& InStack, const FPCGPerNodeVisualLogs& InLogs)
			{
				// Need to verify this, and clarify this when it's a loop item.
				// i.e. should get the graph + loop index and compare
				const bool bIsLocal = Item->GetPCGStack()->GetGraphForCurrentFrame() == InStack.GetGraphForCurrentFrame();

				for (const FPCGNodeLogEntry& LogEntry : InLogs)
				{
					OutMinVerbosity = FMath::Min(OutMinVerbosity, LogEntry.Verbosity);

					if (bIsLocal)
					{
						OutMinLocalVerbosity = FMath::Min(OutMinLocalVerbosity, LogEntry.Verbosity);
					}
				}

				OutLogs.Append(InLogs);
				return true;
			});
		}
	}

	/** Recursively check all descendants for errors/warnings, propagating min verbosity and unique logs upward.
	 *  Uses AddUnique to avoid duplicate tooltip entries when a parent's ForAllMatchingLogs (BeginsWith) already captured the same logs that a child query would return.
	 */
	void GetSubtreeErrorInfo(const FPCGEditorGraphDebugObjectItemPtr& Item, ELogVerbosity::Type& OutMinVerbosity, FPCGPerNodeVisualLogs& OutLogs)
	{
		for (const FPCGEditorGraphDebugObjectItemPtr& Child : Item->GetChildren())
		{
			// Discard local verbosity from children. It controls row text color which should only reflect errors at the item's own graph level, not inherited from descendants.
			ELogVerbosity::Type DummyLocalVerbosity = ELogVerbosity::All;
			FPCGPerNodeVisualLogs ChildLogs;
			GetErrorInfoInternal(Child, OutMinVerbosity, DummyLocalVerbosity, ChildLogs);

			for (const FPCGNodeLogEntry& Entry : ChildLogs)
			{
				OutLogs.AddUnique(Entry);
			}

			GetSubtreeErrorInfo(Child, OutMinVerbosity, OutLogs);
		}
	}

	void GetErrorInfo(FPCGEditorGraphDebugObjectItemPtr Item, EVisibility& OutIconVisibility, FText& OutIconTooltipText, FLinearColor& OutIconColorAndOpacity, FLinearColor& OutRowTextColorAndOpacity)
	{
		OutIconVisibility = EVisibility::Hidden;
		OutIconTooltipText = FText();
		OutIconColorAndOpacity = FLinearColor::White;
		OutRowTextColorAndOpacity = FLinearColor::White;

		ELogVerbosity::Type MinLocalVerbosity = ELogVerbosity::All;
		ELogVerbosity::Type MinVerbosity = ELogVerbosity::All;
		FPCGPerNodeVisualLogs Logs;

		if (Item)
		{
			// Root generation items (actors) have no PCG stack of their own. Their errors live on child component items and are collected by the subtree walk below.
			// For non-root items, capture the item's own logs so MinLocalVerbosity reflects this graph level.
			if (!Item->IsRootGenerationItem())
			{
				GetErrorInfoInternal(Item, MinVerbosity, MinLocalVerbosity, Logs);
			}

			// Propagate errors/warnings from all descendants so parent rows show the error icon badge and tooltip.
			GetSubtreeErrorInfo(Item, MinVerbosity, Logs);
		}

		if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
		{
			OutIconTooltipText = PCGEditorModule->GetNodeVisualLogs().GetSummaryTextWithSources(Logs, nullptr);
		}

		OutIconVisibility = OutIconTooltipText.IsEmpty() ? EVisibility::Hidden : EVisibility::Visible;

		constexpr FLinearColor WarningColor(1.0f, 0.75f, 0.0f, 0.9f);
		constexpr FLinearColor ErrorColor(1.0f, 0.0f, 0.0f, 0.9f);

		if (MinLocalVerbosity < ELogVerbosity::All)
		{
			OutRowTextColorAndOpacity = MinLocalVerbosity <= ELogVerbosity::Error ? ErrorColor : WarningColor;
		}

		if (MinVerbosity < ELogVerbosity::All)
		{
			OutIconColorAndOpacity = MinVerbosity <= ELogVerbosity::Error ? ErrorColor : WarningColor;
		}
	}

	FString GetStringFromName(FName InName, bool bForSorting)
	{
		if (bForSorting)
		{
			return InName.GetPlainNameString() + FString::Printf(TEXT("%06d"), InName.GetNumber());
		}
		else
		{
			return InName.ToString();
		}
	}

	FSlateFontInfo GetRowFont(FPCGEditorGraphDebugObjectItemPtr Item)
	{
		const bool bUseBold = Item->IsSelected();
		const bool bUseItalic = Item->IsDynamic();

		if (!bUseBold && !bUseItalic)
		{
			return FAppStyle::GetFontStyle("NormalFont");
		}
		else if (bUseBold && !bUseItalic)
		{
			return FAppStyle::GetFontStyle("NormalFontBold");
		}
		else if (!bUseBold && bUseItalic)
		{
			return FAppStyle::GetFontStyle("NormalFontItalic");
		}
		else
		{
			return FAppStyle::GetFontStyle("NormalFontBoldItalic");
		}
	}
}

void FPCGEditorGraphDebugObjectItem::AddChild(TSharedRef<FPCGEditorGraphDebugObjectItem> InChild)
{
	check(!Children.Contains(InChild));
	InChild->Parent = AsShared();
	Children.Add(MoveTemp(InChild));
}

const FSlateBrush* FPCGEditorGraphDebugObjectItem::GetIcon() const
{
	return FSlateIconFinder::FindIconBrushForClass(UPCGGraph::StaticClass());
}

const TSet<TSharedPtr<FPCGEditorGraphDebugObjectItem>>& FPCGEditorGraphDebugObjectItem::GetChildren() const
{
	return Children;
}

FPCGEditorGraphDebugObjectItemPtr FPCGEditorGraphDebugObjectItem::GetParent() const
{
	return Parent.Pin();
}

void FPCGEditorGraphDebugObjectItem::SortChildren(bool bIsAscending, bool bIsRecursive)
{
	Children.Sort([bIsAscending](const FPCGEditorGraphDebugObjectItemPtr& InLHS, const FPCGEditorGraphDebugObjectItemPtr& InRHS)
	{
		// If both items have an explicit sort priority like a loop index, this is the primary sort key.
		const int32 IndexLHS = InLHS->GetSortPriority();
		const int32 IndexRHS = InRHS->GetSortPriority();
		if (IndexLHS != INDEX_NONE && IndexRHS != INDEX_NONE)
		{
			return (IndexLHS < IndexRHS) == bIsAscending;
		}

		// Next sort priority is presence or not of children. Items without children are shown first to reduce the possibility that
		// a child item ends up displayed far away from its parent item when the tree is expanded.
		const int HasChildrenLHS = InLHS->Children.Num() ? 1 : 0;
		const int HasChildrenRHS = InRHS->Children.Num() ? 1 : 0;
		if (HasChildrenLHS != HasChildrenRHS)
		{
			return (HasChildrenLHS < HasChildrenRHS) == bIsAscending;
		}

		// Otherwise fall back to alphanumeric order.
		return (InLHS->GetLabel(/*bForSorting=*/true) < InRHS->GetLabel(/*bForSorting=*/true)) == bIsAscending;
	});

	if (bIsRecursive)
	{
		for (const FPCGEditorGraphDebugObjectItemPtr& Child : Children)
		{
			Child->SortChildren(bIsAscending, bIsRecursive);
		}
	}
}

FString FPCGEditorGraphDebugObjectItem_Actor::GetLabel(bool bForSorting) const
{
	if (Actor.IsValid())
	{
		return PCGHelpers::IsServer(Actor->GetWorld()) ? (Actor->GetActorNameOrLabel() + FString(TEXT(" (Server)"))) : Actor->GetActorNameOrLabel();
	}
	else
	{
		return FString();
	}
}

const FSlateBrush* FPCGEditorGraphDebugObjectItem_Actor::GetIcon() const
{
	const UObject* Object = Actor.Get();
	return FSlateIconFinder::FindIconBrushForClass(Object ? Object->GetClass() : AActor::StaticClass());
}

FString FPCGEditorGraphDebugObjectItem_PCGSource::GetLabel(bool bForSorting) const
{
	// Always prefer a label override if it exists.
	if (LabelOverride.IsSet())
	{
		return LabelOverride.GetValue();
	}

	const IPCGGraphExecutionSource* Source = Cast<IPCGGraphExecutionSource>(PCGSource.GetObject());
	if (!Source)
	{
		return FString();
	}

	// In hierarchical mode, the owning actor and graph name are shown by parent items, so the label only needs to identify the grid level and cell.
	if (bUseHierarchicalLabel)
	{
		if (Source->GetExecutionState().IsLocalSource())
		{
			const uint32 GridSize = Source->GetExecutionState().GetGenerationGridSize();
			const FIntVector GridCoords = Source->GetExecutionState().GetGenerationGridCoords();
			return FString::Printf(TEXT("%gm (%d, %d, %d)"), GridSize / 100.0, GridCoords.X, GridCoords.Y, GridCoords.Z);
		}

		if (Source->GetExecutionState().IsPartitioned())
		{
			return TEXT("Unbounded (UB)");
		}
	}

	if (PCGGraph.IsValid())
	{
		return Source->GetExecutionState().GetDebugName() +
			FString(TEXT(" - ")) +
			(PCGGraph->GetTitleOverride().IsSet() ? PCGGraph->GetTitleOverride().GetValue().ToString() : PCGEditorGraphDebugObjectTree::GetStringFromName(PCGGraph->GetFName(), bForSorting));
	}

	return FString();
}

FString FPCGEditorGraphDebugObjectItem_PCGSubgraph::GetLabel(bool bForSorting) const
{
	if (PCGNode.IsValid() && PCGGraph.IsValid())
	{
		return (PCGGraph->GetTitleOverride().IsSet() ? PCGGraph->GetTitleOverride().GetValue().ToString() : PCGEditorGraphDebugObjectTree::GetStringFromName(PCGGraph->GetFName(), bForSorting)) +
			FString(TEXT(" - ")) +
			(PCGNode->HasAuthoredTitle() ? PCGNode->GetAuthoredTitleLine().ToString() : PCGEditorGraphDebugObjectTree::GetStringFromName(PCGNode->GetFName(), bForSorting));
	}

	return FString();
}

const FSlateBrush* FPCGEditorGraphDebugObjectItem_PCGSubgraph::GetIcon() const
{
	if (SubgraphType == EPCGEditorSubgraphNodeType::LoopSubgraph)
	{
		return FAppStyle::Get().GetBrush("GraphEditor.Macro.Loop_16x");
	}
	else
	{
		return FPCGEditorGraphDebugObjectItem::GetIcon();
	}
}

FString FPCGEditorGraphDebugObjectItem_PCGLoopIndex::GetLabel(bool bForSorting) const
{
	return FString::Format(TEXT("{0}"), { LoopIndex });
}

void SPCGEditorGraphDebugObjectItemRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, FPCGEditorGraphDebugObjectItemPtr InItem)
{
	Item = InItem;

	// This function should auto-expand the row and select the deepest entry as the debug object if it is unambiguous (the only entry at its level in the tree).
	OnDoubleClick = InArgs._OnDoubleClick;
	OnJumpTo = InArgs._OnJumpTo;
	OnFocus = InArgs._OnFocus;
	CanJumpTo = InArgs._CanJumpTo;
	CanFocus = InArgs._CanFocus;

	// Computed once during construction as the tree is refreshed on relevant events.
	const FSlateBrush* RowIcon = nullptr;
	FLinearColor RowIconColorAndOpacity;
	PCGEditorGraphDebugObjectTree::GetRowIconState(Item, RowIcon, RowIconColorAndOpacity);

	// Icon indicating warnings and errors. Tree refreshes after execution so computing once here is sufficient.
	FText ErrorIconTooltipText;
	EVisibility ErrorIconVisibility;
	FLinearColor ErrorIconColorAndOpacity;
	FLinearColor RowTextColorAndOpacity;
	PCGEditorGraphDebugObjectTree::GetErrorInfo(InItem, ErrorIconVisibility, ErrorIconTooltipText, ErrorIconColorAndOpacity, RowTextColorAndOpacity);

	// Transfer warning/error color to icon if there is a local error, but preserve original opacity (otherwise it will look weird for non-local graphs).
	if (ErrorIconVisibility != EVisibility::Hidden && RowTextColorAndOpacity != FLinearColor::White)
	{
		const float OriginalOpacity = RowIconColorAndOpacity.A;
		RowIconColorAndOpacity = RowTextColorAndOpacity;
		RowIconColorAndOpacity.A = OriginalOpacity;
	}
	
	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(6.0f, 0.0f)
		[
			SNew(SImage)
			.Visibility(EVisibility::HitTestInvisible)
			.ColorAndOpacity(RowIconColorAndOpacity)
			.Image(RowIcon)
		]
		+SHorizontalBox::Slot()
		[
			SNew(STextBlock)
			.Text(FText::FromString(Item->GetLabel()))
			.ToolTipText(FText::FromString(Item->GetLabel()))
			.Font_Lambda([this]() { return PCGEditorGraphDebugObjectTree::GetRowFont(Item); })
			// Highlight based on data available for currently inspected node. Computed dynamically in lambda to respond to inspection changes.
			.ColorAndOpacity_Lambda([this, RowTextColorAndOpacity](){ return Item->IsGrayedOut() ? FColor(75, 75, 75) : RowTextColorAndOpacity; })
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		[
			SNew(SImage)
			.Visibility(ErrorIconVisibility)
			.ToolTipText(ErrorIconTooltipText)
			.Image(FAppStyle::Get().GetBrush("Icons.Error"))
			.ColorAndOpacity(ErrorIconColorAndOpacity)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.Visibility_Lambda([this]() { return IsHovered() ? EVisibility::Visible : EVisibility::Hidden; })
			.OnClicked(this, &SPCGEditorGraphDebugObjectItemRow::FocusClicked)
			.IsEnabled(this, &SPCGEditorGraphDebugObjectItemRow::IsFocusEnabled)
			.ToolTipText(LOCTEXT("FocusOnNode", "Show the calling node in the current graph."))
			.ContentPadding(0.0f)
			[
				SNew(SImage)
				.Image(FPCGEditorStyle::Get().GetBrush("PCG.Editor.ZoomToSelection"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.Visibility_Lambda([this]() { return IsHovered() ? EVisibility::Visible : EVisibility::Hidden; })
			.OnClicked(this, &SPCGEditorGraphDebugObjectItemRow::JumpToClicked)
			.IsEnabled(this, &SPCGEditorGraphDebugObjectItemRow::IsJumpToEnabled)
			.ToolTipText(LOCTEXT("JumpToGraph", "Jump to graph, with this debug object context."))
			.ContentPadding(0.0f)
			[
				SNew(SImage)
				.Image(FPCGEditorStyle::Get().GetBrush("PCG.Editor.JumpTo"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];
}

FReply SPCGEditorGraphDebugObjectItemRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (Item.IsValid() && OnDoubleClick.IsBound())
	{
		OnDoubleClick.Execute(Item);
	}

	return FReply::Handled();
}

FReply SPCGEditorGraphDebugObjectItemRow::FocusClicked() const
{
	if (Item.IsValid() && OnFocus.IsBound())
	{
		OnFocus.Execute(Item);
	}

	return FReply::Handled();
}

FReply SPCGEditorGraphDebugObjectItemRow::JumpToClicked() const
{
	if (Item.IsValid() && OnJumpTo.IsBound())
	{
		OnJumpTo.Execute(Item);
	}

	return FReply::Handled();
}

bool SPCGEditorGraphDebugObjectItemRow::IsJumpToEnabled() const
{
	if (Item.IsValid() && CanJumpTo.IsBound())
	{
		return CanJumpTo.Execute(Item);
	}
	else
	{
		return true;
	}
}

bool SPCGEditorGraphDebugObjectItemRow::IsFocusEnabled() const
{
	if (Item.IsValid() && CanFocus.IsBound())
	{
		return CanFocus.Execute(Item);
	}
	else
	{
		return true;
	}
}

SPCGEditorGraphDebugObjectTree::~SPCGEditorGraphDebugObjectTree()
{
	USelection::SelectionChangedEvent.RemoveAll(this);
}

void SPCGEditorGraphDebugObjectTree::Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor)
{
	check(InPCGEditor);
	PCGEditor = InPCGEditor;

	GConfig->GetBool(TEXT("PCGDebugObjectTree"), TEXT("bUseHierarchicalView"), bUseHierarchicalView, GEditorPerProjectIni);

	USelection::SelectionChangedEvent.AddSP(this, &SPCGEditorGraphDebugObjectTree::OnEditorSelectionChanged);

	const TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(12.0f, 12.0f));

	DebugObjectTreeView = SNew(STreeView<FPCGEditorGraphDebugObjectItemPtr>)
		.TreeItemsSource(&RootItems)
		.OnGenerateRow(this, &SPCGEditorGraphDebugObjectTree::MakeTreeRowWidget)
		.OnGetChildren(this, &SPCGEditorGraphDebugObjectTree::OnGetChildren)
		.OnSelectionChanged(this, &SPCGEditorGraphDebugObjectTree::OnSelectionChanged)
		.OnExpansionChanged(this, &SPCGEditorGraphDebugObjectTree::OnExpansionChanged)
		.OnSetExpansionRecursive(this, &SPCGEditorGraphDebugObjectTree::OnSetExpansionRecursive)
		.SelectionMode(ESelectionMode::SingleToggle)
		.AllowOverscroll(EAllowOverscroll::No)
		.ExternalScrollbar(VerticalScrollBar)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always)
		.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &SPCGEditorGraphDebugObjectTree::OpenContextMenu));

	const TSharedRef<SWidget> SetButton = PropertyCustomizationHelpers::MakeUseSelectedButton(
		FSimpleDelegate::CreateSP(this, &SPCGEditorGraphDebugObjectTree::SetDebugObjectFromSelection_OnClicked),
		LOCTEXT("SetDebugObject", "Set debug object from Level Editor selection."),
		TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SPCGEditorGraphDebugObjectTree::IsSetDebugObjectFromSelectionButtonEnabled)));

	TSharedPtr<SLayeredImage> FilterImage = SNew(SLayeredImage)
		.Image(FAppStyle::Get().GetBrush("DetailsView.ViewOptions"))
		.ColorAndOpacity(FSlateColor::UseForeground());

	FilterImage->AddLayer(TAttribute<const FSlateBrush*>(this, &SPCGEditorGraphDebugObjectTree::GetFilterBadgeIcon));

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SetButton
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SPCGEditorGraphDebugObjectTree::FocusOnDebugObject_OnClicked)
				.IsEnabled(this, &SPCGEditorGraphDebugObjectTree::IsFocusOnDebugObjectButtonEnabled)
				.ToolTipText(LOCTEXT("DebugSelectActor", "Select and frame the debug actor in the Level Editor."))
				.ContentPadding(0.0f)
				[
					SNew(SImage)
					.Image(FPCGEditorStyle::Get().GetBrush("PCG.Editor.ZoomToSelection"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SNew(SCheckBox)
				.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
				.IsChecked(this, &SPCGEditorGraphDebugObjectTree::IsUsingHierarchicalView)
				.OnCheckStateChanged_Lambda([this](ECheckBoxState) { ToggleHierarchicalView(); })
				.ToolTipText(LOCTEXT("ToggleHierarchicalView", "Hierarchical grid view: groups sources by original actor with nested grid hierarchy instead of showing partition actors."))
				.Padding(4.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("LevelEditor.Tabs.Outliner"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SNew(SComboButton)
				.HasDownArrow(false)
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnGetMenuContent(this, &SPCGEditorGraphDebugObjectTree::OpenFilterMenu)
				.ContentPadding(0.0f)
				.ButtonContent()
				[
					FilterImage.ToSharedRef()
				]
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				DebugObjectTreeView.ToSharedRef()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				VerticalScrollBar
			]
		]
	];

	RequestRefresh();
}

void SPCGEditorGraphDebugObjectTree::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	OnTick(InCurrentTime);
}

void SPCGEditorGraphDebugObjectTree::OnTick(double InCurrentTime, bool bForceRefresh)
{
	bool bRefreshDone = false;
	if (bNeedsRefresh && (bForceRefresh || InCurrentTime >= NextRefreshTime))
	{
		// Updating the tree while the inspected component is generating can be bad as the selection can
		// be lost. Don't change the tree if we're inspecting something that is generating.
		const IPCGGraphExecutionSource* InspectedSource = SelectedStack.GetRootSource();
		if(InspectedSource == nullptr || !InspectedSource->GetExecutionState().IsGenerating())
		{
			bNeedsRefresh = false;
			RefreshTree();

			constexpr double RefreshCooldownTime = 0.25f;
			NextRefreshTime = InCurrentTime + RefreshCooldownTime;
			bRefreshDone = true;
		}
	}

	if (!IsSetDebugObjectFromSelectionEnabled.IsSet())
	{
		UpdateIsSetDebugObjectFromSelectionEnabled();
	}

	if (bRefreshDone && bShouldSelectStackOnNextRefresh)
	{
		if (SelectedStack.GetStackFrames().IsEmpty())
		{
			if (IsSetDebugObjectFromSelectionEnabled.IsSet() && IsSetDebugObjectFromSelectionEnabled.GetValue())
			{
				SetDebugObjectFromSelection_OnClicked();
			}
			else // Select first occurrence otherwise
			{
				for (const FPCGEditorGraphDebugObjectItemPtr& Item : AllGraphItems)
				{
					if (Item->GetPCGStack() && IsMainOrEmbeddedGraph(Item->GetPCGStack()->GetGraphForCurrentFrame()))
					{
						ExpandAndSelectDebugObject(Item);
						break;
					}
				}
			}
		}

		bShouldSelectStackOnNextRefresh = false;
	}
}

void SPCGEditorGraphDebugObjectTree::RequestRefresh(bool bForce)
{
	bNeedsRefresh = true;

	if (!IsOpen() && bForce)
	{
		// Force a refresh with a current time equal to next refresh time
		OnTick(NextRefreshTime, /*bForceRefresh=*/true);
	}
}

void SPCGEditorGraphDebugObjectTree::SetNodeBeingInspected(const UPCGNode* InPCGNode)
{
	PCGNodeBeingInspected = IsValid(InPCGNode) ? InPCGNode : nullptr;

	if (PCGNodeBeingInspected && SelectedStack.GetStackFrames().IsEmpty())
	{
		bShouldSelectStackOnNextRefresh = true;
	}

	RequestRefresh(/*bForce=*/true);
}

FReply SPCGEditorGraphDebugObjectTree::FocusOnDebugObject_OnClicked() const
{
	if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(PCGEditor.Pin()->GetPCGSourceBeingInspected()))
	{
		AActor* Actor = PCGComponent->GetOwner();
		if (Actor && GEditor && GUnrealEd && GEditor->CanSelectActor(Actor, /*bInSelected=*/true))
		{
			GEditor->SelectNone(/*bNoteSelectionChange=*/false, /*bDeselectBSPSurfs=*/true, /*WarnAboutManyActors=*/false);
			GEditor->SelectActor(Actor, /*bInSelected=*/true, /*bNotify=*/true, /*bSelectEvenIfHidden=*/true);
			GUnrealEd->Exec(Actor->GetWorld(), TEXT("CAMERA ALIGN ACTIVEVIEWPORTONLY"));
			GEditor->SelectComponent(PCGComponent, /*bInSelected=*/true, /*bNotify=*/true, /*bSelectEvenIfHidden=*/true);
		}
	}

	return FReply::Handled();
}

bool SPCGEditorGraphDebugObjectTree::IsFocusOnDebugObjectButtonEnabled() const
{
	return PCGEditor.IsValid() && (Cast<UPCGComponent>(PCGEditor.Pin()->GetPCGSourceBeingInspected()) != nullptr);
}

void SPCGEditorGraphDebugObjectTree::SetDebugObjectFromSelection_OnClicked()
{
	if (FPCGEditorGraphDebugObjectItemPtr Item = GetFirstDebugObjectFromSelection())
	{
		ExpandAndSelectFirstGraphDebugObject(Item);
	}
}

bool SPCGEditorGraphDebugObjectTree::SetDebugObjectFromStackFromAnotherEditor(const FPCGStack& InStack)
{
	RefreshTree();

	// If the given stack is already valid for this editor, try to select it as-is.
	if (IsMainOrEmbeddedGraph(InStack.GetGraphForCurrentFrame()))
	{
		if (FPCGEditorGraphDebugObjectItemPtr Item = GetItemFromStack(InStack))
		{
			ExpandAndSelectDebugObject(Item);
			return true;
		}
	}

	// Find first instance that starts with the given stack but has the current graph at its end.
	for (const FPCGEditorGraphDebugObjectItemPtr& Item : AllGraphItems)
	{
		const FPCGStack* ItemStack = Item->GetPCGStack();

		if (ItemStack && ItemStack->BeginsWith(InStack) && IsMainOrEmbeddedGraph(ItemStack->GetGraphForCurrentFrame()))
		{
			ExpandAndSelectDebugObject(Item);
			return true;
		}
	}

	return false;
}

void SPCGEditorGraphDebugObjectTree::ExpandAndSelectDebugObject(const FPCGEditorGraphDebugObjectItemPtr& InItem)
{
	if (!InItem)
	{
		return;
	}

	FPCGEditorGraphDebugObjectItemPtr Parent = InItem->GetParent();
	while (Parent)
	{
		DebugObjectTreeView->SetItemExpansion(Parent, true);
		Parent = Parent->GetParent();
	}

	DebugObjectTreeView->SetSelection(InItem);
	DebugObjectTreeView->RequestScrollIntoView(InItem);
}

FPCGEditorGraphDebugObjectItemPtr SPCGEditorGraphDebugObjectTree::GetItemFromStack(const FPCGStack& InStack) const
{
	for (const FPCGEditorGraphDebugObjectItemPtr& Item : AllGraphItems)
	{
		if (Item->GetPCGStack() && *(Item->GetPCGStack()) == InStack)
		{
			return Item;
		}
	}

	return nullptr;
}

bool SPCGEditorGraphDebugObjectTree::GetFirstStackFromSelection(const UPCGNode* InNode, const UPCGGraph* InGraph, FPCGStack& OutStack) const
{
	if (SelectedStack.GetStackFrames().IsEmpty())
	{
		return false;
	}

	for (const FPCGEditorGraphDebugObjectItemPtr& Item : AllGraphItems)
	{
		const FPCGStack* ItemStack = Item->GetPCGStack();

		// Early validation - common stack
		if (!ItemStack || !ItemStack->BeginsWith(SelectedStack))
		{
			continue;
		}

		// Then let's find the occurrence of the node
		const TArray<FPCGStackFrame>& ItemStackFrames = ItemStack->GetStackFrames();
		
		const UPCGNode* ItemNode = nullptr;
		int NodeIndex = SelectedStack.GetStackFrames().Num();
		while (NodeIndex < ItemStackFrames.Num())
		{
			ItemNode = ItemStackFrames[NodeIndex].GetObject_GameThread<UPCGNode>();
			if (ItemNode)
			{
				break;
			}
			else
			{
				++NodeIndex;
			}
		}

		if (ItemNode != InNode)
		{
			continue;
		}

		// And make sure the graph that's somewhere after it in the stack is present
		// Implementation note: the graph is either the next (static) or the second next in the stack.
		const UPCGGraph* ItemGraph = nullptr;

		if (ItemStackFrames.IsValidIndex(NodeIndex + 1))
		{
			ItemGraph = ItemStackFrames[NodeIndex + 1].GetObject_GameThread<UPCGGraph>();
		}

		if (!ItemGraph && ItemStackFrames.IsValidIndex(NodeIndex + 2))
		{
			ItemGraph = ItemStackFrames[NodeIndex + 2].GetObject_GameThread<UPCGGraph>();
		}

		if (ItemGraph && (!InGraph || ItemGraph == InGraph))
		{
			OutStack = *Item->GetPCGStack();
			return true;
		}
	}

	return false;
}

bool SPCGEditorGraphDebugObjectTree::IsOpen() const
{
	return PCGEditor.IsValid() && PCGEditor.Pin()->IsPanelCurrentlyForeground(PCGEditorTabs::DebugObjectID);
}

FPCGEditorGraphDebugObjectItemPtr SPCGEditorGraphDebugObjectTree::GetFirstDebugObjectFromSelection() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::GetFirstDebugObjectFromSelection);
	const UPCGGraph* MainGraph = GetMainGraph();
	if (!MainGraph)
	{
		return nullptr;
	}

	check(GEditor);
	USelection* SelectedActors = GEditor->GetSelectedActors();
	if (!IsValid(SelectedActors))
	{
		return nullptr;
	}

	IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get();

	for (FSelectionIterator It(*SelectedActors); It; ++It)
	{
		const AActor* SelectedActor = Cast<AActor>(*It);
		if (!IsValid(SelectedActor))
		{
			continue;
		}

		TArray<const UPCGComponent*> PCGComponents;
		SelectedActor->GetComponents<const UPCGComponent>(PCGComponents, /*bIncludeFromChildActors=*/true);

		for (const UPCGComponent* PCGComponent : PCGComponents)
		{
			if (!IsValid(PCGComponent))
			{
				continue;
			}

			// Look for graph in static stacks.
			FPCGStackContext StackContext;
			if (PCGComponent->GetStackContext(StackContext))
			{
				for (const FPCGStack& Stack : StackContext.GetStacks())
				{
					if (FPCGEditorGraphDebugObjectItemPtr Item = GetItemFromStack(Stack))
					{
						return Item;
					}
				}
			}

			// Look for graph in dynamic stacks.
			if (PCGEditorModule)
			{
				TArray<FPCGStackSharedPtr> ExecutedStacks = PCGEditorModule->GetExecutedStacksPtrs(PCGComponent, MainGraph);
				for (const FPCGStackSharedPtr& Stack : ExecutedStacks)
				{
					if (FPCGEditorGraphDebugObjectItemPtr Item = GetItemFromStack(*Stack))
					{
						return Item;
					}
				}
			}
		}
	}

	return nullptr;
}

void SPCGEditorGraphDebugObjectTree::UpdateIsSetDebugObjectFromSelectionEnabled()
{
	IsSetDebugObjectFromSelectionEnabled = (GetFirstDebugObjectFromSelection() != nullptr);
}

void SPCGEditorGraphDebugObjectTree::AddStacksToTree(const TArray<FPCGStackSharedPtr>& Stacks, FHierarchyContext& InOutContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::AddStacksToTree);
	const UPCGGraph* MainGraph = GetMainGraph();
	if (!MainGraph)
	{
		return;
	}

	for (const FPCGStackSharedPtr& Stack : Stacks)
	{
		AddStackToTree(*Stack, MainGraph, InOutContext);
	}
}

void SPCGEditorGraphDebugObjectTree::AddStacksToTree(const TArray<FPCGStack>& Stacks, FHierarchyContext& InOutContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::AddStacksToTree);
	const UPCGGraph* MainGraph = GetMainGraph();
	if (!MainGraph)
	{
		return;
	}

	for (const FPCGStack& Stack : Stacks)
	{
		AddStackToTree(Stack, MainGraph, InOutContext);
	}
}

void SPCGEditorGraphDebugObjectTree::AddStackToTree(const FPCGStack& Stack, const UPCGGraph* GraphBeingEdited, FHierarchyContext& InOutContext)
{
	check(GraphBeingEdited);

	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::AddStacksToTree::PerStackLoop);
		if (!Stack.HasObject(GraphBeingEdited) ||
			(!bShowDownstream && Stack.GetGraphForCurrentFrame() != GraphBeingEdited) ||
			(bShowOnlyErrorsAndWarnings && !PCGEditorGraphDebugObjectTree::StackContainsErrorOrWarning(&Stack)))
		{
			return;
		}

		IPCGGraphExecutionSource* PCGSource = const_cast<IPCGGraphExecutionSource*>(Stack.GetRootSource());

		if (!PCGSource)
		{
			return;
		}

		// Prevent duplicate entries from the editor world while in PIE.
		if (PCGHelpers::IsRuntimeOrPIE())
		{
			if (UWorld* World = PCGSource->GetExecutionState().GetWorld())
			{
				if (!World->IsGameWorld())
				{
					return;
				}
			}
		}

		int32 TopGraphIndex = INDEX_NONE;
		UPCGGraph* TopGraph = const_cast<UPCGGraph*>(Stack.GetRootGraph(&TopGraphIndex));
		if (!TopGraph)
		{
			return;
		}

		// If we're inspecting a node which has not logged inspection data in a previous execution, display grayed out.
		bool bDisplayGrayedOut = false;
		if (PCGNodeBeingInspected.IsValid())
		{
			const UPCGSettings* Settings = PCGNodeBeingInspected->GetSettings();
			const bool bGPUNode = Settings && Settings->bEnabled && Settings->ShouldExecuteOnGPU();

			// Display grayed out if no inspection data has been stored for this node, and node does not run on GPU,
			// because we don't opportunistically store inspection data for GPU nodes.
			bDisplayGrayedOut = !bGPUNode && !PCGSource->GetExecutionState().GetInspection().HasNodeProducedData(PCGNodeBeingInspected.Get(), Stack);
		}

		UPCGComponent* PCGComponent = Cast<UPCGComponent>(PCGSource);

		// Don't show actor-componentless components in the tree view, as they will never be generated and won't have anything to inspect.
		if (PCGComponent && PCGComponent->UseActorComponentlessGeneration() && PCGComponent->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			return;
		}

		// Resolve the actor to use as the tree root and the grid hierarchy parent.
		FPCGEditorGraphDebugObjectItemPtr OwnerItem;
		uint32 SourceGridSize = PCGHiGenGrid::UnboundedGridSize();
		FIntVector SourceGridCoords = FIntVector::ZeroValue;

		if (InOutContext.bHierarchical)
		{
			// Hierarchical mode: resolve to the original owner (not PA) and nest by grid.
			IPCGGraphExecutionSource* OriginalSource = PCGSource->GetExecutionState().GetOriginalSource();
			AActor* OriginalActor = OriginalSource ? OriginalSource->GetExecutionState().GetTypedTarget<AActor>() : nullptr;
			SourceGridSize = PCGSource->GetExecutionState().IsLocalSource() ? PCGSource->GetExecutionState().GetGenerationGridSize() : PCGHiGenGrid::UnboundedGridSize();
			SourceGridCoords = PCGSource->GetExecutionState().GetGenerationGridCoords();

			if (OriginalActor)
			{
				TObjectKey<UObject> OriginalOwnerKey(OriginalActor);

				if (FPCGEditorGraphDebugObjectItemPtr* FoundActorItem = InOutContext.OwnerItems.Find(OriginalOwnerKey))
				{
					OwnerItem = *FoundActorItem;
					OwnerItem->UpdateGrayedOut(bDisplayGrayedOut);
				}
				else
				{
					OwnerItem = InOutContext.OwnerItems.Emplace(OriginalOwnerKey, MakeShared<FPCGEditorGraphDebugObjectItem_Actor>(OriginalActor, bDisplayGrayedOut));
					AllGraphItems.Add(OwnerItem);
				}

				// For local sources, find the parent source item at the next-larger grid size via spatial containment.
				if (PCGSource->GetExecutionState().IsLocalSource())
				{
					if (const TArray<uint32>* OwnerGridSizes = InOutContext.OwnerGridSizes.Find(OriginalOwnerKey))
					{
						for (const uint32 CandidateGridSize : *OwnerGridSizes)
						{
							if (CandidateGridSize <= SourceGridSize)
							{
								continue;
							}

							const FVector SourcePosition = FVector(SourceGridCoords) * SourceGridSize;
							const FIntVector ParentCoords = (CandidateGridSize != PCGHiGenGrid::UnboundedGridSize())
								? UPCGActorHelpers::GetCellCoord(SourcePosition, CandidateGridSize, PCGSource->GetExecutionState().Use2DGrid())
								: FIntVector::ZeroValue;

							FSourceItemKey ParentKey{ TObjectKey<UObject>(OriginalActor), CandidateGridSize, ParentCoords };
							if (FPCGEditorGraphDebugObjectItemPtr* ParentSourceItem = InOutContext.SourceItems.Find(ParentKey))
							{
								OwnerItem = *ParentSourceItem;
								break;
							}
						}
					}
				}
			}
		}
		else
		{
			// Actor mode: use direct actor owner (including partition actors).
			if (AActor* Actor = PCGSource ? PCGSource->GetExecutionState().GetTypedTarget<AActor>() : nullptr)
			{
				TObjectKey<UObject> ActorKey(Actor);
				if (FPCGEditorGraphDebugObjectItemPtr* FoundActorItem = InOutContext.OwnerItems.Find(ActorKey))
				{
					OwnerItem = *FoundActorItem;
					OwnerItem->UpdateGrayedOut(bDisplayGrayedOut);
				}
				else
				{
					OwnerItem = InOutContext.OwnerItems.Emplace(ActorKey, MakeShared<FPCGEditorGraphDebugObjectItem_Actor>(Actor, bDisplayGrayedOut));
					AllGraphItems.Add(OwnerItem);
				}
			}
			else if (PCGComponent)
			{
				return;
			}
		}

		const TArray<FPCGStackFrame>& StackFrames = Stack.GetStackFrames();

		// Generic function to attach an item to the parent at the correct place
		auto AddToGraphItemsAndAttachItemToParent = [&InOutContext, this](FPCGStack& GraphStack, FPCGEditorGraphDebugObjectItemPtr GraphItem)
		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::AddStacksToTree::PerStackLoop::PerFrame::AddToGraphItemsAndAttachItemToParent);
			AllGraphItems.Add(GraphItem);

			TArray<FPCGStackFrame>& LoopGraphStackFrames = GraphStack.GetStackFramesMutable();
			while (LoopGraphStackFrames.Num() > 0)
			{
				LoopGraphStackFrames.SetNum(LoopGraphStackFrames.Num() - 1, EAllowShrinking::No);

				if (FPCGEditorGraphDebugObjectItemPtr* ParentItem = InOutContext.StackToItem.Find(GraphStack))
				{
					(*ParentItem)->AddChild(GraphItem.ToSharedRef());
					break;
				}
			}
		};

		// Generic function to create the subgraph/loop item and hook it up to the parent properly
		auto AddSubgraphOrLoopItemToStack = [&Stack, &InOutContext, &AddToGraphItemsAndAttachItemToParent](const UPCGGraph* InGraph, const UPCGNode* InSubgraphNode, int FrameCutoff, bool bInIsDebuggable, bool bInDisplayGrayedOut, EPCGEditorSubgraphNodeType InSubgraphType)
		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::AddStacksToTree::PerStackLoop::PerFrame::AddSubgraphOrLoopItemToStack);
			FPCGStack GraphStack = Stack;
			GraphStack.GetStackFramesMutable().SetNum(FrameCutoff, EAllowShrinking::No);

			if (FPCGEditorGraphDebugObjectItemPtr* ExistingGraphItem = InOutContext.StackToItem.Find(GraphStack))
			{
				(*ExistingGraphItem)->UpdateGrayedOut(bInDisplayGrayedOut);
			}
			else
			{
				FPCGEditorGraphDebugObjectItemPtr GraphItem = InOutContext.StackToItem.Emplace(
					GraphStack,
					MakeShared<FPCGEditorGraphDebugObjectItem_PCGSubgraph>(InSubgraphNode, InGraph, GraphStack, bInIsDebuggable, bInDisplayGrayedOut, InSubgraphType));

				AddToGraphItemsAndAttachItemToParent(GraphStack, GraphItem);
			}
		};

		// Example stack:
		//     Component/TopGraph/SubgraphNode/Subgraph/LoopSubgraphNode/LoopIndex/LoopSubgraph
		//                                       ^ static subgraph
		//     Component/TopGraph/SubgraphNode/INDEX_NONE/Subgraph/...
		//                                       ^ dynamic subgraph
		// The loop below adds tree items for component & top graph, and then whenever a graph is encountered
		// we look a previous frames to determine whether to add a subgraph item or loop subgraph item.
		int FrameIndex = StackFrames.Num() - 1;

		// Rollback from the last frames to check if there are matching stacks that would correspond to the current stack, so we don't do duplicate work.
		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::AddStacksToTree::PerStackLoop::RollbackLoop);
			FPCGStack UnrollStack = Stack;
			TArray<FPCGStackFrame>& UnrollFrames = UnrollStack.GetStackFramesMutable();

			while (FrameIndex > 1)
			{
				if (InOutContext.StackToItem.Contains(UnrollStack))
				{
					break;
				}
				else
				{
					--FrameIndex;
					UnrollFrames.Pop(EAllowShrinking::No);
				}
			}

			// If the data we're adding would not gray out the node, then we need to propagate upwards that information.
			if (PCGNodeBeingInspected.IsValid() && !bDisplayGrayedOut)
			{
				while (!UnrollFrames.IsEmpty())
				{
					if (FPCGEditorGraphDebugObjectItemPtr* ExistingItem = InOutContext.StackToItem.Find(UnrollStack))
					{
						// If we update something upstream and it was already not grayed out, then we can rest assured that anything upstream
						// is already not grayed out
						if (!(*ExistingItem)->UpdateGrayedOut(bDisplayGrayedOut))
						{
							break;
						}
					}
					UnrollFrames.Pop(EAllowShrinking::No);
				}
			}
		}

		for(; FrameIndex < StackFrames.Num(); ++FrameIndex)
		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::AddStacksToTree::PerStackLoop::PerFrame);
			const FPCGStackFrame& StackFrame = StackFrames[FrameIndex];
			const FPCGStackFrame& PreviousStackFrame = StackFrames[FrameIndex - 1];
			FPCGEditorGraphDebugObjectItemPtr CurrentItem;

			// When we encounter a graph, we look at the frame index and/or preceding frames to determine the graph type.
			if (const UPCGGraph* StackGraph = StackFrame.GetObject_GameThread<UPCGGraph>())
			{
				const bool bIsDebuggable = IsMainOrEmbeddedGraph(StackGraph);

				// Top graph.
				if (FrameIndex == TopGraphIndex && StackGraph == TopGraph)
				{
					FPCGStack GraphStack = Stack;
					GraphStack.GetStackFramesMutable().SetNum(FrameIndex + 1);

					if (FPCGEditorGraphDebugObjectItemPtr* ExistingTopGraphItem = InOutContext.StackToItem.Find(GraphStack))
					{
						(*ExistingTopGraphItem)->UpdateGrayedOut(bDisplayGrayedOut);
					}
					else
					{
						TSharedPtr<FPCGEditorGraphDebugObjectItem_PCGSource> Item = MakeShared<FPCGEditorGraphDebugObjectItem_PCGSource>(PCGSource, StackGraph, GraphStack, bIsDebuggable, bDisplayGrayedOut, InOutContext.bHierarchical);
						if (UObject* DefaultExecutionSource = PCGEditor.Pin()->GetDefaultExecutionSource(); PCGSource == Cast<IPCGGraphExecutionSource>(DefaultExecutionSource))
						{
							Item->SetLabel(TEXT("Standalone Graph"));
						}

						FPCGEditorGraphDebugObjectItemPtr TopGraphItem = InOutContext.StackToItem.Emplace(
							GraphStack,
							MoveTemp(Item));

						AllGraphItems.Add(TopGraphItem);

						if (OwnerItem)
						{
							OwnerItem->AddChild(TopGraphItem.ToSharedRef());
						}
						else
						{
							// If we don't have a component but an execution source, this becomes our owner item.
							InOutContext.OwnerItems.Emplace(TObjectKey<UObject>(Cast<UObject>(PCGSource)), TopGraphItem);
							OwnerItem = TopGraphItem;
						}

						// Register in source items lookup for hierarchical grid parenting.
						if (InOutContext.bHierarchical)
						{
							if (UObject* OriginalOwner = PCGSource->GetExecutionState().GetOriginalSource() ? PCGSource->GetExecutionState().GetOriginalSource()->GetExecutionState().GetTarget() : nullptr)
							{
								FSourceItemKey SourceKey{ TObjectKey<UObject>(OriginalOwner), SourceGridSize, SourceGridCoords };
								InOutContext.SourceItems.Add(SourceKey, TopGraphItem);
							}
						}
					}
				}
				// Previous stack was node, therefore static subgraph.
				else if (const UPCGNode* SubgraphNode = PreviousStackFrame.GetObject_GameThread<UPCGNode>())
				{
					AddSubgraphOrLoopItemToStack(StackGraph, SubgraphNode, FrameIndex + 1, bIsDebuggable, bDisplayGrayedOut, EPCGEditorSubgraphNodeType::StaticSubgraph);
				}
				// Previous stack was loop index, therefore loop subgraph.
				else if (FrameIndex >= 2 && (PreviousStackFrame.LoopIndex != INDEX_NONE))
				{
					const UPCGNode* LoopSubgraphNode = StackFrames[FrameIndex - 2].GetObject_GameThread<UPCGNode>();
					if (ensure(LoopSubgraphNode))
					{
						// Take the stack up to the looped subgraph node, add a item for the node + graph
						AddSubgraphOrLoopItemToStack(StackGraph, LoopSubgraphNode, FrameIndex - 1, /*bIsDebuggable=*/false, /*bIsGrayedOut=*/bDisplayGrayedOut, EPCGEditorSubgraphNodeType::LoopSubgraph);

						// Take full stack up until this point which will be the unique stack for the loop iteration.
						FPCGStack LoopIterationStack = Stack;
						LoopIterationStack.GetStackFramesMutable().SetNum(FrameIndex + 1);

						if (FPCGEditorGraphDebugObjectItemPtr* ExistingLoopIterationItem = InOutContext.StackToItem.Find(LoopIterationStack))
						{
							(*ExistingLoopIterationItem)->UpdateGrayedOut(bDisplayGrayedOut);
						}
						else
						{
							FPCGEditorGraphDebugObjectItemPtr LoopIterationItem = InOutContext.StackToItem.Emplace(
								LoopIterationStack,
								MakeShared<FPCGEditorGraphDebugObjectItem_PCGLoopIndex>(PreviousStackFrame.LoopIndex, StackGraph, LoopIterationStack, bIsDebuggable, bDisplayGrayedOut));

							AddToGraphItemsAndAttachItemToParent(LoopIterationStack, LoopIterationItem);
						}
					}
				}
				// Previous stack was invalid node / node with a INDEX_NONE loop, therefore most likely a dynamic subgraph
				else if (FrameIndex >= 2 && !PreviousStackFrame.IsValid())
				{
					const UPCGNode* DynamicSubgraphNode = StackFrames[FrameIndex - 2].GetObject_GameThread<UPCGNode>();
					if (DynamicSubgraphNode)
					{
						AddSubgraphOrLoopItemToStack(StackGraph, DynamicSubgraphNode, FrameIndex + 1, bIsDebuggable, bDisplayGrayedOut, EPCGEditorSubgraphNodeType::DynamicSubgraph);
					}
				}
			}
		}
	}
}

void SPCGEditorGraphDebugObjectTree::RefreshTree()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::RefreshTree);
	RootItems.Empty();
	AllGraphItems.Empty();
	DebugObjectTreeView->RequestTreeRefresh();

	UPCGGraph* MainGraph = GetMainGraph();
	if (!MainGraph)
	{
		return;
	}

	IPCGBaseSubsystem* Subsystem = PCGEditor.Pin()->GetSubsystem();
	if (!Subsystem)
	{
		return;
	}

	FPCGGraphExecutionRegistry& GraphExecutionRegistry = FPCGModule::GetGraphExecutionRegistry();
	TArray<FPCGGraphExecutionSourceDescriptor> ExecutionSourceDescriptors = GraphExecutionRegistry.GatherExecutionSources();

	if (!ExecutionSourceDescriptors.IsEmpty())
	{
		// Filter out sources that should not appear in the debug tree.
		ExecutionSourceDescriptors.RemoveAll([](const FPCGGraphExecutionSourceDescriptor& Desc)
		{
			if (!Desc.ExecutionSource || !Desc.bShowInDebugger)
			{
				return true;
			}

			const UWorld* ExecutionStateWorld = Desc.ExecutionSource->GetExecutionState().GetWorld();
			return ExecutionStateWorld && (ExecutionStateWorld->WorldType == EWorldType::Inactive || ExecutionStateWorld->WorldType == EWorldType::EditorPreview);
		});

		FHierarchyContext HierarchyContext;
		HierarchyContext.bHierarchical = bUseHierarchicalView;

		using FGraphContextMapKey = TTuple<UPCGGraph*, uint32 /*GenerationGridSize*/, bool /*IsPartitioned*/>;
		using FGraphContextMapValue = TTuple<FPCGStackContext, bool /*bContainsGraph*/>;
		TMap<FGraphContextMapKey, FGraphContextMapValue> GraphToContexts;

		// Pre-collection: resolve original actors and collect grid sizes per actor so that AddStackToTree can find parent
		// source items for hierarchical grid nesting. Also sort descriptors by grid size descending (UB first) so parent
		// sources are created before child sources.
		if (bUseHierarchicalView)
		{
			auto GetSourceGridSize = [](IPCGGraphExecutionSource* InSource) -> uint32
			{
				return InSource->GetExecutionState().IsLocalSource() ? InSource->GetExecutionState().GetGenerationGridSize() : PCGHiGenGrid::UnboundedGridSize();
			};

			for (const FPCGGraphExecutionSourceDescriptor& Desc : ExecutionSourceDescriptors)
			{
				// RemoveAll above filters out null ExecutionSource entries.
				IPCGGraphExecutionSource* PCGSource = Desc.ExecutionSource;
				check(PCGSource);
				IPCGGraphExecutionSource* OriginalSource = PCGSource->GetExecutionState().GetOriginalSource();
				UObject* OriginalOwner = OriginalSource ? OriginalSource->GetExecutionState().GetTarget() : nullptr;

				if (OriginalOwner)
				{
					TArray<uint32>& GridSizes = HierarchyContext.OwnerGridSizes.FindOrAdd(TObjectKey<UObject>(OriginalOwner));
					GridSizes.AddUnique(GetSourceGridSize(PCGSource));
				}
			}

			// Sort each owner's grid sizes ascending (we iterate from smallest to find next-larger parent).
			for (auto& [OwnerKey, GridSizes] : HierarchyContext.OwnerGridSizes)
			{
				GridSizes.Sort();
			}

			// Sort execution source descriptors by grid size descending so parent (larger grid) sources are processed before child (smaller grid) sources.
			ExecutionSourceDescriptors.Sort([&GetSourceGridSize](const FPCGGraphExecutionSourceDescriptor& A, const FPCGGraphExecutionSourceDescriptor& B)
			{
				const uint32 GridA = A.ExecutionSource ? GetSourceGridSize(A.ExecutionSource) : 0;
				const uint32 GridB = B.ExecutionSource ? GetSourceGridSize(B.ExecutionSource) : 0;
				return GridA > GridB;
			});
		}

		for (const FPCGGraphExecutionSourceDescriptor& ExecutionSourceDescriptor : ExecutionSourceDescriptors)
		{
			IPCGGraphExecutionSource* PCGSource = ExecutionSourceDescriptor.ExecutionSource;

			const bool bIsPartitioned = PCGSource->GetExecutionState().IsPartitioned();
			const uint32 GridSize = PCGSource->GetExecutionState().IsLocalSource() ? PCGSource->GetExecutionState().GetGenerationGridSize() : PCGHiGenGrid::UnboundedGridSize();

			// Process static stacks that can be read from the compiled graph.
			{
				//TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::RefreshTree::StaticStacks);
				// Considering it's likely for multiple components to have the same graph, we'll do a per-graph check first prior to building the component + graph stack.
				UPCGGraph* ComponentGraph = PCGSource->GetExecutionState().GetGraph();
				FGraphContextMapKey MapKey(ComponentGraph, GridSize, bIsPartitioned);
				FGraphContextMapValue* MapValue = GraphToContexts.Find(MapKey);

				if (!MapValue)
				{
					FPCGStackContext GraphStackContext;
					bool bSomeStacksContainCurrentGraph = false;
					if (Subsystem->GetStackContext(MapKey.Get<0>(), MapKey.Get<1>(), MapKey.Get<2>(), GraphStackContext))
					{
						bSomeStacksContainCurrentGraph = Algo::AnyOf(GraphStackContext.GetStacks(), [MainGraph](const FPCGStack& Stack) { return Stack.HasObject(MainGraph); });
					}

					MapValue = &(GraphToContexts.Add(MapKey, FGraphContextMapValue(std::move(GraphStackContext), bSomeStacksContainCurrentGraph)));
				}

				check(MapValue);
				if (MapValue->Get<1>())
				{
					FPCGStack SourceStack;
					SourceStack.PushFrame(Cast<UObject>(PCGSource));
					FPCGStackContext SourceContext(MapValue->Get<0>(), SourceStack);

					// In hierarchical mode, the compiled graph stack context is not grid-size-aware and contains subgraph paths for
					// all grid levels. Only create top-graph items from static stacks to avoid duplicating subgraphs. Dynamic stacks
					// populate them correctly.
					const bool bFilterStaticStacks = bUseHierarchicalView && (bIsPartitioned || PCGSource->GetExecutionState().IsLocalSource());
					if (bFilterStaticStacks)
					{
						for (const FPCGStack& Stack : SourceContext.GetStacks())
						{
							int32 RootGraphIndex = INDEX_NONE;
							Stack.GetRootGraph(&RootGraphIndex);
							if (RootGraphIndex != INDEX_NONE && Stack.GetStackFrames().Num() == RootGraphIndex + 1)
							{
								AddStackToTree(Stack, MainGraph, HierarchyContext);
							}
						}
					}
					else
					{
						AddStacksToTree(SourceContext.GetStacks(), HierarchyContext);
					}
				}
			}
		}

		// Process stacks encountered during execution so far, which will include dynamic subgraphs & loop subgraphs.
		// There will be overlaps with the static stacks but only unique entries will be added to the tree.
		if(IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::RefreshTree::DynamicStacks);
			TArray<FPCGStackSharedPtr> GraphStacks = PCGEditorModule->GetExecutedStacksPtrs(nullptr, MainGraph, /*bOnlyWithSubgraphAsCurrentFrame=*/!bShowDownstream);
			AddStacksToTree(GraphStacks, HierarchyContext);
		}

		for (auto& [OwnerKey, OwnerItem] : HierarchyContext.OwnerItems)
		{
			RootItems.Add(MoveTemp(OwnerItem));
		}
	}

	SortTreeItems();
	RestoreTreeState();
}

void SPCGEditorGraphDebugObjectTree::SortTreeItems(bool bIsAscending, bool bIsRecursive)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::SortTreeItems);
	RootItems.Sort([bIsAscending](const FPCGEditorGraphDebugObjectItemPtr& InLHS, const FPCGEditorGraphDebugObjectItemPtr& InRHS)
	{
		return (InLHS->GetLabel(/*bForSorting=*/true) < InRHS->GetLabel(/*bForSorting=*/true)) == bIsAscending;
	});

	if (bIsRecursive)
	{
		for (const FPCGEditorGraphDebugObjectItemPtr& Item : RootItems)
		{
			Item->SortChildren(bIsAscending, bIsRecursive);
		}
	}
}

void SPCGEditorGraphDebugObjectTree::RestoreTreeState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphDebugObjectTree::RestoreTreeState);
	// Try to restore user item expansion. We can't modify ExpandedStack, so we'll rebuild it.
	TSet<FPCGStack> ExpandedStacksBefore = std::move(ExpandedStacks);
	ExpandedStacks.Empty(ExpandedStacksBefore.Num());
	for (const FPCGStack& ExpandedStack : ExpandedStacksBefore)
	{
		if (FPCGEditorGraphDebugObjectItemPtr Item = GetItemFromStack(ExpandedStack))
		{
			DebugObjectTreeView->SetItemExpansion(Item, true);
		}
	}

	bool bFoundMatchingStack = false;

	// Try to restore user item selection by exact matching.
	if (FPCGEditorGraphDebugObjectItemPtr ExactMatchItem = GetItemFromStack(SelectedStack))
	{
		if (!DebugObjectTreeView->IsItemSelected(ExactMatchItem))
		{
			DebugObjectTreeView->SetItemSelection(ExactMatchItem, true);
		}

		bFoundMatchingStack = true;
	}

	// Try to restore user item selection by fuzzy matching (e.g. share the same owner) if no exactly matching stack was found.
	if (!bFoundMatchingStack)
	{
		for (FPCGEditorGraphDebugObjectItemPtr& Item : AllGraphItems)
		{
			const FPCGStack* ItemStack = Item->GetPCGStack();

			bool bFuzzyMatch = false;

			if (ItemStack && SelectedGraph.Get() == ItemStack->GetRootGraph())
			{
				const IPCGGraphExecutionSource* RootSource = ItemStack->GetRootSource();
				const UPCGComponent* RootComponent = Cast<UPCGComponent>(RootSource);
				const AActor* RootOwner = RootComponent ? RootComponent->GetOwner() : nullptr;
				const APCGPartitionActor* RootPartitionActor = Cast<APCGPartitionActor>(RootOwner);

				if (RootComponent && RootPartitionActor)
				{
					const UPCGComponent* SelectedOriginalComponent = Cast<UPCGComponent>(SelectedOriginalSource.Get());
					// For local components, we can fuzzy match as long as the GridSize, GridCoord, OriginalComponent, and ExecutionDomain are the same.
					// This is equivalent to saying they are on the same partition actor and come from the same original component.
					bFuzzyMatch = SelectedGridSize == RootComponent->GetGenerationGridSize()
						&& SelectedGridCoord == RootPartitionActor->GetGridCoord()
						&& SelectedOriginalComponent == RootPartitionActor->GetOriginalComponent(RootComponent)
						&& (IsValid(SelectedOriginalComponent) && SelectedOriginalComponent->IsManagedByRuntimeGenSystem() == RootComponent->IsManagedByRuntimeGenSystem());
				}
				else
				{
					// For original components, we can fuzzy match as long as the owning actor is the same.
					// Note: This fails for multiple original components with the same graph on the same actor, since there is no way to know which one to pick.
					if (SelectedOwner.Get() == RootOwner && Item->GetParent() && Item->GetParent()->GetChildren().Num() == 1)
					{
						int32 ItemRootGraphIndex = INDEX_NONE;
						int32 SelectedRootGraphIndex = INDEX_NONE;

						ItemStack->GetRootGraph(&ItemRootGraphIndex);
						SelectedStack.GetRootGraph(&SelectedRootGraphIndex);

						const TArray<FPCGStackFrame>& ItemStackFrames = ItemStack->GetStackFrames();
						const TArray<FPCGStackFrame>& SelectedStackFrames = SelectedStack.GetStackFrames();

						// If the stacks match from the RootGraph onwards, then our fuzzy match should succeed.
						if (ItemRootGraphIndex != INDEX_NONE && ItemRootGraphIndex == SelectedRootGraphIndex && ItemStackFrames.Num() == SelectedStackFrames.Num())
						{
							bool bAllStackFramesMatch = true;

							for (int I = ItemRootGraphIndex; I < ItemStackFrames.Num(); ++I)
							{
								if (!ItemStackFrames[I].IsValid() || ItemStackFrames[I] != SelectedStackFrames[I])
								{
									bAllStackFramesMatch = false;
									break;
								}
							}

							if (bAllStackFramesMatch)
							{
								bFuzzyMatch = true;
							}
						}
					}
				}
			}

			if (bFuzzyMatch)
			{
				// Force the selected object to re-expand.
				FPCGEditorGraphDebugObjectItemPtr Parent = Item->GetParent();
				while (Parent)
				{
					DebugObjectTreeView->SetItemExpansion(Parent, true);
					Parent = Parent->GetParent();
				}

				if (!DebugObjectTreeView->IsItemSelected(Item))
				{
					DebugObjectTreeView->SetItemSelection(Item, true);
				}
				
				break;
			}
		}
	}
}

void SPCGEditorGraphDebugObjectTree::OnEditorSelectionChanged(UObject* InObject)
{
	IsSetDebugObjectFromSelectionEnabled.Reset();
}

UPCGGraph* SPCGEditorGraphDebugObjectTree::GetFocusedGraph() const
{
	const TSharedPtr<FPCGEditor> PCGEditorPtr = PCGEditor.Pin();
	return PCGEditorPtr.IsValid() ? PCGEditorPtr->GetFocusedGraph() : nullptr;
}

UPCGGraph* SPCGEditorGraphDebugObjectTree::GetMainGraph() const
{
	const TSharedPtr<FPCGEditor> PCGEditorPtr = PCGEditor.Pin();
	return PCGEditorPtr.IsValid() ? PCGEditorPtr->GetMainGraph() : nullptr;
}

TSharedRef<ITableRow> SPCGEditorGraphDebugObjectTree::MakeTreeRowWidget(FPCGEditorGraphDebugObjectItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew( STableRow<TSharedPtr<SPCGEditorGraphDebugObjectItemRow>>, InOwnerTable)
	[
		SNew(SPCGEditorGraphDebugObjectItemRow, InOwnerTable, InItem)
		.OnDoubleClick(this, &SPCGEditorGraphDebugObjectTree::ExpandAndSelectFirstGraphDebugObject)
		.OnJumpTo(this, &SPCGEditorGraphDebugObjectTree::JumpToGraphInTree)
		.CanJumpTo(this, &SPCGEditorGraphDebugObjectTree::CanJumpToGraphInTree)
		.OnFocus(this, &SPCGEditorGraphDebugObjectTree::FocusOnItem)
		.CanFocus(this, &SPCGEditorGraphDebugObjectTree::CanFocusOnItem)
	];
}

void SPCGEditorGraphDebugObjectTree::OnGetChildren(FPCGEditorGraphDebugObjectItemPtr InItem, TArray<FPCGEditorGraphDebugObjectItemPtr>& OutChildren) const
{
	if (InItem.IsValid())
	{
		for (TWeakPtr<FPCGEditorGraphDebugObjectItem> ChildItem : InItem->GetChildren())
		{
			FPCGEditorGraphDebugObjectItemPtr ChildItemPtr = ChildItem.Pin();
			OutChildren.Add(ChildItemPtr);
		}
	}
}

void SPCGEditorGraphDebugObjectTree::OnSelectionChanged(FPCGEditorGraphDebugObjectItemPtr InItem, ESelectInfo::Type InSelectInfo)
{
	// Unmark the previously selected object
	if (FPCGEditorGraphDebugObjectItemPtr PreviousItem = GetItemFromStack(SelectedStack))
	{
		PreviousItem->SetSelected(false);
	}

	// Reset selected item information.
	SelectedStack = FPCGStack();
	SelectedGraph = nullptr;
	SelectedOwner = nullptr;
	SelectedGridSize = PCGHiGenGrid::UnboundedGridSize();
	SelectedGridCoord = FIntVector::ZeroValue;
	SelectedOriginalSource.Reset();

	if (const FPCGStack* Stack = InItem ? InItem->GetPCGStack() : nullptr)
	{
		SelectedStack = *Stack;
		SelectedGraph = SelectedStack.GetRootGraph();

		if (const IPCGGraphExecutionSource* RootSource = SelectedStack.GetRootSource())
		{
			const UPCGComponent* RootComponent = Cast<UPCGComponent>(RootSource);
			SelectedOwner = RootComponent ? RootComponent->GetOwner() : nullptr;

			if (const APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(SelectedOwner.Get()))
			{
				check(RootComponent);
				SelectedGridSize = RootComponent->GetGenerationGridSize();
				SelectedGridCoord = PartitionActor->GetGridCoord();
				SelectedOriginalSource = PartitionActor->GetOriginalComponent(RootComponent);
			}
			else
			{
				SelectedOriginalSource = RootSource;
			}
		}
	}

	const UPCGGraph* MainGraph = GetMainGraph();
	bool bStackInspectedSet = false;

	// Only attempt to inspect stacks that correspond to the edited graph. Other graphs need to be inspected in their own editor.
	if (InItem)
	{
		// Basically - if the selection is something from "upstream", then we can clear the stack being inspected.
		if (SelectedStack.HasObject(MainGraph))
		{
			// If the last graph in the stack is the current graph, then we can just use that as the selected stack
			TArray<FPCGStackFrame>& StackFrames = SelectedStack.GetStackFramesMutable();
			while(!StackFrames.IsEmpty() && !IsMainOrEmbeddedGraph(StackFrames.Last().GetObject_GameThread<UPCGGraph>()))
			{
				StackFrames.Pop();
			}

			if (ensure(!SelectedStack.GetStackFrames().IsEmpty()))
			{
				PCGEditor.Pin()->SetStackBeingInspected(SelectedStack);
				bStackInspectedSet = true;
			}
		}
	}

	if (!bStackInspectedSet)
	{
		PCGEditor.Pin()->ClearStackBeingInspected();
	}

	// Finally mark the selected item as selected
	if (IsMainOrEmbeddedGraph(SelectedStack.GetGraphForCurrentFrame()))
	{
		if (FPCGEditorGraphDebugObjectItemPtr CurrentSelectedItem = GetItemFromStack(SelectedStack))
		{
			CurrentSelectedItem->SetSelected(true);
		}
	}
}

void SPCGEditorGraphDebugObjectTree::OnExpansionChanged(FPCGEditorGraphDebugObjectItemPtr InItem, bool bInIsExpanded)
{
	if (!InItem.IsValid())
	{
		return;
	}

	InItem->SetExpanded(bInIsExpanded);

	if (const FPCGStack* Stack = InItem->GetPCGStack())
	{
		if (bInIsExpanded)
		{
			ExpandedStacks.Add(*Stack);
		}
		else
		{
			ExpandedStacks.Remove(*Stack);
		}
	}
}

void SPCGEditorGraphDebugObjectTree::OnSetExpansionRecursive(FPCGEditorGraphDebugObjectItemPtr InItem, bool bInExpand) const
{
	if (!InItem.IsValid() || !DebugObjectTreeView.IsValid())
	{
		return;
	}

	DebugObjectTreeView->SetItemExpansion(InItem, bInExpand);
	InItem->SetExpanded(bInExpand);

	for (const FPCGEditorGraphDebugObjectItemPtr& ChildItem : InItem->GetChildren())
	{
		if (ChildItem.IsValid())
		{
			OnSetExpansionRecursive(ChildItem, bInExpand);
		}
	}
}

bool SPCGEditorGraphDebugObjectTree::IsMainOrEmbeddedGraph(const UPCGGraph* InGraph) const
{
	if (!InGraph)
	{
		return false;
	}

	const UPCGGraph* MainGraph = GetMainGraph();
	return InGraph == MainGraph || InGraph->GetEmbeddedParentGraph() == MainGraph;
}

void SPCGEditorGraphDebugObjectTree::ExpandAndSelectFirstGraphDebugObject(const FPCGEditorGraphDebugObjectItemPtr& InItem)
{
	if (!InItem.IsValid())
	{
		return;
	}

	// Regardless of what happens, we'll expand the currently selected item.
	DebugObjectTreeView->SetItemExpansion(InItem, true);

	// BFS starting from InItem itself for the first item whose stack ends at the current graph.
	FPCGEditorGraphDebugObjectItemPtr ItemToSelect;

	TArray<FPCGEditorGraphDebugObjectItemPtr> ToVisit({ InItem });
	int32 VisitIndex = 0;

	while (VisitIndex < ToVisit.Num())
	{
		const FPCGEditorGraphDebugObjectItemPtr& Item = ToVisit[VisitIndex];
		++VisitIndex;

		const FPCGStack* ItemStack = Item ? Item->GetPCGStack() : nullptr;
		if (ItemStack && !ItemStack->GetStackFrames().IsEmpty() && IsMainOrEmbeddedGraph(ItemStack->GetStackFrames().Last().GetObject_GameThread<UPCGGraph>()))
		{
			ItemToSelect = Item;
			break;
		}

		ToVisit.Append(Item->GetChildren().Array());
	}

	// If we've found nothing, select the original item.
	ExpandAndSelectDebugObject(ItemToSelect.IsValid() ? ItemToSelect : InItem);
}

const FSlateBrush* SPCGEditorGraphDebugObjectTree::GetFilterBadgeIcon() const
{
	if (!bShowDownstream || bShowOnlyErrorsAndWarnings)
	{
		return FAppStyle::Get().GetBrush("Icons.BadgeModified");
	}
	else
	{
		return nullptr;
	}
}

TSharedRef<SWidget> SPCGEditorGraphDebugObjectTree::OpenFilterMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, /*InCommandList=*/nullptr);
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToggleShowOnlyErrorsAndWarnings", "Show only errors/warnings"),
		LOCTEXT("ToggleShowOnlyErrorsAndWarningsTooltip", "Toggles whether only executions that had errors and warnings are shown."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphDebugObjectTree::ToggleShowOnlyErrorsAndWarnings),
			FCanExecuteAction(),
			FGetActionCheckState::CreateSP(this, &SPCGEditorGraphDebugObjectTree::IsShowingOnlyErrorsAndWarnings)),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToggleShowGraphsDownstream", "Show downstream"),
		LOCTEXT("ToggleShowGraphsDownstreamTooltip", "Toggles whether all graphs downstream to this current graph are shown."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphDebugObjectTree::ToggleShowDownstream),
			FCanExecuteAction(),
			FGetActionCheckState::CreateSP(this, &SPCGEditorGraphDebugObjectTree::IsShowingDownstream)),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	return MenuBuilder.MakeWidget();
}

TSharedPtr<SWidget> SPCGEditorGraphDebugObjectTree::OpenContextMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, /*InCommandList=*/nullptr);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("JumpToSelectedGraph", "Jump To"),
		LOCTEXT("JumpToSelectedGraphTooltip", "Jumps to the selected graph."),
		FSlateIcon(FPCGEditorStyle::Get().GetStyleSetName(), "PCG.Editor.JumpTo"),
		FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphDebugObjectTree::ContextMenu_JumpToGraphInTree), FCanExecuteAction::CreateSP(this, &SPCGEditorGraphDebugObjectTree::ContextMenu_JumpToGraphInTree_CanExecute))
	);

	return MenuBuilder.MakeWidget();
}

void SPCGEditorGraphDebugObjectTree::ContextMenu_JumpToGraphInTree()
{
	const TArray<FPCGEditorGraphDebugObjectItemPtr> SelectedItems = DebugObjectTreeView->GetSelectedItems();
	for (const FPCGEditorGraphDebugObjectItemPtr& SelectedItem : SelectedItems)
	{
		JumpToGraphInTree(SelectedItem);
	}
}

void SPCGEditorGraphDebugObjectTree::JumpToGraphInTree(const FPCGEditorGraphDebugObjectItemPtr& Item)
{
	if (!Item || !Item->GetPCGStack())
	{
		return;
	}

	FPCGStack Stack = *Item->GetPCGStack();

	// Example stack:
	//     Component/TopGraph/SubgraphNode/Subgraph/LoopSubgraphNode/LoopIndex/LoopSubgraph
	//                                       ^ static subgraph
	//     Component/TopGraph/SubgraphNode/INDEX_NONE/Subgraph/...
	//                                       ^ dynamic subgraph

	UPCGGraph* JumpToPCGGraph = nullptr;
	UPCGNode* JumpToPCGNode = nullptr;

	// If the item is about this graph, then we'll jump to this instance's caller - we'll walk back until we find a graph (e.g. parent)
	// and the next entry should be our caller node.
	if (Item->IsDebuggable() && !IsMainOrEmbeddedGraph(Item->GetPCGGraph()))
	{
		// Debuggable target graphs correspond to the currently edited graph. For this case open the parent graph and jump to the corresponding subgraph node.
		// Search the stack for the parent graph.
		for (int i = Stack.GetStackFrames().Num() - 2; i > 0; --i) 
		{
			if (const UPCGGraph* PCGGraph = Stack.GetStackFrames()[i].GetObject_GameThread<UPCGGraph>())
			{
				JumpToPCGGraph = const_cast<UPCGGraph*>(PCGGraph);
				JumpToPCGNode = const_cast<UPCGNode*>(Stack.GetStackFrames()[i + 1].GetObject_GameThread<UPCGNode>());

				// Cull remaining remaining frames so the stack selection into the parent graph editor works.
				Stack.GetStackFramesMutable().SetNum(i);
				
				break;
			}
		}
	}
	else 
	{
		// Two cases here: upstream or downstream. In both cases, there's no need to jump to a given node as there is no concept of caller here.
		JumpToPCGGraph = const_cast<UPCGGraph*>(Item->GetPCGGraph());
	}

	if (JumpToPCGGraph)
	{
		if (TSharedPtr<FPCGEditor> PinnedEditor = PCGEditor.Pin())
		{
			FPCGEditor* CurrentPCGEditor = PinnedEditor->OpenEditorForGraph(JumpToPCGGraph);
			if (!CurrentPCGEditor)
			{
				return;
			}

			if (CurrentPCGEditor != PinnedEditor.Get())
			{
				// Implementation note - for selection purposes we had culled the stack to be in the local referential
				// however in the case of the jump here we want the full stack
				CurrentPCGEditor->SetStackBeingInspectedFromAnotherEditor(Stack);
			}

			if (JumpToPCGNode)
			{
				CurrentPCGEditor->JumpToNode(JumpToPCGNode);
			}
		}
	}
}

bool SPCGEditorGraphDebugObjectTree::ContextMenu_JumpToGraphInTree_CanExecute() const
{
	const TArray<FPCGEditorGraphDebugObjectItemPtr> SelectedItems = DebugObjectTreeView->GetSelectedItems();
	for (const FPCGEditorGraphDebugObjectItemPtr& SelectedItem : SelectedItems)
	{
		if (CanJumpToGraphInTree(SelectedItem))
		{
			return true;
		}
	}

	return false;
}

bool SPCGEditorGraphDebugObjectTree::CanJumpToGraphInTree(const FPCGEditorGraphDebugObjectItemPtr& Item) const
{
	// Offer jump-to command if any selected item is a graph or a subgraph loop iteration.
	return Item && (Item->GetPCGGraph() || Item->IsLoopIteration());
}

void SPCGEditorGraphDebugObjectTree::FocusOnItem(const FPCGEditorGraphDebugObjectItemPtr& Item)
{
	if (!Item || !Item->GetPCGStack())
	{
		return;
	}

	const FPCGStack& Stack = *Item->GetPCGStack();
	const UPCGGraph* FocusedGraph = GetFocusedGraph();
	bool bFoundCurrentGraph = false;
	const UPCGNode* LastNode = nullptr;

	// Find first node after the current graph in the stack.
	for (int i = Stack.GetStackFrames().Num() - 1; i >= 0; --i)
	{
		const FPCGStackFrame& StackFrame = Stack.GetStackFrames()[i];
		const UObject* Object = StackFrame.GetObject_GameThread();

		if (Object == FocusedGraph)
		{
			bFoundCurrentGraph = true;
			break;
		}
		else if (const UPCGNode* Node = Cast<const UPCGNode>(Object))
		{
			LastNode = Node;
		}
	}

	if (bFoundCurrentGraph && LastNode)
	{
		PCGEditor.Pin()->JumpToNode(LastNode);
	}
}

bool SPCGEditorGraphDebugObjectTree::CanFocusOnItem(const FPCGEditorGraphDebugObjectItemPtr& Item) const
{
	if (!Item)
	{
		return false;
	}

	// Anything downstream from the currently editable graph (e.g. "IsDebuggable" is true) is avaiable for focus.
	FPCGEditorGraphDebugObjectItemPtr Parent = Item->GetParent();
	while (Parent)
	{
		if (Parent->IsDebuggable())
		{
			return true;
		}
		else
		{
			Parent = Parent->GetParent();
		}
	}

	return false;
}

void SPCGEditorGraphDebugObjectTree::ToggleShowOnlyErrorsAndWarnings()
{
	bShowOnlyErrorsAndWarnings = !bShowOnlyErrorsAndWarnings;
	RequestRefresh();
}

ECheckBoxState SPCGEditorGraphDebugObjectTree::IsShowingOnlyErrorsAndWarnings() const
{
	return bShowOnlyErrorsAndWarnings ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SPCGEditorGraphDebugObjectTree::ToggleShowDownstream()
{
	bShowDownstream = !bShowDownstream;
	RequestRefresh();
}

ECheckBoxState SPCGEditorGraphDebugObjectTree::IsShowingDownstream() const
{
	return bShowDownstream ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SPCGEditorGraphDebugObjectTree::ToggleHierarchicalView()
{
	bUseHierarchicalView = !bUseHierarchicalView;
	GConfig->SetBool(TEXT("PCGDebugObjectTree"), TEXT("bUseHierarchicalView"), bUseHierarchicalView, GEditorPerProjectIni);
	RequestRefresh();
}

ECheckBoxState SPCGEditorGraphDebugObjectTree::IsUsingHierarchicalView() const
{
	return bUseHierarchicalView ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

#undef LOCTEXT_NAMESPACE
