// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "Graph/PCGStackContext.h"

#include "ToolMenus.h"
#include "GameFramework/Actor.h"
#include "Misc/Optional.h"
#include "UObject/ObjectKey.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class FPCGEditorGraphDebugObjectItem;
class FPCGEditor;

typedef TSharedPtr<FPCGEditorGraphDebugObjectItem> FPCGEditorGraphDebugObjectItemPtr;

class FPCGEditorGraphDebugObjectItem : public TSharedFromThis<FPCGEditorGraphDebugObjectItem>
{
public:
	FPCGEditorGraphDebugObjectItem() = default;

	explicit FPCGEditorGraphDebugObjectItem(bool bInGrayedOut)
		: bGrayedOut(bInGrayedOut)
	{
	}
	
	virtual ~FPCGEditorGraphDebugObjectItem() = default;

	void AddChild(TSharedRef<FPCGEditorGraphDebugObjectItem> InChild);
	const TSet<TSharedPtr<FPCGEditorGraphDebugObjectItem>>& GetChildren() const;
	FPCGEditorGraphDebugObjectItemPtr GetParent() const;
	void SortChildren(bool bIsAscending, bool bIsRecursive);

	bool IsExpanded() const { return bIsExpanded; }
	void SetExpanded(bool bInIsExpanded) { bIsExpanded = bInIsExpanded; }

	bool IsGrayedOut() const { return bGrayedOut; }
	bool UpdateGrayedOut(bool bInGrayedOut) 
	{ 
		const bool bWasGrayedOut = bGrayedOut;
		bGrayedOut &= bInGrayedOut; 
		return bWasGrayedOut;
	}

	bool IsSelected() const { return bSelected; }
	void SetSelected(bool bInSelected) { bSelected = bInSelected; }

	/** Optional sort priority, if returns INDEX_NONE then sort will fall back to alphabetical. */
	virtual int32 GetSortPriority() const { return INDEX_NONE; }

	/** Whether this item represents a currently debuggable object for the current edited graph. */
	virtual bool IsDebuggable() const { return false; }
	/** Whether this item represents a dynamically executed element (as in dynamic subgraph). */
	virtual bool IsDynamic() const { return false; }
	/** Whether this is an actor/root item, which doesn't have stack information to its components. */
	virtual bool IsRootGenerationItem() const { return false; }

	virtual FString GetLabel(bool bForSorting = false) const = 0;
	virtual const FPCGStack* GetPCGStack() const { return nullptr; }
	virtual const UPCGGraph* GetPCGGraph() const { return nullptr; }
	virtual bool IsLoopIteration() const { return false; }
	virtual const FSlateBrush* GetIcon() const;

protected:
	virtual FPCGStack* GetMutablePCGStack() { return nullptr; }

	TWeakPtr<FPCGEditorGraphDebugObjectItem> Parent;
	TSet<TSharedPtr<FPCGEditorGraphDebugObjectItem>> Children;
	bool bIsExpanded = false;
	bool bGrayedOut = false;
	bool bSelected = false;
};

class FPCGEditorGraphDebugObjectItem_Actor : public FPCGEditorGraphDebugObjectItem
{
public:
	explicit FPCGEditorGraphDebugObjectItem_Actor(TWeakObjectPtr<AActor> InActor, bool bInHasInspectionData)
		: FPCGEditorGraphDebugObjectItem(bInHasInspectionData)
		, Actor(InActor)
	{
		PCGStack.PushFrame(InActor.Get());
	}

	virtual FString GetLabel(bool bForSorting = false) const override;
	virtual const FPCGStack* GetPCGStack() const override { return &PCGStack; }
	virtual const FSlateBrush* GetIcon() const override;
	virtual bool IsRootGenerationItem() const override { return true; }

protected:
	virtual FPCGStack* GetMutablePCGStack() override { return &PCGStack; }

	TWeakObjectPtr<AActor> Actor = nullptr;

	FPCGStack PCGStack;
};

class FPCGEditorGraphDebugObjectItem_PCGSource : public FPCGEditorGraphDebugObjectItem
{
public:
	explicit FPCGEditorGraphDebugObjectItem_PCGSource(
		IPCGGraphExecutionSource* InSource,
		const UPCGGraph* InPCGGraph,
		const FPCGStack& InPCGStack,
		bool bInIsDebuggable,
		bool bInHasInspectionData,
		bool bInUseHierarchicalLabel)
		: FPCGEditorGraphDebugObjectItem(bInHasInspectionData)
		, PCGSource(InSource)
		, PCGGraph(InPCGGraph)
		, PCGStack(InPCGStack)
		, bIsDebuggable(bInIsDebuggable)
		, bUseHierarchicalLabel(bInUseHierarchicalLabel)
	{}

	virtual FString GetLabel(bool bForSorting = false) const override;
	void SetLabel(const FString& InLabel) { LabelOverride = InLabel; }
	virtual const FPCGStack* GetPCGStack() const override { return &PCGStack; }
	virtual bool IsDebuggable() const override { return bIsDebuggable; }
	virtual const UPCGGraph* GetPCGGraph() const override { return PCGGraph.Get(); }

protected:
	virtual FPCGStack* GetMutablePCGStack() override { return &PCGStack; }

	FPCGSoftGraphExecutionSource PCGSource;
	TSoftObjectPtr<const UPCGGraph> PCGGraph;

	FPCGStack PCGStack;
	bool bIsDebuggable = false;
	bool bUseHierarchicalLabel = false;
	TOptional<FString> LabelOverride;
};

enum class EPCGEditorSubgraphNodeType : uint8
{
	StaticSubgraph = 0,
	DynamicSubgraph,
	LoopSubgraph
};

class FPCGEditorGraphDebugObjectItem_PCGSubgraph : public FPCGEditorGraphDebugObjectItem
{
public:
	explicit FPCGEditorGraphDebugObjectItem_PCGSubgraph(
		TWeakObjectPtr<const UPCGNode> InPCGNode,
		TWeakObjectPtr<const UPCGGraph> InPCGGraph,
		const FPCGStack& InPCGStack,
		bool bInIsDebuggable,
		bool bInHasInspectionData,
		EPCGEditorSubgraphNodeType InSubgraphType)
		: FPCGEditorGraphDebugObjectItem(bInHasInspectionData)
		, PCGNode(InPCGNode)
		, PCGGraph(InPCGGraph)
		, PCGStack(InPCGStack)
		, bIsDebuggable(bInIsDebuggable)
		, SubgraphType(InSubgraphType)
	{}

	virtual FString GetLabel(bool bForSorting = false) const override;
	virtual const FPCGStack* GetPCGStack() const override { return &PCGStack; }
	virtual bool IsDebuggable() const override { return bIsDebuggable; }
	virtual bool IsDynamic() const override { return SubgraphType != EPCGEditorSubgraphNodeType::StaticSubgraph; }
	virtual const UPCGGraph* GetPCGGraph() const override { return PCGGraph.Get(); }
	virtual const FSlateBrush* GetIcon() const override;

protected:
	virtual FPCGStack* GetMutablePCGStack() override { return &PCGStack; }

	TWeakObjectPtr<const UPCGNode> PCGNode = nullptr;
	TWeakObjectPtr<const UPCGGraph> PCGGraph = nullptr;

	FPCGStack PCGStack;
	bool bIsDebuggable = false;
	EPCGEditorSubgraphNodeType SubgraphType = EPCGEditorSubgraphNodeType::StaticSubgraph;
};

class FPCGEditorGraphDebugObjectItem_PCGLoopIndex : public FPCGEditorGraphDebugObjectItem
{
public:
	explicit FPCGEditorGraphDebugObjectItem_PCGLoopIndex(
		int32 InLoopIndex,
		TWeakObjectPtr<const UObject> InLoopedPCGGraph,
		const FPCGStack& InPCGStack,
		bool bInIsDebuggable,
		bool bInHasInspectionData)
		: FPCGEditorGraphDebugObjectItem(bInHasInspectionData)
		, LoopIndex(InLoopIndex)
		, LoopedPCGGraph(InLoopedPCGGraph)
		, PCGStack(InPCGStack)
		, bIsDebuggable(bInIsDebuggable)
	{}

	virtual int32 GetLoopIndex() const { return LoopIndex; }

	virtual FString GetLabel(bool bForSorting = false) const override;
	virtual const FPCGStack* GetPCGStack() const override { return &PCGStack; }
	virtual int32 GetSortPriority() const override { return LoopIndex; }
	virtual bool IsDebuggable() const override { return bIsDebuggable; }
	virtual bool IsLoopIteration() const override { return true; }
	virtual const UPCGGraph* GetPCGGraph() const override { return Cast<UPCGGraph>(LoopedPCGGraph.Get()); }

protected:
	virtual FPCGStack* GetMutablePCGStack() override { return &PCGStack; }

	int32 LoopIndex = INDEX_NONE;
	TWeakObjectPtr<const UObject> LoopedPCGGraph = nullptr;

	FPCGStack PCGStack;
	bool bIsDebuggable = false;
};

class SPCGEditorGraphDebugObjectItemRow : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FPCGDebugObjectItemRowAction, const FPCGEditorGraphDebugObjectItemPtr&);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FPCGDebugObjectItemRowPredicate, const FPCGEditorGraphDebugObjectItemPtr&);

	SLATE_BEGIN_ARGS(SPCGEditorGraphDebugObjectItemRow)
	{}
		SLATE_EVENT(FPCGDebugObjectItemRowAction, OnDoubleClick)
		SLATE_EVENT(FPCGDebugObjectItemRowAction, OnJumpTo)
		SLATE_EVENT(FPCGDebugObjectItemRowPredicate, CanJumpTo)
		SLATE_EVENT(FPCGDebugObjectItemRowAction, OnFocus)
		SLATE_EVENT(FPCGDebugObjectItemRowPredicate, CanFocus)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, FPCGEditorGraphDebugObjectItemPtr InItem);

	//~Begin SWidget Interface
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	//~End SWidget Interface

	FReply FocusClicked() const;
	FReply JumpToClicked() const;
	bool IsJumpToEnabled() const;
	bool IsFocusEnabled() const;

private:
	FPCGEditorGraphDebugObjectItemPtr Item;

	/** Invoked when the user double clicks on the row. */
	FPCGDebugObjectItemRowAction OnDoubleClick;
	/** Invoked when the 'Jump To' action is clicked on the row buttons. */
	FPCGDebugObjectItemRowAction OnJumpTo;
	/** Invoked when the 'Go to node' action is clicked on the row buttons. */
	FPCGDebugObjectItemRowAction OnFocus;
	/** Controls whether the jump to button will be enabled or not. */
	FPCGDebugObjectItemRowPredicate CanJumpTo;
	/** Controls whether the 'go to node' button will be enabled or not. */
	FPCGDebugObjectItemRowPredicate CanFocus;
};

class SPCGEditorGraphDebugObjectTree : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphDebugObjectTree) {}
	SLATE_END_ARGS()

	virtual ~SPCGEditorGraphDebugObjectTree();

	void Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// Request a refresh of the widget. If bForce is true, will ask for refresh even if the widget is closed.
	void RequestRefresh(bool bForce = false);

	void SetNodeBeingInspected(const UPCGNode* InPCGNode);

	/** If the stack matches to an item, Expands the tree view to make that item visible and select it, and return true. Otherwise, return false. */
	bool SetDebugObjectFromStackFromAnotherEditor(const FPCGStack& InStack);

	/** Return the first stack downstream of the currently selected stack with the provided node & graph. Return false if nothing is found. */
	bool GetFirstStackFromSelection(const UPCGNode* InNode, const UPCGGraph* InGraph, FPCGStack& OutStack) const;

private:
	void OnTick(double InCurrentTime, bool bForceRefresh = false);
	
	bool IsOpen() const;
	
	FReply FocusOnDebugObject_OnClicked() const;
	bool IsFocusOnDebugObjectButtonEnabled() const;

	FPCGEditorGraphDebugObjectItemPtr GetFirstDebugObjectFromSelection() const;
	void SetDebugObjectFromSelection_OnClicked();
	bool IsSetDebugObjectFromSelectionButtonEnabled() const { return IsSetDebugObjectFromSelectionEnabled.IsSet() && IsSetDebugObjectFromSelectionEnabled.GetValue(); }
	void UpdateIsSetDebugObjectFromSelectionEnabled();

	void RefreshTree();
	void SortTreeItems(bool bIsAscending = true, bool bIsRecursive = true);
	void RestoreTreeState();

	/** Key for looking up source items by original owner, grid size, and grid coordinates. */
	struct FSourceItemKey
	{
		TObjectKey<UObject> OriginalOwner;
		uint32 GridSize = 0;
		FIntVector GridCoords = FIntVector::ZeroValue;

		friend bool operator==(const FSourceItemKey& A, const FSourceItemKey& B)
		{
			return A.OriginalOwner == B.OriginalOwner && A.GridSize == B.GridSize && A.GridCoords == B.GridCoords;
		}

		friend uint32 GetTypeHash(const FSourceItemKey& Key)
		{
			return HashCombine(GetTypeHash(Key.OriginalOwner), HashCombine(GetTypeHash(Key.GridSize), GetTypeHash(Key.GridCoords)));
		}
	};

	/** Accumulated state used while building the tree in RefreshTree/AddStackToTree. Tracks owner items,
	 *  stack-to-item mappings, and per-actor grid size information needed for hierarchical grid nesting. */
	struct FHierarchyContext
	{
		/** When true, sources are grouped by original actor with grid nesting. When false, flat view with partition actors. */
		bool bHierarchical = true;
		/** Maps owner to root tree item. In hierarchical mode, keyed by original owner. In legacy mode, keyed by direct actor (including PAs). */
		TMap<TObjectKey<UObject>, FPCGEditorGraphDebugObjectItemPtr> OwnerItems;
		/** Maps stack to tree item. */
		TMap<const FPCGStack, FPCGEditorGraphDebugObjectItemPtr> StackToItem;
		/** Maps (OriginalOwner, GridSize, GridCoords) to source tree item for parent lookup. Hierarchical mode only. */
		TMap<FSourceItemKey, FPCGEditorGraphDebugObjectItemPtr> SourceItems;
		/** Sorted (ascending) grid sizes per original owner. Pre-collected before processing stacks. Hierarchical mode only. */
		TMap<TObjectKey<UObject>, TArray<uint32>> OwnerGridSizes;
	};

	void AddStacksToTree(const TArray<FPCGStackSharedPtr>& Stacks, FHierarchyContext& InOutContext);
	void AddStacksToTree(const TArray<FPCGStack>& Stacks, FHierarchyContext& InOutContext);
	void AddStackToTree(const FPCGStack& Stack, const UPCGGraph* GraphBeingEdited, FHierarchyContext& InOutContext);

	void OnEditorSelectionChanged(UObject* InObject);

	UPCGGraph* GetFocusedGraph() const;
	UPCGGraph* GetMainGraph() const;
	bool IsMainOrEmbeddedGraph(const UPCGGraph* InGraph) const;

	TSharedRef<ITableRow> MakeTreeRowWidget(FPCGEditorGraphDebugObjectItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable);
	void OnGetChildren(FPCGEditorGraphDebugObjectItemPtr InItem, TArray<FPCGEditorGraphDebugObjectItemPtr>& OutChildren) const;
	void OnSelectionChanged(FPCGEditorGraphDebugObjectItemPtr InItem, ESelectInfo::Type InSelectInfo);
	void OnExpansionChanged(FPCGEditorGraphDebugObjectItemPtr InItem, bool bInIsExpanded);
	void OnSetExpansionRecursive(FPCGEditorGraphDebugObjectItemPtr InItem, bool bInExpand) const;

	/** Returns the matching item from the AllGraphItems array if any. */
	FPCGEditorGraphDebugObjectItemPtr GetItemFromStack(const FPCGStack& InStack) const;
	/** Expands the tree view to make an item visible and select it. */
	void ExpandAndSelectDebugObject(const FPCGEditorGraphDebugObjectItemPtr& InItem);

	/** Expand the given row and select the next occurrence of the current graph. */
	void ExpandAndSelectFirstGraphDebugObject(const FPCGEditorGraphDebugObjectItemPtr& InItem);
	
	TSharedRef<SWidget> OpenFilterMenu();
	const FSlateBrush* GetFilterBadgeIcon() const;

	TSharedPtr<SWidget> OpenContextMenu();

	/** Jump-to context menu command. */
	void ContextMenu_JumpToGraphInTree();
	bool ContextMenu_JumpToGraphInTree_CanExecute() const;

	void JumpToGraphInTree(const FPCGEditorGraphDebugObjectItemPtr& Item);
	bool CanJumpToGraphInTree(const FPCGEditorGraphDebugObjectItemPtr& Item) const;
	void FocusOnItem(const FPCGEditorGraphDebugObjectItemPtr& Item);
	bool CanFocusOnItem(const FPCGEditorGraphDebugObjectItemPtr& Item) const;

	void ToggleShowOnlyErrorsAndWarnings();
	ECheckBoxState IsShowingOnlyErrorsAndWarnings() const;
	void ToggleShowDownstream();
	ECheckBoxState IsShowingDownstream() const;
	void ToggleHierarchicalView();
	ECheckBoxState IsUsingHierarchicalView() const;

	TWeakPtr<FPCGEditor> PCGEditor;

	TSharedPtr<STreeView<FPCGEditorGraphDebugObjectItemPtr>> DebugObjectTreeView;
	TArray<FPCGEditorGraphDebugObjectItemPtr> RootItems;
	TArray<FPCGEditorGraphDebugObjectItemPtr> AllGraphItems;

	bool bNeedsRefresh = false;
	bool bShouldSelectStackOnNextRefresh = false;

	double NextRefreshTime = 0.0;

	/** Latest value for IsSetDebugObjectFromSelectionButtonEnabled */
	TOptional<bool> IsSetDebugObjectFromSelectionEnabled;

	/** Used to retain item expansion state across tree refreshes. */
	TSet<FPCGStack> ExpandedStacks;

	/** Used to retain item selection state across tree refreshes. */
	FPCGStack SelectedStack;

	/** Used to retain item selection state across tree refreshes if the SelectedStack is invalidated (e.g. through BP reconstruction). */
	TSoftObjectPtr<const UPCGGraph> SelectedGraph = nullptr;
	TSoftObjectPtr<const AActor> SelectedOwner = nullptr;
	uint32 SelectedGridSize = PCGHiGenGrid::UnboundedGridSize();
	FIntVector SelectedGridCoord = FIntVector::ZeroValue;
	FPCGSoftGraphExecutionSource SelectedOriginalSource;
	TSoftObjectPtr<const UPCGNode> PCGNodeBeingInspected = nullptr;

	/** Controls whether only stacks containing errors/warnings should be shown. */
	bool bShowOnlyErrorsAndWarnings = false;

	/** Controls whether downstream graphs are going to be shown. */
	bool bShowDownstream = true;

	/** Controls whether sources are grouped by original actor with nested grid hierarchy or shown flat with partition actors. */
	bool bUseHierarchicalView = false;
};
