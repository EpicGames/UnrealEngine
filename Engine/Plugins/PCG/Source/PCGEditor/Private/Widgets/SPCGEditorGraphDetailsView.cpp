// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphDetailsView.h"

#include "PCGEditor.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGSettings.h"
#include "Nodes/PCGEditorGraphNode.h"

#include "DetailsViewArgs.h"
#include "IDetailPropertyExtensionHandler.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Layout/Visibility.h"
#include "Modules/ModuleManager.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PCGEditorGraphDetailsView"

namespace PCGEditorGraphDetailsView
{
	namespace Constants
	{
		constexpr TCHAR DefaultDebugDensityMaterialPath[] = TEXT("/PCG/DebugObjects/PCG_DebugMaterial.PCG_DebugMaterial");
		constexpr TCHAR DefaultDebugMaterialColorPath[] = TEXT("/PCG/DebugObjects/PCG_DebugMaterial_Color.PCG_DebugMaterial_Color");

		static const FText DebugMaterialDensityButtonText = LOCTEXT("DebugMaterialDensityButtonText", "Density");
		static const FText DebugMaterialColorButtonText = LOCTEXT("DebugMaterialColorButtonText", "Color");

		static const FText DebugMaterialButtonToastText = LOCTEXT("DebugMaterialButtonToastText", "[PCG] Assigned Debug Material");

		static const FText DebugMaterialDensityButtonToastSubText = LOCTEXT("DebugMaterialDensityButtonToastSubText", "Density - PCG_DebugMaterial");
		static const FText DebugMaterialColorButtonToastSubText = LOCTEXT("DebugMaterialColorButtonToastSubText", "Color - PCG_DebugMaterial_Color");

		static const FText DefaultDebugMaterialDensityButtonTooltip = LOCTEXT("DefaultDebugMaterialDensityButtonTooltip", "Select the default debug material - 'PCG_DebugMaterial'.");
		static const FText DefaultDebugMaterialColorButtonTooltip = LOCTEXT("DefaultDebugMaterialColorButtonTooltip", "Select the default  debug material - 'PCG_DebugMaterial_Color'");
	}
}

class FPCGDebugDetailsExtension : public IDetailPropertyExtensionHandler
{
	virtual bool IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const override
	{
		if (const FProperty* Property = PropertyHandle.GetProperty())
		{
			return Property->GetFName() == GET_MEMBER_NAME_CHECKED(FPCGDebugVisualizationSettings, MaterialOverride);
		}
		else
		{
			return false;
		}
	}

	virtual void ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle) override
	{
		using namespace PCGEditorGraphDetailsView;
		if (PropertyHandle.IsValid())
		{
			check(IsPropertyExtendable(InObjectClass, *PropertyHandle));

			InWidgetRow.ExtensionContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.MinWidth(0)
				.MaxWidth(80)
				.HAlign(HAlign_Right)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SButton)
						.Text(Constants::DebugMaterialDensityButtonText)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.IsEnabled_Lambda([PropertyHandle]
						{
							FString CurrentString;
							PropertyHandle->GetValueAsFormattedString(CurrentString);
							return CurrentString != Constants::DefaultDebugDensityMaterialPath;
						})
						.OnClicked_Lambda([PropertyHandle]
						{
							PropertyHandle->SetValueFromFormattedString(Constants::DefaultDebugDensityMaterialPath);
							FPCGEditorCommon::Helpers::DispatchEditorToast(Constants::DebugMaterialButtonToastText, Constants::DebugMaterialDensityButtonToastSubText);
							return FReply::Handled();
						})
						.ToolTipText(Constants::DefaultDebugMaterialDensityButtonTooltip)
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SButton)
						.Text(Constants::DebugMaterialColorButtonText)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.IsEnabled_Lambda([PropertyHandle]
						{
							FString CurrentString;
							PropertyHandle->GetValueAsFormattedString(CurrentString);
							return CurrentString != Constants::DefaultDebugMaterialColorPath;
						})
						.OnClicked_Lambda([PropertyHandle]
						{
							PropertyHandle->SetValueFromFormattedString(Constants::DefaultDebugMaterialColorPath);
							FPCGEditorCommon::Helpers::DispatchEditorToast(Constants::DebugMaterialButtonToastText, Constants::DebugMaterialColorButtonToastSubText);

							return FReply::Handled();
						})
						.ToolTipText(Constants::DefaultDebugMaterialColorButtonTooltip)
					]
				]
			];
		}
	}
};

void SPCGEditorGraphDetailsView::Construct(const FArguments& InArgs)
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bCustomFilterAreaLocation = true;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	// Filter out the graph parameters from the details view
	DetailsViewArgs.ShouldForceHideProperty.BindLambda(
		[](const TSharedRef<IPropertyHandle>& PropertyNode)
		{
			if (const FProperty* Property = PropertyNode->GetProperty())
			{
				if (const FStructProperty* PropertyStruct = CastField<FStructProperty>(Property))
				{
					// Explicit because FInstancedPropertyBag holds the Custom DetailsView for the graphs.
					return PropertyStruct->Struct == FInstancedPropertyBag::StaticStruct() && PropertyStruct->GetOwnerClass() == UPCGGraph::StaticClass();
				}
			}

			return false;
		});

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SPCGEditorGraphDetailsView::IsReadOnlyProperty));
	DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SPCGEditorGraphDetailsView::IsVisibleProperty));
	// Adds helper buttons to the Debug Details View display.
	const TSharedRef<FPCGDebugDetailsExtension> DebugDetailsExtension = MakeShared<FPCGDebugDetailsExtension>();
	DetailsView->SetExtensionHandler(DebugDetailsExtension);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(0.f, 0.f, 0.f, 0.f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SPCGEditorGraphDetailsView::OnLockButtonClicked)
				.ContentPadding(FMargin(4, 2))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ToolTipText(LOCTEXT("LockSelectionButton_ToolTip", "Locks the current attribute list view to this selection"))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &SPCGEditorGraphDetailsView::GetLockIcon)
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SPCGEditorGraphDetailsView::OnNameClicked)
				.Visibility(this, &SPCGEditorGraphDetailsView::GetNameVisibility)
				[
					SNew(STextBlock)
					.Text(this, &SPCGEditorGraphDetailsView::GetName)
				]
			]
			+SHorizontalBox::Slot()
			[
				DetailsView->GetFilterAreaWidget().ToSharedRef()
			]
		]
		+SVerticalBox::Slot()
		[
			DetailsView.ToSharedRef()
		]
	];
}

void SPCGEditorGraphDetailsView::SetObject(UObject* InObject, bool bForceRefresh)
{
	TArray<TWeakObjectPtr<UObject>> InObjects;
	InObjects.Add(InObject);

	SetObjects(InObjects, bForceRefresh);
}

void SPCGEditorGraphDetailsView::SetObjects(const TArray<TWeakObjectPtr<UObject>>& InObjects, bool bForceRefresh, bool bOverrideLock)
{
	if (bOverrideLock || !bIsLocked)
	{
		SelectedObjects = InObjects;

		// Filter only the types we're interested in, e.g. the settings and not the nodes themselves
		TArray<TWeakObjectPtr<UObject>> ObjectsToView;
		ObjectsToView.Reserve(InObjects.Num());

		for (const TWeakObjectPtr<UObject>& InObject : InObjects)
		{
			if (!InObject.IsValid())
			{
				continue;
			}

			if (UPCGEditorGraphNodeBase* PCGGraphNode = Cast<UPCGEditorGraphNodeBase>(InObject.Get()))
			{
				if (UPCGNode* PCGNode = PCGGraphNode->GetPCGNode())
				{
					if (PCGNode->IsInstance())
					{
						ObjectsToView.Add(PCGNode->GetSettingsInterface());
					}
					else
					{
						ObjectsToView.Add(PCGNode->GetSettings());
					}

					continue;
				}
			}

			ObjectsToView.Add(InObject);
		}

		DetailsView->SetObjects(ObjectsToView);

		if (InObjects.IsEmpty())
		{
			bIsLocked = false;
		}
	}
}

const TArray<TWeakObjectPtr<UObject>>& SPCGEditorGraphDetailsView::GetSelectedObjects() const
{
	return SelectedObjects;
}

FReply SPCGEditorGraphDetailsView::OnLockButtonClicked()
{
	bIsLocked = !bIsLocked;
	return FReply::Handled();
}

FReply SPCGEditorGraphDetailsView::OnNameClicked()
{
	if (SelectedObjects.Num() != 1)
	{
		return FReply::Handled();
	}

	if (UEdGraphNode* GraphNode = Cast<UEdGraphNode>(SelectedObjects[0].Get()))
	{
		if (FPCGEditor* Editor = EditorPtr.Pin().Get())
		{
			Editor->JumpToNode(GraphNode);
		}
	}

	return FReply::Handled();
}

const FSlateBrush* SPCGEditorGraphDetailsView::GetLockIcon() const
{
	return FAppStyle::GetBrush(bIsLocked ? TEXT("PropertyWindow.Locked") : TEXT("PropertyWindow.Unlocked"));
}

EVisibility SPCGEditorGraphDetailsView::GetNameVisibility() const
{
	return bIsLocked ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SPCGEditorGraphDetailsView::GetName() const
{
	const TArray<TWeakObjectPtr<UObject>>& DetailsSelectedObjects = DetailsView->GetSelectedObjects();
	if (DetailsSelectedObjects.IsEmpty())
	{
		// Implementation note: this doesn't really happen since we reselect the graph settings when unselecting
		return LOCTEXT("NoObjectsSelected", "Empty selection");
	}
	else if (DetailsSelectedObjects.Num() == 1)
	{
		UObject* SelectedObject = DetailsSelectedObjects[0].Get();
		UPCGNode* OwnerNode = nullptr;

		if (Cast<UPCGSettings>(SelectedObject) != nullptr || Cast<UPCGSettingsInstance>(SelectedObject) != nullptr)
		{
			OwnerNode = Cast<UPCGNode>(SelectedObject->GetOuter());
		}

		if (OwnerNode)
		{
			return OwnerNode->GetNodeTitle(EPCGNodeTitleType::ListView);
		}
		else if (UPCGGraph* Graph = Cast<UPCGGraph>(SelectedObject))
		{
			return LOCTEXT("GraphSettingsSelected", "Graph Settings");
		}
		else
		{
			return FText::FromName(SelectedObject->GetFName());
		}
	}
	else
	{
		return FText::Format(LOCTEXT("MultipleObjectsSelectedFmt", "{0} nodes"), FText::AsNumber(DetailsSelectedObjects.Num()));
	}
}

bool SPCGEditorGraphDetailsView::IsReadOnlyProperty(const FPropertyAndParent& InPropertyAndParent) const
{
	// Everything is writable when not in an instance
	if (!DetailsView ||
		InPropertyAndParent.ParentProperties.IsEmpty() ||
		InPropertyAndParent.ParentProperties.Last()->GetFName() != GET_MEMBER_NAME_CHECKED(UPCGSettingsInstance, Settings))
	{
		return false;
	}

	const TArray<TWeakObjectPtr<UObject>>& DetailsSelectedObjects = DetailsView->GetSelectedObjects();
	for (const TWeakObjectPtr<UObject>& SelectedObject : DetailsSelectedObjects)
	{
		if (!SelectedObject.IsValid())
		{
			continue;
		}

		if (UPCGSettingsInstance* Instance = Cast<UPCGSettingsInstance>(SelectedObject.Get()))
		{
			return true;
		}
	}

	return false;
}

bool SPCGEditorGraphDetailsView::IsVisibleProperty(const FPropertyAndParent& InPropertyAndParent) const
{
	if (!DetailsView)
	{
		return true;
	}

	// Currently never hide anything from the graph settings
	if (InPropertyAndParent.Objects.Num() == 1 && Cast<UPCGGraph>(InPropertyAndParent.Objects[0]))
	{
		return true;
	}

	// Always hide asset info information
	if (InPropertyAndParent.Property.HasMetaData(TEXT("Category")) &&
		InPropertyAndParent.Property.GetMetaData(TEXT("Category")) == TEXT("AssetInfo"))
	{
		return false;
	}

	// Otherwise, everything is visible when not in an instance
	if (InPropertyAndParent.ParentProperties.IsEmpty() ||
		InPropertyAndParent.ParentProperties.Last()->GetFName() != GET_MEMBER_NAME_CHECKED(UPCGSettingsInstance, Settings))
	{
		return true;
	}

	// Hide debug settings from the setting when showing the instance settings.
	if (InPropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(UPCGSettings, bEnabled) ||
		InPropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(UPCGSettings, bDebug) ||
		(InPropertyAndParent.ParentProperties.Num() >= 2 &&
			InPropertyAndParent.ParentProperties[InPropertyAndParent.ParentProperties.Num() - 2]->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGSettings, DebugSettings)))
	{
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE