// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FPCGEditor;
class IDetailsView;
struct FPropertyAndParent;

/** Thin wrapper on top of a SDetailsView to add some PCG-specific logic around locking and improve layout */
class SPCGEditorGraphDetailsView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphDetailsView)
		{}	
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	/** Sets the editor associated to this details view, used for interaction from the details view to the editor */
	void SetEditor(TWeakPtr<FPCGEditor> InEditorPtr) { EditorPtr = InEditorPtr; }

	/** Returns the details view under this widget */
	TSharedPtr<IDetailsView> GetDetailsView() const { return DetailsView; }

	//~Begin IDetailsView-like interface
	/** Returns the lock state of the details view, e.g. whether the selected object should change on a new selection */
	bool IsLocked() const { return bIsLocked; }

	/** Changes the selected objects in the details view. Will unlock automatically. */
	void SetObject(UObject* InObject, bool bForceRefresh = false);

	/** Changes the selected objects in the details view. Will unlock automatically. */
	void SetObjects(const TArray<TWeakObjectPtr<UObject>>& InObjects, bool bForceRefresh = false, bool bOverrideLock = false);

	/** Retrieves the list of objects currently shown in the details view */
	const TArray<TWeakObjectPtr<UObject>>& GetSelectedObjects() const;
	//~End IDetailsView-like interface

	/** Controls lock from the editor */
	void SetIsLocked(bool bInIsLocked) { bIsLocked = bInIsLocked; }

protected:
	/** Returns whether a property should be readonly (used for instances) */
	bool IsReadOnlyProperty(const FPropertyAndParent& InPropertyAndParent) const;
	/** Returns whether a property should be visible (used for instance vs. settings properties) */
	bool IsVisibleProperty(const FPropertyAndParent& InPropertyAndParent) const;

	FReply OnLockButtonClicked();
	FReply OnNameClicked();
	const FSlateBrush* GetLockIcon() const;

	/** Gets the current visibility state of the name section, which is related to whether the details view is locked. */
	EVisibility GetNameVisibility() const;
	/** Returns the name to show in the details-view name section, e.g. the settings title most of the time. */
	FText GetName() const;

	TSharedPtr<IDetailsView> DetailsView;
	bool bIsLocked : 1 = false;

	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	TWeakPtr<FPCGEditor> EditorPtr = nullptr;
};