// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphDeterminism.h"

#include "Framework/Views/TableViewMetadata.h"
#include "Tests/Determinism/PCGDeterminismTestsCommon.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "PCGDeterminism"

namespace
{
	const FName NAME_Index(TEXT("Index_ColumnID"));
	const FName NAME_NodeTitle(TEXT("NodeTitle_ColumnID"));
	const FName NAME_NodeName(TEXT("NodeName_ColumnID"));
	const FName NAME_Seed(TEXT("Seed_ColumnID"));
	const FName NAME_DataTypesTested(TEXT("DataTypesTested_ColumnID"));
	const FName NAME_AdditionalDetails(TEXT("AdditionalDetails_ColumnID"));

	const FText TEXT_Index;
	const FText TEXT_NodeTitle(LOCTEXT("NodeTitle_Label", "Title"));
	const FText TEXT_NodeName(LOCTEXT("NodeName_Label", "Name"));
	const FText TEXT_Seed(LOCTEXT("Seed_Label", "Seed"));
	const FText TEXT_DataTypesTested(LOCTEXT("DataTypesTested_Label", "Input Data"));
	const FText TEXT_AdditionalDetails(LOCTEXT("AdditionalDetails_Label", "Additional Details"));

	const FText TEXT_NotDeterministic(LOCTEXT("NotDeterministic", "Fail"));
	const FText TEXT_Consistent(LOCTEXT("OrderConsistent", "Order Consistent"));
	const FText TEXT_Independent(LOCTEXT("OrderIndependent", "Order Independent"));
	const FText TEXT_Orthogonal(LOCTEXT("OrderOrthogonal", "Order Orthogonal"));
	const FText TEXT_Basic(LOCTEXT("BasicDeterminism", "Pass"));

	constexpr float SmallManualWidth = 25.f;
	constexpr float MediumManualWidth = 80.f;
	constexpr float LargeManualWidth = 160.f;
}

void SPCGEditorGraphDeterminismRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const FPCGNodeTestResultPtr& Item, int32 ItemIndex)
{
	CurrentItem = Item;
	SMultiColumnTableRow<FPCGNodeTestResultPtr>::Construct(SMultiColumnTableRow::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SPCGEditorGraphDeterminismRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	check(CurrentItem);

	FText CellText;

	auto ReturnColorCodedResultBlock = [this](const FText& OutCellText)
	{
		return SNew(STextBlock)
			.Text(OutCellText)
			.ColorAndOpacity(CurrentItem->bFlagRaised ? FColor::Red : FColor::Green);
	};

	// Permanent columns
	if (ColumnId == NAME_Index)
	{
		return ReturnColorCodedResultBlock(FText::FromString(FString::FromInt(CurrentItem->Index)));
	}
	else if (ColumnId == NAME_NodeTitle)
	{
		return ReturnColorCodedResultBlock(FText::FromName(CurrentItem->TestResultTitle));
	}
	else if (ColumnId == NAME_NodeName)
	{
		CellText = FText::FromString(CurrentItem->TestResultName);
	}
	else if (ColumnId == NAME_Seed)
	{
		CellText = FText::AsNumber(CurrentItem->Seed);
	}
	else if (ColumnId == NAME_DataTypesTested)
	{
		CellText = FText::FromString(CurrentItem->DataTypesTested.ToString());
	}
	else if (ColumnId == NAME_AdditionalDetails)
	{
		// TODO: Add tooltip widget to display multiple error messages, but only display the first in the row for now
		FString FullDetails = CurrentItem->AdditionalDetails.Num() > 0 ? CurrentItem->AdditionalDetails[0] : "";
		CellText = FText::FromString(FullDetails);
	}
	// Test columns
	else if (EDeterminismLevel* DeterminismLevel = CurrentItem->TestResults.Find(ColumnId))
	{
		switch (*DeterminismLevel)
		{
		case EDeterminismLevel::OrderOrthogonal:
			return SNew(STextBlock)
				.Text(TEXT_Orthogonal)
				.ColorAndOpacity(FColor::Orange);
		case EDeterminismLevel::OrderConsistent:
			return SNew(STextBlock)
				.Text(TEXT_Consistent)
				.ColorAndOpacity(FColor::Yellow);
		case EDeterminismLevel::OrderIndependent:
			return SNew(STextBlock)
				.Text(TEXT_Independent)
				.ColorAndOpacity(FColor::Green);
		case EDeterminismLevel::Basic:
			return SNew(STextBlock)
				.Text(TEXT_Basic)
				.ColorAndOpacity(FColor::Turquoise);
		case EDeterminismLevel::NoDeterminism:
		default:
			return SNew(STextBlock)
				.Text(TEXT_NotDeterministic)
				.ColorAndOpacity(FColor::Red);
		}
	}

	return SNew(STextBlock)
		.Text(CellText);
}

void SPCGEditorGraphDeterminismListView::Construct(const FArguments& InArgs, TWeakPtr<FPCGEditor> InPCGEditor)
{
	check(InPCGEditor.IsValid() && !bIsConstructed);
	PCGEditorPtr = InPCGEditor;

	GeneratedHeaderRow = SNew(SHeaderRow);
	SortMode = EColumnSortMode::None;
	SortingColumn = NAME_None;

	SAssignNew(ListView, SListView<FPCGNodeTestResultPtr>)
		.ListItemsSource(&ListViewItems)
		.OnGenerateRow(this, &SPCGEditorGraphDeterminismListView::OnGenerateRow)
		.HeaderRow(GeneratedHeaderRow);

	ChildSlot
	[
		ListView->AsShared()
	];

	bIsConstructed = true;
}

void SPCGEditorGraphDeterminismListView::AddItem(const FPCGNodeTestResultPtr& Item)
{
	check(Item.IsValid());
	ListViewItems.Emplace(Item);
}

void SPCGEditorGraphDeterminismListView::ClearItems()
{
	ItemIndexCounter = -1;
	ListViewItems.Empty();
	RefreshItems();
}

void SPCGEditorGraphDeterminismListView::RefreshItems()
{
	ListView->RequestListRefresh();
}

void SPCGEditorGraphDeterminismListView::AddColumn(const FTestColumnInfo& ColumnInfo)
{
	SHeaderRow::FColumn::FArguments Arguments;
	Arguments.ColumnId(ColumnInfo.ColumnID);
	Arguments.DefaultLabel(ColumnInfo.ColumnLabel);
	if (ColumnInfo.Width > 0.f)
	{
		Arguments.ManualWidth(ColumnInfo.Width);
	}
	Arguments.HAlignHeader(ColumnInfo.HAlign);
	Arguments.HAlignCell(ColumnInfo.HAlign);
	Arguments.SortMode(this, &SPCGEditorGraphDeterminismListView::GetColumnSortMode, ColumnInfo.ColumnID);
	Arguments.OnSort(this, &SPCGEditorGraphDeterminismListView::OnSortColumnHeader);
	GeneratedHeaderRow->AddColumn(Arguments);
}

void SPCGEditorGraphDeterminismListView::BuildBaseColumns()
{
	ClearColumns();
	// Permanent columns
	AddColumn({NAME_Index, TEXT_Index, SmallManualWidth, HAlign_Center});
	AddColumn({NAME_NodeTitle, TEXT_NodeTitle, LargeManualWidth, HAlign_Left});
	AddColumn({NAME_NodeName, TEXT_NodeName, LargeManualWidth, HAlign_Left});
	AddColumn({NAME_Seed, TEXT_Seed, MediumManualWidth, HAlign_Center});
	AddColumn({NAME_DataTypesTested, TEXT_DataTypesTested, MediumManualWidth, HAlign_Center});
}

void SPCGEditorGraphDeterminismListView::AddDetailsColumn()
{
	// Final details column
	AddColumn({NAME_AdditionalDetails, TEXT_AdditionalDetails, 0.f, HAlign_Left});
}

bool SPCGEditorGraphDeterminismListView::WidgetIsConstructed() const
{
	return bIsConstructed;
}

void SPCGEditorGraphDeterminismListView::ClearColumns()
{
	GeneratedHeaderRow->ClearColumns();
}

TSharedRef<ITableRow> SPCGEditorGraphDeterminismListView::OnGenerateRow(const FPCGNodeTestResultPtr Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SPCGEditorGraphDeterminismRow, OwnerTable, Item, ++ItemIndexCounter);
}

void SPCGEditorGraphDeterminismListView::OnSortColumnHeader(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode)
{
	if (SortingColumn == ColumnId)
	{
		// Circling
		SortMode = static_cast<EColumnSortMode::Type>((SortMode + 1) % 3);
	}
	else
	{
		SortingColumn = ColumnId;
		SortMode = NewSortMode;
	}

	if (SortingColumn != NAME_None && SortMode != EColumnSortMode::None)
	{
		Algo::Sort(ListViewItems, [this, ColumnId](const FPCGNodeTestResultPtr& A, const FPCGNodeTestResultPtr& B)
		{
			bool bIsLess = false;
			if (SortingColumn == NAME_Index)
			{
				bIsLess = A->Index < B->Index;
			}
			else if (SortingColumn == NAME_NodeTitle)
			{
				bIsLess = A->TestResultTitle.ToString() < B->TestResultTitle.ToString();
			}
			else if (SortingColumn == NAME_NodeName)
			{
				bIsLess = A->TestResultName < B->TestResultName;
			}
			else if (SortingColumn == NAME_Seed)
			{
				bIsLess = A->Seed < B->Seed;
			}
			else if (SortingColumn == NAME_DataTypesTested)
			{
				bIsLess = A->DataTypesTested.ToString() < B->DataTypesTested.ToString();
			}
			else if (SortingColumn == NAME_AdditionalDetails)
			{
				bIsLess = (A->AdditionalDetails.IsEmpty() && !B->AdditionalDetails.IsEmpty()) ||
					(!A->AdditionalDetails.IsEmpty() && !B->AdditionalDetails.IsEmpty() && A->AdditionalDetails[0] < B->AdditionalDetails[0]);
			}
			else
			{
				const EDeterminismLevel* DeterminismLevelA = A->TestResults.Find(ColumnId);
				const EDeterminismLevel* DeterminismLevelB = B->TestResults.Find(ColumnId);
				bIsLess = (DeterminismLevelA && !DeterminismLevelB) ||
					(DeterminismLevelA && DeterminismLevelB && *DeterminismLevelA < *DeterminismLevelB);
			}

			return SortMode == EColumnSortMode::Ascending ? bIsLess : !bIsLess;
		});
	}

	RefreshItems();
}

EColumnSortMode::Type SPCGEditorGraphDeterminismListView::GetColumnSortMode(const FName ColumnId) const
{
	if (SortingColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

#undef LOCTEXT_NAMESPACE
