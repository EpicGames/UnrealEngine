// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;
struct FDeterminismTestResult;

class FPCGEditor;

typedef TSharedPtr<FDeterminismTestResult> FPCGNodeTestResultPtr;

struct FTestColumnInfo
{
	FTestColumnInfo(FName ColumnID, FText ColumnLabel, float Width = 0.f, EHorizontalAlignment HAlign = HAlign_Left) :
		ColumnID(ColumnID),
		ColumnLabel(ColumnLabel),
		Width(Width),
		HAlign(HAlign) {}

	FName ColumnID = TEXT("UnnamedColumn_ID");
	FText ColumnLabel = NSLOCTEXT("PCGDeterminism", "Unnamed_Column", "Unnamed Column");
	float Width = 0.f;
	EHorizontalAlignment HAlign = HAlign_Left;
};

class SPCGEditorGraphDeterminismRow final : public SMultiColumnTableRow<FPCGNodeTestResultPtr>
{
	SLATE_BEGIN_ARGS(SPCGEditorGraphDeterminismRow) {}
	SLATE_END_ARGS()

	/** Construct a row of the ListView */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const FPCGNodeTestResultPtr& Item, int32 ItemIndex);

	/** Generates a column, given the column's ID */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

protected:
	FPCGNodeTestResultPtr CurrentItem;
};

class SPCGEditorGraphDeterminismListView final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphDeterminismListView) {}
	SLATE_END_ARGS()

	/** Construct the ListView */
	void Construct(const FArguments& InArgs, TWeakPtr<FPCGEditor> InPCGEditor);
	/** Add an item to the ListView */
	void AddItem(const FPCGNodeTestResultPtr& Item);
	/** Clear all items from the ListView */
	void ClearItems();
	/** Refreshes the ListView */
	void RefreshItems();
	/** Adds a test column to the ListView */
	void AddColumn(const FTestColumnInfo& ColumnInfo);
	/** Builds the default columns for the widget */
	void BuildBaseColumns();
	/** Adds the additional details column */
	void AddDetailsColumn();

	/** Validates if the ListView has been constructed */
	bool WidgetIsConstructed() const;

private:
	/** Clears the columns of the widget */
	void ClearColumns();
	/** Generate the row widget */
	TSharedRef<ITableRow> OnGenerateRow(const FPCGNodeTestResultPtr Item, const TSharedRef<STableViewBase>& OwnerTable) const;
	void OnSortColumnHeader(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode);
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	TSharedPtr<SHeaderRow> GeneratedHeaderRow;

	TWeakPtr<FPCGEditor> PCGEditorPtr;

	TSharedPtr<SListView<FPCGNodeTestResultPtr>> ListView;
	TArray<FPCGNodeTestResultPtr> ListViewItems;

	// To allow sorting
	FName SortingColumn = NAME_None;
	EColumnSortMode::Type SortMode = EColumnSortMode::Type::None;

	bool bIsConstructed = false;
	mutable int32 ItemIndexCounter = -1;
};
