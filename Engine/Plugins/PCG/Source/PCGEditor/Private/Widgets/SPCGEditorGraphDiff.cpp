// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SPCGEditorGraphDiff.h"

#include "Details/PCGGraphDetails.h"
#include "Nodes/PCGEditorGraphNodeBase.h"
#include "PCGEditorGraph.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "Schema/PCGEditorGraphSchema.h"

#include "DetailsViewArgs.h"
#include "DiffUtils.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphUtilities.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GraphDiffControl.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailsView.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "PropertyBagDiff.h"
#include "PropertyEditorModule.h"
#include "StructUtils/PropertyBag.h"
#include "AsyncDetailViewDiff.h"
#include "SDetailsDiff.h"
#include "SDetailsSplitter.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SPCGEditorGraphDiff"

//////////////////////////////////////////////////////////////////////////
// FPCGDiffResultItem

TSharedRef<SWidget> FPCGDiffResultItem::GenerateWidget() const
{
	const FText& ToolTip = Result.ToolTip;
	const FSlateColor Color = Result.GetDisplayColor();
	const FText& Text = Result.DisplayString;

	return SNew(STextBlock)
		.ToolTipText(Text.IsEmpty() ? LOCTEXT("UnknownDiffTooltip", "There is an unspecified difference") : ToolTip)
		.ColorAndOpacity(Color)
		.Text(Text.IsEmpty() ? LOCTEXT("UnknownDiff", "Unknown Diff") : Text);
}

//////////////////////////////////////////////////////////////////////////
// FDiffListCommands

class FPCGDiffListCommands : public TCommands<FPCGDiffListCommands>
{
public:
	FPCGDiffListCommands()
		: TCommands<FPCGDiffListCommands>(TEXT("PCGGraphDiffList"), LOCTEXT("PCGDiff", "PCG Graph Diff"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override
	{
		UI_COMMAND(Previous, "Prev", "Go to previous difference", EUserInterfaceActionType::Button, FInputChord(EKeys::F7, EModifierKey::Control));
		UI_COMMAND(Next, "Next", "Go to next difference", EUserInterfaceActionType::Button, FInputChord(EKeys::F7));
	}

	TSharedPtr<FUICommandInfo> Previous;
	TSharedPtr<FUICommandInfo> Next;
};

//////////////////////////////////////////////////////////////////////////
// SPCGEditorGraphDiff

UPCGEditorGraph* SPCGEditorGraphDiff::CreateEditorGraphForDiff(UPCGGraph* InPCGGraph)
{
	if (!InPCGGraph)
	{
		return nullptr;
	}

	// Create a new editor graph for this PCG graph (typical for loaded revision packages)
	UPCGEditorGraph* EditorGraph = NewObject<UPCGEditorGraph>(InPCGGraph, UPCGEditorGraph::StaticClass(), NAME_None, RF_Transient);
	EditorGraph->Schema = UPCGEditorGraphSchema::StaticClass();
	EditorGraph->InitFromNodeGraph(InPCGGraph);
	return EditorGraph;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SPCGEditorGraphDiff::Construct(const FArguments& InArgs)
{
	LastPinTarget = nullptr;
	LastOtherPinTarget = nullptr;
	FoundDiffs = MakeShared<TArray<FDiffSingleResult>>();

	FPCGDiffListCommands::Register();

	PanelOld.Graph = InArgs._GraphOld;
	PanelNew.Graph = InArgs._GraphNew;

	PanelOld.RevisionInfo = InArgs._OldRevision;
	PanelNew.RevisionInfo = InArgs._NewRevision;

	PanelOld.bShowAssetName = InArgs._ShowAssetNames;
	PanelNew.bShowAssetName = InArgs._ShowAssetNames;

	OpenInDefaults = InArgs._OpenInDefaults;

	// Construct editor graphs from the PCG graphs
	if (const UPCGGraph* OldGraph = PanelOld.Graph)
	{
		PanelOld.EditorGraph.Reset(CreateEditorGraphForDiff(const_cast<UPCGGraph*>(OldGraph)));
	}
	if (const UPCGGraph* NewGraph = PanelNew.Graph)
	{
		PanelNew.EditorGraph.Reset(CreateEditorGraphForDiff(const_cast<UPCGGraph*>(NewGraph)));
	}

	// Create the details views for the settings diff
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowDifferingPropertiesOption = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsViewArgs.ViewIdentifier = FPCGGraphDetails::DiffViewIdentifier;

	const auto AlwaysReadOnly = FIsPropertyEditingEnabled::CreateLambda([]
	{
		return false;
	});

	OldDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	OldDetailsView->SetIsPropertyEditingEnabledDelegate(AlwaysReadOnly);
	OldDetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SPCGEditorGraphDiff::IsPropertyVisibleInCurrentMode));
	OldDetailsView->ShowAllAdvancedProperties();

	NewDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	NewDetailsView->SetIsPropertyEditingEnabledDelegate(AlwaysReadOnly);
	NewDetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SPCGEditorGraphDiff::IsPropertyVisibleInCurrentMode));
	NewDetailsView->ShowAllAdvancedProperties();

	DetailViewDiff = MakeShared<FAsyncDetailViewDiff>(OldDetailsView.ToSharedRef(), NewDetailsView.ToSharedRef());

	const TSharedRef<SHorizontalBox> DefaultEmptyPanel = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SelectGraphTip", "Select a difference to focus"))
		];

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Content()
		[
			SNew(SSplitter)
			+ SSplitter::Slot()
			.Value(0.2f)
			[
				SNew(SBorder)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SButton)
						.OnClicked(this, &SPCGEditorGraphDiff::OnOpenInDefaults)
						.Content()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("DefaultDiff", "Default Diff"))
						]
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.f)
					[
						GenerateDiffListWidget()
					]
				]
			]
			+ SSplitter::Slot()
			.Value(0.8f)
			[
				SNew(SSplitter)
				.Orientation(Orient_Vertical)
				+ SSplitter::Slot()
				.Value(0.7f)
				[
					SNew(SSplitter)
					+ SSplitter::Slot()
					.Value(0.5f)
					[
						SAssignNew(PanelOld.GraphEditorBorder, SBorder)
						.VAlign(VAlign_Fill)
						[
							DefaultEmptyPanel
						]
					]
					+ SSplitter::Slot()
					.Value(0.5f)
					[
						SAssignNew(PanelNew.GraphEditorBorder, SBorder)
						.VAlign(VAlign_Fill)
						[
							DefaultEmptyPanel
						]
					]
				]
				+ SSplitter::Slot()
				.Value(0.3f)
				[
					SAssignNew(DetailsSplitter, SDetailsSplitter)
					+ SDetailsSplitter::Slot()
					.DetailsView(OldDetailsView)
					.DifferencesWithRightPanel_Lambda([this]() { return DetailViewDiff; })
					.Value(0.5f)
					+ SDetailsSplitter::Slot()
					.DetailsView(NewDetailsView)
					.Value(0.5f)
				]
			]
		]
	];

	PanelOld.GeneratePanel(PanelOld.EditorGraph.Get(), FoundDiffs);
	PanelNew.GeneratePanel(PanelNew.EditorGraph.Get(), FoundDiffs);

	ApplyViewLockState();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SPCGEditorGraphDiff::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (DetailViewDiff)
	{
		constexpr float MaxTickTimeMs = 0.01f;
		DetailViewDiff->Tick(MaxTickTimeMs);
	}
}

TSharedPtr<SWindow> SPCGEditorGraphDiff::CreateDiffWindow(const UPCGGraph* OldGraph, const UPCGGraph* NewGraph, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision)
{
	// Determine window title
	FText WindowTitle;
	const bool bIsSingleAsset = !NewGraph || !OldGraph || (NewGraph->GetFName() == OldGraph->GetFName());
	if (bIsSingleAsset)
	{
		const FString GraphName = NewGraph ? NewGraph->GetName() : (OldGraph ? OldGraph->GetName() : TEXT(""));
		WindowTitle = FText::Format(LOCTEXT("PCGGraphDiffTitle", "{0} - PCG Graph Diff"), FText::FromString(GraphName));
	}
	else
	{
		WindowTitle = LOCTEXT("PCGGraphDiffTitleNameless", "PCG Graph Diff");
	}

	const TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(FVector2f(1000.f, 800.f));

	Window->SetContent(SNew(SPCGEditorGraphDiff)
		.GraphOld(OldGraph)
		.GraphNew(NewGraph)
		.OldRevision(OldRevision)
		.NewRevision(NewRevision)
		.ShowAssetNames(!bIsSingleAsset));

	FSlateApplication::Get().AddWindow(Window.ToSharedRef());
	return Window;
}

FReply SPCGEditorGraphDiff::OnOpenInDefaults()
{
	OpenInDefaults.ExecuteIfBound(PanelOld.Graph, PanelNew.Graph);
	return FReply::Handled();
}

TSharedRef<SWidget> SPCGEditorGraphDiff::GenerateDiffListWidget()
{
	BuildDiffSourceArray();

	if (DiffListSource.Num() > 0)
	{
		Algo::SortBy(DiffListSource, [](const FSharedDiffOnGraph& Data) { return Data->Result.Diff; });

		const FPCGDiffListCommands& Commands = FPCGDiffListCommands::Get();
		KeyCommands = MakeShared<FUICommandList>();

		KeyCommands->MapAction(Commands.Previous, FExecuteAction::CreateSP(this, &SPCGEditorGraphDiff::PrevDiff));
		KeyCommands->MapAction(Commands.Next, FExecuteAction::CreateSP(this, &SPCGEditorGraphDiff::NextDiff));

		FToolBarBuilder ToolbarBuilder(KeyCommands.ToSharedRef(), FMultiBoxCustomization::None);
		ToolbarBuilder.AddToolBarButton(Commands.Previous, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.PrevDiff"));
		ToolbarBuilder.AddToolBarButton(Commands.Next, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.NextDiff"));
		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SPCGEditorGraphDiff::OnToggleLockView)),
			NAME_None,
			LOCTEXT("LockGraphsLabel", "Lock/Unlock"),
			LOCTEXT("LockGraphsTooltip", "Pan and zoom both panels together, or scroll/zoom them independently"),
			TAttribute<FSlateIcon>(this, &SPCGEditorGraphDiff::GetLockViewImage));

		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(0.f)
				.AutoHeight()
				[
					ToolbarBuilder.MakeWidget()
				]
				+ SVerticalBox::Slot()
				.Padding(0.f)
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("PropertyWindow.CategoryBackground"))
					.Padding(FMargin(2.0f))
					.ForegroundColor(FAppStyle::GetColor("PropertyWindow.CategoryForeground"))
					.ToolTipText(LOCTEXT("DifferencesToolTip", "List of differences found between revisions, click to select"))
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("RevisionDifferences", "Revision Differences"))
					]
				]
				+ SVerticalBox::Slot()
				.Padding(1.f)
				.FillHeight(1.f)
				[
					SAssignNew(DiffList, SDiffListViewType)
					.ListItemsSource(&DiffListSource)
					.OnGenerateRow(this, &SPCGEditorGraphDiff::OnGenerateRow)
					.SelectionMode(ESelectionMode::Single)
					.OnSelectionChanged(this, &SPCGEditorGraphDiff::OnSelectionChanged)
				]
			];
	}
	else
	{
		return SNew(SBorder)
			.Padding(FMargin(2.0f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoDifferences", "No differences found"))
			];
	}
}

void SPCGEditorGraphDiff::BuildDiffSourceArray()
{
	FoundDiffs->Empty();
	DiffListSource.Empty();

	if (!PanelOld.EditorGraph || !PanelNew.EditorGraph)
	{
		return;
	}

	FGraphDiffControl::DiffGraphs(PanelOld.EditorGraph.Get(), PanelNew.EditorGraph.Get(), *FoundDiffs);

	BuildGraphSettingsDiffs();
	BuildGraphParameterDiffs();

	for (const FDiffSingleResult& Diff : *FoundDiffs)
	{
		DiffListSource.Add(MakeShared<FPCGDiffResultItem>(Diff));
	}
}

void SPCGEditorGraphDiff::NextDiff()
{
	if (DiffListSource.Num() == 0)
	{
		return;
	}

	int32 Index = GetCurrentDiffIndex();
	if (Index == INDEX_NONE)
	{
		// None selected, so select the first one
		DiffList->SetSelection(DiffListSource[0]);
		return;
	}

	Index = (Index + 1) % DiffListSource.Num();
	DiffList->SetSelection(DiffListSource[Index]);
}

void SPCGEditorGraphDiff::PrevDiff()
{
	if (DiffListSource.Num() == 0)
	{
		return;
	}

	int32 Index = GetCurrentDiffIndex();
	if (Index == INDEX_NONE)
	{
		// None selected, so select the first one
		DiffList->SetSelection(DiffListSource[0]);
		return;
	}

	Index = (Index + DiffListSource.Num() - 1) % DiffListSource.Num();

	DiffList->SetSelection(DiffListSource[Index]);
}

int32 SPCGEditorGraphDiff::GetCurrentDiffIndex() const
{
	const TArray<FSharedDiffOnGraph> Selected = DiffList->GetSelectedItems();
	if (Selected.Num() == 1)
	{
		for (int32 Index = 0; Index < DiffListSource.Num(); ++Index)
		{
			if (DiffListSource[Index] == Selected[0])
			{
				return Index;
			}
		}
	}

	return INDEX_NONE;
}

void SPCGEditorGraphDiff::OnToggleLockView()
{
	bLockViews = !bLockViews;
	ApplyViewLockState();
}

FSlateIcon SPCGEditorGraphDiff::GetLockViewImage() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), bLockViews ? "Icons.Lock" : "Icons.Unlock");
}

void SPCGEditorGraphDiff::ApplyViewLockState()
{
	const TSharedPtr<SGraphEditor> OldEditor = PanelOld.GraphEditor.Pin();
	const TSharedPtr<SGraphEditor> NewEditor = PanelNew.GraphEditor.Pin();
	if (!OldEditor || !NewEditor)
	{
		return;
	}

	if (bLockViews)
	{
		OldEditor->LockToGraphEditor(PanelNew.GraphEditor);
		NewEditor->LockToGraphEditor(PanelOld.GraphEditor);
	}
	else
	{
		OldEditor->UnlockFromGraphEditor(PanelNew.GraphEditor);
		NewEditor->UnlockFromGraphEditor(PanelOld.GraphEditor);
	}
}

TSharedRef<ITableRow> SPCGEditorGraphDiff::OnGenerateRow(FSharedDiffOnGraph Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<FSharedDiffOnGraph>, OwnerTable)
		.Content()
		[
			Item->GenerateWidget()
		];
}

void SPCGEditorGraphDiff::OnSelectionChanged(FSharedDiffOnGraph Item, ESelectInfo::Type SelectionType)
{
	if (!Item.IsValid())
	{
		return;
	}

	const FDiffSingleResult& Result = Item->Result;

	if (Result.Pin1)
	{
		if (const TSharedPtr<SGraphEditor> OldEditor = PanelOld.GraphEditor.Pin())
		{
			OldEditor->ClearSelectionSet();
		}
		if (const TSharedPtr<SGraphEditor> NewEditor = PanelNew.GraphEditor.Pin())
		{
			NewEditor->ClearSelectionSet();
		}

		auto FocusPin = [this](UEdGraphPin* InPin)
		{
			if (InPin)
			{
				LastPinTarget = InPin;
				UEdGraph* NodeGraph = InPin->GetOwningNode()->GetGraph();
				if (SGraphEditor* NodeGraphEditor = GetGraphEditorForGraph(NodeGraph))
				{
					NodeGraphEditor->JumpToPin(InPin);
				}
			}
		};

		FocusPin(Result.Pin1);
		FocusPin(Result.Pin2);
	}
	else if (Result.Node1)
	{
		if (const TSharedPtr<SGraphEditor> OldEditor = PanelOld.GraphEditor.Pin())
		{
			OldEditor->ClearSelectionSet();
		}
		if (const TSharedPtr<SGraphEditor> NewEditor = PanelNew.GraphEditor.Pin())
		{
			NewEditor->ClearSelectionSet();
		}

		auto FocusNode = [this](UEdGraphNode* InNode)
		{
			if (InNode)
			{
				UEdGraph* NodeGraph = InNode->GetGraph();
				if (SGraphEditor* NodeGraphEditor = GetGraphEditorForGraph(NodeGraph))
				{
					NodeGraphEditor->JumpToNode(InNode, false);
				}
			}
		};

		FocusNode(Result.Node1);
		FocusNode(Result.Node2);
		
		// If the node was added, only the Node1 will be set but it would be the NewNode, so swap them in that specific case.
		UEdGraphNode* OldNode = Result.Diff == EDiffType::NODE_ADDED ? Result.Node2 : Result.Node1;
		UEdGraphNode* NewNode = Result.Diff == EDiffType::NODE_ADDED ? Result.Node1 : Result.Node2;

		ShowSettingsForNodes(OldNode, NewNode);
	}
	else if (Result.Object1 || Result.Object2)
	{
		// Graph-level entry. The tag stamped at construction time tells us which sub-view to surface.
		if (Result.OwningObjectPath == TEXT("PCG.Parameter"))
		{
			ShowGraphParametersDetails();
		}
		else
		{
			ShowGraphSettingsDetails();
		}
	}
}

void SPCGEditorGraphDiff::ShowSettingsForNodes(UEdGraphNode* OldNode, UEdGraphNode* NewNode)
{
	UObject* OldSettings = nullptr;
	UObject* NewSettings = nullptr;

	if (const UPCGEditorGraphNodeBase* OldPCGEdNode = Cast<UPCGEditorGraphNodeBase>(OldNode))
	{
		if (const UPCGNode* PCGNode = OldPCGEdNode->GetPCGNode())
		{
			OldSettings = PCGNode->GetSettings();
		}
	}

	if (const UPCGEditorGraphNodeBase* NewPCGEdNode = Cast<UPCGEditorGraphNodeBase>(NewNode))
	{
		if (const UPCGNode* PCGNode = NewPCGEdNode->GetPCGNode())
		{
			NewSettings = PCGNode->GetSettings();
		}
	}

	CurrentDetailsMode = EDetailsMode::Node;
	OldDetailsView->SetObject(OldSettings);
	NewDetailsView->SetObject(NewSettings);
}

void SPCGEditorGraphDiff::ShowGraphSettingsDetails()
{
	CurrentDetailsMode = EDetailsMode::GraphSettings;
	OldDetailsView->SetObject(const_cast<UPCGGraph*>(PanelOld.Graph));
	NewDetailsView->SetObject(const_cast<UPCGGraph*>(PanelNew.Graph));
}

void SPCGEditorGraphDiff::ShowGraphParametersDetails()
{
	CurrentDetailsMode = EDetailsMode::GraphParameters;
	OldDetailsView->SetObject(const_cast<UPCGGraph*>(PanelOld.Graph));
	NewDetailsView->SetObject(const_cast<UPCGGraph*>(PanelNew.Graph));
}

bool SPCGEditorGraphDiff::IsPropertyVisibleInCurrentMode(const FPropertyAndParent& PropertyAndParent) const
{
	// Test whether a property is FInstancedPropertyBag — the same condition used by
	// SPCGEditorGraphUserParametersView to discriminate UserParameters from the rest of UPCGGraph.
	auto IsBagProperty = [](const FProperty* Property)
	{
		const FStructProperty* StructProp = CastField<FStructProperty>(Property);
		return StructProp && StructProp->Struct == FInstancedPropertyBag::StaticStruct();
	};

	auto IsInUserParametersBag = [&IsBagProperty](const FPropertyAndParent& InProp)
	{
		if (IsBagProperty(&InProp.Property))
		{
			return true;
		}
		for (const FProperty* Parent : InProp.ParentProperties)
		{
			if (IsBagProperty(Parent))
			{
				return true;
			}
		}
		return false;
	};

	switch (CurrentDetailsMode)
	{
	case EDetailsMode::GraphParameters:
		return IsInUserParametersBag(PropertyAndParent);
	case EDetailsMode::GraphSettings:
		return !IsInUserParametersBag(PropertyAndParent);
	case EDetailsMode::Node: // fall-through
	default:
		return true;
	}
}

void SPCGEditorGraphDiff::BuildGraphSettingsDiffs()
{
	const UPCGGraph* OldGraph = PanelOld.Graph;
	const UPCGGraph* NewGraph = PanelNew.Graph;
	if (!OldGraph || !NewGraph)
	{
		return;
	}

	// Head property names already covered by the node-graph diff or the per-parameter diff,
	// or noisy editor-only state we deliberately exclude from the graph-settings panel.
	static const TSet<FName> SkippedHeadProperties = {
		// Covered by FGraphDiffControl::DiffGraphs (UEdGraph nodes/pins).
		TEXT("Nodes"),
		TEXT("ExtraEditorNodes"),
		TEXT("InputNode"),
		TEXT("OutputNode"),
		// Covered by BuildGraphParameterDiffs.
		TEXT("UserParameters"),
		// Editor cosmetics that rarely warrant a diff entry.
		TEXT("GraphCustomization"),
		// @todo_pcg: To be supported eventually.
		TEXT("ToolData"),
	};

	TArray<FSingleObjectDiffEntry> PropertyDiffs;
	DiffUtils::CompareUnrelatedObjects(OldGraph, NewGraph, PropertyDiffs);

	for (const FSingleObjectDiffEntry& Entry : PropertyDiffs)
	{
		const FString HeadName = Entry.Identifier.GetRootProperty(1).ToDisplayName();
		if (SkippedHeadProperties.Contains(FName(*HeadName)))
		{
			continue;
		}

		FDiffSingleResult Result;
		Result.Object1 = OldGraph;
		Result.Object2 = NewGraph;

		const FString FullPath = Entry.Identifier.ToDisplayName();
		Result.Diff = EDiffType::OBJECT_PROPERTY;
		Result.Category = EDiffType::MODIFICATION;
		Result.DisplayString = FText::Format(LOCTEXT("GraphSettingsChanged", "[Graph Settings] {0}"), FText::FromString(FullPath));
		Result.ToolTip = Result.DisplayString;
		Result.OwningObjectPath = TEXT("PCG.Settings");
		FoundDiffs->Add(MoveTemp(Result));
	}
}

void SPCGEditorGraphDiff::BuildGraphParameterDiffs()
{
	const UPCGGraph* OldGraph = PanelOld.Graph;
	const UPCGGraph* NewGraph = PanelNew.Graph;
	if (!OldGraph || !NewGraph)
	{
		return;
	}

	const FInstancedPropertyBag* OldBag = OldGraph->GetUserParametersStruct();
	const FInstancedPropertyBag* NewBag = NewGraph->GetUserParametersStruct();
	if (!OldBag || !NewBag)
	{
		return;
	}

	TArray<UE::StructUtils::FPropertyBagDiffEntry> ParameterDiffs;
	UE::StructUtils::DiffPropertyBags(*OldBag, *NewBag, ParameterDiffs);

	for (const UE::StructUtils::FPropertyBagDiffEntry& Entry : ParameterDiffs)
	{
		const FText NameForDisplay = FText::FromName(!Entry.NewName.IsNone() ? Entry.NewName : Entry.OldName);

		FDiffSingleResult Result;
		Result.Object1 = OldGraph;
		Result.Object2 = NewGraph;

		switch (Entry.DiffType)
		{
		case UE::StructUtils::EPropertyBagDiffType::Added:
			Result.Diff = EDiffType::OBJECT_ADDED;
			Result.Category = EDiffType::ADDITION;
			Result.DisplayString = FText::Format(LOCTEXT("ParamAdded", "[Parameter] Added: {0}"), NameForDisplay);
			break;
		case UE::StructUtils::EPropertyBagDiffType::Removed:
			Result.Diff = EDiffType::OBJECT_REMOVED;
			Result.Category = EDiffType::SUBTRACTION;
			Result.DisplayString = FText::Format(LOCTEXT("ParamRemoved", "[Parameter] Removed: {0}"), NameForDisplay);
			break;
		case UE::StructUtils::EPropertyBagDiffType::Renamed:
			Result.Diff = EDiffType::OBJECT_PROPERTY;
			Result.Category = EDiffType::MODIFICATION;
			Result.DisplayString = FText::Format(LOCTEXT("ParamRenamed", "[Parameter] Renamed: {0} -> {1}"), FText::FromName(Entry.OldName), FText::FromName(Entry.NewName));
			break;
		case UE::StructUtils::EPropertyBagDiffType::TypeChanged:
			Result.Diff = EDiffType::OBJECT_PROPERTY;
			Result.Category = EDiffType::MODIFICATION;
			Result.DisplayString = FText::Format(LOCTEXT("ParamTypeChanged", "[Parameter] Type changed: {0}"), NameForDisplay);
			break;
		case UE::StructUtils::EPropertyBagDiffType::ValueChanged:
			Result.Diff = EDiffType::OBJECT_PROPERTY;
			Result.Category = EDiffType::MODIFICATION;
			Result.DisplayString = FText::Format(LOCTEXT("ParamValueChanged", "[Parameter] Value changed: {0}"), NameForDisplay);
			break;
		case UE::StructUtils::EPropertyBagDiffType::CategoryChanged:
			Result.Diff = EDiffType::OBJECT_PROPERTY;
			Result.Category = EDiffType::MODIFICATION;
			Result.DisplayString = FText::Format(LOCTEXT("ParamCategoryChanged", "[Parameter] Category changed: {0}"), NameForDisplay);
			break;
		case UE::StructUtils::EPropertyBagDiffType::MetaDataChanged:
			Result.Diff = EDiffType::OBJECT_PROPERTY;
			Result.Category = EDiffType::MODIFICATION;
			Result.DisplayString = FText::Format(LOCTEXT("ParamMetaDataChanged", "[Parameter] Metadata changed ({0}): {1}"), FText::FromName(Entry.MetaDataKey), NameForDisplay);
			break;
		case UE::StructUtils::EPropertyBagDiffType::FlagsChanged:
			Result.Diff = EDiffType::OBJECT_PROPERTY;
			Result.Category = EDiffType::MODIFICATION;
			Result.DisplayString = FText::Format(LOCTEXT("ParamFlagsChanged", "[Parameter] Flags changed: {0}"), NameForDisplay);
			break;
		case UE::StructUtils::EPropertyBagDiffType::Reordered:
			Result.Diff = EDiffType::OBJECT_PROPERTY;
			Result.Category = EDiffType::MINOR;
			Result.DisplayString = FText::Format(LOCTEXT("ParamReordered", "[Parameter] Reordered: {0}"), NameForDisplay);
			break;
		default:
			Result.Diff = EDiffType::OBJECT_PROPERTY;
			Result.Category = EDiffType::MODIFICATION;
			Result.DisplayString = FText::Format(LOCTEXT("ParamChanged", "[Parameter] Changed: {0}"), NameForDisplay);
			break;
		}
		Result.ToolTip = Result.DisplayString;
		Result.OwningObjectPath = TEXT("PCG.Parameter");
		FoundDiffs->Add(MoveTemp(Result));
	}
}

SGraphEditor* SPCGEditorGraphDiff::GetGraphEditorForGraph(UEdGraph* Graph) const
{
	if (const TSharedPtr<SGraphEditor> OldEditor = PanelOld.GraphEditor.Pin())
	{
		if (OldEditor->GetCurrentGraph() == Graph)
		{
			return OldEditor.Get();
		}
	}
	if (const TSharedPtr<SGraphEditor> NewEditor = PanelNew.GraphEditor.Pin())
	{
		if (NewEditor->GetCurrentGraph() == Graph)
		{
			return NewEditor.Get();
		}
	}
	return nullptr;
}

//////////////////////////////////////////////////////////////////////////
// FPCGDiffPanel

SPCGEditorGraphDiff::FPCGDiffPanel::FPCGDiffPanel()
	: Graph(nullptr)
	, EditorGraph(nullptr)
{
}

void SPCGEditorGraphDiff::FPCGDiffPanel::GeneratePanel(UEdGraph* InGraph, TSharedPtr<TArray<FDiffSingleResult>> DiffResults)
{
	TSharedPtr<SWidget> Widget = SNew(SBorder)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock).Text(LOCTEXT("PCGDiffPanelNoGraph", "Graph does not exist in this revision"))
		];

	if (InGraph)
	{
		FGraphAppearanceInfo AppearanceInfo;
		AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText", "DIFF");

		TSharedRef<SGraphEditor> Editor = SNew(SGraphEditor)
			.GraphToEdit(InGraph)
			.DiffResults(DiffResults)
			.IsEditable(false)
			.TitleBar(
				SNew(SBorder)
				.HAlign(HAlign_Center)
				.Padding(0.0f, 16.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock).Text(GetTitle())
				])
			.Appearance(AppearanceInfo);

		GraphEditor = Editor;
		Widget = Editor;
	}

	GraphEditorBorder->SetContent(Widget.ToSharedRef());
}

FText SPCGEditorGraphDiff::FPCGDiffPanel::GetTitle() const
{
	FText Title = LOCTEXT("CurrentRevision", "Current Revision");

	if (!RevisionInfo.Revision.IsEmpty())
	{
		const FText DateText = FText::AsDate(RevisionInfo.Date, EDateTimeStyle::Short);
		const FText RevisionText = FText::FromString(RevisionInfo.Revision);
		const FText ChangelistText = FText::AsNumber(RevisionInfo.Changelist, &FNumberFormattingOptions::DefaultNoGrouping());

		if (bShowAssetName && Graph)
		{
			const FString AssetName = Graph->GetName();
			if (ISourceControlModule::Get().GetProvider().UsesChangelists())
			{
				Title = FText::Format(LOCTEXT("NamedRevisionFmtCL", "{0} - Revision {1}, CL {2}, {3}"), FText::FromString(AssetName), RevisionText, ChangelistText, DateText);
			}
			else
			{
				Title = FText::Format(LOCTEXT("NamedRevisionFmt", "{0} - Revision {1}, {2}"), FText::FromString(AssetName), RevisionText, DateText);
			}
		}
		else
		{
			if (ISourceControlModule::Get().GetProvider().UsesChangelists())
			{
				Title = FText::Format(LOCTEXT("PreviousRevisionFmtCL", "Revision {0}, CL {1}, {2}"), RevisionText, ChangelistText, DateText);
			}
			else
			{
				Title = FText::Format(LOCTEXT("PreviousRevisionFmt", "Revision {0}, {1}"), RevisionText, DateText);
			}
		}
	}
	else if (bShowAssetName && Graph)
	{
		Title = FText::Format(LOCTEXT("NamedCurrentRevisionFmt", "{0} - Current Revision"), FText::FromString(Graph->GetName()));
	}

	return Title;
}

#undef LOCTEXT_NAMESPACE
