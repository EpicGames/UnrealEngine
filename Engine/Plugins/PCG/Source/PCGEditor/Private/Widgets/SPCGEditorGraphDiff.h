// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DiffResults.h"
#include "GraphEditor.h"
#include "IAssetTypeActions.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FAsyncDetailViewDiff;
class FUICommandList;
class IDetailsView;
class ITableRow;
class SBorder;
class SDetailsSplitter;
class STableViewBase;
class SWindow;
class UEdGraph;
class UEdGraphPin;
class UPCGEditorGraph;
class UPCGGraph;
struct FPropertyAndParent;
template <typename ItemType> class SListView;

struct FPCGDiffResultItem : public TSharedFromThis<FPCGDiffResultItem>
{
	explicit FPCGDiffResultItem(const FDiffSingleResult& InResult)
		: Result(InResult)
	{
	}

	TSharedRef<SWidget> GenerateWidget() const;

	const FDiffSingleResult Result;
};

/** Visual diff between two PCG Graphs. */
class SPCGEditorGraphDiff : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_TwoParams(FOpenInDefaults, const UPCGGraph*, const UPCGGraph*);

	SLATE_BEGIN_ARGS(SPCGEditorGraphDiff) {}
		SLATE_ARGUMENT(const UPCGGraph*, GraphOld)
		SLATE_ARGUMENT(const UPCGGraph*, GraphNew)
		SLATE_ARGUMENT(FRevisionInfo, OldRevision)
		SLATE_ARGUMENT(FRevisionInfo, NewRevision)
		SLATE_ARGUMENT(bool, ShowAssetNames)
		SLATE_EVENT(FOpenInDefaults, OpenInDefaults)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Create a diff window for two PCG graph interfaces. */
	static TSharedPtr<SWindow> CreateDiffWindow(
		const UPCGGraph* OldGraph,
		const UPCGGraph* NewGraph,
		const FRevisionInfo& OldRevision,
		const FRevisionInfo& NewRevision);

private:
	/** Panel displaying one side of the diff. */
	struct FPCGDiffPanel
	{
		FPCGDiffPanel();

		/** Generate the graph editor panel with diff highlighting. */
		void GeneratePanel(UEdGraph* Graph, TSharedPtr<TArray<FDiffSingleResult>> DiffResults);

		/** Get a title describing this revision. */
		FText GetTitle() const;

		/** The PCG graph being displayed. */
		const UPCGGraph* Graph = nullptr;

		/** The editor graph constructed for diff visualization. */
		TStrongObjectPtr<UPCGEditorGraph> EditorGraph = nullptr;

		FRevisionInfo RevisionInfo;
		TSharedPtr<SBorder> GraphEditorBorder;
		TWeakPtr<SGraphEditor> GraphEditor;
		bool bShowAssetName = false;
	};

	using FSharedDiffOnGraph = TSharedPtr<FPCGDiffResultItem>;
	using SDiffListViewType = SListView<FSharedDiffOnGraph>;

	/** Construct a UPCGEditorGraph from a UPCGGraph for diff purposes. */
	static UPCGEditorGraph* CreateEditorGraphForDiff(UPCGGraph* InPCGGraph);

	/** What the details splitter is currently showing — drives the visibility filter applied to OldDetailsView/NewDetailsView. */
	enum class EDetailsMode : uint8
	{
		Node,             // Showing per-node settings UObjects. No filter applied.
		GraphSettings,    // Showing UPCGGraph itself, with the UserParameters bag hidden.
		GraphParameters,  // Showing UPCGGraph itself, filtered to only the UserParameters bag.
	};

	/** Update the details splitter to show settings for the given node pair. */
	void ShowSettingsForNodes(UEdGraphNode* OldNode, UEdGraphNode* NewNode);

	/** Point the details splitter at the two UPCGGraph objects with the UserParameters bag hidden. */
	void ShowGraphSettingsDetails();

	/** Point the details splitter at the two UPCGGraph objects filtered to the UserParameters bag. */
	void ShowGraphParametersDetails();

	/** Visibility filter applied to both details views; driven by CurrentDetailsMode. */
	bool IsPropertyVisibleInCurrentMode(const FPropertyAndParent& PropertyAndParent) const;

	/** Append graph-level UPROPERTY diffs (HiGen, cooking, runtime flags, etc.) to FoundDiffs. */
	void BuildGraphSettingsDiffs();

	/** Append per-parameter diffs from the two graphs' UserParameters property bags to FoundDiffs. */
	void BuildGraphParameterDiffs();

	FReply OnOpenInDefaults();
	TSharedRef<SWidget> GenerateDiffListWidget();
	void BuildDiffSourceArray();

	void NextDiff();
	void PrevDiff();
	int32 GetCurrentDiffIndex() const;

	/** Toggle handler hooked up to the lock toolbar button. */
	void OnToggleLockView();

	/** Icon attribute — Lock when locked, Unlock when unlocked. */
	FSlateIcon GetLockViewImage() const;

	/** Apply the current bLockViews state to both panels' SGraphEditors. */
	void ApplyViewLockState();

	void OnSelectionChanged(FSharedDiffOnGraph Item, ESelectInfo::Type SelectionType);
	TSharedRef<ITableRow> OnGenerateRow(FSharedDiffOnGraph Item, const TSharedRef<STableViewBase>& OwnerTable);
	SGraphEditor* GetGraphEditorForGraph(UEdGraph* Graph) const;

	FOpenInDefaults OpenInDefaults;
	FPCGDiffPanel PanelOld;
	FPCGDiffPanel PanelNew;

	TArray<FSharedDiffOnGraph> DiffListSource;
	TSharedPtr<TArray<FDiffSingleResult>> FoundDiffs;
	TSharedPtr<FUICommandList> KeyCommands;
	TSharedPtr<SDiffListViewType> DiffList;

	/** Details diff infrastructure. */
	TSharedPtr<IDetailsView> OldDetailsView;
	TSharedPtr<IDetailsView> NewDetailsView;
	TSharedPtr<SDetailsSplitter> DetailsSplitter;
	TSharedPtr<FAsyncDetailViewDiff> DetailViewDiff;

	UEdGraphPin* LastPinTarget = nullptr;
	UEdGraphPin* LastOtherPinTarget = nullptr;

	EDetailsMode CurrentDetailsMode = EDetailsMode::Node;

	/** When true, panning/zooming either graph editor mirrors the other. Default on, matches Blueprint Diff. */
	bool bLockViews = true;
};
