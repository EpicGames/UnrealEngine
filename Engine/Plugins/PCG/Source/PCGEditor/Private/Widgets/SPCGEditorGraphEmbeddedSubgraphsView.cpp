// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphEmbeddedSubgraphsView.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "PCGEditorStyle.h"
#include "PCGGraph.h"
#include "Schema/PCGEditorGraphSchema.h"

#include "GraphEditorDragDropAction.h"
#include "ScopedTransaction.h"
#include "Misc/MessageDialog.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Serialization/FindObjectReferencers.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SPCGEditorGraphEmbeddedSubgraphsView"

namespace PCGEditorGraphEmbeddedSubgraphsView
{
	const FText RenameEmbeddedSubgraphText = LOCTEXT("RenameEmbeddedSubgraph", "Rename Embedded Subgraph");

	DECLARE_DELEGATE(FOnRenameRequest);

	struct FEmbeddedSubgraphItem
	{
		TWeakObjectPtr<UPCGGraph> WeakEmbeddedSubgraph;
		TWeakPtr<FPCGEditor> WeakEditor;
		void Delete();
		void RenameRequest() { OnRenameRequest.ExecuteIfBound(); }

		FOnRenameRequest OnRenameRequest;
	};

	class SEmbeddedSubgraphRow : public STableRow<TSharedPtr<FEmbeddedSubgraphItem>>
	{
	public:
		SLATE_BEGIN_ARGS(SEmbeddedSubgraphRow) {}
			SLATE_ARGUMENT(TSharedPtr<FEmbeddedSubgraphItem>, Item)
		SLATE_END_ARGS()

		void Construct(const FArguments&, TSharedRef<STableViewBase>);
		FText GetItemLabel() const;
		EVisibility GetItemVisibility() const;
		void OnTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);
		bool OnVerifyTextChanged(const FText& InNewText, FText& OutErrorMessage);
		virtual FReply OnDragDetected(const FGeometry&, const FPointerEvent&) override;
		FReply OnDeleteClicked();
		void OnRenameRequest();

	private:
		TSharedPtr<FEmbeddedSubgraphItem> Item;
		TSharedPtr<SInlineEditableTextBlock> TextBlock;
	};

	class FEmbeddedSubgraphDragDropAction : public FGraphEditorDragDropAction
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FEmbeddedSubgraphDragDropAction, FGraphEditorDragDropAction);

		TSharedPtr<FEmbeddedSubgraphItem> Item;

		static TSharedRef<FEmbeddedSubgraphDragDropAction> New(TSharedPtr<FEmbeddedSubgraphItem> InItem)
		{
			TSharedRef<FEmbeddedSubgraphDragDropAction> Operation = MakeShareable(new FEmbeddedSubgraphDragDropAction);
			Operation->Item = InItem;
			Operation->Construct();
			return Operation;
		}
		
		virtual FReply DroppedOnPanel(const TSharedRef<SWidget>&, const FVector2f&, const FVector2f&, UEdGraph&) override;
		virtual FReply DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
		virtual void HoverTargetChanged() override;
		FText GetHoverText() const;
	};
}

void PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem::Delete()
{
	UPCGGraph* EmbeddedSubgraph = WeakEmbeddedSubgraph.Get();
	if (!EmbeddedSubgraph)
	{
		return;
	}

	TArray<UObject*> Referencers;
	// Find all refs
	for (auto Referencer : TFindObjectReferencers<UPCGGraph>({ EmbeddedSubgraph }, /*PackageToCheck =*/EmbeddedSubgraph->GetPackage(), /*bIgnoreTemplates =*/false))
	{
		// Skip outer references
		if (Referencer.Key->IsIn(Referencer.Value) || Referencer.Value->IsIn(Referencer.Key))
		{
			continue;
		}
		Referencers.Add(Referencer.Value);
	}

	TMap<UPCGGraph*, UPCGGraph*> Redirects;
	Redirects.Add(EmbeddedSubgraph, nullptr);

	FScopedTransaction Transaction(LOCTEXT("DeleteEmbeddedSubgraph", "Delete Embedded Subgraph"));
	if (!Referencers.IsEmpty())
	{
		EAppReturnType::Type Ret = FMessageDialog::Open(
			EAppMsgType::YesNoCancel, LOCTEXT("DeleteWithReferencesMsg", "Embedded subgraph is referenced. Deleting the subgraph will invalidate those subgraph nodes. Are you sure?"),
			LOCTEXT("DeleteWithReferencesTitle", "Delete embedded subgraph with references?"));
		if (Ret != EAppReturnType::Yes)
		{
			Transaction.Cancel();
			return;
		}

		for (UObject* Referencer : Referencers)
		{
			Referencer->Modify();
			FArchiveReplaceObjectRef<UPCGGraph>(Referencer, Redirects);
		}
	}

	if (TSharedPtr<FPCGEditor> PCGEditor = WeakEditor.Pin())
	{
		PCGEditor->CloseDocument(EmbeddedSubgraph);
	}

	EmbeddedSubgraph->GetTypedOuter<UPCGGraph>()->DeleteEmbeddedSubgraph(EmbeddedSubgraph);
}

FReply PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphDragDropAction::DroppedOnPanel(const TSharedRef<SWidget>& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	if (!Graph.GetSchema()->IsA<UPCGEditorGraphSchema>())
	{
		return FReply::Unhandled();
	}

	UPCGEditorGraph* EditorGraph = Cast<UPCGEditorGraph>(&Graph);
	if (!ensure(EditorGraph))
	{
		return FReply::Unhandled();
	}

	FAssetData Asset(Item->WeakEmbeddedSubgraph.Get());
	Graph.GetSchema()->DroppedAssetsOnGraph({ Asset }, GraphPosition, &Graph);
	
	return FReply::Handled();
}

FReply PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphDragDropAction::DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	UEdGraphNode* CurrentHoveredNode = GetHoveredNode();
	if (!CurrentHoveredNode || !CurrentHoveredNode->GetGraph() || !CurrentHoveredNode->GetGraph()->GetSchema())
	{
		return FReply::Unhandled();
	}

	FAssetData Asset(Item->WeakEmbeddedSubgraph.Get());
	CurrentHoveredNode->GetGraph()->GetSchema()->DroppedAssetsOnNode({ Asset }, GraphPosition, CurrentHoveredNode);
	return FReply::Handled();
}

FText PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphDragDropAction::GetHoverText() const
{
	return Item.IsValid() && Item->WeakEmbeddedSubgraph.IsValid() ? Item->WeakEmbeddedSubgraph->Title : LOCTEXT("InvalidHoverText", "Invalid");
}

void PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphDragDropAction::HoverTargetChanged()
{
	SetSimpleFeedbackMessage(FPCGEditorStyle::Get().GetBrush("ClassIcon.PCGGraph"), FLinearColor::White, GetHoverText());
}

void PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::Construct(const FArguments& InArgs, TSharedRef<STableViewBase> InOwner)
{
	Item = InArgs._Item;

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	HorizontalBox->AddSlot()
	.AutoWidth()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	.Padding(4.f)
	[
		SNew(SImage)
		.ColorAndOpacity(FSlateColor::UseForeground())
		.Image(FPCGEditorStyle::Get().GetBrush("ClassIcon.PCGGraph"))
	];

	HorizontalBox->AddSlot()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.Padding(4.f)
	[
		SAssignNew(TextBlock, SInlineEditableTextBlock)
		.Text(this, &PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::GetItemLabel)
		.HintText(LOCTEXT("EmbeddedSubgraphLabelHint", "Label"))
		.OnTextCommitted(this, &PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::OnTextCommitted)
		.OnVerifyTextChanged(this, &PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::OnVerifyTextChanged)
		.IsSelected(this, &PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::IsSelected)
		.MultiLine(false)
	];

	HorizontalBox->AddSlot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(4.f)
	[
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.Visibility(this, &PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::GetItemVisibility)
		.OnClicked(this, &PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::OnDeleteClicked)
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::GetBrush("Icons.Delete"))
		]
	];

	STableRow<TSharedPtr<FEmbeddedSubgraphItem>>::Construct(
		STableRow::FArguments()
		[
			HorizontalBox
		],
		InOwner
	);

	Item->OnRenameRequest = FOnRenameRequest::CreateSP(this, &PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::OnRenameRequest);
}

FText PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::GetItemLabel() const
{
	if (UPCGGraph* Graph = Item->WeakEmbeddedSubgraph.Get())
	{
		return Graph->Title;
	}
	else
	{
		return LOCTEXT("UnknownEmbeddedSubgraphLabel", "Unknown Graph");
	}
}

EVisibility PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::GetItemVisibility() const
{
	return IsSelected() ? EVisibility::Visible : EVisibility::Hidden;
}

FReply PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::OnDeleteClicked()
{
	if (Item.IsValid())
	{
		Item->Delete();
		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

void PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::OnRenameRequest()
{
	if (TextBlock.IsValid())
	{
		TextBlock->EnterEditingMode();
	}
}

void PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::OnTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (UPCGGraph* EmbeddedSubgraph = Item->WeakEmbeddedSubgraph.Get())
	{
		FScopedTransaction Transaction(PCGEditorGraphEmbeddedSubgraphsView::RenameEmbeddedSubgraphText);
		EmbeddedSubgraph->Modify();
		EmbeddedSubgraph->Title = InNewText;
	}
}

bool PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::OnVerifyTextChanged(const FText& InNewText, FText& OutErrorMessage)
{
	const UPCGGraph* EmbeddedSubgraph = Item->WeakEmbeddedSubgraph.Get();
	if (!EmbeddedSubgraph)
	{
		return false;
	}

	if (InNewText.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmbeddedSubgraphLabelInvalidError", "Invalid label");
		return false;
	}

	if (UPCGGraph* OuterGraph = EmbeddedSubgraph->GetTypedOuter<UPCGGraph>())
	{
		for (UPCGGraph* CurrentEmbeddedSubgraph : OuterGraph->GetEmbeddedSubgraphs())
		{
			if (CurrentEmbeddedSubgraph != EmbeddedSubgraph && CurrentEmbeddedSubgraph->Title.EqualToCaseIgnored(InNewText))
			{
				OutErrorMessage = LOCTEXT("EmbeddedSubgraphLabelAlreadyExistsError", "Local graph with same label already exists");
				return false;
			}
		}

		return true;
	}
	else
	{
		return false;
	}
}

FReply PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow::OnDragDetected(const FGeometry&, const FPointerEvent&)
{
	if (Item.IsValid())
	{
		if (TSharedPtr<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphDragDropAction> DragDropOp = PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphDragDropAction::New(Item))
		{
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}
	}
	
	return FReply::Unhandled();
}

UPCGGraph* SPCGEditorGraphEmbeddedSubgraphsView::GetEditorGraph() const
{
	if (TSharedPtr<FPCGEditor> PinnedEditor = PCGEditorPtr.Pin())
	{
		return PinnedEditor.IsValid() ? PinnedEditor->GetMainGraph() : nullptr;
	}

	return nullptr;
}

void SPCGEditorGraphEmbeddedSubgraphsView::Construct(const FArguments& InArgs, const TSharedPtr<FPCGEditor>& InPCGEditor)
{
	PCGEditorPtr = InPCGEditor;
	
	UPCGGraph* EditorGraph = GetEditorGraph();
	if (!EditorGraph)
	{
		return;
	}

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	ChildSlot
	[
		VerticalBox
	];

	// Setup toolbar
	{
		// Setup toolbar
		TSharedRef<SHorizontalBox> ToolbarBox = SNew(SHorizontalBox);

		ToolbarBox->AddSlot()
		.HAlign(HAlign_Fill)
		.FillWidth(1.f)
		[
			SNew(SSearchBox)
			.OnTextChanged(this, &SPCGEditorGraphEmbeddedSubgraphsView::OnFilterTextChanged)
		];

		ToolbarBox->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(4.f, 0.f, 0.f, 0.f)
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("NewEmbeddedSubgraphToolTip", "New Embedded subgraph"))
			.OnClicked(this, &SPCGEditorGraphEmbeddedSubgraphsView::NewEmbeddedSubgraphClicked)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			]
		];

		VerticalBox->AddSlot()
		.AutoHeight()
		.Padding(8.f, 8.f, 8.f, 4.f)
		[
			ToolbarBox
		];
	}

	// Setup list view
	{
		RefreshEmbeddedSubgraphsList();

		VerticalBox->AddSlot()
		[
			SAssignNew(ListView, SListView<TSharedPtr<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem>>)
			.OnGenerateRow(this, &SPCGEditorGraphEmbeddedSubgraphsView::GenerateEmbeddedSubgraphRow)
			.SelectionMode(ESelectionMode::Single)
			.ListItemsSource(&Items)
			.OnMouseButtonDoubleClick(this, &SPCGEditorGraphEmbeddedSubgraphsView::OnEmbeddedSubgraphItemDoubleClick)
			.OnKeyDownHandler(this, &SPCGEditorGraphEmbeddedSubgraphsView::OnEmbeddedSubgraphItemKeyDownHandler)
			.OnContextMenuOpening(this, &SPCGEditorGraphEmbeddedSubgraphsView::OpenContextMenu)
		];
	}
}

void SPCGEditorGraphEmbeddedSubgraphsView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bRefreshRequested)
	{
		RefreshEmbeddedSubgraphsList();
		bRefreshRequested = false;
	}
}

FReply SPCGEditorGraphEmbeddedSubgraphsView::NewEmbeddedSubgraphClicked()
{
	if (UPCGGraph* EditorGraph = GetEditorGraph())
	{
		{
			FScopedTransaction Transaction(LOCTEXT("AddEmbeddedSubgraph", "New Embedded subgraph"));
			if (UPCGGraph* NewGraph = EditorGraph->AddNewEmbeddedSubgraph())
			{
				if (TSharedPtr<FPCGEditor> PinnedEditor = PCGEditorPtr.Pin())
				{
					PinnedEditor->OpenDocument(NewGraph, FDocumentTracker::EOpenDocumentCause::OpenNewDocument);
				}
			}
		}
		RefreshEmbeddedSubgraphsList();
	}

	return FReply::Handled();
}

void SPCGEditorGraphEmbeddedSubgraphsView::OnFilterTextChanged(const FText& InText)
{
	FilterText = InText;
	RequestRefresh();
}

void SPCGEditorGraphEmbeddedSubgraphsView::RefreshEmbeddedSubgraphsList()
{
	Items.Empty();

	if (UPCGGraph* EditorGraph = GetEditorGraph())
	{
		FString FilterTextStr = FilterText.ToString();
		for (const auto& EmbeddedSubgraph : EditorGraph->GetEmbeddedSubgraphs())
		{
			if (FilterText.IsEmpty() || EmbeddedSubgraph->Title.ToString().Contains(FilterTextStr))
			{
				Items.Add(MakeShared<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem>(EmbeddedSubgraph, PCGEditorPtr));
			}
		}
		
	}
	
	if (ListView.IsValid())
	{
		ListView->RebuildList();
	}
}

TSharedRef<ITableRow> SPCGEditorGraphEmbeddedSubgraphsView::GenerateEmbeddedSubgraphRow(TSharedPtr<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem> InItem, const TSharedRef<STableViewBase>& InOwner)
{
	return SNew(PCGEditorGraphEmbeddedSubgraphsView::SEmbeddedSubgraphRow, InOwner).Item(InItem);
}

void SPCGEditorGraphEmbeddedSubgraphsView::OnEmbeddedSubgraphItemDoubleClick(TSharedPtr<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem> InItem)
{
	if (TSharedPtr<FPCGEditor> PinnedEditor = PCGEditorPtr.Pin())
	{
		UPCGGraph* EmbeddedSubgraph = InItem->WeakEmbeddedSubgraph.Get();
		
		PinnedEditor->OpenDocument(EmbeddedSubgraph, FDocumentTracker::EOpenDocumentCause::OpenNewDocument);
	}
}

FReply SPCGEditorGraphEmbeddedSubgraphsView::OnEmbeddedSubgraphItemKeyDownHandler(const FGeometry&, const FKeyEvent& InKeyEvent) const
{
	if (ListView->GetNumItemsSelected() != 1)
	{
		return FReply::Unhandled();
	}

	if (InKeyEvent.GetKey() == EKeys::F2)
	{
		TSharedPtr<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem> Item = ListView->GetSelectedItems()[0];
		Item->RenameRequest();
		return FReply::Handled();
	}

	if (InKeyEvent.GetKey() == EKeys::Delete || InKeyEvent.GetKey() == EKeys::BackSpace)
	{
		TSharedPtr<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem> Item = ListView->GetSelectedItems()[0];
		Item->Delete();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedPtr<SWidget> SPCGEditorGraphEmbeddedSubgraphsView::OpenContextMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, /*InCommandList=*/nullptr);
	MenuBuilder.AddMenuEntry(
		PCGEditorGraphEmbeddedSubgraphsView::RenameEmbeddedSubgraphText,
		LOCTEXT("RenameEmbeddedSubgraph_Tooltip", "Allows renaming the selected embedded subgraph."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSPLambda(this, [this]()
			{ 
					if (ListView->GetNumItemsSelected() == 1) 
					{ 
						ListView->GetSelectedItems()[0]->RenameRequest(); 
					}
			}),
			FCanExecuteAction::CreateSPLambda(this, [this]() { return ListView->GetNumItemsSelected() == 1; }))
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DeleteEmbeddedSubgraphs", "Delete Embedded Subgraph"),
		LOCTEXT("DeleteEmbeddedSubgraphs_Tooltip", "Removes the selected embedded subgraphs from the current graph."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSPLambda(this, [this]()
			{
				TArray<TSharedPtr<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem>> SelectedItems = ListView->GetSelectedItems();
				for (auto& SelectedItem : SelectedItems)
				{
					SelectedItem->Delete();
				}
			}),
			FCanExecuteAction::CreateSPLambda(this, [this]() { return !ListView->GetSelectedItems().IsEmpty(); }))
	);

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE

