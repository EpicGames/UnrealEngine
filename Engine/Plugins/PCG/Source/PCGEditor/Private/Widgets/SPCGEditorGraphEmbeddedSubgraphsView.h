// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#include "Widgets/SCompoundWidget.h"

struct FGeometry;
struct FKeyEvent;
class FPCGEditor;
class ITableRow;
class STableViewBase;
class UPCGGraph;

template <typename ItemType> class SListView;

namespace PCGEditorGraphEmbeddedSubgraphsView
{
	struct FEmbeddedSubgraphItem;
}

/** Displays a PCGGraph's Local graphs. */
class SPCGEditorGraphEmbeddedSubgraphsView final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphEmbeddedSubgraphsView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<FPCGEditor>& InPCGEditor);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	void RequestRefresh() { bRefreshRequested = true; }

private:
	void OnFilterTextChanged(const FText& InText);
	FReply NewEmbeddedSubgraphClicked();
	TSharedRef<ITableRow> GenerateEmbeddedSubgraphRow(TSharedPtr<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem> InItem, const TSharedRef<STableViewBase>& InOwner);
	void OnEmbeddedSubgraphItemDoubleClick(TSharedPtr<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem> InItem);
	FReply OnEmbeddedSubgraphItemKeyDownHandler(const FGeometry&, const FKeyEvent& InKeyEvent) const;
	UPCGGraph* GetEditorGraph() const;
	void RefreshEmbeddedSubgraphsList();
	TSharedPtr<SWidget> OpenContextMenu();

	TArray<TSharedPtr<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem>> Items;
	TSharedPtr<SListView<TSharedPtr<PCGEditorGraphEmbeddedSubgraphsView::FEmbeddedSubgraphItem>>> ListView;
	TWeakPtr<FPCGEditor> PCGEditorPtr;
	bool bRefreshRequested = false;
	FText FilterText;
};
