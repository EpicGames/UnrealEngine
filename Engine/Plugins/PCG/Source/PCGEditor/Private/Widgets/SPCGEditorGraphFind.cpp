// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphFind.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "PCGEditorStyle.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGSubgraph.h"
#include "PCGSubsystem.h"
#include "Editor/IPCGEditorModule.h"
#include "Elements/PCGUserParameterGet.h"
#include "Managers/PCGEditorInspectionDataManager.h"
#include "Nodes/PCGEditorGraphNodeBase.h"

#include "Algo/Contains.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/WidgetPath.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "PCGEditorGraphFind"

namespace PCGEditorGraphFindResult
{
	// List of nodes that should output result for their pins.
	static const TStaticArray<TSubclassOf<UPCGSettings>, 1> ExceptionNodesToShowPins = { UPCGUserParameterGetSettings::StaticClass() };
}

FPCGEditorGraphFindResult::FPCGEditorGraphFindResult(const FString& InValue)
: Parent(nullptr)
, Value(InValue)
, GraphNode(nullptr)
, RootGraphNode(nullptr)
, Pin()
{
}

FPCGEditorGraphFindResult::FPCGEditorGraphFindResult(const FText& InValue)
: FPCGEditorGraphFindResult(InValue.ToString())
{
}

FPCGEditorGraphFindResult::FPCGEditorGraphFindResult(const FString& InValue, const TSharedPtr<FPCGEditorGraphFindResult>& InParent, UEdGraphNode* InNode)
: Parent(InParent)
, Value(InValue)
, GraphNode(InNode)
, Pin()
{
	RootGraphNode = (Parent.IsValid() && Parent.Pin()->RootGraphNode.IsValid()) ? Parent.Pin()->RootGraphNode : InNode;
}

FPCGEditorGraphFindResult::FPCGEditorGraphFindResult(const FString& InValue, const TSharedPtr<FPCGEditorGraphFindResult>& InParent, UEdGraphPin* InPin)
: Parent(InParent)
, Value(InValue)
, GraphNode(nullptr)
, Pin(InPin)
{
	RootGraphNode = (Parent.IsValid() && Parent.Pin()->RootGraphNode.IsValid()) ? Parent.Pin()->RootGraphNode : nullptr;
}

FReply FPCGEditorGraphFindResult::OnClick(TWeakPtr<FPCGEditor> InPCGEditorPtr)
{
	TSharedPtr<FPCGEditor> Editor = InPCGEditorPtr.Pin();

	if (!Editor)
	{
		return FReply::Handled();
	}

	UEdGraphPin* ResolvedPin = Pin.Get();
	if (ResolvedPin && ResolvedPin->GetOwningNode()->GetGraph() == Editor->GetFocusedEditorGraph())
	{
		Editor->JumpToNode(ResolvedPin->GetOwningNode());
	}
	else
	{
		Editor->JumpToNode(RootGraphNode.Get());
	}

	return FReply::Handled();
}

FReply FPCGEditorGraphFindResult::OnDoubleClick(TWeakPtr<FPCGEditor> InPCGEditorPtr)
{
	TSharedPtr<FPCGEditor> CurrentEditor = InPCGEditorPtr.Pin();
	TSharedPtr<FPCGEditor> Editor = CurrentEditor;
	if (!Editor)
	{
		return FReply::Unhandled();
	}

	if (ParentGraph)
	{
		Editor->OpenEditorForGraph(ParentGraph->GetPCGGraph());
		Editor = ParentGraph->GetEditor().Pin();
	}

	if (Editor)
	{
		if (Editor != CurrentEditor && CurrentEditor->GetStackBeingInspected())
		{
			Editor->SetStackBeingInspectedFromAnotherEditor(*CurrentEditor->GetStackBeingInspected());
		}

		if (UEdGraphPin* ResolvedPin = Pin.Get())
		{
			Editor->JumpToNode(ResolvedPin->GetOwningNode());
		}
		else if (GraphNode.IsValid())
		{
			Editor->JumpToNode(GraphNode.Get());
		}
	}

	return FReply::Handled();
}

FText FPCGEditorGraphFindResult::GetToolTip() const
{
	FText ToolTip;

	if (UEdGraphPin* ResolvedPin = Pin.Get())
	{
		if (UEdGraphNode* OwningNode = ResolvedPin->GetOwningNode())
		{
			FString ToolTipString;
			OwningNode->GetPinHoverText(*ResolvedPin, ToolTipString);
			ToolTip = FText::FromString(ToolTipString);
		}
	}
	else if (GraphNode.IsValid())
	{
		ToolTip = GraphNode->GetTooltipText();
	}

	return ToolTip;
}

FText FPCGEditorGraphFindResult::GetCategory() const
{
	if (Pin.Get())
	{
		return LOCTEXT("PinCategory", "Pin");
	}
	else if (GraphNode.IsValid())
	{
		return LOCTEXT("NodeCategory", "Node");
	}
	return FText::GetEmpty();
}

FText FPCGEditorGraphFindResult::GetComment() const
{
	if (GraphNode.IsValid())
	{
		const FString NodeComment = GraphNode->NodeComment;
		if (!NodeComment.IsEmpty())
		{
			return FText::Format(LOCTEXT("NodeCommentFmt", "Node Comment:[{0}]"), FText::FromString(NodeComment));
		}
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> FPCGEditorGraphFindResult::CreateIcon() const
{
	FSlateColor IconColor = FSlateColor::UseForeground();
	const FSlateBrush* Brush = nullptr;

	// TODO: consider the pin connection state (connected or not + single + multi)
	// TODO: consider node state (enabled or not)
	if (UEdGraphPin* ResolvedPin = Pin.Get())
	{
		// TODO get pin icon from nodebase?
		Brush = FAppStyle::GetBrush(TEXT("GraphEditor.PinIcon"));
		const UEdGraphSchema* Schema = ResolvedPin->GetSchema();
		IconColor = Schema->GetPinColor(ResolvedPin);
	}
	else if (GraphNode.IsValid())
	{
		// TODO get icon and tint from nodebase?
		Brush = FAppStyle::GetBrush(TEXT("GraphEditor.NodeGlyph"));

		if (const UPCGEditorGraphNodeBase* PCGNode = Cast<UPCGEditorGraphNodeBase>(GraphNode))
		{
			if (Cast<UPCGBaseSubgraphNode>(PCGNode->GetPCGNode()))
			{
				Brush = FPCGEditorStyle::Get().GetBrush(TEXT("ClassIcon.PCGGraphInterface"));
			}

			IconColor = FSlateColor(PCGNode->GetNodeTitleColor());
		}
	}

	return SNew(SImage)
		.Image(Brush)
		.ColorAndOpacity(IconColor)
		.ToolTipText(GetCategory());
}

SPCGEditorGraphFind::~SPCGEditorGraphFind()
{
	if (PCGEditorPtr.IsValid())
	{
		PCGEditorPtr.Pin()->GetInspectionDataManager().OnInspectedStackChangedDelegate.RemoveAll(this);
	}
}

void SPCGEditorGraphFind::Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor)
{
	PCGEditorPtr = InPCGEditor;

	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();

	if (PCGEditor)
	{
		PCGEditor->GetInspectionDataManager().OnInspectedStackChangedDelegate.AddSP(this, &SPCGEditorGraphFind::OnInspectedStackChanged);
	}

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SComboButton)
				.ForegroundColor(FSlateColor::UseStyle())
				.HasDownArrow(false)
				.OnGetMenuContent(this, &SPCGEditorGraphFind::OnFindFilterMenu)
				.ContentPadding(1)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(SearchTextField, SSearchBox)
				.HintText(LOCTEXT("PCGGraphSearchHint", "Enter text to find nodes..."))
				.OnTextChanged(this, &SPCGEditorGraphFind::OnSearchTextChanged)
				.OnTextCommitted(this, &SPCGEditorGraphFind::OnSearchTextCommitted)
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.f, 4.f, 0.f, 0.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				SAssignNew(TreeView, STreeViewType)
				.TreeItemsSource(&ItemsFound)
				.OnGenerateRow(this, &SPCGEditorGraphFind::OnGenerateRow)
				.OnGetChildren(this, &SPCGEditorGraphFind::OnGetChildren)
				.OnSelectionChanged(this, &SPCGEditorGraphFind::OnTreeSelectionChanged)
				.OnMouseButtonDoubleClick(this, &SPCGEditorGraphFind::OnTreeDoubleClick)
				.OnKeyDownHandler(this, &SPCGEditorGraphFind::OnTreeViewKeyDown)
				.SelectionMode(ESelectionMode::Single)
			]
		]
	];
}

void SPCGEditorGraphFind::FocusForUse()
{
	// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
	FWidgetPath FilterTextBoxWidgetPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchTextField.ToSharedRef(), FilterTextBoxWidgetPath);

	// Set keyboard focus directly
	FSlateApplication::Get().SetKeyboardFocus(FilterTextBoxWidgetPath, EFocusCause::SetDirectly);
}

void SPCGEditorGraphFind::OnSearchTextChanged(const FText& InText)
{
	SearchValue = InText.ToString();
	InitiateSearch();
}

void SPCGEditorGraphFind::OnSearchTextCommitted(const FText& /*InText*/, ETextCommit::Type InCommitType)
{
	// Since we already initiate a search when the text changes, there's no real need to do anything on commit
	if (InCommitType != ETextCommit::Type::OnUserMovedFocus)
	{
		InitiateSearch();
	}
}

void SPCGEditorGraphFind::OnInspectedStackChanged(const FPCGStack& InPCGStack)
{
	if (!SearchValue.IsEmpty())
	{
		InitiateSearch();
	}
}

void SPCGEditorGraphFind::OnGetChildren(FPCGEditorGraphFindResultPtr InItem, TArray<FPCGEditorGraphFindResultPtr>& OutChildren)
{
	OutChildren += InItem->Children;
}

void SPCGEditorGraphFind::OnTreeSelectionChanged(FPCGEditorGraphFindResultPtr Item, ESelectInfo::Type)
{
	if (Item.IsValid())
	{
		Item->OnClick(PCGEditorPtr);
	}
}

void SPCGEditorGraphFind::OnTreeDoubleClick(FPCGEditorGraphFindResultPtr Item)
{
	if (Item.IsValid())
	{
		Item->OnDoubleClick(PCGEditorPtr);
	}
}

FReply SPCGEditorGraphFind::OnTreeViewKeyDown(const FGeometry& /*InGeometry*/, const FKeyEvent& InKeyEvent) const
{
	if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		TArray<FPCGEditorGraphFindResultPtr> SelectedItems = TreeView->GetSelectedItems();
		if (SelectedItems.Num() == 1 && SelectedItems[0].IsValid())
		{
			return SelectedItems[0]->OnDoubleClick(PCGEditorPtr);
		}
	}

	return FReply::Unhandled();
}

void SPCGEditorGraphFind::SetFindMode(EPCGGraphFindMode InFindMode)
{
	if (InFindMode != FindMode)
	{
		FindMode = InFindMode;
		InitiateSearch();
	}
}

bool SPCGEditorGraphFind::IsCurrentFindMode(EPCGGraphFindMode InFindMode) const
{
	return FindMode == InFindMode;
}

TSharedRef<SWidget> SPCGEditorGraphFind::OnFindFilterMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MinimalExpansion", "Show minimum tree"),
		LOCTEXT("MinimalExpansionTooltip", "Shows minimum subset of visited tree to perform search."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPCGEditorGraphFind::SetFindMode, EPCGGraphFindMode::ShowMinimumTree),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SPCGEditorGraphFind::IsCurrentFindMode, EPCGGraphFindMode::ShowMinimumTree)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("FullExpansion", "Show full tree"),
		LOCTEXT("FullExpansionTooltip", "Shows the search results from the fully expanded tree, e.g. will have every occurrence."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPCGEditorGraphFind::SetFindMode, EPCGGraphFindMode::ShowFullTree),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SPCGEditorGraphFind::IsCurrentFindMode, EPCGGraphFindMode::ShowFullTree)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("FlatList", "Show flat graph list"),
		LOCTEXT("FlatListTooltip", "Shows the occurrences in all visited graphs but no hierarchy."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPCGEditorGraphFind::SetFindMode, EPCGGraphFindMode::ShowFlatList),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SPCGEditorGraphFind::IsCurrentFindMode, EPCGGraphFindMode::ShowFlatList)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SearchPins", "Include pin names"),
		LOCTEXT("SearchPinsTooltip", "TODO"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				bShowPinResults = !bShowPinResults;
				InitiateSearch();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]()
			{
				return bShowPinResults;
			})
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	return MenuBuilder.MakeWidget();
}

TSharedRef<ITableRow> SPCGEditorGraphFind::OnGenerateRow(FPCGEditorGraphFindResultPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(STableRow<TSharedPtr<FPCGEditorGraphFindResult>>, InOwnerTable)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			InItem->CreateIcon()
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(2, 0)
		[
			SNew(STextBlock)
			.Text(FText::FromString(InItem->Value))
			.HighlightText(HighlightText)
			.ToolTipText(InItem->GetToolTip())
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(2, 0)
		[
			SNew(STextBlock)
			.Text(InItem->GetComment()) 
			.ColorAndOpacity(FLinearColor::Yellow)
			.HighlightText(HighlightText)
		]
	];
}

void SPCGEditorGraphFind::InitiateSearch()
{
	TArray<FString> Tokens;
	SearchValue.ParseIntoArray(Tokens, TEXT(" "), true);

	ItemsFound.Empty();
	if (Tokens.Num() > 0)
	{
		HighlightText = FText::FromString(SearchValue);
		MatchTokens(Tokens);
	}

	// Insert a fake result to inform user if none found
	if (ItemsFound.Num() == 0)
	{
		ItemsFound.Add(MakeShared<FPCGEditorGraphFindResult>(LOCTEXT("PCGGraphSearchNoResults", "No Results found")));
	}

	TreeView->RequestTreeRefresh();

	// Expand so that all items that contain a matched token are shown
	auto ExpandItem = [this](const FPCGEditorGraphFindResultPtr& Item, auto&& RecursiveCall) -> bool
	{
		check(Item);
		bool bShouldExpandItem = Item->bIsMatch;

		for (const FPCGEditorGraphFindResultPtr& Child : Item->Children)
		{
			bShouldExpandItem = RecursiveCall(Child, RecursiveCall);
		}

		if (bShouldExpandItem)
		{
			TreeView->SetItemExpansion(Item, true);
		}

		return bShouldExpandItem;
	};

	for (const FPCGEditorGraphFindResultPtr& Item : ItemsFound)
	{
		ExpandItem(Item, ExpandItem);
	}
}

namespace PCGEditorGraphFind
{
	// Convenience struct to vist graphs & stacks and build a proper hierarchy
	struct GraphTree
	{
		UPCGGraph* Graph = nullptr;
		UEdGraphNode* Node = nullptr;

		const TArray<GraphTree>& GetChildren() const { return Children; }
		const GraphTree* GetParent() const { return Parent; }

		bool HasVisitedGraph(UPCGGraph* InGraph) const
		{
			return Graph == InGraph || (Parent && Parent->HasVisitedGraph(InGraph));
		}

		GraphTree* FindOrCreateChild(UEdGraphNode* InNode, UPCGGraph* InGraph)
		{
			for (GraphTree& Child : Children)
			{
				if (Child.Node == InNode && Child.Graph == InGraph)
				{
					return &Child;
				}
			}

			// Not found -> add new child
			GraphTree& NewChild = AddChild();
			NewChild.Node = InNode;
			NewChild.Graph = InGraph;

			return &NewChild;
		}

		// Collapse current subtree so that it is attached to the same parent graph, but closest to the root.
		bool Collapse()
		{
			for (int ChildIndex = Children.Num() - 1; ChildIndex >= 0; --ChildIndex)
			{
				if (Children[ChildIndex].Collapse())
				{
					Children.RemoveAt(ChildIndex, EAllowShrinking::No);
				}
			}

			GraphTree* CurrentParent = Parent;
			GraphTree* TopmostOccurrenceOfThisGraph = nullptr;
			while (CurrentParent)
			{
				if (CurrentParent->Graph == Graph)
				{
					TopmostOccurrenceOfThisGraph = CurrentParent;
				}

				CurrentParent = CurrentParent->Parent;
			}

			if (TopmostOccurrenceOfThisGraph)
			{
				TopmostOccurrenceOfThisGraph->AppendChildren(Children);
				return true;
			}
			else
			{
				return false;
			}
		}

		GraphTree* FindFirst(UPCGGraph* InGraph)
		{
			TArray<GraphTree*> BFSExpansion = GetBreadthFirstExpansion();
			for (GraphTree* ExpandedNode : BFSExpansion)
			{
				if (ExpandedNode->Graph == InGraph)
				{
					return ExpandedNode;
				}
			}

			return nullptr;
		}

	private:
		TArray<GraphTree> Children;
		GraphTree* Parent = nullptr;

		TArray<GraphTree*> GetBreadthFirstExpansion()
		{
			TArray<GraphTree*> Expansion;
			Expansion.Add(this);
			int ExpansionIndex = 0;
			while (ExpansionIndex < Expansion.Num())
			{
				GraphTree* Current = Expansion[ExpansionIndex];
				for (GraphTree& Child : Current->Children)
				{
					Expansion.Add(&Child);
				}

				++ExpansionIndex;
			}

			return Expansion;
		}

		GraphTree& AddChild()
		{
			SIZE_T PreviouslyAllocatedSize = Children.GetAllocatedSize();
			GraphTree& NewChild = Children.Emplace_GetRef();
			NewChild.Parent = this;

			if (Children.GetAllocatedSize() != PreviouslyAllocatedSize)
			{
				UpdateParentPointers();
			}

			return NewChild;
		}

		void AppendChildren(const TArray<GraphTree>& InChildren)
		{
			Children.Append(InChildren);
			// Since we might have resized the array and because we're moving the children here, we must update the parent pointers.
			UpdateParentPointers();
		}

		void UpdateParentPointers()
		{
			for (GraphTree& Child : Children)
			{
				Child.Parent = this;
				Child.UpdateParentPointers();
			}
		}
	};
} // namespace PCGEditorGraphFind

void SPCGEditorGraphFind::MatchTokens(const TArray<FString>& InTokens)
{
	TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();
	UPCGEditorGraph* PCGEditorGraph = PCGEditor ? PCGEditor->GetMainEditorGraph() : nullptr;
	
	if (!PCGEditor || !PCGEditorGraph)
	{
		return;
	}

	// Start by gathering all graphs to search through.
	TSet<UPCGGraph*> AllGraphs;
	UPCGGraph* ThisGraph = PCGEditorGraph->GetPCGGraph();
	AllGraphs.Add(ThisGraph);

	PCGEditorGraphFind::GraphTree GraphRoot;
	GraphRoot.Graph = ThisGraph;
	GraphRoot.Node = nullptr;

	auto VisitAllNodes = [&AllGraphs, PCGEditor](UPCGEditorGraph* EditorGraph, PCGEditorGraphFind::GraphTree& Parent, auto&& RecursiveCall)
	{
		if (!EditorGraph)
		{
			return;
		}

		for (UEdGraphNode* Node : EditorGraph->Nodes)
		{
			if (UPCGEditorGraphNodeBase* PCGEditorNode = Cast<UPCGEditorGraphNodeBase>(Node))
			{
				if (const UPCGBaseSubgraphNode* SubgraphNode = Cast<UPCGBaseSubgraphNode>(PCGEditorNode->GetPCGNode()))
				{
					if (UPCGGraph* Subgraph = SubgraphNode->GetSubgraph())
					{
						// Check if this subgraph is already present in the hierarchy
						if (!Parent.HasVisitedGraph(Subgraph))
						{
							AllGraphs.Add(Subgraph);
							if (PCGEditorGraphFind::GraphTree* Child = Parent.FindOrCreateChild(Node, Subgraph))
							{
								RecursiveCall(PCGEditor->GetOrCreatePCGEditorGraph(Subgraph), *Child, RecursiveCall);
							}
						}
					}
				}
			}
		}
	};

	// Visit the static graph(s) starting from the root.
	VisitAllNodes(PCGEditorGraph, GraphRoot, VisitAllNodes);

	// Then gather all stacks starting from the stack being inspected, and link them to their matching subgraph node.
	IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get();
	if (PCGEditor->GetStackBeingInspected() && PCGEditorModule)
	{
		const FPCGStack& CurrentStack = *PCGEditor->GetStackBeingInspected();
		TArray<FPCGStackSharedPtr> Substacks = PCGEditorModule->GetExecutedStacksPtrs(CurrentStack);
		// For each stack, find the first subgraph node from the start, then get all graphs under it.
		for (const FPCGStackSharedPtr& SubstackPtr : Substacks)
		{
			const FPCGStack& Substack = *SubstackPtr;

			if (Substack.GetStackFrames().Num() <= CurrentStack.GetStackFrames().Num())
			{
				continue;
			}

			// We'll be trying to navigate down the stack and find the node + graph pairs, creating what we need in the graph tree.
			PCGEditorGraphFind::GraphTree* Current = &GraphRoot;
			int FrameIndex = CurrentStack.GetStackFrames().Num();

			UEdGraphNode* SubgraphEditorNode = nullptr;
			UPCGGraph* Subgraph = nullptr;
			UPCGEditorGraph* CurrentEditorGraph = PCGEditorGraph;

			// There are two kind of subgraph frame structures we are looking for:
			// Subgraph node (UPCGNode) > Subgraph (UPCGGraph) -> this is when we have a static subgraphs
			// Subgraph node (UPCGNode) > Loop Index / -1 -> this is when we have a dynamic subgraph or a loop
			auto CheckFramePair = [&Substack, &SubgraphEditorNode, &Subgraph, &CurrentEditorGraph](int NodeFrameIndex, int SubgraphFrameIndex) -> bool
			{
				if(SubgraphFrameIndex >= Substack.GetStackFrames().Num())
				{
					return false;
				}

				const FPCGStackFrame& SubgraphNodeFrame = Substack.GetStackFrames()[NodeFrameIndex];
				const UPCGNode* SubgraphNode = SubgraphNodeFrame.GetObject_GameThread<UPCGNode>();
				SubgraphEditorNode = (SubgraphNode && CurrentEditorGraph) ? const_cast<UPCGEditorGraphNodeBase*>(CurrentEditorGraph->GetEditorNodeFromPCGNode(SubgraphNode)) : nullptr;
				const FPCGStackFrame& SubgraphFrame = Substack.GetStackFrames()[SubgraphFrameIndex];
				Subgraph = const_cast<UPCGGraph*>(SubgraphFrame.GetObject_GameThread<UPCGGraph>());

				return SubgraphEditorNode && Subgraph;
			};

			while (FrameIndex < Substack.GetStackFrames().Num() && Current)
			{
				bool bProcessPair = false;

				// Test node + graph in a static subgraph configuration
				if (CheckFramePair(FrameIndex, FrameIndex + 1))
				{
					bProcessPair = true;
					FrameIndex += 2;
				}
				// Test node + graph in a dynamic subgraph/loop configuration
				else if (CheckFramePair(FrameIndex, FrameIndex + 2))
				{
					bProcessPair = true;
					FrameIndex += 3;
				}
				else
				{
					++FrameIndex;
				}

				if (bProcessPair)
				{
					AllGraphs.Add(Subgraph);
					// Find/Create child on current
					Current = Current->FindOrCreateChild(SubgraphEditorNode, Subgraph);
					CurrentEditorGraph = Current && Subgraph ? PCGEditor->GetOrCreatePCGEditorGraph(Subgraph) : nullptr;

					if (Current && CurrentEditorGraph)
					{
						VisitAllNodes(CurrentEditorGraph, *Current, VisitAllNodes);
					}
					else
					{
						// We can't progress here so leave the loop.
						break;
					}
				}
			}
		}
	}

	// Implementation note: at this point, we could visit each graph only once to search for the tokens
	// but we'd need additional mechanisms to do deep copies, which is not super significant at this point in time.
	// Perform search by graph based on the tree expansion we'll visit
	if (FindMode == EPCGGraphFindMode::ShowMinimumTree)
	{
		// Moves subtrees to their earliest appearance in the breadth-first tree
		GraphRoot.Collapse();
	}

	if(FindMode == EPCGGraphFindMode::ShowMinimumTree || FindMode == EPCGGraphFindMode::ShowFullTree)
	{
		auto RecurseInGraphTree = [this, &InTokens, PCGEditor](const PCGEditorGraphFind::GraphTree& Node, TFunctionRef<FPCGEditorGraphFindResultPtr()> GetParentFunc, auto&& RecurseCall) -> void
		{
			check(Node.Graph);

			FString NodeString;
			FPCGEditorGraphFindResultPtr ThisResult;

			if (Node.Node)
			{
				NodeString = FString::Printf(TEXT("%s (%s)"), *FName::NameToDisplayString(Node.Graph->GetName(), /*bIsBool=*/false), *Node.Node->GetNodeTitle(ENodeTitleType::ListView).ToString());
			}

			auto GetOrCreateNodeResult = [this, &Node, &NodeString, &ThisResult, &GetParentFunc, PCGEditor]()
			{
				FPCGEditorGraphFindResultPtr Parent = GetParentFunc();
				check(Parent);

				if(!ThisResult.IsValid() && Node.Node)
				{
					ThisResult = MakeShared<FPCGEditorGraphFindResult>(NodeString, Parent, Node.Node);
					ThisResult->ParentGraph = PCGEditor->GetOrCreatePCGEditorGraph(Node.Graph);
					Parent->Children.Add(ThisResult);
				}

				return ThisResult ? ThisResult : Parent;
			};

			// If current node (including dynamic graph name) matches tokens, add it
			if (StringMatchesSearchTokens(InTokens, NodeString))
			{
				GetOrCreateNodeResult()->bIsMatch = true;
			}

			// Create local elements
			MatchTokensInternal(InTokens, PCGEditor->GetOrCreatePCGEditorGraph(Node.Graph), GetOrCreateNodeResult/*, MatchedTokens*/);

			// Continue through the tree
			for (const PCGEditorGraphFind::GraphTree& Child : Node.GetChildren())
			{
				RecurseCall(Child, GetOrCreateNodeResult, RecurseCall);
			}
		};

		FPCGEditorGraphFindResultPtr RootFindResult = MakeShared<FPCGEditorGraphFindResult>(FString("PCGTreeRoot"));
		auto GetRootPtr = [&RootFindResult]() { return RootFindResult; };

		RecurseInGraphTree(GraphRoot, GetRootPtr, RecurseInGraphTree);
		ItemsFound.Append(RootFindResult->Children);
	}
	// - flat list (this graph + all downstream graphs after)
	else if (FindMode == EPCGGraphFindMode::ShowFlatList)
	{
		for(UPCGGraph* Graph : AllGraphs)
		{
			FPCGEditorGraphFindResultPtr GraphNodePtr;
			const FString GraphString = FName::NameToDisplayString(Graph->GetName(), /*bIsBool=*/false);

			auto CreateGraphNode = [Graph, &GraphString, &GraphNodePtr, &GraphRoot, PCGEditor]()
			{
				if (!GraphNodePtr.IsValid())
				{
					PCGEditorGraphFind::GraphTree* GraphTreeNode = GraphRoot.FindFirst(Graph);

					UEdGraphNode* EditorNode = nullptr;
					UEdGraphNode* RootEditorNode = nullptr;

					if (GraphTreeNode)
					{
						EditorNode = GraphTreeNode->Node;

						const PCGEditorGraphFind::GraphTree* GraphRootNode = GraphTreeNode;
						while (GraphRootNode->GetParent() && GraphRootNode->GetParent() != &GraphRoot)
						{
							GraphRootNode = GraphRootNode->GetParent();
						}

						RootEditorNode = GraphRootNode->Node;
					}

					GraphNodePtr = MakeShared<FPCGEditorGraphFindResult>(GraphString, FPCGEditorGraphFindResultPtr(), EditorNode);
					GraphNodePtr->ParentGraph = PCGEditor->GetOrCreatePCGEditorGraph(Graph);
					GraphNodePtr->RootGraphNode = RootEditorNode;
				}

				return GraphNodePtr;
			};

			// If current node (including dynamic graph name) matches tokens, add it
			if (StringMatchesSearchTokens(InTokens, GraphString))
			{
				CreateGraphNode()->bIsMatch = true;
			}

			MatchTokensInternal(InTokens,  PCGEditor->GetOrCreatePCGEditorGraph(Graph), CreateGraphNode/*, Results*/);

			if (GraphNodePtr)
			{
				ItemsFound.Add(GraphNodePtr);
			}
		}
	}
}

void SPCGEditorGraphFind::MatchTokensInternal(const TArray<FString>& InTokens, UPCGEditorGraph* PCGEditorGraph, TFunctionRef<FPCGEditorGraphFindResultPtr()> GetParentFunc) const
{
	if (!PCGEditorGraph)
	{
		return;
	}

	for (UEdGraphNode* Node : PCGEditorGraph->Nodes)
	{
		check(Node);

		const FString NodeString = Node->GetNodeTitle(ENodeTitleType::ListView).ToString();

		// Search string has full title (both lines).
		FString NodeSearchString = Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString() + Node->NodeComment;

		// Add internal object name which will still display here and there.
		const UPCGEditorGraphNodeBase* PCGEditorGraphNodeBase = Cast<UPCGEditorGraphNodeBase>(Node);
		if (const UPCGNode* PCGNode = PCGEditorGraphNodeBase ? PCGEditorGraphNodeBase->GetPCGNode() : nullptr)
		{
			NodeSearchString.Append(PCGNode->GetName());
		}

		NodeSearchString = NodeSearchString.Replace(TEXT(" "), TEXT(""));

		FPCGEditorGraphFindResultPtr NodeResult;
		auto GetOrCreateNodeResult = [&NodeResult, &NodeString, Node, PCGEditorGraph, &GetParentFunc]() -> FPCGEditorGraphFindResultPtr&
		{
			if (!NodeResult.IsValid())
			{
				NodeResult = MakeShared<FPCGEditorGraphFindResult>(NodeString, GetParentFunc(), Node);
				NodeResult->ParentGraph = PCGEditorGraph;
				GetParentFunc()->Children.Add(NodeResult);
			}

			return NodeResult;
		};

		if (StringMatchesSearchTokens(InTokens, NodeSearchString))
		{
			GetOrCreateNodeResult()->bIsMatch = true;
		}

		const UPCGEditorGraphNodeBase* PCGNode = Cast<const UPCGEditorGraphNodeBase>(Node);
		const UPCGSettings* NodeSettings = PCGNode ? PCGNode->GetSettings() : nullptr;
		TSubclassOf<UPCGSettings> NodeSettingsClass(NodeSettings ? NodeSettings->GetClass() : nullptr);
		
		if (bShowPinResults || (NodeSettingsClass && Algo::Contains(PCGEditorGraphFindResult::ExceptionNodesToShowPins, NodeSettingsClass)))
		{
			for (UEdGraphPin* Pin : Node->Pins)
			{
				if (Pin && Pin->PinFriendlyName.CompareTo(FText::FromString(TEXT(" "))) != 0)
				{
					const FText PinName = Pin->GetSchema()->GetPinDisplayName(Pin);
					FString PinSearchString = Pin->PinName.ToString() + Pin->PinFriendlyName.ToString() + Pin->DefaultValue + Pin->PinType.PinCategory.ToString() + Pin->PinType.PinSubCategory.ToString() + (Pin->PinType.PinSubCategoryObject.IsValid() ? Pin->PinType.PinSubCategoryObject.Get()->GetFullName() : TEXT(""));
					PinSearchString = PinSearchString.Replace(TEXT(" "), TEXT(""));
					if (StringMatchesSearchTokens(InTokens, PinSearchString))
					{
						FPCGEditorGraphFindResultPtr PinResult(MakeShared<FPCGEditorGraphFindResult>(PinName.ToString(), GetOrCreateNodeResult(), Pin));
						PinResult->ParentGraph = PCGEditorGraph;
						PinResult->bIsMatch = true;
						NodeResult->Children.Add(PinResult);
					}
				}
			}
		}
	}
}

bool SPCGEditorGraphFind::StringMatchesSearchTokens(const TArray<FString>& InTokens, const FString& InComparisonString)
{
	bool bFoundAllTokens = true;

	//search the entry for each token, it must have all of them to pass
	for (const FString& Token : InTokens)
	{
		if (!InComparisonString.Contains(Token))
		{
			bFoundAllTokens = false;
			break;
		}
	}
	return bFoundAllTokens;
}

#undef LOCTEXT_NAMESPACE
