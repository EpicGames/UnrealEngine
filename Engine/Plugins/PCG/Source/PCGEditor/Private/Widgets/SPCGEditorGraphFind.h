// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphPin.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class SSearchBox;
class STableViewBase;
namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }
template <typename ItemType> class STreeView;

class FPCGEditor;
struct FPCGStack;
class UEdGraphNode;
class UPCGEditorGraph;
class UPCGGraph;

struct FPCGEditorGraphFindResult
{
	/** Create a root (or only text) result */
	FPCGEditorGraphFindResult(const FString& InValue);

	/** Create a root (or only text) result */
	FPCGEditorGraphFindResult(const FText& InValue);

	/** Create a listing for a node result */
	FPCGEditorGraphFindResult(const FString& InValue, const TSharedPtr<FPCGEditorGraphFindResult>& InParent, UEdGraphNode* InNode);

	/** Create a listing for a pin result */
	FPCGEditorGraphFindResult(const FString& InValue, const TSharedPtr<FPCGEditorGraphFindResult>& InParent, UEdGraphPin* InPin);

	/** Called when user clicks on the search item */
	FReply OnClick(TWeakPtr<FPCGEditor> InPCGEditorPtr);

	/** Called when user double-clicks on the search item */
	FReply OnDoubleClick(TWeakPtr<FPCGEditor> InPCGEditorPtr);

	/** Get ToolTip for this search result */
	FText GetToolTip() const;

	/** Get Category for this search result */
	FText GetCategory() const;

	/** Get Comment for this search result */
	FText GetComment() const;

	/** Create an icon to represent the result */
	TSharedRef<SWidget> CreateIcon() const;

	/** ParentGraph if not current */
	UPCGEditorGraph* ParentGraph = nullptr;

	/** Search result parent */
	TWeakPtr<FPCGEditorGraphFindResult> Parent;

	/** Any children listed under this category */
	TArray<TSharedPtr<FPCGEditorGraphFindResult>> Children;
	
	/** The string value for this result */
	FString Value;

	/** The graph node that this search result refers to */
	TWeakObjectPtr<UEdGraphNode> GraphNode;

	/** The graph node that`s in the current editor graph upstream. */
	TWeakObjectPtr<UEdGraphNode> RootGraphNode;

	/** The pin that this search result refers to */
	FEdGraphPinReference Pin;

	/** Whether this result contains a found token or not. */
	bool bIsMatch = false;
};

enum class EPCGGraphFindMode : uint8
{
	ShowMinimumTree,
	ShowFullTree,
	ShowFlatList
};

class SPCGEditorGraphFind : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphFind) {}
	SLATE_END_ARGS()

	~SPCGEditorGraphFind();
	void Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor);

	/** Focuses this widget's search box */
	void FocusForUse();

private:
	typedef TSharedPtr<FPCGEditorGraphFindResult> FPCGEditorGraphFindResultPtr;
	typedef STreeView<FPCGEditorGraphFindResultPtr> STreeViewType;

	/** Called when user changes the text they are searching for */
	void OnSearchTextChanged(const FText& InText);

	/** Called when user changes commits text to the search box */
	void OnSearchTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

	/** Called when the debug object selected changes, which should trigger a new search */
	void OnInspectedStackChanged(const FPCGStack& InPCGStack);

	/** Get the children of a row */
	void OnGetChildren(FPCGEditorGraphFindResultPtr InItem, TArray<FPCGEditorGraphFindResultPtr>& OutChildren);

	/** Called when user clicks on a new result */
	void OnTreeSelectionChanged(FPCGEditorGraphFindResultPtr Item, ESelectInfo::Type);

	/** Called when an element is double-clicked */
	void OnTreeDoubleClick(FPCGEditorGraphFindResultPtr InItem);

	/** Called when keys are entered when in the tree selection */
	FReply OnTreeViewKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) const;

	/** Called when the user clicks on the filter icon */
	TSharedRef<SWidget> OnFindFilterMenu();

	/** Called when a new row is being generated */
	TSharedRef<ITableRow> OnGenerateRow(FPCGEditorGraphFindResultPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	/** Begins the search based on the SearchValue */
	void InitiateSearch();

	/** Find any results that contain all of the tokens */
	void MatchTokens(const TArray<FString>& InTokens);

	/** Recursive internal implementation of the MatchTokens */
	void MatchTokensInternal(const TArray<FString>& InTokens, UPCGEditorGraph* PCGEditorGraph, TFunctionRef<FPCGEditorGraphFindResultPtr()> GetParentFunc) const;

	/** Determines if a string matches the search tokens */
	static bool StringMatchesSearchTokens(const TArray<FString>& InTokens, const FString& InComparisonString);

	/** Called when the find mode is changed */
	void SetFindMode(EPCGGraphFindMode InFindMode);

	/** Returns true if the current find mode is the same as the one provided */
	bool IsCurrentFindMode(EPCGGraphFindMode InFindMode) const;

	/** Pointer back to the PCG editor that owns us */
	TWeakPtr<FPCGEditor> PCGEditorPtr;

	/** The tree view displays the results */
	TSharedPtr<STreeViewType> TreeView;

	/** The search text box */
	TSharedPtr<SSearchBox> SearchTextField;

	/** This buffer stores the currently displayed results */
	TArray<FPCGEditorGraphFindResultPtr> ItemsFound;

	/** The string to highlight in the results */
	FText HighlightText;

	/** The string to search for */
	FString SearchValue;

	/** Controls the way find results are presented. */
	EPCGGraphFindMode FindMode = EPCGGraphFindMode::ShowMinimumTree;

	/** Controls whether pin names are show in the find results. */
	bool bShowPinResults = false;
};
