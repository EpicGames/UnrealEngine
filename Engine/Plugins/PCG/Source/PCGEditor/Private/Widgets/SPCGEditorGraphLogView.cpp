// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphLogView.h"

#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGSubgraph.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "Managers/PCGEditorInspectionDataManager.h"
#include "Nodes/PCGEditorGraphNode.h"

#include "Styling/StyleColors.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SPCGEditorGraphLogView"

namespace PCGEditorGraphLogView
{	
	const FName NAME_Order = FName(TEXT("Order"));
	const FName NAME_Node = FName(TEXT("Node"));
	const FName NAME_Namespace = FName(TEXT("Namespace"));
	const FName NAME_Message = FName(TEXT("Message"));

	/** Labels of the columns */
	const FText TEXT_OrderLabel = LOCTEXT("Order", "Order");
	const FText TEXT_NodeLabel = LOCTEXT("Node", "Node");
	const FText TEXT_NamespaceLabel = LOCTEXT("Namespace", "Namespace");
	const FText TEXT_MessageLabel = LOCTEXT("Message", "Message");
}

void SPCGLogListViewItemRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const PCGLogListViewItemPtr& Item)
{
	InternalItem = Item;

	SMultiColumnTableRow<PCGLogListViewItemPtr>::Construct(
		SMultiColumnTableRow::FArguments()
		.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
		InOwnerTableView);
}

TSharedRef<SWidget> SPCGLogListViewItemRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	FText ColumnData = LOCTEXT("ColumnError", "Unrecognized Column");

	if (InternalItem.IsValid())
	{
		if (ColumnId == PCGEditorGraphLogView::NAME_Order)
		{
			ColumnData = FText::AsNumber(InternalItem->Order);
		}
		else if (ColumnId == PCGEditorGraphLogView::NAME_Node)
		{
			return SNew(STextBlock)
				.ColorAndOpacity(InternalItem->EditorNode->GetNodeTitleColor())
				.Text(FText::FromName(InternalItem->NodeName));
		}
		else if (ColumnId == PCGEditorGraphLogView::NAME_Namespace)
		{
			ColumnData = FText::FromName(InternalItem->Namespace);
		}
		else if (ColumnId == PCGEditorGraphLogView::NAME_Message)
		{
			EStyleColor Color = EStyleColor::Foreground;

			switch (InternalItem->Verbosity)
			{
			case ELogVerbosity::Error:
				Color = EStyleColor::Error;
				break;

			case ELogVerbosity::Warning:
				Color = EStyleColor::Warning;
				break;
			}

			return SNew(STextBlock)
				.ColorAndOpacity(Color)
				.Text(FText::FromString(InternalItem->Message));
		}
	}

	return SNew(STextBlock)
		.Text(ColumnData);
}

void SPCGEditorGraphLogView::OnItemDoubleClicked(PCGLogListViewItemPtr Item)
{
	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();
	if (!PCGEditor.IsValid())
	{
		return;
	}

	if (!Item.IsValid() || !Item->EditorNode)
	{
		return;
	}

	PCGEditor->JumpToNode(Item->EditorNode);
}

SPCGEditorGraphLogView::~SPCGEditorGraphLogView()
{
	if (PCGEditorPtr.IsValid())
	{
		PCGEditorPtr.Pin()->GetInspectionDataManager().OnInspectedStackChangedDelegate.RemoveAll(this);
	}
}

void SPCGEditorGraphLogView::Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor)
{
	PCGEditorPtr = InPCGEditor;

	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();
	if (PCGEditor)
	{
		PCGEditorGraph = PCGEditor->GetMainEditorGraph();
		PCGEditor->GetInspectionDataManager().OnInspectedStackChangedDelegate.AddSP(this, &SPCGEditorGraphLogView::OnDebugStackChanged);
	}

	ListViewHeader = CreateHeaderRowWidget();

	const TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal)
		.Thickness(FVector2D(12.0f, 12.0f));

	const TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(12.0f, 12.0f));
	
	SAssignNew(ListView, SListView<PCGLogListViewItemPtr>)
		.ListItemsSource(&ListViewItems)
		.HeaderRow(ListViewHeader)
		.OnGenerateRow(this, &SPCGEditorGraphLogView::OnGenerateRow)
		.OnMouseButtonDoubleClick(this, &SPCGEditorGraphLogView::OnItemDoubleClicked)
		.AllowOverscroll(EAllowOverscroll::No)
		.ExternalScrollbar(VerticalScrollBar)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always);
	
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("ClearButton", "Clear"))
				.OnClicked(this, &SPCGEditorGraphLogView::Clear)
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				.ExternalScrollbar(HorizontalScrollBar)
				+SScrollBox::Slot()
				[
					ListView->AsShared()
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				VerticalScrollBar
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				HorizontalScrollBar
			]
		]
	];

	Refresh();
}

void SPCGEditorGraphLogView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bNeedsRefresh)
	{
		bNeedsRefresh = false;
		Refresh();
	}
}

TSharedRef<SHeaderRow> SPCGEditorGraphLogView::CreateHeaderRowWidget()
{
	return SNew(SHeaderRow)
		.ResizeMode(ESplitterResizeMode::FixedPosition)
		.CanSelectGeneratedColumn(true)
		+ SHeaderRow::Column(PCGEditorGraphLogView::NAME_Order)
			.ManualWidth(64)
			.DefaultLabel(PCGEditorGraphLogView::TEXT_OrderLabel)
			.HAlignHeader(HAlign_Center)
			.HAlignCell(HAlign_Center)
			.SortMode(this, &SPCGEditorGraphLogView::GetColumnSortMode, PCGEditorGraphLogView::NAME_Order)
			.OnSort(this, &SPCGEditorGraphLogView::OnSortColumnHeader)
		+ SHeaderRow::Column(PCGEditorGraphLogView::NAME_Namespace)
			.ManualWidth(180)
			.DefaultLabel(PCGEditorGraphLogView::TEXT_NamespaceLabel)
			.HAlignHeader(HAlign_Center)
			.HAlignCell(HAlign_Left)
			.SortMode(this, &SPCGEditorGraphLogView::GetColumnSortMode, PCGEditorGraphLogView::NAME_Namespace)
			.OnSort(this, &SPCGEditorGraphLogView::OnSortColumnHeader)
		+ SHeaderRow::Column(PCGEditorGraphLogView::NAME_Node)
			.ManualWidth(180)
			.DefaultLabel(PCGEditorGraphLogView::TEXT_NodeLabel)
			.HAlignHeader(HAlign_Center)
			.HAlignCell(HAlign_Left)
			.SortMode(this, &SPCGEditorGraphLogView::GetColumnSortMode, PCGEditorGraphLogView::NAME_Node)
			.OnSort(this, &SPCGEditorGraphLogView::OnSortColumnHeader)
		+ SHeaderRow::Column(PCGEditorGraphLogView::NAME_Message)
			.FillWidth(1.0)
			.DefaultLabel(PCGEditorGraphLogView::TEXT_MessageLabel)
			.HAlignHeader(HAlign_Center)
			.HAlignCell(HAlign_Left)
			.SortMode(this, &SPCGEditorGraphLogView::GetColumnSortMode, PCGEditorGraphLogView::NAME_Message)
			.OnSort(this, &SPCGEditorGraphLogView::OnSortColumnHeader);
}

void SPCGEditorGraphLogView::OnSortColumnHeader(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode)
{
	if (SortingColumn == ColumnId)
	{
		// Circling
		SortMode = EColumnSortMode::Type((SortMode + 1) % 3);
	}
	else
	{
		SortingColumn = ColumnId;
		SortMode = NewSortMode;
	}

	Refresh();
}

EColumnSortMode::Type SPCGEditorGraphLogView::GetColumnSortMode(const FName ColumnId) const
{
	if (SortingColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

FReply SPCGEditorGraphLogView::Clear()
{
	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();
	if (!PCGEditor.IsValid())
	{
		return FReply::Handled();
	}

	if (!PCGEditorGraph)
	{
		return FReply::Handled();
	}

	if (IPCGGraphExecutionSource* ExecutionSource = GetExecutionSource())
	{
		ExecutionSource->GetExecutionState().GetExtraCapture().ResetCapturedMessages();
		Refresh();
	}

	return FReply::Handled();
}

FReply SPCGEditorGraphLogView::Refresh()
{
	ListViewItems.Empty();
	ListView->RequestListRefresh();

	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();
	if (!PCGEditor.IsValid())
	{
		return FReply::Handled();
	}

	if (!PCGEditorGraph)
	{
		return FReply::Handled();
	}

	const IPCGGraphExecutionSource* ExecutionSource = GetExecutionSource();

	if (!ExecutionSource)
	{
		return FReply::Handled();
	}

	TArray<UPCGEditorGraphNode*> EditorNodes;
	PCGEditorGraph->GetNodesOfClass<UPCGEditorGraphNode>(EditorNodes);

	for (const UPCGEditorGraphNode* PCGEditorNode : EditorNodes)
	{
		const UPCGNode* PCGNode = PCGEditorNode ? PCGEditorNode->GetPCGNode() : nullptr;
		if (PCGNode)
		{
			CreateAndAddItem(PCGEditorNode, PCGNode, *PCGNode->GetNodeTitle(EPCGNodeTitleType::ListView).ToString());
		}
	}

	if (SortingColumn != NAME_None && SortMode != EColumnSortMode::None)
	{
		Algo::Sort(ListViewItems, [this](const PCGLogListViewItemPtr& A, const PCGLogListViewItemPtr& B)
			{
				bool isLess = false;
				if (SortingColumn == PCGEditorGraphLogView::NAME_Order)
				{
					isLess = A->Order < B->Order;
				}
				else if (SortingColumn == PCGEditorGraphLogView::NAME_Namespace)
				{
					isLess = A->Namespace.LexicalLess(B->Namespace);
				}
				else if (SortingColumn == PCGEditorGraphLogView::NAME_Node)
				{
					isLess = A->NodeName.LexicalLess(B->NodeName);
				}
				else if (SortingColumn == PCGEditorGraphLogView::NAME_Message)
				{
					isLess = A->Message < B->Message;
				}

				return SortMode == EColumnSortMode::Ascending ? isLess : !isLess;
			});
	}

	ListView->SetItemsSource(&ListViewItems);

	return FReply::Handled();
}

void SPCGEditorGraphLogView::CreateAndAddItem(const UPCGEditorGraphNode* InPCGEditorNode, const UPCGNode* InPCGNode, const FName& InName)
{
	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();

	check(PCGEditor);

	IPCGGraphExecutionSource* ExecutionSource = GetExecutionSource();

	if (!ExecutionSource || !InPCGEditorNode || !InPCGNode)
	{
		return;
	}

	const TArray<PCGUtils::FCapturedMessage>* CapturedMessagesPtr = ExecutionSource->GetExecutionState().GetExtraCapture().GetCapturedMessages().Find(InPCGNode);
	if (!CapturedMessagesPtr)
	{
		return;
	}

	for (const PCGUtils::FCapturedMessage& Message : *CapturedMessagesPtr)
	{
		PCGLogListViewItemPtr ListViewItem = MakeShared<FPCGLogListViewItem>();
		ListViewItem->EditorNode = InPCGEditorNode;
		ListViewItem->PCGNode = InPCGNode;
		ListViewItem->NodeName = InName;
		ListViewItem->Order = Message.Index;
		ListViewItem->Namespace = Message.Namespace;
		ListViewItem->Message = Message.Message;
		ListViewItem->Verbosity = Message.Verbosity;

		ListViewItems.Add(ListViewItem);
	}

	// Also if the node is a subgraph, recurse on all child nodes
	const UPCGSettings* Settings = InPCGNode->GetSettings();
	if (Settings && Settings->IsA<UPCGBaseSubgraphSettings>())
	{
		if (const UPCGGraph* Graph = CastChecked<const UPCGBaseSubgraphSettings>(Settings)->GetSubgraph())
		{
			const FString Prefix = InName.ToString() + "/";
			for (const UPCGNode* ChildNode : Graph->GetNodes())
			{
				if (!ChildNode)
				{
					continue;
				}

				const FText ChildTitle = ChildNode->GetNodeTitle(EPCGNodeTitleType::ListView);

				CreateAndAddItem(InPCGEditorNode, ChildNode, FName(Prefix + ChildTitle.ToString()));
			}
		}
	}
}

IPCGGraphExecutionSource* SPCGEditorGraphLogView::GetExecutionSource() const
{
	return PCGEditorPtr.IsValid() ? PCGEditorPtr.Pin()->GetPCGSourceBeingInspected() : nullptr;
}

void SPCGEditorGraphLogView::OnDebugStackChanged(const FPCGStack& InPCGStack)
{
	RequestRefresh();
}

TSharedRef<ITableRow> SPCGEditorGraphLogView::OnGenerateRow(PCGLogListViewItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SPCGLogListViewItemRow, OwnerTable, Item);
}

#undef LOCTEXT_NAMESPACE
