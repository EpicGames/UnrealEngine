// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;

class FPCGEditor;
class UPCGEditorGraphNode;
class UPCGEditorGraph;
class IPCGGraphExecutionSource;
class UPCGNode;
struct FPCGStack;

namespace PCGEditorGraphLogView
{	
	/** Names of the columns in the attribute list */
	extern const FName NAME_Order;
}

struct FPCGLogListViewItem
{
	const UPCGNode* PCGNode = nullptr;	
	const UPCGEditorGraphNode* EditorNode = nullptr;

	int32 Order = 0;
	FName NodeName;
	FName Namespace;
	FString Message;

	ELogVerbosity::Type Verbosity = ELogVerbosity::NoLogging;
};

typedef TSharedPtr<FPCGLogListViewItem> PCGLogListViewItemPtr;

class SPCGLogListViewItemRow : public SMultiColumnTableRow<PCGLogListViewItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SPCGLogListViewItemRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const PCGLogListViewItemPtr& Item);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

protected:
	PCGLogListViewItemPtr InternalItem;
};

class SPCGEditorGraphLogView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphLogView) { }
	SLATE_END_ARGS()

	~SPCGEditorGraphLogView();

	void Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	void RequestRefresh() { bNeedsRefresh = true; }

private:
	TSharedRef<SHeaderRow> CreateHeaderRowWidget();
	void CreateAndAddItem(const UPCGEditorGraphNode* InPCGEditorNode, const UPCGNode* InPCGNode, const FName& InName);

	void OnDebugStackChanged(const FPCGStack& InPCGStack);

	// Callbacks
	FReply Refresh();
	FReply Clear();
	TSharedRef<ITableRow> OnGenerateRow(PCGLogListViewItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable) const;
	void OnItemDoubleClicked(PCGLogListViewItemPtr Item);
	void OnSortColumnHeader(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode);
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/** Pointer back to the PCG editor that owns us */
	TWeakPtr<FPCGEditor> PCGEditorPtr;

	/** Cached PCGGraph being viewed */
	UPCGEditorGraph* PCGEditorGraph = nullptr;

	/** Cached PCG Execution source being viewed */
	IPCGGraphExecutionSource* GetExecutionSource() const;

	TSharedPtr<SHeaderRow> ListViewHeader;
	TSharedPtr<SListView<PCGLogListViewItemPtr>> ListView;
	TArray<PCGLogListViewItemPtr> ListViewItems;

	bool bNeedsRefresh : 1 = false;

	// To allow sorting
	FName SortingColumn = PCGEditorGraphLogView::NAME_Order;
	EColumnSortMode::Type SortMode = EColumnSortMode::Type::Ascending;
};
