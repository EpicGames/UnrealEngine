// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphNodePalette.h"

#include "PCGGraph.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "PCGEditorUtils.h"
#include "PCGSettingsDragDropAction.h"
#include "Schema/PCGEditorGraphSchema.h"
#include "Schema/PCGEditorGraphSchemaActions.h"
#include "Widgets/SPCGEditorGraphActionWidget.h"

#include "SEnumCombo.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "SPCGEditorGraphNodePalette"

void SPCGEditorGraphNodePaletteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData)
{
	check(InCreateData->Action.IsValid());

	TSharedPtr<FEdGraphSchemaAction> GraphAction = InCreateData->Action;
	ActionPtr = InCreateData->Action;

	this->ChildSlot
	[
		SNew(SPCGGraphActionWidget, InCreateData)
		.NameWidget(CreateTextSlotWidget(InCreateData, InCreateData->bIsReadOnly).ToSharedPtr())
	];
}

FText SPCGEditorGraphNodePaletteItem::GetItemTooltip() const
{
	return ActionPtr.Pin()->GetTooltipDescription();
}

void SPCGEditorGraphNodePalette::Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor)
{
	const UEnum* PCGElementTypeEnum = StaticEnum<EPCGElementType>();
	PCGEditor = InPCGEditor;
	
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TypeTextBlock", "Type:"))
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(SEnumComboBox, PCGElementTypeEnum)
				.ContentPadding(FMargin(4, 0))
				.OnEnumSelectionChanged(this, &SPCGEditorGraphNodePalette::OnTypeSelectionChanged)
				.CurrentValue(this, &SPCGEditorGraphNodePalette::GetTypeValue)
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(GraphActionMenu, SGraphActionMenu)
			.OnActionDragged(this, &SPCGEditorGraphNodePalette::OnActionDragged)
			.OnCreateWidgetForAction(this, &SPCGEditorGraphNodePalette::OnCreateWidgetForAction)
			.OnCollectAllActions(this, &SPCGEditorGraphNodePalette::CollectAllActions)
			.AutoExpandActionMenu(true)
		]
	];

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetAdded().AddSP(this, &SPCGEditorGraphNodePalette::OnAssetChanged);
	AssetRegistryModule.Get().OnAssetRemoved().AddSP(this, &SPCGEditorGraphNodePalette::OnAssetChanged);
	AssetRegistryModule.Get().OnAssetUpdated().AddSP(this, &SPCGEditorGraphNodePalette::OnAssetChanged); // Todo, evaluate if this triggers too often
	AssetRegistryModule.Get().OnAssetRenamed().AddSP(this, &SPCGEditorGraphNodePalette::OnAssetRenamed);
}

SPCGEditorGraphNodePalette::~SPCGEditorGraphNodePalette()
{
	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
	{
		IAssetRegistry* AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
		if (AssetRegistry) // AssetRegistry can be null during EngineShutdown even if module is available
		{
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
			AssetRegistry->OnAssetUpdated().RemoveAll(this);
			AssetRegistry->OnAssetRenamed().RemoveAll(this);
		}
	}
}

void SPCGEditorGraphNodePalette::RequestRefresh()
{
	bNeedsRefresh = true;
}

TSharedRef<SWidget> SPCGEditorGraphNodePalette::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SPCGEditorGraphNodePaletteItem, InCreateData);
}

FReply SPCGEditorGraphNodePalette::OnActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& MouseEvent)
{
	if (InActions.Num() > 0 && InActions[0].IsValid())
	{
		TSharedPtr<FEdGraphSchemaAction> InAction = InActions[0];
		if (InAction->GetTypeId() == FPCGEditorGraphSchemaAction_NewSettingsElement::StaticGetTypeId())
		{
			FPCGEditorGraphSchemaAction_NewSettingsElement* SettingsAction = static_cast<FPCGEditorGraphSchemaAction_NewSettingsElement*>(InAction.Get());
			return FReply::Handled().BeginDragDrop(FPCGSettingsDragDropAction::New(InAction, SettingsAction->SettingsObjectPath));
		}
	}

	return SGraphPalette::OnActionDragged(InActions, MouseEvent);
}

void SPCGEditorGraphNodePalette::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	const UPCGEditorGraphSchema* PCGSchema = GetDefault<UPCGEditorGraphSchema>();

	FGraphActionMenuBuilder ActionMenuBuilder;
	FPCGEditor* Editor = PCGEditor.Pin().Get();
	check(Editor);
	FPCGActionsFilter ActionFilter(Editor->GetMainEditorGraph(), ElementType & PCGSchema->GetElementTypeFiltering(), PCGSchema->GetGraphEditorCustomization(Editor->GetMainEditorGraph()));
	
	PCGSchema->GetPaletteActions(ActionMenuBuilder, ActionFilter);
	OutAllActions.Append(ActionMenuBuilder);
}

void SPCGEditorGraphNodePalette::OnAssetChanged(const FAssetData& InAssetData)
{
	if (InAssetData.IsInstanceOf<UPCGGraphInterface>() ||
		InAssetData.IsInstanceOf<UPCGSettings>() ||
		PCGEditorUtils::IsAssetPCGBlueprint(InAssetData))
	{
		bNeedsRefresh = true;
	}
}

void SPCGEditorGraphNodePalette::OnAssetRenamed(const FAssetData& InAssetData, const FString& /*InNewAssetName*/)
{
	OnAssetChanged(InAssetData);
}

void SPCGEditorGraphNodePalette::OnTypeSelectionChanged(int32 InValue, ESelectInfo::Type /*SelectInfo*/)
{
	ElementType = static_cast<EPCGElementType>(InValue);
	bNeedsRefresh = true;
}

void SPCGEditorGraphNodePalette::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SGraphPalette::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bNeedsRefresh)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SPCGEditorGraphNodePalette::RefreshActionsList);
		RefreshActionsList(true);
		bNeedsRefresh = false;
	}
}

int32 SPCGEditorGraphNodePalette::GetTypeValue() const
{
	return static_cast<int32>(ElementType);
}

#undef LOCTEXT_NAMESPACE
 
