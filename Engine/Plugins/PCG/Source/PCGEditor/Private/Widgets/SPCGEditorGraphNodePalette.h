// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGEditorCommon.h"

#include "SGraphPalette.h"

class FPCGEditor;

class SPCGEditorGraphNodePaletteItem : public SGraphPaletteItem
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphNodePaletteItem) {};
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData);

protected:
	//~ Begin SGraphPaletteItem Interface
	virtual FText GetItemTooltip() const override;
	//~ End SGraphPaletteItem Interface
};


class SPCGEditorGraphNodePalette : public SGraphPalette
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphNodePalette) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor);
	virtual ~SPCGEditorGraphNodePalette();

	void RequestRefresh();

protected:
	//~ Begin SGraphPalette Interface
	virtual TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData) override;
	virtual FReply OnActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& MouseEvent) override;
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;
	//~ End SGraphPalette Interface

	//~ Begin SWidget interface
	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget interface

private:
	void OnAssetChanged(const FAssetData& InAssetData);
	void OnAssetRenamed(const FAssetData& InAssetData, const FString& InNewAssetName);
	void OnTypeSelectionChanged(int32, ESelectInfo::Type SelectInfo);
	
	int32 GetTypeValue() const;
	
	TWeakPtr<FPCGEditor> PCGEditor;
	EPCGElementType ElementType = EPCGElementType::All;
	bool bNeedsRefresh = false;
};
