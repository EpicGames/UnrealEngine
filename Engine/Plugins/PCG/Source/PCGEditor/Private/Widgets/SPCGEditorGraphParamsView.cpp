// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphParamsView.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "PCGGraph.h"

#include "PropertyBagDetails.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Modules/ModuleManager.h"

void SPCGEditorGraphUserParametersView::Construct(const FArguments& InArgs, const TSharedPtr<FPCGEditor>& InPCGEditor)
{
	if (!InPCGEditor.IsValid())
	{
		return;
	}

	UPCGGraph* PCGGraph = InPCGEditor.IsValid() ? InPCGEditor->GetFocusedGraph() : nullptr;

	if (!PCGGraph)
	{
		return;
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;
		DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bShowScrollBar = true;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bShowObjectLabel = true;

		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bShowDifferingPropertiesOption = false;
		DetailsViewArgs.bShowHiddenPropertiesWhilePlayingOption = false;
		DetailsViewArgs.bShowKeyablePropertiesOption = false;
		DetailsViewArgs.bShowAnimatedPropertiesOption = false;
		DetailsViewArgs.bShowCustomFilterOption = false;
		DetailsViewArgs.bShowSectionSelector = false;
		DetailsViewArgs.bShowLooseProperties = false;

		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bForceHiddenPropertyVisibility = false;
		// Filter all but the graph parameters
		DetailsViewArgs.ShouldForceHideProperty.BindLambda(
			[](const TSharedRef<IPropertyHandle>& PropertyNode)
			{
				auto PropertyIsPropertyBag = [](const FProperty* Property)
				{
					const FStructProperty* PropertyStruct = CastField<FStructProperty>(Property);
					// Explicitly check against FInstancedPropertyBag, because that is what holds the Custom DetailsView.
					return PropertyStruct && PropertyStruct->Struct == FInstancedPropertyBag::StaticStruct();
				};

				if (const FProperty* Property = PropertyNode->GetProperty())
				{
					const bool bPropertyIsPropertyBag = PropertyIsPropertyBag(Property);
					bool bPropertyIsInPropertyBag = false;
					// Traverse the parentage to see if the property is contained within the property bag.
					for (TSharedPtr<IPropertyHandle> ParentNode = PropertyNode->GetParentHandle(); ParentNode != nullptr; ParentNode = ParentNode->GetParentHandle())
					{
						if (PropertyIsPropertyBag(ParentNode->GetProperty()))
						{
							bPropertyIsInPropertyBag = true;
							break;
						}
					}

					// Filter if the property isn't the property bag and the property is not inside the property bag.
					return !(bPropertyIsPropertyBag || bPropertyIsInPropertyBag);
				}

				return true;
			});
	}

	// Note: Single Property View Header notes that it doesn't work with arrays or structs, so using filters for now.
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	// Important: since this is using the graph object and that has details customization, we want to ensure we use the default layout instead.
	// Otherwise we'll get buttons and more.
	DetailsView->RegisterInstancedCustomPropertyLayout(UPCGGraph::StaticClass(), DetailsView->GetGenericLayoutDetailsDelegate());

	DetailsView->SetObject(PCGGraph);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			DetailsView.ToSharedRef()
		]
	];
}

void SPCGEditorGraphUserParametersView::SetGraph(UPCGGraph* InGraph)
{
	if (DetailsView)
	{
		DetailsView->SetObject(InGraph);
	}
}
