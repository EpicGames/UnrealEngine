// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#include "Widgets/SCompoundWidget.h"

struct FPropertyAndParent;
class FPCGEditor;
class IDetailsView;
class UPCGGraph;

/** Displays a PCGGraph's User Parameters details view. */
class SPCGEditorGraphUserParametersView final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphUserParametersView)	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<FPCGEditor>& InPCGEditor);
	void SetGraph(UPCGGraph* InGraph);
private:
	TSharedPtr<IDetailsView> DetailsView;
};