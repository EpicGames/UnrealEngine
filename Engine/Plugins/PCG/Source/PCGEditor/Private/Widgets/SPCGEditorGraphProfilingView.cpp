// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphProfilingView.h"

#include "PCGComponent.h"
#include "Graph/PCGStackContext.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"
#include "Managers/PCGEditorInspectionDataManager.h"
#include "Nodes/PCGEditorGraphNode.h"

#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SPCGEditorGraphProfilingView"

namespace PCGEditorGraphProfilingView
{
	const FText NoDataAvailableText = LOCTEXT("NoDataAvailableText", "N/A");
	
	/** Names of the columns in the attribute list */
	const FName NAME_Node = FName(TEXT("Node"));
	const FName NAME_PrepareDataTime = FName(TEXT("PrepareDataTime"));
	const FName NAME_PrepareDataWallTime = FName(TEXT("PrepareData_WallTime"));
	const FName NAME_NbPrepareFrames = FName(TEXT("NbPrepareFrames"));
	const FName NAME_MinExecutionFrameTime = FName(TEXT("MinFrameTime"));
	const FName NAME_MaxExecutionFrameTime = FName(TEXT("MaxFrameTime"));
	const FName NAME_ExecutionTime = FName(TEXT("ExecutionTime"));
	const FName NAME_ExecutionWallTime = FName(TEXT("Execution_WallTime"));
	const FName NAME_NbExecutionFrames = FName(TEXT("NbExecutionFrames"));
	const FName NAME_TotalTime = FName(TEXT("TotalTime"));
	const FName NAME_TotalWallTime = FName(TEXT("Total_WallTime"));
	const FName NAME_CPUMemoryInMB = FName(TEXT("CPU data size (MB)"));
	const FName NAME_GPUMemoryInMB = FName(TEXT("GPU data size (MB)"));
	const FName NAME_GPUTimeMs = FName(TEXT("GPUTime"));
	const FName NAME_NumberOfTasks = FName(TEXT("NumberOfTasks"));

	/** Labels of the columns */
	const FText TEXT_NodeLabel = LOCTEXT("NodeLabel", "Node");
	const FText TEXT_PrepareDataTimeLabel = LOCTEXT("PrepareDataTimeLabel", "Prepare");
	const FText TEXT_PrepareDataWallTimeLabel = LOCTEXT("PrepareDataWallTimeLabel", "Prepare WallTime");
	const FText TEXT_NbPrepareFramesLabel = LOCTEXT("NbPrepareFramesLabel", "Prep Frames");
	const FText TEXT_MinExecutionFrameTimeLabel = LOCTEXT("MinExecutionFrameTimeLabel", "Min FrameTime");
	const FText TEXT_MaxExecutionFrameTimeLabel = LOCTEXT("MaxExecutionFrameTimeLabel", "Max FrameTime");
	const FText TEXT_ExecutionTimeLabel = LOCTEXT("ExecutionTimeLabel", "Exec");
	const FText TEXT_ExecutionWallTimeLabel = LOCTEXT("ExecutionWallTimeLabel", "Exec WallTime");
	const FText TEXT_NbExecutionFramesLabel = LOCTEXT("NbExecutionFramesLabel", "Exec Frames");
	const FText TEXT_TotalTimeLabel = LOCTEXT("TotalTimeLabel", "Total");
	const FText TEXT_TotalWallTimeLabel = LOCTEXT("TotalWallTimeLabel", "Total WallTime");
	const FText TEXT_CPUMemoryInMBLabel = LOCTEXT("CPUMemoryInMBLabel", "CPU data (MB)");
	const FText TEXT_GPUMemoryInMBLabel = LOCTEXT("GPUMemoryInMBLabel", "GPU data (MB)");
	const FText TEXT_GPUTimeMsLabel = LOCTEXT("GPUTimeMsLabel", "GPU Time (ms)");
	const FText TEXT_NumberOfTasks = LOCTEXT("NumberOfTasksLabel", "Num Subtasks");

	/** Tooltips */
	const FText TEXT_PrepareDataTimeTooltip = LOCTEXT("PrepareDataTimeTooltip", "Cost of the PrepareData execution phase which some nodes use to process the incoming data, in ms.");
	const FText TEXT_PrepareDataWallTimeTooltip = LOCTEXT("PrepareDataWallTimeTooltip", "Total real time elapsed between prepare data first being called until completion, including any wait/sleep time, in ms.");
	const FText TEXT_NbPrepareFramesTooltip = LOCTEXT("NbPrepareFramesTooltip", "The number of frames in which one or more prepare data phases were executed.");
	const FText TEXT_MinExecutionFrameTimeTooltip = LOCTEXT("MinExecutionFrameTimeTooltip", "The minimum time spent of all execution frames, in ms.");
	const FText TEXT_MaxExecutionFrameTimeTooltip = LOCTEXT("MaxExecutionFrameTimeTooltip", "The maximum time spent of all execution frames, in ms.");
	const FText TEXT_ExecutionTimeTooltip = LOCTEXT("ExecutionTimeTooltip", "The total time spent for execution, summed over all execution frames, in ms.");
	const FText TEXT_ExecutionWallTimeTooltip = LOCTEXT("ExecutionWallTimeTooltip", "Total real time elapsed between execute first being called until completion, including any wait/sleep time, in ms.");
	const FText TEXT_NbExecutionFramesTooltip = LOCTEXT("NbExecutionFramesTooltip", "The number of frames in which one or more execution phases were executed.");
	const FText TEXT_TotalTimeTooltip = LOCTEXT("TotalTimeTooltip", "The total time spent in this node, summed over all execution and prepare data frames, in ms.");
	const FText TEXT_TotalWallTimeTooltip = LOCTEXT("TotalWallTimeTooltip", "Total real time elapsed between the first call until completion, including any wait/sleep time, in ms.");
	const FText TEXT_CPUMemoryInMBTooltip = LOCTEXT("CPUMemoryInMBTooltip", "Size in MB of newly created CPU system memory data.");
	const FText TEXT_GPUMemoryInMBTooltip = LOCTEXT("GPUMemoryInMBTooltip", "Size in MB of output data allocated in GPU memory while executing kernels emitted by the node. This memory is freed once any downstream consumers have executed.");
	const FText TEXT_GPUTimeMsTooltip = LOCTEXT("GPUTimeMsTooltip", "GPU kernel execution time in ms, measured via GPU hardware counters. Only available for GPU nodes in editor builds with GPU stats enabled.");
	const FText TEXT_NumberOfTasksTooltip = LOCTEXT("NumberOfTasksTooltip", "Number of subtasks executed for this subgraph. It's an approximation to give an order of magnitude. Only show a value for subgraph nodes.");
}

FText FPCGProfilingTotalData::GetTotalTimeLabel() const
{
	return FText::Format(LOCTEXT("GraphTotalTimeLabel", "Total Time: {0} s"), TotalTime);
}

FText FPCGProfilingTotalData::GetTotalWallTimeLabel() const
{
	return FText::Format(LOCTEXT("GraphTotalWallTimeLabel", "Total Wall Time: {0} s"), TotalWallTime);
}

FText FPCGProfilingTotalData::GetTotalCPUDataSizeInMBLabel() const
{
	return FText::Format(LOCTEXT("GraphTotalCPUDataSizeInMBLabel", "Total CPU data (MB): {0} MB"), TotalCPUDataSizeInMB);
}

FText FPCGProfilingTotalData::GetTotalGPUDataSizeInMBLabel() const
{
	return FText::Format(LOCTEXT("GraphTotalGPUDataSizeInMBLabel", "Total GPU data (MB): {0} MB"), TotalGPUDataSizeInMB);
}

FText FPCGProfilingTotalData::GetTotalNumTasksLabel() const
{
	return FText::Format(LOCTEXT("GraphTotalNumTasksLabel", "Total Num Tasks: {0}"), TotalNumTasks);
}

TSharedRef<SWidget> FPCGProfilingTotalData::MakeOptionsWidget()
{
	TSharedPtr<SComboButton> OptionsWidget = SNew(SComboButton)
	.ForegroundColor(FSlateColor::UseStyle())
	.HasDownArrow(false)
	.OnGetMenuContent(FOnGetContent::CreateLambda([this](){ return OnOptionsMenu();}))
	.ContentPadding(1)
	.ButtonContent()
	[
		SNew(SImage)
		.Image(FAppStyle::GetBrush("EditorViewportToolBar.OptionsDropdown"))
		.ColorAndOpacity(FSlateColor::UseForeground())
	];

	return OptionsWidget.ToSharedRef();
}

TSharedRef<SWidget> FPCGProfilingTotalData::OnOptionsMenu()
{
	FMenuBuilder MenuBuilder(false, nullptr);

	auto AddEntry = [&MenuBuilder](FText Text, bool* bValue)
	{
		MenuBuilder.AddMenuEntry(
			MoveTemp(Text),
			FText{},
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([bValue]() { *bValue = !*bValue;}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([bValue](){ return *bValue;})),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	};

	AddEntry(LOCTEXT("TotalTimeOption", "Show Total Time"), &bShowTotalTime);
	AddEntry(LOCTEXT("TotalWallTimeOption", "Show Total Wall Time"), &bShowTotalWallTime);
	AddEntry(LOCTEXT("TotalCPUDataSizeOption", "Show Total CPU Data Size"), &bShowTotalCPUDataSize);
	AddEntry(LOCTEXT("TotalGPUDataSizeOption", "Show Total GPU Data Size"), &bShowTotalGPUDataSize);
	AddEntry(LOCTEXT("TotalNumTasksOption", "Show Total Num Tasks"), &bShowTotalTasks);

	return MenuBuilder.MakeWidget();
}

void FPCGProfilingTotalData::Reset()
{
	TotalNumTasks = 0;
	TotalCPUDataSizeInMB = 0.0;
	TotalGPUDataSizeInMB = 0.0;
	TotalTime = 0;
	TotalWallTime = 0;
}

static FText FormatMemoryMB(uint64 Bytes)
{
	const double ValueMB = Bytes / (1024.0 * 1024.0);

	// For zero or values >= 1 MB, use 2 decimal places. For smaller values, use enough decimal places to show 2 significant figures.
	const int32 FractionalDigits = (ValueMB == 0.0 || ValueMB >= 1.0) ? 2 : 1 - FMath::FloorToInt(FMath::LogX(10.0, ValueMB));
	FNumberFormattingOptions Opts;
	Opts.MinimumFractionalDigits = FractionalDigits;
	Opts.MaximumFractionalDigits = FractionalDigits;

	return FText::AsNumber(ValueMB, &Opts);
}

void SPCGProfilingListViewItemRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const PCGProfilingListViewItemPtr& Item)
{
	InternalItem = Item;

	SMultiColumnTableRow<PCGProfilingListViewItemPtr>::Construct(
		SMultiColumnTableRow::FArguments()
		.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
		InOwnerTableView);
}

FText FPCGProfilingListViewItem::GetTextForColumn(FName ColumnId, bool bNoGrouping) const
{
	const FNumberFormattingOptions* NumberFormattingOptions = (bNoGrouping ? &FNumberFormattingOptions::DefaultNoGrouping() : nullptr);

	if (ColumnId == PCGEditorGraphProfilingView::NAME_Node)
	{
		const UPCGSettings* PCGSettings = PCGNode ? PCGNode->GetSettings() : nullptr;

		if (PCGSettings)
		{
			if (PCGSettings->ShouldExecuteOnGPU())
			{
				return FText::FromString(Name + TEXT(" (GPU)"));
			}
			else if (!PCGSettings->bEnabled)
			{
				return FText::FromString(Name + TEXT(" (disabled)"));
			}
		}

		if (bHasCPUTimingData)
		{
			return FText::FromString(Name);
		}
		else
		{
			return FText::FromString(Name + TEXT(" (cached)"));
		}
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_NbExecutionFrames)
	{
		return ((CallTime.ExecutionFrameCount >= 0) ? FText::AsNumber(CallTime.ExecutionFrameCount, NumberFormattingOptions) : FText());
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_NbPrepareFrames)
	{
		return ((CallTime.PrepareDataFrameCount >= 0) ? FText::AsNumber(CallTime.PrepareDataFrameCount, NumberFormattingOptions) : FText());
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_CPUMemoryInMB)
	{
		return CallTime.OutputCPUMemorySize.IsSet() ? FormatMemoryMB(CallTime.OutputCPUMemorySize.GetValue()) : PCGEditorGraphProfilingView::NoDataAvailableText;
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_GPUMemoryInMB)
	{
		return CallTime.OutputGPUMemorySize.IsSet() ? FormatMemoryMB(CallTime.OutputGPUMemorySize.GetValue()) : PCGEditorGraphProfilingView::NoDataAvailableText;
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_GPUTimeMs)
	{
		return CallTime.GPUTime.IsSet() ? FText::AsNumber(CallTime.GPUTime.GetValue() * 1000.0, NumberFormattingOptions) : PCGEditorGraphProfilingView::NoDataAvailableText;
	}
	else if (!bHasCPUTimingData)
	{
		return PCGEditorGraphProfilingView::NoDataAvailableText;
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_MinExecutionFrameTime)
	{
		// In ms
		return ((CallTime.MinExecutionFrameTime >= 0) ? FText::AsNumber(CallTime.MinExecutionFrameTime * 1000.0, NumberFormattingOptions) : FText());
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_MaxExecutionFrameTime)
	{
		// In ms
		return ((CallTime.MaxExecutionFrameTime >= 0) ? FText::AsNumber(CallTime.MaxExecutionFrameTime * 1000.0, NumberFormattingOptions) : FText());
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_ExecutionTime)
	{
		// In ms
		return FText::AsNumber(CallTime.ExecutionTime * 1000.0, NumberFormattingOptions);
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_ExecutionWallTime)
	{
		// In ms
		return FText::AsNumber(CallTime.ExecutionWallTime() * 1000.0, NumberFormattingOptions);
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_PrepareDataTime)
	{
		// In ms
		return FText::AsNumber(CallTime.PrepareDataTime * 1000.0, NumberFormattingOptions);
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_PrepareDataWallTime)
	{
		// In ms
		return FText::AsNumber(CallTime.PrepareDataWallTime() * 1000.0, NumberFormattingOptions);
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_TotalTime)
	{
		// In ms
		return FText::AsNumber((CallTime.TotalTime()) * 1000.0, NumberFormattingOptions);
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_TotalWallTime)
	{
		// In ms
		return FText::AsNumber((CallTime.TotalWallTime()) * 1000.0, NumberFormattingOptions);
	}
	else if (ColumnId == PCGEditorGraphProfilingView::NAME_NumberOfTasks)
	{
		// Do not show the number of tasks if it is not higher than 1
		return NumberOfTasks > 1 ? FText::AsNumber(NumberOfTasks) : FText{};
	}
	else
	{
		return LOCTEXT("ItemColumnError", "Unrecognized Column");
	}
}

TSharedRef<SWidget> SPCGProfilingListViewItemRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	FText ColumnData = LOCTEXT("ColumnError", "Unrecognized Column");
	FColor ColumnDataColor = FColor::White;

	if (InternalItem.IsValid())
	{
		ColumnData = InternalItem->GetTextForColumn(ColumnId, /*bNoGrouping=*/false);

		if (!InternalItem->bHasCPUTimingData && !InternalItem->CallTime.OutputCPUMemorySize.IsSet() && !InternalItem->CallTime.OutputGPUMemorySize.IsSet() && !InternalItem->CallTime.GPUTime.IsSet())
		{
			ColumnDataColor = FColor(75, 75, 75);
		}
	}

	TSharedRef<SWidget> DataWidget = 
		SNew(STextBlock)
		.Text(ColumnData)
		.ColorAndOpacity(ColumnDataColor)
		.OverflowPolicy(ETextOverflowPolicy::Ellipsis);

	// Add the internal name of the node as tooltip
	if (ColumnId == PCGEditorGraphProfilingView::NAME_Node && InternalItem->PCGNode)
	{
		DataWidget->SetToolTipText(FText::FromString(InternalItem->PCGNode->GetName()));
	}

	return DataWidget;
}

void SPCGEditorGraphProfilingView::OnItemDoubleClicked(PCGProfilingListViewItemPtr Item)
{
	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();
	if (!PCGEditor.IsValid())
	{
		return;
	}

	if (!Item.IsValid() || !Item->EditorNode)
	{
		return;
	}

	PCGEditor->JumpToNode(Item->EditorNode);
}

SPCGEditorGraphProfilingView::~SPCGEditorGraphProfilingView()
{
	if (PCGEditorPtr.IsValid())
	{
		PCGEditorPtr.Pin()->GetInspectionDataManager().OnInspectedStackChangedDelegate.RemoveAll(this);
	}
}

void SPCGEditorGraphProfilingView::Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor)
{
	PCGEditorPtr = InPCGEditor;

	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();
	if (PCGEditor)
	{
		PCGEditorGraph = PCGEditor->GetMainEditorGraph();
		PCGEditor->GetInspectionDataManager().OnInspectedStackChangedDelegate.AddSP(this, &SPCGEditorGraphProfilingView::OnDebugStackChanged);
	}

	SortingColumn = PCGEditorGraphProfilingView::NAME_TotalTime;
	SortMode = EColumnSortMode::Descending;
	ListViewHeader = CreateHeaderRowWidget();

	ListViewCommands = MakeShareable(new FUICommandList);
	ListViewCommands->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SPCGEditorGraphProfilingView::CopySelectionToClipboard),
		FCanExecuteAction::CreateSP(this, &SPCGEditorGraphProfilingView::CanCopySelectionToClipboard));

	const TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal)
		.Thickness(FVector2D(12.0f, 12.0f));

	const TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(12.0f, 12.0f));
	
	SAssignNew(ListView, SListView<PCGProfilingListViewItemPtr>)
		.ListItemsSource(&ListViewItems)
		.HeaderRow(ListViewHeader)
		.OnGenerateRow(this, &SPCGEditorGraphProfilingView::OnGenerateRow)
		.OnMouseButtonDoubleClick(this, &SPCGEditorGraphProfilingView::OnItemDoubleClicked)
		.AllowOverscroll(EAllowOverscroll::No)
		.ExternalScrollbar(VerticalScrollBar)
		.OnKeyDownHandler(this, &SPCGEditorGraphProfilingView::OnListViewKeyDown)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always);
	
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSearchBox)
				.HintText(LOCTEXT("PCGGraphSearchHint", "Enter text to find nodes..."))
				.OnTextChanged(this, &SPCGEditorGraphProfilingView::OnSearchTextChanged)
				.OnTextCommitted(this, &SPCGEditorGraphProfilingView::OnSearchTextCommitted)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(1.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ExpandSubgraphDepth", "Expand Subgraph Depth"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(1.0f, 0.0f)
			[
				SNew(SSpinBox<int32>)
				.Value(this, &SPCGEditorGraphProfilingView::GetSubgraphExpandDepth)
				.OnValueChanged(this, &SPCGEditorGraphProfilingView::OnSubgraphExpandDepthChanged)
				.MinValue(0)
				.MaxValue(20)
			]
			+SHorizontalBox::Slot()
			.Padding(FMargin(30.0f, 0.0f, 2.0f, 0.0f))
			.VAlign(EVerticalAlignment::VAlign_Center)
			.AutoWidth()
			[
				ProfilingTotalData.MakeOptionsWidget()
			]
			+SHorizontalBox::Slot()
			.Padding(FMargin(30.0f, 0.0f, 2.0f, 0.0f))
			.VAlign(EVerticalAlignment::VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text_Lambda([this]() { return ProfilingTotalData.GetTotalTimeLabel();})
				.Visibility_Lambda([this]() { return ProfilingTotalData.bShowTotalTime ? EVisibility::Visible : EVisibility::Collapsed;})
			]
			+SHorizontalBox::Slot()
			.Padding(FMargin(30.0f, 0.0f, 2.0f, 0.0f))
			.VAlign(EVerticalAlignment::VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text_Lambda([this]() { return ProfilingTotalData.GetTotalWallTimeLabel();})
				.Visibility_Lambda([this]() { return ProfilingTotalData.bShowTotalWallTime ? EVisibility::Visible : EVisibility::Collapsed;})
			]
			+SHorizontalBox::Slot()
			.Padding(FMargin(30.0f, 0.0f, 2.0f, 0.0f))
			.VAlign(EVerticalAlignment::VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text_Lambda([this]() { return ProfilingTotalData.GetTotalCPUDataSizeInMBLabel();})
				.Visibility_Lambda([this]() { return ProfilingTotalData.bShowTotalCPUDataSize ? EVisibility::Visible : EVisibility::Collapsed;})
			]
			+SHorizontalBox::Slot()
			.Padding(FMargin(30.0f, 0.0f, 2.0f, 0.0f))
			.VAlign(EVerticalAlignment::VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text_Lambda([this]() { return ProfilingTotalData.GetTotalGPUDataSizeInMBLabel();})
				.Visibility_Lambda([this]() { return ProfilingTotalData.bShowTotalGPUDataSize ? EVisibility::Visible : EVisibility::Collapsed;})
			]
			+SHorizontalBox::Slot()
			.Padding(FMargin(30.0f, 0.0f, 2.0f, 0.0f))
			.VAlign(EVerticalAlignment::VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text_Lambda([this]() { return ProfilingTotalData.GetTotalNumTasksLabel();})
				.Visibility_Lambda([this]() { return ProfilingTotalData.bShowTotalTasks ? EVisibility::Visible : EVisibility::Collapsed;})
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				.ExternalScrollbar(HorizontalScrollBar)
				+SScrollBox::Slot()
				[
					ListView->AsShared()
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				VerticalScrollBar
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				HorizontalScrollBar
			]
		]
	];

	Refresh();
}

void SPCGEditorGraphProfilingView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bNeedsRefresh)
	{
		bNeedsRefresh = false;
		Refresh();
	}
}

TSharedRef<SHeaderRow> SPCGEditorGraphProfilingView::CreateHeaderRowWidget()
{
	return SNew(SHeaderRow)
		.ResizeMode(ESplitterResizeMode::FixedPosition)
		.CanSelectGeneratedColumn(true)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_Node)
		.ManualWidth(300.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_NodeLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Left)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_Node)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_PrepareDataTime)
		.ManualWidth(75.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_PrepareDataTimeLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_PrepareDataTime)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_PrepareDataTimeTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_PrepareDataWallTime)
		.ManualWidth(110.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_PrepareDataWallTimeLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_PrepareDataWallTime)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_PrepareDataWallTimeTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_NbPrepareFrames)
		.ManualWidth(90.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_NbPrepareFramesLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_NbPrepareFrames)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_NbPrepareFramesTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_NbExecutionFrames)
		.ManualWidth(90.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_NbExecutionFramesLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_NbExecutionFrames)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_NbExecutionFramesTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_MinExecutionFrameTime)
		.ManualWidth(110.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_MinExecutionFrameTimeLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_MinExecutionFrameTime)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_MinExecutionFrameTimeTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_MaxExecutionFrameTime)
		.ManualWidth(110.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_MaxExecutionFrameTimeLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_MaxExecutionFrameTime)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_MaxExecutionFrameTimeTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_ExecutionTime)
		.ManualWidth(75.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_ExecutionTimeLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_ExecutionTime)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_ExecutionTimeTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_ExecutionWallTime)
		.ManualWidth(100.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_ExecutionWallTimeLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_ExecutionWallTime)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_ExecutionWallTimeTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_TotalTime)
		.ManualWidth(75.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_TotalTimeLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_TotalTime)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_TotalTimeTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_TotalWallTime)
		.ManualWidth(100.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_TotalWallTimeLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_TotalWallTime)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_TotalWallTimeTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_CPUMemoryInMB)
		.ManualWidth(100.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_CPUMemoryInMBLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_CPUMemoryInMB)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_CPUMemoryInMBTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_GPUMemoryInMB)
		.ManualWidth(100.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_GPUMemoryInMBLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_GPUMemoryInMB)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_GPUMemoryInMBTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_GPUTimeMs)
		.ManualWidth(100.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_GPUTimeMsLabel)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_GPUTimeMs)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_GPUTimeMsTooltip)
		+ SHeaderRow::Column(PCGEditorGraphProfilingView::NAME_NumberOfTasks)
		.ManualWidth(110.0f)
		.DefaultLabel(PCGEditorGraphProfilingView::TEXT_NumberOfTasks)
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Right)
		.SortMode(this, &SPCGEditorGraphProfilingView::GetColumnSortMode, PCGEditorGraphProfilingView::NAME_NumberOfTasks)
		.OnSort(this, &SPCGEditorGraphProfilingView::OnSortColumnHeader)
		.InitialSortMode(EColumnSortMode::Descending)
		.DefaultTooltip(PCGEditorGraphProfilingView::TEXT_NumberOfTasksTooltip);
}

void SPCGEditorGraphProfilingView::OnSortColumnHeader(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode)
{
	if (SortingColumn == ColumnId)
	{
		// Circling
		SortMode = static_cast<EColumnSortMode::Type>((SortMode + 1) % 3);
	}
	else
	{
		SortingColumn = ColumnId;
		SortMode = NewSortMode;
	}

	RequestRefresh();
}

void SPCGEditorGraphProfilingView::OnSubgraphExpandDepthChanged(int32 NewValue)
{
	ExpandSubgraphDepth = NewValue;
	RequestRefresh();
}

FReply SPCGEditorGraphProfilingView::OnListViewKeyDown(const FGeometry& /*InGeometry*/, const FKeyEvent& InKeyEvent) const
{
	if (ListViewCommands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SPCGEditorGraphProfilingView::CopySelectionToClipboard() const
{
	constexpr TCHAR Delimiter = TEXT(',');
	constexpr TCHAR LineEnd = TEXT('\n');

	TStringBuilder<2048> CSVExport;
	TArray<FName> VisibleColumns;

	// Write column header row
	bool bHasWrittenAColumn = false;
	const TIndirectArray<SHeaderRow::FColumn>& Columns = ListViewHeader->GetColumns();
	for (auto Column : Columns)
	{
		if (!Column.bIsVisible)
		{
			continue;
		}

		VisibleColumns.Add(Column.ColumnId);
		const FText& ColumnTitle = Column.DefaultText.Get();

		if (bHasWrittenAColumn)
		{
			CSVExport += Delimiter;
		}

		CSVExport += ColumnTitle.ToString();
		bHasWrittenAColumn = true;
	}

	// Gather selected rows and sort them to match the display order instead of selection order
	TArray<PCGProfilingListViewItemPtr> SelectedListViewItems = ListView->GetSelectedItems();

	TArray<int32> SelectedListViewItemsIndices;
	Algo::Transform(SelectedListViewItems, SelectedListViewItemsIndices, [this](const PCGProfilingListViewItemPtr& InItem) { return ListViewItems.IndexOfByKey(InItem); });
	Algo::Sort(SelectedListViewItemsIndices, [](const int32& A, const int32& B) { return A < B; });

	SelectedListViewItems.Reset();
	Algo::Transform(SelectedListViewItemsIndices, SelectedListViewItems, [this](const int32& InItemIndex) { return ListViewItems[InItemIndex]; });

	// Write each row
	for (const PCGProfilingListViewItemPtr& ListViewItem : SelectedListViewItems)
	{
		CSVExport += LineEnd;

		for (int ColumnIndex = 0; ColumnIndex < VisibleColumns.Num(); ++ColumnIndex)
		{
			if (ColumnIndex > 0)
			{
				CSVExport += Delimiter;
			}

			CSVExport += ListViewItem->GetTextForColumn(VisibleColumns[ColumnIndex], /*bNoGrouping=*/true).ToString();
		}
	}

	FPlatformApplicationMisc::ClipboardCopy(*CSVExport);
}

bool SPCGEditorGraphProfilingView::CanCopySelectionToClipboard() const
{
	return ListView->GetNumItemsSelected() > 0;
}

EColumnSortMode::Type SPCGEditorGraphProfilingView::GetColumnSortMode(const FName ColumnId) const
{
	if (SortingColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

namespace PCGEditorGraphProfilingView
{
	void AddListItems(
		TArray<PCGProfilingListViewItemPtr>& OutListViewItems,
		const TArray<PCGUtils::FCallTreeInfo>& TreeInfo,
		const TMap<const UPCGNode*, UPCGEditorGraphNode*>& EditorNodeLookup,
		int ExpandSubgraphDepth,
		const FString& FolderName,
		const FString& SearchString,
		UPCGEditorGraphNode* CurrentEditorNode = nullptr)
	{
		for (const PCGUtils::FCallTreeInfo& Info : TreeInfo)
		{
			UPCGEditorGraphNode*const* EditorNodeItr = EditorNodeLookup.Find(Info.Node);
			UPCGEditorGraphNode* EditorNode = EditorNodeItr ? *EditorNodeItr : CurrentEditorNode;

			FString Fullname = FolderName;
			if (!Info.Name.IsEmpty())
			{
				Fullname += Info.Name;
			}
			else if (Info.Node)
			{
				Fullname += Info.Node->GetNodeTitle(EPCGNodeTitleType::ListView).ToString();
			}
			else if (Info.LoopIndex != INDEX_NONE)
			{
				Fullname += FString::Printf(TEXT("Loop_%d"), Info.LoopIndex);
			}

			const bool bFilteredIn = SearchString.IsEmpty() || Fullname.Find(SearchString) != INDEX_NONE || (Info.Node && Info.Node->GetName().Find(SearchString) != INDEX_NONE);

			if (bFilteredIn && (Info.Children.IsEmpty() || ExpandSubgraphDepth == 0))
			{
				PCGProfilingListViewItemPtr ListViewItem = MakeShared<FPCGProfilingListViewItem>();

				ListViewItem->PCGNode = Info.Node;
				ListViewItem->EditorNode = EditorNode;
				ListViewItem->Name = Fullname;
				ListViewItem->bHasCPUTimingData = (Info.CallTime.MaxExecutionFrameTime > 0);

				ListViewItem->CallTime = Info.CallTime;
				ListViewItem->NumberOfTasks = Info.NumberOfTasks;

				OutListViewItems.Add(ListViewItem);
			}

			if(ExpandSubgraphDepth > 0 && !Info.Children.IsEmpty())
			{
				AddListItems(OutListViewItems, Info.Children, EditorNodeLookup, ExpandSubgraphDepth - 1, Fullname + "/", SearchString, EditorNode);
			}
		}
	}
}

void SPCGEditorGraphProfilingView::RequestRefresh()
{
	bNeedsRefresh = true;
}

FReply SPCGEditorGraphProfilingView::Refresh()
{
	ProfilingTotalData.Reset();

	ListViewItems.Empty();
	ListView->RequestListRefresh();

	const TSharedPtr<FPCGEditor> PCGEditor = PCGEditorPtr.Pin();
	if (!PCGEditor.IsValid())
	{
		return FReply::Handled();
	}

	if (!PCGEditorGraph)
	{
		return FReply::Handled();
	}

	const IPCGGraphExecutionSource* ExecutionSource = GetExecutionSource();
	if (!ExecutionSource)
	{
		return FReply::Handled();
	}

	TArray<UPCGEditorGraphNode*> EditorNodes;
	PCGEditorGraph->GetNodesOfClass<UPCGEditorGraphNode>(EditorNodes);

	TMap<const UPCGNode*, UPCGEditorGraphNode*> EditorNodeLookup;
	for (UPCGEditorGraphNode* EditorNode : EditorNodes)
	{
		EditorNodeLookup.Add(EditorNode->GetPCGNode(), EditorNode);
	}

	PCGUtils::FCallTreeInfo TreeInfo = ExecutionSource->GetExecutionState().GetExtraCapture().CalculateCallTreeInfo(ExecutionSource, PCGStack);

	ListViewItems.Reserve(TreeInfo.Children.Num());

	if (TreeInfo.Children.Num() > 0)
	{
		ProfilingTotalData.TotalTime = TreeInfo.CallTime.TotalTime();
		ProfilingTotalData.TotalWallTime = TreeInfo.CallTime.TotalWallTime();
		ProfilingTotalData.TotalCPUDataSizeInMB = TreeInfo.CallTime.OutputCPUMemorySize.Get(0) / (1024.0 * 1024.0);
		ProfilingTotalData.TotalGPUDataSizeInMB = TreeInfo.CallTime.OutputGPUMemorySize.Get(0) / (1024.0 * 1024.0);
		ProfilingTotalData.TotalNumTasks = TreeInfo.NumberOfTasks;
	}

	//TODO: could turn this into a tree instead of expanding into a list
	PCGEditorGraphProfilingView::AddListItems(ListViewItems, TreeInfo.Children, EditorNodeLookup, ExpandSubgraphDepth, FString(), SearchValue);

	if (SortingColumn != NAME_None && SortMode != EColumnSortMode::None)
	{
		Algo::Sort(ListViewItems, [this](const PCGProfilingListViewItemPtr& A, const PCGProfilingListViewItemPtr& B)
			{
				bool isLess = false;

				// N/A always sorts to the bottom.
				auto IsNA = [](const PCGProfilingListViewItemPtr& Item, FName Column) -> bool
				{
					if (Column == PCGEditorGraphProfilingView::NAME_CPUMemoryInMB)
					{
						return !Item->CallTime.OutputCPUMemorySize.IsSet();
					}
					else if (Column == PCGEditorGraphProfilingView::NAME_GPUMemoryInMB)
					{
						return !Item->CallTime.OutputGPUMemorySize.IsSet();
					}
					else if (Column == PCGEditorGraphProfilingView::NAME_GPUTimeMs)
					{
						return !Item->CallTime.GPUTime.IsSet();
					}
					else if (Column != PCGEditorGraphProfilingView::NAME_Node && Column != PCGEditorGraphProfilingView::NAME_NumberOfTasks)
					{
						return !Item->bHasCPUTimingData;
					}
					return false;
				};

				// Early out if one or both are N/A.
				const bool bAIsNA = IsNA(A, SortingColumn);
				if (bAIsNA != IsNA(B, SortingColumn) || bAIsNA)
				{
					return !bAIsNA;
				}

				if (SortingColumn == PCGEditorGraphProfilingView::NAME_Node)
				{
					isLess = A->Name < B->Name;
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_PrepareDataTime)
				{
					isLess = A->CallTime.PrepareDataTime < B->CallTime.PrepareDataTime;
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_PrepareDataWallTime)
				{
					isLess = A->CallTime.PrepareDataWallTime() < B->CallTime.PrepareDataWallTime();
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_NbPrepareFrames)
				{
					isLess = A->CallTime.PrepareDataFrameCount < B->CallTime.PrepareDataFrameCount;
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_MinExecutionFrameTime)
				{
					isLess = A->CallTime.MinExecutionFrameTime < B->CallTime.MinExecutionFrameTime;
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_MaxExecutionFrameTime)
				{
					isLess = A->CallTime.MaxExecutionFrameTime < B->CallTime.MaxExecutionFrameTime;
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_ExecutionTime)
				{
					isLess = A->CallTime.ExecutionTime < B->CallTime.ExecutionTime;
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_ExecutionWallTime)
				{
					isLess = A->CallTime.ExecutionWallTime() < B->CallTime.ExecutionWallTime();
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_NbExecutionFrames)
				{
					isLess = A->CallTime.ExecutionFrameCount < B->CallTime.ExecutionFrameCount;
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_TotalTime)
				{
					isLess = A->CallTime.TotalTime() < B->CallTime.TotalTime();
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_TotalWallTime)
				{
					isLess = A->CallTime.TotalWallTime() < B->CallTime.TotalWallTime();
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_CPUMemoryInMB)
				{
					isLess = A->CallTime.OutputCPUMemorySize.Get(0) < B->CallTime.OutputCPUMemorySize.Get(0);
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_GPUMemoryInMB)
				{
					isLess = A->CallTime.OutputGPUMemorySize.Get(0) < B->CallTime.OutputGPUMemorySize.Get(0);
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_GPUTimeMs)
				{
					isLess = A->CallTime.GPUTime.Get(0.0) < B->CallTime.GPUTime.Get(0.0);
				}
				else if (SortingColumn == PCGEditorGraphProfilingView::NAME_NumberOfTasks)
				{
					isLess = A->NumberOfTasks < B->NumberOfTasks;
				}

				return SortMode == EColumnSortMode::Ascending ? isLess : !isLess;
			});
	}

	ListView->SetItemsSource(&ListViewItems);

	return FReply::Handled();
}

void SPCGEditorGraphProfilingView::OnDebugStackChanged(const FPCGStack& InPCGStack)
{
	PCGStack = InPCGStack;
	RequestRefresh();
}

IPCGGraphExecutionSource* SPCGEditorGraphProfilingView::GetExecutionSource() const
{
	return PCGEditorPtr.IsValid() ? PCGEditorPtr.Pin()->GetPCGSourceBeingInspected() : nullptr;
}

TSharedRef<ITableRow> SPCGEditorGraphProfilingView::OnGenerateRow(PCGProfilingListViewItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SPCGProfilingListViewItemRow, OwnerTable, Item);
}

void SPCGEditorGraphProfilingView::OnSearchTextChanged(const FText& InText)
{
	SearchValue = InText.ToString();
	Refresh();
}

void SPCGEditorGraphProfilingView::OnSearchTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	OnSearchTextChanged(InText);
}

#undef LOCTEXT_NAMESPACE
