// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/PCGStackContext.h"
#include "Utils/PCGExtraCapture.h"

#include "Framework/Commands/GenericCommands.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FPCGEditor;
class STableViewBase;
class UPCGEditorGraphNode;
class UPCGEditorGraph;
class UPCGNode;

struct FPCGProfilingListViewItem
{
	FText GetTextForColumn(FName ColumnId, bool bNoGrouping) const;

	const UPCGNode* PCGNode = nullptr;
	const UPCGEditorGraphNode* EditorNode = nullptr;

	FString Name;

	PCGUtils::FCallTime CallTime;
	int32 NumberOfTasks = 0;

	bool bHasCPUTimingData = false;
};

struct FPCGProfilingTotalData
{
	bool bShowTotalTime = true;
	bool bShowTotalWallTime = true;
	bool bShowTotalCPUDataSize = true;
	bool bShowTotalGPUDataSize = true;
	bool bShowTotalTasks = true;
	
	int32 TotalNumTasks = 0;
	double TotalTime = 0.0;
	double TotalWallTime = 0.0;
	double TotalCPUDataSizeInMB = 0.0;
	double TotalGPUDataSizeInMB = 0.0;

	FText GetTotalTimeLabel() const;
	FText GetTotalWallTimeLabel() const;
	FText GetTotalCPUDataSizeInMBLabel() const;
	FText GetTotalGPUDataSizeInMBLabel() const;
	FText GetTotalNumTasksLabel() const;

	TSharedRef<SWidget> MakeOptionsWidget();
	TSharedRef<SWidget> OnOptionsMenu();

	void Reset();
};

typedef TSharedPtr<FPCGProfilingListViewItem> PCGProfilingListViewItemPtr;

class SPCGProfilingListViewItemRow : public SMultiColumnTableRow<PCGProfilingListViewItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SPCGProfilingListViewItemRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const PCGProfilingListViewItemPtr& Item);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

protected:
	PCGProfilingListViewItemPtr InternalItem;
};

class SPCGEditorGraphProfilingView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorGraphProfilingView) { }
	SLATE_END_ARGS()

	~SPCGEditorGraphProfilingView();

	void Construct(const FArguments& InArgs, TSharedPtr<FPCGEditor> InPCGEditor);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	TSharedRef<SHeaderRow> CreateHeaderRowWidget();

	int32 GetSubgraphExpandDepth() const { return ExpandSubgraphDepth; }
	void OnSubgraphExpandDepthChanged(int32 NewValue);

	void OnDebugStackChanged(const FPCGStack& InPCGStack);

	// Callbacks
	void RequestRefresh();
	FReply Refresh();
	TSharedRef<ITableRow> OnGenerateRow(PCGProfilingListViewItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable) const;
	void OnItemDoubleClicked(PCGProfilingListViewItemPtr Item);
	void OnSortColumnHeader(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode);
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	FReply OnListViewKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) const;
	void CopySelectionToClipboard() const;
	bool CanCopySelectionToClipboard() const;

	/** Called when user changes the text they are searching for */
	void OnSearchTextChanged(const FText& InText);

	/** Called when user changes commits text to the search box */
	void OnSearchTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

	/** Returns the current PCG execution source */
	IPCGGraphExecutionSource* GetExecutionSource() const;

	/** Pointer back to the PCG editor that owns us */
	TWeakPtr<FPCGEditor> PCGEditorPtr;

	/** Cached PCGGraph being viewed */
	UPCGEditorGraph* PCGEditorGraph = nullptr;

	/** Current stack being viewed */
	FPCGStack PCGStack;

	TSharedPtr<SHeaderRow> ListViewHeader;
	TSharedPtr<SListView<PCGProfilingListViewItemPtr>> ListView;
	TArray<PCGProfilingListViewItemPtr> ListViewItems;
	TSharedPtr<FUICommandList> ListViewCommands;

	// To allow sorting
	FName SortingColumn = NAME_None;
	EColumnSortMode::Type SortMode = EColumnSortMode::Type::None;

	FPCGProfilingTotalData ProfilingTotalData;

	bool bNeedsRefresh = false;

	/** Currently depth of entry exposition - if 0, only the nodes in the currently debugged graph will be shown. */
	int32 ExpandSubgraphDepth = 0;

	/** The string to search for */
	FString SearchValue;
};
