// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGEditorGraphTitleBar.h"

#include "PCGEditor.h"
#include "PCGEditorGraph.h"

#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SPCGGraphEditorTitleBar"

FText SPCGGraphEditorTitleBar::GetTitleForOneCrumb(TWeakObjectPtr<const UPCGEditorGraph> InWeakEditorGraph)
{
	if (const UPCGEditorGraph* EditorGraph = InWeakEditorGraph.Get())
	{
		return EditorGraph->GetDisplayName();
	}

	return LOCTEXT("PCGGraphEditorTitleBarInvalidCrumb", "Invalid");
}

void SPCGGraphEditorTitleBar::Construct(const FArguments& InArgs)
{
	EditorGraph = InArgs._Graph;
	check(EditorGraph.IsValid());
	check(InArgs._HistoryNavigationWidget.IsValid());

	// Set-up shared breadcrumb defaults
	FMargin BreadcrumbTrailPadding = FMargin(4.f, 2.f);
	const FSlateBrush* BreadcrumbButtonImage = FAppStyle::GetBrush("BreadcrumbTrail.Delimiter");

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
		.HAlign(HAlign_Fill)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("EventGraphTitleBar")))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				InArgs._HistoryNavigationWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]
			// Title text/icon
			+ SHorizontalBox::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SAssignNew(BreadcrumbTrailScrollBox, SScrollBox)
					.Orientation(Orient_Horizontal)
					.ScrollBarVisibility(EVisibility::Collapsed)
					+ SScrollBox::Slot()
					.Padding(0.f)
					.VAlign(VAlign_Center)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SAssignNew(BreadcrumbTrail, SBreadcrumbTrail<SPCGGraphEditorTitleBar::FBreadcrumbItem>)
							.ButtonStyle(FAppStyle::Get(), "GraphBreadcrumbButton")
							.TextStyle(FAppStyle::Get(), "GraphBreadcrumbButtonText")
							.ButtonContentPadding(BreadcrumbTrailPadding)
							.DelimiterImage(BreadcrumbButtonImage)
							.PersistentBreadcrumbs(true)
							.OnCrumbClicked(this, &SPCGGraphEditorTitleBar::OnBreadcrumbClicked)
						]
					]
				]
			]
		]
	];

	RebuildBreadcrumbTrail();
	BreadcrumbTrailScrollBox->ScrollToEnd();
}

void SPCGGraphEditorTitleBar::OnBreadcrumbClicked(const FBreadcrumbItem& Crumb)
{
	const UPCGEditorGraph* CrumbGraph = Crumb.EditorGraph.Get();
	if (!CrumbGraph)
	{
		return;
	}
	
	TSharedPtr<FPCGEditor> Editor = CrumbGraph->GetEditor().Pin();
	if (!Editor || !CrumbGraph->GetPCGGraph())
	{
		return;
	}

	Editor->OpenDocument(CrumbGraph->GetPCGGraph(), FDocumentTracker::EOpenDocumentCause::OpenNewDocument);
}

void SPCGGraphEditorTitleBar::RebuildBreadcrumbTrail()
{
	if (!EditorGraph.IsValid() || !EditorGraph->GetPCGGraph())
	{
		BreadcrumbTrail->ClearCrumbs(false);
		return;
	}

	TArray<TWeakObjectPtr<const UPCGEditorGraph>> Stack;
	Stack.Push(EditorGraph);

	if (UPCGGraph* ParentGraph = EditorGraph->GetPCGGraph()->GetEmbeddedParentGraph(); ParentGraph && ParentGraph->GetEditorGraph())
	{
		Stack.Push(ParentGraph->GetEditorGraph());
	}

	BreadcrumbTrail->ClearCrumbs(false);

	while (Stack.Num() > 0)
	{
		TWeakObjectPtr<const UPCGEditorGraph> WeakGraph(Stack.Pop());
		TAttribute<FText> CrumbText = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&SPCGGraphEditorTitleBar::GetTitleForOneCrumb, WeakGraph));
		BreadcrumbTrail->PushCrumb(CrumbText, { WeakGraph });
	}
}

#undef LOCTEXT_NAMESPACE
