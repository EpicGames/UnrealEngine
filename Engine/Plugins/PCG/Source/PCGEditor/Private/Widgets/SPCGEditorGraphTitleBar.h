// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/SCompoundWidget.h"

class FPCGEditor;
class UPCGEditorGraph;

class SPCGGraphEditorTitleBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGGraphEditorTitleBar)
		: _Graph(nullptr)
		, _Editor()
	{
	}
		SLATE_ARGUMENT(UPCGEditorGraph*, Graph)
		SLATE_ARGUMENT(TWeakPtr<FPCGEditor>, Editor)
		SLATE_ARGUMENT(TSharedPtr<SWidget>, HistoryNavigationWidget)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:

	struct FBreadcrumbItem
	{
		TWeakObjectPtr<const UPCGEditorGraph> EditorGraph;
	};

	void RebuildBreadcrumbTrail();
	void OnBreadcrumbClicked(const FBreadcrumbItem& Crumb);

	static FText GetTitleForOneCrumb(TWeakObjectPtr<const UPCGEditorGraph> InWeakEditorGraph);

	/** Edited graph */
	TWeakObjectPtr<UPCGEditorGraph> EditorGraph = nullptr;

	TSharedPtr<SScrollBox> BreadcrumbTrailScrollBox;

	/** Breadcrumb trail widget */
	TSharedPtr<SBreadcrumbTrail<FBreadcrumbItem>> BreadcrumbTrail;
};
