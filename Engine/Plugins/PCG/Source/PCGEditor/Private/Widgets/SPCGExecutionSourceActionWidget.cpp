// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SPCGExecutionSourceActionWidget.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGEditorStyle.h"
#include "Subsystems/IPCGBaseSubsystem.h"

#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "LevelEditorSubsystem.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PCGExecutionSourceActionWidget"

namespace PCGExecutionSourceActionWidget
{
	void TriggerSelectionUpdate(IPCGGraphExecutionSource* ExecutionSource, FPCGTaskId TaskId)
	{
		if (!GEditor || !ExecutionSource || TaskId == InvalidPCGTaskId)
		{
			return;
		}

		if (IPCGBaseSubsystem* PCGSubsystem = ExecutionSource->GetExecutionState().GetSubsystem())
		{
			FPCGScheduleGenericParams Params(
				/*InOperation=*/ [](FPCGContext*) -> bool
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(PCGExecutionSourceActionWidget::TriggerSelectionUpdateTask);
					if (GEditor && GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
					{
						// Since we might have been moving things around (component wise) we need to force a selection set change, 
						// Which in turn will trigger a component visualizer update.
						if (UTypedElementSelectionSet* SelectionSet = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>()->GetSelectionSet())
						{
							SelectionSet->OnChanged().Broadcast(SelectionSet);
						}
					}

					return true;
				},
				/*InExecutionSource= */ ExecutionSource,
				/*InExecutionDependencies=*/ { TaskId },
				/*InDataDependencies=*/ {},
				/*bSupportBasePointDataInput=*/ true);

			PCGSubsystem->ScheduleGeneric(Params);
		}
	}
}

void SPCGExecutionSourceActionWidget::Construct(const FArguments& InArgs)
{
	ExecutionSources = InArgs._ExecutionSources.Get();

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		.VAlign(VAlign_Fill)
		[
			SNew(SButton)
			.OnClicked(this, &SPCGExecutionSourceActionWidget::OnGenerateClicked)
			.ToolTipText(FText::FromString("Generates graph data. \nCtrl + Click flushes the cache and force generates."))
			.Visibility(this, &SPCGExecutionSourceActionWidget::GetGenerateButtonVisibility)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 6.0f, 0.0f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16, 16))
					.Image_Lambda([]() { return FSlateApplication::Get().GetModifierKeys().IsControlDown() ? FPCGEditorStyle::Get().GetBrush("PCG.Command.ForceRegenClearCache") : FPCGEditorStyle::Get().GetBrush("PCG.Command.ForceRegen"); })
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text_Lambda([]() { return FSlateApplication::Get().GetModifierKeys().IsControlDown() ? LOCTEXT("ForceRegenerateButton", "Force Generate") : LOCTEXT("GenerateButton", "Generate"); })
				]
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		.VAlign(VAlign_Fill)
		[
			SNew(SButton)
			.OnClicked(this, &SPCGExecutionSourceActionWidget::OnCancelClicked)
			.Visibility(this, &SPCGExecutionSourceActionWidget::GetCancelButtonVisibility)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 6.0f, 0.0f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16, 16))
					.Image(FPCGEditorStyle::Get().GetBrush("PCG.Command.StopRegen"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("CancelButton", "Cancel"))
				]
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Fill)
		.Padding(2.0f, 0.0f)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SPCGExecutionSourceActionWidget::OnCleanupClicked)
			.ToolTipText(FText::FromString("Cleans up graph data. \nCtrl + Click purges all generated artifacts tagged as created by PCG."))
			.Visibility(this, &SPCGExecutionSourceActionWidget::GetCleanupButtonVisibility)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("CleanupButton", "Cleanup"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Fill)
		.Padding(2.0f, 0.0f)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SPCGExecutionSourceActionWidget::OnRefreshClicked)
			.Visibility(this, &SPCGExecutionSourceActionWidget::GetRefreshButtonVisibility)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("RefreshButton", "Refresh"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Fill)
		.Padding(2.0f, 0.0f)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SPCGExecutionSourceActionWidget::OnClearPCGLinkClicked)
			.Visibility(this, &SPCGExecutionSourceActionWidget::GetClearPCGLinkButtonVisibility)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("ClearPCGLinkButton", "Clear PCG Link"))
			]
		]
	];
}

EVisibility SPCGExecutionSourceActionWidget::GetGenerateButtonVisibility() const
{
	for (const TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSource : ExecutionSources)
	{
		// If execution source is runtime generated then generate/cleanup is managed by the scheduler.
		if (ExecutionSource.IsValid() && !ExecutionSource->GetExecutionState().IsGenerating() && !ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility SPCGExecutionSourceActionWidget::GetCancelButtonVisibility() const
{
	for (const TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSource : ExecutionSources)
	{
		// If execution source is runtime generated then generate/cleanup is managed by the scheduler.
		if (ExecutionSource.IsValid() && ExecutionSource->GetExecutionState().IsGenerating() && !ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility SPCGExecutionSourceActionWidget::GetCleanupButtonVisibility() const
{
	for (const TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSource : ExecutionSources)
	{
		// If execution source is runtime generated then generate/cleanup is managed by the scheduler.
		if (ExecutionSource.IsValid() && !ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility SPCGExecutionSourceActionWidget::GetRefreshButtonVisibility() const
{
	for (const TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSource : ExecutionSources)
	{
		if (ExecutionSource.IsValid() && ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility SPCGExecutionSourceActionWidget::GetClearPCGLinkButtonVisibility() const
{
	for (const TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSource : ExecutionSources)
	{
		// @todo_pcg: support this in IPCGGraphExecutionSource
		if (UPCGComponent* Component = Cast<UPCGComponent>(ExecutionSource.Get()))
		{
			if (IsValid(Component) && !Component->IsManagedByRuntimeGenSystem())
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}


FReply SPCGExecutionSourceActionWidget::OnGenerateClicked()
{
	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	const bool bIsControlDown = ModifierKeys.IsControlDown();

	bool bFlushedCache = false;

	for (TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSourcePtr : ExecutionSources)
	{
		IPCGGraphExecutionSource* ExecutionSource = ExecutionSourcePtr.Get();

		if (ExecutionSource && !ExecutionSource->GetExecutionState().IsGenerating() && !ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			// Flush cache if needed
			if (bIsControlDown && !bFlushedCache)
			{
				if (IPCGBaseSubsystem* PCGSubsystem = ExecutionSource->GetExecutionState().GetSubsystem())
				{
					PCGSubsystem->FlushCache();
					bFlushedCache = true;
				}
			}

			IPCGGraphExecutionState::FGenerateParams Params;
			Params.bEvenIfAlreadyGenerated = bIsControlDown;
			FPCGTaskId GenerationTask = ExecutionSource->GetExecutionState().Generate(Params);
						
			PCGExecutionSourceActionWidget::TriggerSelectionUpdate(ExecutionSource, GenerationTask);
		}
	}

	return FReply::Handled();
}

FReply SPCGExecutionSourceActionWidget::OnCancelClicked()
{
	for (TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSource : ExecutionSources)
	{
		if (ExecutionSource.IsValid() && ExecutionSource->GetExecutionState().IsGenerating() && !ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			ExecutionSource->GetExecutionState().Cancel();
		}
	}

	return FReply::Handled();
}

FReply SPCGExecutionSourceActionWidget::OnRefreshClicked()
{
	for (TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSource : ExecutionSources)
	{
		// @todo_pcg: support this in IPCGGraphExecutionSource
		if (UPCGComponent* Component = Cast<UPCGComponent>(ExecutionSource.Get()))
		{
			if (IsValid(Component) && Component->IsManagedByRuntimeGenSystem())
			{
				// Trigger the deepest refresh - re-initialize the PAs.
				Component->Refresh(EPCGChangeType::Structural | EPCGChangeType::GenerationGrid, /*bCancelExistingRefresh=*/true);
			}
		}
	}

	return FReply::Handled();
}

FReply SPCGExecutionSourceActionWidget::OnClearPCGLinkClicked()
{
	for (TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSource : ExecutionSources)
	{
		// @todo_pcg: support this in IPCGGraphExecutionSource
		if (UPCGComponent* Component = Cast<UPCGComponent>(ExecutionSource.Get()))
		{
			if (IsValid(Component) && !Component->IsManagedByRuntimeGenSystem())
			{
				Component->ClearPCGLink();
			}
		}
	}

	return FReply::Handled();
}

FReply SPCGExecutionSourceActionWidget::OnCleanupClicked()
{
	for (TWeakInterfacePtr<IPCGGraphExecutionSource>& ExecutionSource : ExecutionSources)
	{
		if (ExecutionSource.IsValid() && !ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();

			FPCGTaskId CleanupTaskId = InvalidPCGTaskId;

			UPCGComponent* Component = Cast<UPCGComponent>(ExecutionSource.Get());
			if (ModifierKeys.IsControlDown() && Component)
			{
				Component->CleanupLocalDeleteAllGeneratedObjects({});
				CleanupTaskId = Component->GetCleanupTaskId();
			}
			else
			{
				CleanupTaskId = ExecutionSource->GetExecutionState().Cleanup();
			}

			PCGExecutionSourceActionWidget::TriggerSelectionUpdate(ExecutionSource.Get(), CleanupTaskId);
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
