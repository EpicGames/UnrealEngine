// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SPCGMetadataAttributeGenericViewer.h"

#include "PCGData.h"
#include "Metadata/PCGMetadataAttribute.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailsViewArgs.h"
#include "InstancedPropertyBagStructureDataProvider.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"

TSharedRef<IDetailCustomization> FPCGMetadataAttributeGenericViewerDetailCustomization::MakeInstance(TSharedPtr<SPCGMetadataAttributeGenericViewer> Viewer)
{
	return MakeShareable(new FPCGMetadataAttributeGenericViewerDetailCustomization(Viewer));
}

void FPCGMetadataAttributeGenericViewerDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const TArray<TWeakObjectPtr<UObject>>& Objects = DetailBuilder.GetSelectedObjects();
	if(Objects.Num() != 1 || !Objects[0].IsValid())
	{
		return;
	}
	
	if (!WeakViewer.IsValid())
	{
		return;
	}
	
	FInstancedPropertyBag& Struct = WeakViewer.Pin()->TempStruct;
	const UPropertyBag* Bag = Struct.GetPropertyBagStruct();
	if (!Bag || Bag->GetPropertyDescs().IsEmpty())
	{
		return;
	}
	
	const FName PropertyName = Bag->GetPropertyDescs()[0].Name;

	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(TEXT("NoCategory"));
	CategoryBuilder.AddExternalStructureProperty(MakeShared<FInstancePropertyBagStructureDataProvider>(Struct), PropertyName);
}

FPCGMetadataAttributeGenericViewerDetailCustomization::FPCGMetadataAttributeGenericViewerDetailCustomization(TSharedPtr<SPCGMetadataAttributeGenericViewer> Viewer)
	: WeakViewer(Viewer)
{
}
	
void SPCGMetadataAttributeGenericViewer::Construct(const FArguments& InArgs)
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bShowHeaders = false;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	
	DetailsView->RegisterInstancedCustomPropertyLayout(UPCGData::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FPCGMetadataAttributeGenericViewerDetailCustomization::MakeInstance, StaticCastSharedPtr<SPCGMetadataAttributeGenericViewer>(AsShared().ToSharedPtr())));
	
	DetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateLambda([](const FPropertyAndParent& In)
	{
		return true;
	}));
	
	DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda([](const FPropertyAndParent& In)
	{
		// Make sure to remove all the properties that are coming from the Data (aka, has an object attached to it).
		return In.Objects.IsEmpty();
	}));

	ChildSlot
	[
		DetailsView.ToSharedRef()
	];
}

void SPCGMetadataAttributeGenericViewer::Refresh()
{
	if (DetailsView)
	{
		if (Data.IsValid() && Attr)
		{
			TempStruct = Attr->BuildStructForDebug(EntryKey);
		}
		else
		{
			TempStruct.Reset();
		}
		
		DetailsView->SetObject(const_cast<UPCGData*>(Data.Get()), /*bForceRefresh=*/true);
		
		// Make sure the root is expanded by default, but not necessarily all its children.
		DetailsView->SetRootExpansionStates(/*bExpand=*/true, /*bRecurse=*/false);
	}
}

void SPCGMetadataAttributeGenericViewer::Setup(TWeakObjectPtr<const UPCGData> InData, const FPCGMetadataAttributeBase* InAttr, PCGMetadataEntryKey InEntryKey)
{
	Data = InData;
	Attr = InAttr;
	EntryKey = InEntryKey;
	Refresh();
}
