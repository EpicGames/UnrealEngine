// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"


#include "Components/DetailsView.h"
#include "Metadata/PCGMetadataCommon.h"
#include "StructUtils/PropertyBag.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class UPCGData;
class FPCGMetadataAttributeBase;
class FPCGEditor;
class IDetailsView;
class SPCGMetadataAttributeGenericViewer;

/** Header customization for when multiple objects are displayed in the details panel. */
class FPCGMetadataAttributeGenericViewerDetailCustomization final : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance(TSharedPtr<SPCGMetadataAttributeGenericViewer> Viewer);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	FPCGMetadataAttributeGenericViewerDetailCustomization(TSharedPtr<SPCGMetadataAttributeGenericViewer> Viewer);
	TWeakPtr<SPCGMetadataAttributeGenericViewer> WeakViewer; 
};

/** Thin wrapper on top of a SDetailsView to add some PCG-specific logic around locking and improve layout */
class SPCGMetadataAttributeGenericViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGMetadataAttributeGenericViewer)
		{}	
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	/** Sets the editor associated to this details view, used for interaction from the details view to the editor */
	void SetEditor(TWeakPtr<FPCGEditor> InEditorPtr) { EditorPtr = InEditorPtr; }

	/** Returns the details view under this widget */
	TSharedPtr<IDetailsView> GetDetailsView() const { return DetailsView; }
	
	void Refresh();

	/** Controls lock from the editor */
	void Setup(TWeakObjectPtr<const UPCGData> InData, const FPCGMetadataAttributeBase* InAttr, PCGMetadataEntryKey InEntryKey);

protected:
	friend FPCGMetadataAttributeGenericViewerDetailCustomization;
	TSharedPtr<IDetailsView> DetailsView;

	FInstancedPropertyBag TempStruct;
	
	TWeakObjectPtr<const UPCGData> Data;
	const FPCGMetadataAttributeBase* Attr;
	PCGMetadataEntryKey EntryKey;
	TWeakPtr<FPCGEditor> EditorPtr = nullptr;
};