// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPCGShaderTextSearchWidget.h"

#include "PCGEditorStyle.h"

#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SBoxPanel.h"

#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "PCGShaderTextSearchWidget"

SPCGShaderTextSearchWidget::SPCGShaderTextSearchWidget()
{
}

SPCGShaderTextSearchWidget::~SPCGShaderTextSearchWidget()
{
}

void SPCGShaderTextSearchWidget::Construct(const FArguments& InArgs)
{
	const FSearchBoxStyle& SearchBoxStyle = FPCGEditorStyle::Get().GetWidgetStyle<FSearchBoxStyle>(TEXT("TextEditor.SearchBoxStyle"));
	
	SearchBox =
		SNew(SSearchBox)
			.HintText(NSLOCTEXT("SearchBox", "HelpHint", "Search For Text"))
			.Style(&SearchBoxStyle)
			.OnTextChanged(InArgs._OnTextChanged)
			.OnTextCommitted(InArgs._OnTextCommitted)
			.SearchResultData(InArgs._SearchResultData)
			.SelectAllTextWhenFocused(true)
			.DelayChangeNotificationsWhileTyping(true)
			.MinDesiredWidth(200)
			.OnSearch(InArgs._OnResultNavigationButtonClicked);
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(6.f)
		.AutoHeight()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SearchBox.ToSharedRef()
			]
		]
	];
}

void SPCGShaderTextSearchWidget::FocusSearchBox() const
{
	FSlateApplication::Get().SetKeyboardFocus(SearchBox, EFocusCause::SetDirectly);
}

void SPCGShaderTextSearchWidget::TriggerSearch(const FText& InNewSearchText) const
{
	FocusSearchBox();

	// Multiline search is not supported, sanitize the input to be single line text.
	FString SingleLineString = InNewSearchText.ToString();
	{
		SingleLineString.GetCharArray().RemoveAll([&](const TCHAR InChar) -> bool
		{
			if (InChar != 0)
			{
				const bool bIsCharAllowed = !FChar::IsLinebreak(InChar);
				return !bIsCharAllowed;
			}
			return false;
		});
	}
	
	FText SingleLineSearchText = FText::FromString(SingleLineString);
	
	// Clear the text to trigger a fresh search. Sometimes, the search text can be the same but starting from different place.
	SearchBox->SetText(FText::GetEmpty());
	
	if (InNewSearchText.IsEmpty())
	{
		SearchBox->SetText(LastSearchedText);
	}
	else
	{
		SearchBox->SetText(SingleLineSearchText);
	}

	SearchBox->SelectAllText();
}

void SPCGShaderTextSearchWidget::ClearSearchText()
{
	// Save the last searched text so that if the next search is triggered without any selected text, we can initialize the search
	// using it, standard behavior in regular text editors.
	LastSearchedText = SearchBox->GetText();
	SearchBox->SetText(FText::GetEmpty());
}

#undef LOCTEXT_NAMESPACE
