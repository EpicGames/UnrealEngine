// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Input/SSearchBox.h"

class SSearchBox;
class STextBlock;
struct FTextLocation;

/** Search text field. */
class SPCGShaderTextSearchWidget : public SCompoundWidget
{
public:
	SPCGShaderTextSearchWidget();
	~SPCGShaderTextSearchWidget() override;

	SLATE_BEGIN_ARGS(SPCGShaderTextSearchWidget) {};
		/** Invoked whenever the text changes */
		SLATE_EVENT(FOnTextChanged, OnTextChanged )

		/** Invoked whenever the text is committed (e.g. user presses enter) */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )

		/** This will add a next and previous button to your search box */
		SLATE_EVENT(SSearchBox::FOnSearch, OnResultNavigationButtonClicked )
	
		/** Optional search result data to be shown in the search bar. */
		SLATE_ATTRIBUTE(TOptional<SSearchBox::FSearchResultData>, SearchResultData )

		/** Callback delegate to have first chance handling of the OnKeyDown event */
		SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void FocusSearchBox() const;

	void TriggerSearch(const FText& InNewSearchText) const;
	
	void ClearSearchText();
	
private:
	TSharedPtr<SSearchBox> SearchBox;
	FText LastSearchedText;
};
