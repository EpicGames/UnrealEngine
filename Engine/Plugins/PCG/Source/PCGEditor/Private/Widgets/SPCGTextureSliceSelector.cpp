// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SPCGTextureSliceSelector.h"

#include "Data/PCGTexture2DArrayData.h"
#include "DataVisualizations/PCGVisualizationTexture2D.h"

#include "EditorViewportClient.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SPCGTextureSliceSelector"

void SPCGTextureSliceSelector::Construct(const FArguments& InArgs)
{
	WeakData = InArgs._WeakData;
	WeakMaterial = InArgs._WeakMaterial;
	ViewportClient = InArgs._ViewportClient;
	CurrentSlice = 0;

	if (const UPCGTexture2DArrayData* Data = WeakData.Get())
	{
		ResourceType = Data->GetResourceType();
		ArraySize = FMath::Max(1, static_cast<int32>(Data->GetArraySize()));
	}

	if (ArraySize <= 1)
	{
		SetVisibility(EVisibility::Collapsed);
		ChildSlot[SNullWidget::NullWidget];
		return;
	}

	const int32 MaxSlice = ArraySize - 1;

	ChildSlot
	.VAlign(VAlign_Center)
	.Padding(FMargin(2.0f, 0.0f))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.0f, 0.0f, 6.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SliceLabel", "Slice"))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.MinDesiredWidth(56.0f)
			[
				SNew(SSpinBox<int32>)
				.MinValue(0)
				.MaxValue(MaxSlice)
				.MinSliderValue(0)
				.MaxSliderValue(MaxSlice)
				.Value(CurrentSlice)
				.OnValueChanged(this, &SPCGTextureSliceSelector::OnSliceValueChanged)
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(FText::Format(LOCTEXT("SliceMaxFormat", "/ {0}"), FText::AsNumber(MaxSlice)))
		]
	];
}

void SPCGTextureSliceSelector::OnSliceValueChanged(int32 NewSlice)
{
	const int32 ClampedSlice = FMath::Clamp(NewSlice, 0, FMath::Max(0, ArraySize - 1));

	if (ClampedSlice == CurrentSlice)
	{
		return;
	}

	ApplySlice(ClampedSlice);
}

void SPCGTextureSliceSelector::ApplySlice(int32 NewSlice)
{
	UMaterialInstanceDynamic* Material = WeakMaterial.Get();
	if (!Material)
	{
		return;
	}

	if (ResourceType == EPCGTextureResourceType::TextureObject)
	{
		Material->SetScalarParameterValue(FName(TEXT("SliceIndex")), static_cast<float>(NewSlice));
	}
	else if (ResourceType == EPCGTextureResourceType::ExportedTexture)
	{
		const UPCGTexture2DArrayData* Data = WeakData.Get();
		if (!Data)
		{
			return;
		}

		FPCGVisualizationTexture2DParams VisParams;
		VisParams.TextureRHI = Data->GetRHI();
		VisParams.Resolution = Data->GetResolution();
		VisParams.SliceIndex = static_cast<uint16>(NewSlice);

		UPCGVisualizationTexture2D* NewWrapper = UPCGVisualizationTexture2D::Create(VisParams);
		CurrentWrapper.Reset(NewWrapper);

		Material->SetTextureParameterValue(FName(TEXT("DebugTexture")), NewWrapper);
	}

	CurrentSlice = NewSlice;

	if (ViewportClient)
	{
		ViewportClient->Invalidate();
	}
}

#undef LOCTEXT_NAMESPACE
