// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#include "UObject/StrongObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

class FEditorViewportClient;
class UMaterialInstanceDynamic;
class UPCGTexture2DArrayData;
class UPCGVisualizationTexture2D;

/** Toolbar widget for the PCG data viewport that lets the user pick which slice of a UPCGTexture2DArrayData is displayed. */
class SPCGTextureSliceSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGTextureSliceSelector)
			: _ViewportClient(nullptr)
		{}
		SLATE_ARGUMENT(TWeakObjectPtr<const UPCGTexture2DArrayData>, WeakData)
		SLATE_ARGUMENT(TWeakObjectPtr<UMaterialInstanceDynamic>, WeakMaterial)
		SLATE_ARGUMENT(FEditorViewportClient*, ViewportClient)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	void OnSliceValueChanged(int32 NewSlice);
	void ApplySlice(int32 NewSlice);

	TWeakObjectPtr<const UPCGTexture2DArrayData> WeakData;
	TWeakObjectPtr<UMaterialInstanceDynamic> WeakMaterial;
	FEditorViewportClient* ViewportClient = nullptr;
	EPCGTextureResourceType ResourceType = EPCGTextureResourceType::TextureObject;
	int32 ArraySize = 1;
	int32 CurrentSlice = 0;

	/** Holds the currently-displayed wrapper texture alive for the ExportedTexture path. Replaced each time the slice changes. */
	TStrongObjectPtr<UPCGVisualizationTexture2D> CurrentWrapper;
};
