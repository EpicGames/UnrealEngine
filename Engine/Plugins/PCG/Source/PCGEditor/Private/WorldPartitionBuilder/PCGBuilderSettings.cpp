// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGBuilderSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBuilderSettings)

UPCGBuilderSettings::UPCGBuilderSettings()
{
	// By default include this mode
	EditingModes.Add(EPCGEditorDirtyMode::LoadAsPreview);
}
