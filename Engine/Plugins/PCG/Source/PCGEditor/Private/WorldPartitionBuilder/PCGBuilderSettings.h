// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "UObject/Object.h"

#include "PCGBuilderSettings.generated.h"

class UPCGGraphInterface;
class UDataLayerAsset;

UCLASS(hidecategories = Object)
class UPCGBuilderSettings : public UObject
{
	GENERATED_BODY()

public:
	UPCGBuilderSettings();

	/** Graphs that will be generated in the specified order. When left empty, all Graphs will get generated in the order they are discovered. */
	UPROPERTY(EditAnywhere, Category = Build)
	TArray<TSoftObjectPtr<UPCGGraphInterface>> Graphs;

	/** Include components which have the following editing mode. */
	UPROPERTY(EditAnywhere, Category = Build)
	TArray<EPCGEditorDirtyMode> EditingModes;

	/** Filter generated components by owner actor name. */
	UPROPERTY(EditAnywhere, Category = Filter)
	TArray<FString>  FilterByActorNames;
		
	/** Will load editor only data layers without having to specify them as included */
	UPROPERTY(EditAnywhere, Category = DataLayers)
	bool bLoadEditorOnlyDataLayers = true;

	/** Will load runtime data layers that are default activated without having to specify them as included */
	UPROPERTY(EditAnywhere, Category = DataLayers)
	bool bLoadActivatedRuntimeDataLayers = true;

	/** Data layers that need to be loaded when running builder. */
	UPROPERTY(EditAnywhere, Category = DataLayers)
	TArray<TObjectPtr<UDataLayerAsset>> IncludedDataLayers;

	/** Data layers that need to be unloaded when running builder. */
	UPROPERTY(EditAnywhere, Category = DataLayers)
	TArray<TObjectPtr<UDataLayerAsset>> ExcludedDataLayers;

	/** Run builder iteratively loading cell regions instead of the whole map at once. Useful for large maps. */
	UPROPERTY(EditAnywhere, Category = Advanced)
	bool bIterativeCellLoading = false;

	/** Cell size when using iterative cell loading. */
	UPROPERTY(EditAnywhere, Category = Advanced, meta = (EditCondition=bIterativeCellLoading, EditConditionHides, ClampMin=12800, UIMin=12800))
	int32 IterativeCellSize = 25600;

	/** Submit dirty files even if errors occurred during generation. */
	UPROPERTY(EditAnywhere, Category = Advanced)
	bool bIgnoreGenerationErrors = false;

	/** Call generate on each component and wait until completion and any async processes before generating the next. */
	UPROPERTY(EditAnywhere, Category = Advanced, meta = (DisplayName = "One Component at a Time"))
	bool bOneComponentAtATime = false;

	/** Experimental feature that will try and revert PCG Component Actors which haven't changed from their previous generation. */
	UPROPERTY(EditAnywhere, Category = Experimental)
	bool bRevertUnchangedActors = false;

	/** Outputs that can change on every generation but do not have an impact on final result should be listed here so they can be ignored when comparing results for revert. */
	UPROPERTY(EditAnywhere, Category = Experimental, meta= (EditCondition=bRevertUnchangedActors, EditConditionHides))
	TArray<FString> OutputsToIgnoreForRevert;
};