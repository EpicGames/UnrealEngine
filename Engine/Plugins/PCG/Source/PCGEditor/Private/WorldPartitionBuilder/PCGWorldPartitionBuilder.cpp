// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionBuilder/PCGWorldPartitionBuilder.h"

#include "PCGBuilderSettings.h"
#include "PCGComponent.h"
#include "PCGEditorSettings.h"
#include "PCGGraph.h"
#include "PCGManagedResource.h"
#include "PCGSubsystem.h"
#include "SPCGBuilderDialog.h"

#include "Builder/PCGBuilderVolume.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGHelpers.h"

#include "AssetCompilingManager.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "FileHelpers.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"

#include "Algo/Copy.h"
#include "Algo/Transform.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/OutputDevice.h"
#include "Modules/ModuleManager.h"
#include "Serialization/ArchiveCrc32.h"
#include "UObject/Linker.h"
#include "UObject/UObjectIterator.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterShape.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGWorldPartitionBuilder)

DEFINE_LOG_CATEGORY_STATIC(LogPCGWorldPartitionBuilder, Verbose, All);

#define LOCTEXT_NAMESPACE "PCGWorldPartitionBuilder"

struct FPCGWorldPartitionBuilder
{
	/** Generate the given components. Optionally generate only one component at a time with a wait on async processes after each. Optionally apply given filter to select components. */
	static bool GenerateComponents(TArray<TWeakObjectPtr<UPCGComponent>>& Components, UWorld* InWorld, const FPCGWorldPartitionBuilderArgs& InArgs, const TMap<UPCGComponent*, UPCGComponent*>& InTaskDependencies, TSet<FString>& InOutDeletedActorPackages, bool& bOutGenerationErrors);

	/** Generate a component. Applies correct editing mode if necessary. */
	static FPCGTaskId GenerateComponent(UPCGComponent* InComponent, UWorld* InWorld, const FPCGWorldPartitionBuilderArgs& InArgs, FPCGTaskId InTaskDependency);

	/** Groups components into buckets to generate in order */
	static TArray<TArray<TWeakObjectPtr<UPCGComponent>>> GroupComponentsToGenerate(const TArray<TWeakObjectPtr<UPCGComponent>>& InComponentsToGenerate, const FPCGWorldPartitionBuilderArgs& InArgs);

	/** Waits for background activity to be quiet. */
	static void WaitForAllAsyncEditorProcesses(UWorld* InWorld);

	/** Runs builder on current editor world. */
	static bool Build(UWorld* InWorld, const TArray<FString>& Args);

	/** Load Actors this Partition Actor depends on : Original Component Actor and Generated Managed Actors */
	static void LoadPartitionedActorDependencies(UWorld* InWorld, TSet<FWorldPartitionReference>& OutLoadedActors);

	/** Creates missing Partitioned Actors with the original components */
	static bool CreatePartitionedActors(UWorld* InWorld, const FPCGWorldPartitionBuilderArgs& InArgs, TMap<TSoftObjectPtr<UPCGComponent>, FWorldPartitionReference>& InOutPartitionedComponents);

	/** Update mappings to loaded Partitioned Actors */
	static bool UpdatePartitionedActors(UWorld* InWorld, const TMap<TSoftObjectPtr<UPCGComponent>, FWorldPartitionReference>& InPartitionedComponents, TSet<FString>& InOutDeletedActorPackages);

	/** Grab all original PCG components in world, applying given filter. */
	static void CollectComponentsToGenerate(UWorld* InWorld, const FPCGWorldPartitionBuilderArgs& InArgs, TFunctionRef<bool(const UPCGComponent*)> ComponentFilter, TArray<TWeakObjectPtr<UPCGComponent>>& OutComponents);

	/** Grab all sub grid PCG components to generate based on the current collected list */
	static void CollectSubGridComponentsToGenerate(UWorld* InWorld, const FPCGWorldPartitionBuilderArgs& InArgs, const TArray<TWeakObjectPtr<UPCGComponent>>& InComponentsToGenerate, TArray<UPCGComponent*>& OutAdditionalComponentsToGenerate, TMap<UPCGComponent*, UPCGComponent*>& OutDependencies);

	/** Reset loaded package dirty flags */
	static void ResetPackageDirtyFlags();

	/** Extract an Actor Object Path from a PCG Component Soft Object Ptr */
	static FSoftObjectPath GetActorPathFromComponentPtr(TSoftObjectPtr<UPCGComponent> InComponentPtr);

	/** Get the list of Dirty packages */
	static TArray<UPackage*> GetDirtyPackages(const UWorld* InWorld, const TSet<FString>& AllowedAssetPackages);

	/** Set if Partition Actor creation should be disabled for world */
	static void SetDisablePartitionActorCreationForWorld(UWorld* InWorld, bool bDisable);

	/** Expand Loaded Bounds if needed */
	static TUniquePtr<IWorldPartitionActorLoaderInterface::ILoaderAdapter> ExpandLoadedBoundsIfNeeded(UWorld* InWorld, const FBox& InCurrentBounds, const FBox& InNewBounds, const FString& InLoaderName);
		
	/** Get Grid Size information for a given graph and cache it for future calls */
	static void GetGenerationGridSizes(const UPCGGraph* InGraph, const APCGWorldActor* InWorldActor, PCGHiGenGrid::FSizeArray& OutGridSizes, bool& bOutHasUnbounded);

	/** Returns when GraphExecutor is done running */
	static void WaitOnGraphExecutor(UWorld* InWorld);

	/** Allows to temporarly disable the clearing of results */
	static void SetDisableClearResults(UPCGSubsystem* InSubsystem, bool bInDisableClearResults);

	static void GetRevertableActorsFromComponents(TArray<TWeakObjectPtr<UPCGComponent>> InComponentsToGenerate, const FPCGWorldPartitionBuilderArgs& InArgs, TMap<TSoftObjectPtr<AActor>, FPCGCrc>& OutRevertableActors);
	static void GetRevertablePartitionActors(UWorld* InWorld, const FPCGWorldPartitionBuilderArgs& InArgs, TMap<TSoftObjectPtr<AActor>, FPCGCrc>& OutRevertableActors);
	static void GetRevertableActorsFromDirtyPackages(TArray<UPackage*> InDirtyPackages, const FPCGWorldPartitionBuilderArgs& InArgs, TMap<TSoftObjectPtr<AActor>, FPCGCrc>& OutRevertableActors);
	static void RevertUnchangedActors(TArray<UPackage*>& InOutDirtyPackages, const FPCGWorldPartitionBuilderArgs& InArgs, const TMap<TSoftObjectPtr<AActor>, FPCGCrc>& InRevertableActors, TSet<TSoftObjectPtr<AActor>>& InOutRevertedActors);
	static bool IsActorRevertable(AActor* InActor, const FPCGWorldPartitionBuilderArgs& InArgs, FPCGCrc& OutCrc);
	static void LogRevertedActors(const TSet<TSoftObjectPtr<AActor>>& InRevertedActors);

	static void GetModifiedAndNewActorsFromDirtyPackages(const TArray<UPackage*>& InDirtyPackages, TArray<AActor*>& OutModifiedActors, TArray<AActor*>& OutNewActors);
	static void LogModifiedAndNewActorsFromDirtyPackages(const TArray<UPackage*>& InDirtyPackages);

	static void SetChainedDispatchToLocalComponents(UWorld* InWorld, bool bInChainedDispatch);

	static void UpdateEditingMode(UPCGComponent* InComponent, bool bForce = false);

	static FBox GetPCGBuilderVolumeBounds(UWorld* InWorld);
private:
	static void CollectSubGridComponentsToGenerateInternal(UPCGSubsystem* InPCGSubsystem, int32 InGridSizeIndex, const PCGHiGenGrid::FSizeArray& InGridSizes, UPCGComponent* InOriginalComponent, APCGPartitionActor* InPartitionActor, TArray<UPCGComponent*>& OutAdditionalComponentsToGenerate, TMap<UPCGComponent*, UPCGComponent*>& OutDependencies);

	/** Cached information about Partitioned Component Grid sizes */
	struct FGridSizeInfo
	{
		bool bHasUnbounded = false;
		PCGHiGenGrid::FSizeArray GridSizes;
	};

	static TMap<TSoftObjectPtr<const UPCGGraph>, FGridSizeInfo> GraphToGridSizeInfo;
};

TMap<TSoftObjectPtr<const UPCGGraph>, FPCGWorldPartitionBuilder::FGridSizeInfo> FPCGWorldPartitionBuilder::GraphToGridSizeInfo;

namespace PCGWorldPartitionBuilder
{
	static FAutoConsoleCommand CommandBuildComponents(
		TEXT("pcg.BuildComponents"),
		TEXT("Runs PCG world builder on PCG components in current world. Arguments (multiple values separated with ';'), Flags can optionally have a specified '=true or false', true if none provided:\n"
			"\t[-PCGBuilderSettings=/Game/Path/BuilderSettingsAssetName]\n"
			"\t[-IncludeGraphNames=PCG_GraphA;PCG_GraphB] (default: all graphs)\n"
			"\t[-GenerateComponentEditingModeLoadAsPreview] (default: true)\n"
			"\t[-GenerateComponentEditingModeNormal] (default: false)\n"
			"\t[-GenerateComponentEditingModePreview] (default: false)\n"
			"\t[-IgnoreGenerationErrors] (default: false)\n"
			"\t[-IncludeActorIDs=MyActor1_UAID1234678;MyActor2_UAID1234678] (default: all actors)\n"
			"\t[-OneComponentAtATime] (default: false)\n"
			"\t[-IterativeCellLoading] (default: false)\n"
			"\t[-IterativeCellSize=25600] (default: 25600)\n"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args) { FPCGWorldPartitionBuilder::Build(GEditor ? GEditor->GetEditorWorldContext().World() : nullptr, Args); }));
};

/** Scoped object class that tracks if any package was dirtied during its lifetime */
class FPCGDetectDirtyPackageInScope
{
public:
	FPCGDetectDirtyPackageInScope()
	{
		UPackage::PackageDirtyStateChangedEvent.AddRaw(this, &FPCGDetectDirtyPackageInScope::HandleDirtyPackage);
	}

	~FPCGDetectDirtyPackageInScope()
	{
		UPackage::PackageDirtyStateChangedEvent.RemoveAll(this);
	}

	bool AnyDirtyPackage() { return bAnyDirtyPackage; }

private:
	void HandleDirtyPackage(UPackage* InPackage)
	{
		bAnyDirtyPackage = true;
	}

	bool bAnyDirtyPackage = false;
};

/** Scoped object class that tracks deleted actors during its lifetime */
class FPCGDetectDeletedActorInScope
{
public:
	FPCGDetectDeletedActorInScope(UWorld* InWorld, TSet<FString>& InDeletedActorPackages)
		: World(InWorld), DeletedActorPackages(InDeletedActorPackages)
	{
		GEngine->OnLevelActorDeleted().AddRaw(this, &FPCGDetectDeletedActorInScope::HandleDeletedActor);
	}

	~FPCGDetectDeletedActorInScope()
	{
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}

private:
	void HandleDeletedActor(AActor* InActor)
	{
		if (InActor && InActor->IsPackageExternal())
		{
			if (UPackage* ActorPackage = InActor->GetPackage())
			{
				UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Actor '%ls' deleted, package '%ls' added to delete list.",
					*InActor->GetName(),
					*ActorPackage->GetName());

				// Notify World Partition of Deleted Package right away in case Actor gets Garbage Collected before saving
				World->GetWorldPartition()->OnPackageDeleted(ActorPackage);
				DeletedActorPackages.Add(ActorPackage->GetName());
			}
		}
	}

	UWorld* World = nullptr;
	TSet<FString>& DeletedActorPackages;
};

/** Output device to capture presence of errors during generation. */
struct FPCGDetectErrorsInScope : public FOutputDevice
{
	FPCGDetectErrorsInScope()
	{
		if (GLog)
		{
			GLog->AddOutputDevice(this);
		}
	}

	virtual ~FPCGDetectErrorsInScope()
	{
		if (GLog)
		{
			GLog->RemoveOutputDevice(this);
		}
	}

	//~Begin FOutputDevice interface
	virtual bool IsMemoryOnly() const override { return true; }
	virtual bool CanBeUsedOnMultipleThreads() const override { return true; }
	virtual bool CanBeUsedOnAnyThread() const override { return true; }
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category)
	{
		if (Verbosity <= ELogVerbosity::Error)
		{
			bErrorOccurred = true;
		}
	}
	//~End FOutputDevice interface

	bool GetErrorOccurred() const { return bErrorOccurred; }

private:
	std::atomic<bool> bErrorOccurred = false;
};

/** Disable clearing of results while in this scope */
struct FPCGDisableClearResults
{
	FPCGDisableClearResults(UWorld* InWorld)
	{
		Subsystem = UWorld::GetSubsystem<UPCGSubsystem>(InWorld);

		if(Subsystem)
		{
			FPCGWorldPartitionBuilder::SetDisableClearResults(Subsystem, true);
		}
	}

	virtual ~FPCGDisableClearResults()
	{
		if (Subsystem)
		{
			FPCGWorldPartitionBuilder::SetDisableClearResults(Subsystem, false);
		}
	}

	UPCGSubsystem* Subsystem = nullptr;
};

bool FPCGWorldPartitionBuilderArgs::InitializeFrom(const FPCGWorldPartitionCommandlineArgs& CommandlineArgs, FPCGWorldPartitionBuilderArgs& OutArgs)
{
	FPCGWorldPartitionBuilderArgs BuilderArgs;
	TSoftObjectPtr<UPCGBuilderSettings> SettingsPath = CommandlineArgs.SettingAsset.IsSet() ? CommandlineArgs.SettingAsset.GetValue() : GetDefault<UPCGEditorProjectSettings>()->DefaultBuilderSetting;
	
	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Initialize Builder Args");

	bool bUsingBuilderSettings = false;
	// Load Settings Asset if any exists
	if (SettingsPath.IsValid() || SettingsPath.IsPending())
	{
		if (UPCGBuilderSettings* BuilderSettings = Cast<UPCGBuilderSettings>(SettingsPath.LoadSynchronous()))
		{
			BuilderArgs.bGenerateEditingModeLoadAsPreviewComponents = BuilderSettings->EditingModes.Contains(EPCGEditorDirtyMode::LoadAsPreview);
			BuilderArgs.bGenerateEditingModeNormalComponents = BuilderSettings->EditingModes.Contains(EPCGEditorDirtyMode::Normal);
			BuilderArgs.bGenerateEditingModePreviewComponents = BuilderSettings->EditingModes.Contains(EPCGEditorDirtyMode::Preview);
			BuilderArgs.bIgnoreGenerationErrors = BuilderSettings->bIgnoreGenerationErrors;
			BuilderArgs.bOneComponentAtATime = BuilderSettings->bOneComponentAtATime || BuilderSettings->bRevertUnchangedActors;
			BuilderArgs.IncludeActorIDs = BuilderSettings->FilterByActorNames;
			BuilderArgs.IncludeGraphs = BuilderSettings->Graphs;
			BuilderArgs.bIterativeCellLoading = BuilderSettings->bIterativeCellLoading;
			BuilderArgs.IterativeCellSize = BuilderSettings->IterativeCellSize;
			BuilderArgs.bLoadEditorOnlyDataLayers = BuilderSettings->bLoadEditorOnlyDataLayers;
			BuilderArgs.bLoadActivatedRuntimeDataLayers = BuilderSettings->bLoadActivatedRuntimeDataLayers;
			BuilderArgs.IncludedDataLayers = BuilderSettings->IncludedDataLayers;
			BuilderArgs.ExcludedDataLayers = BuilderSettings->ExcludedDataLayers;
			BuilderArgs.bRevertUnchangedActors = BuilderSettings->bRevertUnchangedActors;
			BuilderArgs.OutputsToIgnoreForRevert = BuilderSettings->OutputsToIgnoreForRevert;

			UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Loaded PCGBuilderSettings '%ls'", *SettingsPath.ToString());
			bUsingBuilderSettings = true;
		}
		else
		{
			UE_LOGF(LogPCGWorldPartitionBuilder, Error, "Failed to load PCGBuilderSettings '%ls'", *SettingsPath.ToString());
			return false;
		}
	}
	
	if (!bUsingBuilderSettings)
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Not using any PCGBuilderSettings");
	}

	// Then apply Commandline since it has priority
	const TCHAR* FromCommandline = TEXT("Commandline");
	const TCHAR* FromDefault = bUsingBuilderSettings ? TEXT("PCGBuilderSettings") : TEXT("Default");
		
	// Init 'bGenerateEditingModeLoadAsPreviewComponents'
	bool bFromCommandline = CommandlineArgs.bGenerateEditingModeLoadAsPreviewComponents.IsSet();
	if (bFromCommandline)
	{
		BuilderArgs.bGenerateEditingModeLoadAsPreviewComponents = CommandlineArgs.bGenerateEditingModeLoadAsPreviewComponents.GetValue();
	}

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "bGenerateEditingModeLoadAsPreviewComponents : '%d' (%ls)", 
		BuilderArgs.bGenerateEditingModeLoadAsPreviewComponents ? 1 : 0,
		bFromCommandline ? FromCommandline : FromDefault);
	
	// Init 'bGenerateEditingModeNormalComponents'
	bFromCommandline = CommandlineArgs.bGenerateEditingModeNormalComponents.IsSet();
	if (bFromCommandline)
	{
		BuilderArgs.bGenerateEditingModeNormalComponents = CommandlineArgs.bGenerateEditingModeNormalComponents.GetValue();
	}

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "bGenerateEditingModeNormalComponents : '%d' (%ls)",
		BuilderArgs.bGenerateEditingModeNormalComponents ? 1 : 0,
		bFromCommandline ? FromCommandline : FromDefault);
	
	// Init 'bGenerateEditingModePreviewComponents'
	bFromCommandline = CommandlineArgs.bGenerateEditingModePreviewComponents.IsSet();
	if (bFromCommandline)
	{
		BuilderArgs.bGenerateEditingModePreviewComponents = CommandlineArgs.bGenerateEditingModePreviewComponents.GetValue();
	}

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "bGenerateEditingModePreviewComponents : '%d' (%ls)",
		BuilderArgs.bGenerateEditingModePreviewComponents ? 1 : 0,
		bFromCommandline ? FromCommandline : FromDefault);


	// Init 'bOneComponentAtATime'
	bFromCommandline = CommandlineArgs.bOneComponentAtATime.IsSet();
	if (bFromCommandline)
	{
		BuilderArgs.bOneComponentAtATime = CommandlineArgs.bOneComponentAtATime.GetValue();
	}

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "bOneComponentAtATime : '%d' (%ls)",
		BuilderArgs.bOneComponentAtATime ? 1 : 0,
		bFromCommandline ? FromCommandline : FromDefault);
	
	// Init 'bIgnoreGenerationErrors'
	bFromCommandline = CommandlineArgs.bIgnoreGenerationErrors.IsSet();
	if (bFromCommandline)
	{
		BuilderArgs.bIgnoreGenerationErrors = CommandlineArgs.bIgnoreGenerationErrors.GetValue();
	}

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "bIgnoreGenerationErrors : '%d' (%ls)",
		BuilderArgs.bIgnoreGenerationErrors ? 1 : 0,
		bFromCommandline ? FromCommandline : FromDefault);
		
	// Init 'IncludeActorIDs'
	bFromCommandline = CommandlineArgs.IncludeActorIDs.IsSet();
	if (bFromCommandline)
	{
		BuilderArgs.IncludeActorIDs = CommandlineArgs.IncludeActorIDs.GetValue();
	}

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "IncludeActorIDs : '%ls' (%ls)",
		BuilderArgs.IncludeActorIDs.IsEmpty() ? TEXT("None") : *FString::Join(BuilderArgs.IncludeActorIDs, TEXT(",")),
		bFromCommandline ? FromCommandline : FromDefault);
		
	// Init 'IncludeGraphNames'
	bFromCommandline = CommandlineArgs.IncludeGraphNames.IsSet();
	if (bFromCommandline)
	{
		// Commandline overrides using graph names so we empty whatever graph specified in the settings (it one or the other)
		BuilderArgs.IncludeGraphs.Empty();
		BuilderArgs.IncludeGraphNames = CommandlineArgs.IncludeGraphNames.GetValue();
	}

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "IncludeGraphNames : '%ls' (%ls)",
		BuilderArgs.IncludeGraphNames.IsEmpty() ? TEXT("None") : *FString::Join(BuilderArgs.IncludeGraphNames, TEXT(",")),
		bFromCommandline ? FromCommandline : TEXT("Default"));

	TArray<FString> GraphsToString;
	Algo::Transform(BuilderArgs.IncludeGraphs, GraphsToString, [](const TSoftObjectPtr<UPCGGraphInterface>& Graph) { return Graph.ToString(); });
		
	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "IncludeGraphs '%ls' (%ls)",
		GraphsToString.IsEmpty() ? TEXT("None") : *FString::Join(GraphsToString, TEXT(",")),
		FromDefault);

	// Init 'bIterativeCellLoading'
	bFromCommandline = CommandlineArgs.bIterativeCellLoading.IsSet();
	if (bFromCommandline)
	{
		BuilderArgs.bIterativeCellLoading = CommandlineArgs.bIterativeCellLoading.GetValue();
	}

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "bIterativeCellLoading : '%d' (%ls)",
		BuilderArgs.bIterativeCellLoading ? 1 : 0,
		bFromCommandline ? FromCommandline : FromDefault);

	// Init 'IterativeCellSize'
	bFromCommandline = CommandlineArgs.IterativeCellSize.IsSet();
	if (bFromCommandline)
	{
		BuilderArgs.IterativeCellSize = CommandlineArgs.IterativeCellSize.GetValue();
	}

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "IterativeCellSize : '%d' (%ls)",
		BuilderArgs.IterativeCellSize,
		bFromCommandline ? FromCommandline : FromDefault);

	// Log DataLayers
	TArray<FString> DataLayersToString;
	Algo::Transform(BuilderArgs.IncludedDataLayers, DataLayersToString, [](const TSoftObjectPtr<UDataLayerAsset>& DataLayerAsset) { return DataLayerAsset.ToString(); });

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "IncludedDataLayers '%ls' (%ls)",
		DataLayersToString.IsEmpty() ? TEXT("None") : *FString::Join(DataLayersToString, TEXT(",")),
		FromDefault);

	DataLayersToString.Empty();
	Algo::Transform(BuilderArgs.ExcludedDataLayers, DataLayersToString, [](const TSoftObjectPtr<UDataLayerAsset>& DataLayerAsset) { return DataLayerAsset.ToString(); });

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "ExcludedDataLayers '%ls' (%ls)",
		DataLayersToString.IsEmpty() ? TEXT("None") : *FString::Join(DataLayersToString, TEXT(",")),
		FromDefault);

	// Log Revert parameters
	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "bRevertUnchangedActors '%d' (%ls)",
		BuilderArgs.bRevertUnchangedActors ? 1 : 0,
		FromDefault);

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "OutputsToIgnoreForRevert '%ls' (%ls)",
		BuilderArgs.OutputsToIgnoreForRevert.IsEmpty() ? TEXT("None") : *FString::Join(BuilderArgs.OutputsToIgnoreForRevert, TEXT(",")),
		FromDefault);

	OutArgs = BuilderArgs;
	return true;
}

UPCGWorldPartitionBuilder::UPCGWorldPartitionBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		auto GetBoolParam = [this](const FString& ParamName, TOptional<bool>& OutOptionalParam)
		{
			const FString ParamNameAssignment = ParamName + "=";
			bool bParamValue;
			if (HasParam(ParamName))
			{
				OutOptionalParam = true;
			}
			else if (FParse::Bool(*GetBuilderArgs(), *ParamNameAssignment, bParamValue))
			{
				OutOptionalParam = bParamValue;
			}
		};

		GetBoolParam("GenerateComponentEditingModeLoadAsPreview", CommandlineArgs.bGenerateEditingModeLoadAsPreviewComponents);
		GetBoolParam("GenerateComponentEditingModeNormal", CommandlineArgs.bGenerateEditingModeNormalComponents);
		GetBoolParam("GenerateComponentEditingModePreview", CommandlineArgs.bGenerateEditingModePreviewComponents);
		GetBoolParam("OneComponentAtATime", CommandlineArgs.bOneComponentAtATime);
		GetBoolParam("IgnoreGenerationErrors", CommandlineArgs.bIgnoreGenerationErrors);
		GetBoolParam("IterativeCellLoading", CommandlineArgs.bIterativeCellLoading);
	
		FString IncludeGraphNamesValue;
		if (GetParamValue("IncludeGraphNames=", IncludeGraphNamesValue))
		{
			TArray<FString> IncludeGraphNameStrings;
			if (!IncludeGraphNamesValue.IsEmpty())
			{
				IncludeGraphNamesValue.ParseIntoArray(IncludeGraphNameStrings, TEXT(";"), true);
			}
			CommandlineArgs.IncludeGraphNames = IncludeGraphNameStrings;
		}

		FString IncludeActorIDsValue;
		if (GetParamValue("IncludeActorIDs=", IncludeActorIDsValue))
		{
			TArray<FString> IncludeActorIDStrings;
			if (!IncludeActorIDsValue.IsEmpty())
			{
				IncludeActorIDsValue.ParseIntoArray(IncludeActorIDStrings, TEXT(";"), true);
			}
			CommandlineArgs.IncludeActorIDs = IncludeActorIDStrings;
		}

		FString SettingAsset;
		if (GetParamValue("PCGBuilderSettings=", SettingAsset) && !SettingAsset.IsEmpty())
		{
			CommandlineArgs.SettingAsset = TSoftObjectPtr<UPCGBuilderSettings>(FSoftObjectPath(SettingAsset));
		}

		int32 ParamIterativeCellSize = 25600;
		if (GetParamValue("IterativeCellSize=", ParamIterativeCellSize))
		{
			const int32 MinIterativeCellSize = 12800;
			const int32 MaxIterativeCellSize = FMath::Max(MinIterativeCellSize, ParamIterativeCellSize);
			if (MaxIterativeCellSize == ParamIterativeCellSize)
			{
				CommandlineArgs.IterativeCellSize = ParamIterativeCellSize;
			}
			else
			{
				UE_LOGF(LogPCGWorldPartitionBuilder, Warning, "Minimum value for 'IterativeCellSize' is %d", MinIterativeCellSize);
			}
		}
	}
}

void UPCGWorldPartitionBuilderHelper::AddAssetToSubmitAllowList(const TSoftObjectPtr<UObject>& InPath)
{
	for (TObjectIterator<UPCGWorldPartitionBuilder> It; It; ++It)
	{
		It->AddAssetToSubmitAllowList(InPath);
	}
}

void UPCGWorldPartitionBuilder::AddAssetToSubmitAllowList(const TSoftObjectPtr<UObject>& InPath)
{
	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Added Asset to Submit Allow List: '%ls'", *InPath.ToString());
	AllowedAssetPackages.Add(InPath.GetLongPackageName());
}

bool UPCGWorldPartitionBuilder::PreWorldInitialization(UWorld* World, FPackageSourceControlHelper& PackageHelper)
{
	if (!Super::PreWorldInitialization(World, PackageHelper))
	{
		return false;
	}

	// Prevent Actor Creation outside of run scope
	FPCGWorldPartitionBuilder::SetDisablePartitionActorCreationForWorld(World, /*bDisable=*/true);

	return true;
}

bool UPCGWorldPartitionBuilder::PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper)
{
	if (!Super::PreRun(World, PackageHelper))
	{
		return false;
	}

	if (!FPCGWorldPartitionBuilderArgs::InitializeFrom(CommandlineArgs, Args))
	{
		return false;
	}

	FPCGWorldPartitionBuilder::SetChainedDispatchToLocalComponents(World, Args.bRevertUnchangedActors);

	RevertableActors.Empty();
	RevertedActors.Empty();

	// Reset Allow list
	AllowedAssetPackages.Empty();

	// Set base class member
	IterativeCellSize = Args.IterativeCellSize;

	// If World doesn't have a World Partition do not use iterative cell loading
	Args.bIterativeCellLoading &= World->IsPartitionedWorld();

	// Setup DataLayers
	// Set Base Builder Class fields
	bLoadNonDynamicDataLayers = Args.bLoadEditorOnlyDataLayers;
	bIncludeInitiallyActivatedRuntimeDataLayers = Args.bLoadActivatedRuntimeDataLayers;

	IncludedDataLayers.Append(Args.IncludedDataLayers);
	ExcludedDataLayers.Append(Args.ExcludedDataLayers);

	if (GetLoadingMode() == ELoadingMode::IterativeCells2D)
	{
		FBox PCGBuilderVolumeBounds = FPCGWorldPartitionBuilder::GetPCGBuilderVolumeBounds(World);
		if (PCGBuilderVolumeBounds.IsValid)
		{
			IterativeWorldBounds = PCGBuilderVolumeBounds;
		}
		
		// Disable Source Control until Post Run
		PreviousProviderName = ISourceControlModule::Get().GetProvider().GetName();
		if (PreviousProviderName != NAME_None)
		{
			ISourceControlModule::Get().SetProvider(NAME_None);
		}
	}

	return true;
}

bool UPCGWorldPartitionBuilder::RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{
	if (!ensure(GEngine) || !ensure(World))
	{
		return false;
	}

	// Try to eliminate as much background noise as possible before getting started.
	FPCGWorldPartitionBuilder::WaitForAllAsyncEditorProcesses(World);

	if (GetLoadingMode() == ELoadingMode::EntireWorld)
	{
		return RunEntireWorld(World, PackageHelper);
	}
	else
	{
		return RunPerCell(World, InCellInfo, PackageHelper);
	}
}

bool UPCGWorldPartitionBuilder::RunEntireWorld(UWorld* World, FPackageSourceControlHelper& PackageHelper)
{
	// 1. Collect components of interest once upfront.

	TArray<TWeakObjectPtr<UPCGComponent>> ComponentsToGenerate;
	{
		auto ComponentFilter = [](const UPCGComponent* InComponent)
		{
			// Skip LCs, rely on original component to generate everything.
			return !InComponent->IsLocalComponent();
		};

		FPCGWorldPartitionBuilder::CollectComponentsToGenerate(World, Args, ComponentFilter, ComponentsToGenerate);
	}

	// 2. Clear the dirty flag on any packages that dirtied themselves while loading. We're not interested in saving these.
	FPCGWorldPartitionBuilder::ResetPackageDirtyFlags();

	// Before we run any generation, gather the revertable actors from the components we want to generate and the loaded pcg partition actors.
	// This will give us the pre generation Actorr to CRC mapping to compare against when generation completes.
	FPCGWorldPartitionBuilder::GetRevertableActorsFromComponents(ComponentsToGenerate, Args, RevertableActors);
	FPCGWorldPartitionBuilder::GetRevertablePartitionActors(World, Args, RevertableActors);

	// Create missing Partition Actors
	FPCGWorldPartitionBuilder::SetDisablePartitionActorCreationForWorld(World, /*bDisable=*/false);
	FPCGWorldPartitionBuilder::CreatePartitionedActors(World, Args, PartitionedComponents);
	
	// Update mappings
	TSet<FString> DeletedActorPackages;
	FPCGWorldPartitionBuilder::UpdatePartitionedActors(World, PartitionedComponents, DeletedActorPackages);
	
	// 3. Generate components.
	bool bGeneratedAnyComponent = false;
		
	TArray<TArray<TWeakObjectPtr<UPCGComponent>>> GroupsToGenerate = FPCGWorldPartitionBuilder::GroupComponentsToGenerate(ComponentsToGenerate, Args);
	for (TArray<TWeakObjectPtr<UPCGComponent>>& GroupToGenerate : GroupsToGenerate)
	{
		bool bErrorsOccurred = false;
		bGeneratedAnyComponent |= FPCGWorldPartitionBuilder::GenerateComponents(GroupToGenerate, World, Args, TMap<UPCGComponent*, UPCGComponent*>(), DeletedActorPackages, bErrorsOccurred);
		bErrorOccurredWhileGenerating |= bErrorsOccurred;
	}

	// 4. Get packages that were dirtied during generation and record them for saving later.
	TArray<UPackage*> DirtyPackages;

	if (bGeneratedAnyComponent)
	{
		// Wait for other executions that might have spawned from generating the components
		FPCGWorldPartitionBuilder::WaitOnGraphExecutor(World);
		
		DirtyPackages = FPCGWorldPartitionBuilder::GetDirtyPackages(World, AllowedAssetPackages);
	}
	
	// 5. Revert unchanged actors
	FPCGWorldPartitionBuilder::RevertUnchangedActors(DirtyPackages, Args, RevertableActors, RevertedActors);

	if (!bGeneratedAnyComponent)
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, Display, "No components found to generate. No packages will be saved.");
	}
	else if (DirtyPackages.IsEmpty())
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, Display, "At least one component was generated but no additional packages were dirtied.");
	}
	else
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Generation complete, %d packages dirtied.", DirtyPackages.Num());
	}

	if (!bGeneratedAnyComponent)
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Dirty package detection and save skipped due to trivial run");
		return !bErrorOccurredWhileGenerating;
	}

	return SaveEntireWorldPackages(World, PackageHelper, MoveTemp(DirtyPackages), MoveTemp(DeletedActorPackages));
}

bool UPCGWorldPartitionBuilder::SaveEntireWorldPackages(UWorld* World, FPackageSourceControlHelper& PackageHelper, TArray<UPackage*> DirtyPackages, TSet<FString> DeletedActorPackages)
{
	// Check whether an error was thrown while generating components and fail the builder if the ignore argument is not provided.
	if (bErrorOccurredWhileGenerating)
	{
		if (!Args.bIgnoreGenerationErrors)
		{
			UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Dirty package detection and save skipped due to errors during generation.");

			return !bErrorOccurredWhileGenerating;
		}
		else
		{
			UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Generation errors ignored, dirty packages will be saved.");
		}
	}

	// Save/delete pending packages.

	// Empty packages should be deleted - mirrors logic in InternalPromptForCheckoutAndSave()
	TArray<UPackage*> PackagesToDelete;
	TArray<UPackage*> PackagesToSave;

	for (int i = DirtyPackages.Num() - 1; i >= 0; --i)
	{
		if (UPackage* DirtyPackage = DirtyPackages[i])
		{
			if (UPackage::IsEmptyPackage(DirtyPackage))
			{
				PackagesToDelete.Add(DirtyPackage);
			}
			else
			{
				PackagesToSave.Add(DirtyPackage);

				if (DirtyPackage->HasAnyPackageFlags(PKG_NewlyCreated))
				{
					++NumNewPackages;
				}
				else
				{
					++NumModifiedPackages;
				}
			}

			// Remove from here if package was still part of the loaded DirtyPackages
			DeletedActorPackages.Remove(DirtyPackage->GetName());
		}
	}

	// Log New/Modified Actors
	FPCGWorldPartitionBuilder::LogModifiedAndNewActorsFromDirtyPackages(DirtyPackages);

	// Set final counts
	NumDeletedPackages = DeletedActorPackages.Num() + PackagesToDelete.Num();

	if (!SavePackages(PackagesToSave, PackageHelper))
	{
		return false;
	}

	// Loaded Packages to delete
	if (!PackagesToDelete.IsEmpty())
	{
		if (!PackageHelper.Delete(PackagesToDelete))
		{
			return false;
		}
	}

	// Unloaded Packages to delete
	if (!DeletedActorPackages.IsEmpty())
	{
		PackageHelper.Delete(DeletedActorPackages.Array());
	}

	TArray<FString> FilesToSubmit;
	FilesToSubmit.Append(SourceControlHelpers::PackageFilenames(DirtyPackages));
	FilesToSubmit.Append(SourceControlHelpers::PackageFilenames(PackagesToDelete));
	FilesToSubmit.Append(SourceControlHelpers::PackageFilenames(DeletedActorPackages.Array()));

	const FString ChangeDescription = FString::Printf(TEXT("Generated PCG components for world '%s'"), *World->GetName());
	return OnFilesModified(FilesToSubmit, ChangeDescription);
}

bool UPCGWorldPartitionBuilder::RunPerCell(UWorld* InWorld, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{
	ON_SCOPE_EXIT
	{
		// Since we are potentially releasing actor references when exiting the RunPerCell call it is important to run a GC so that actors can be properly unloaded (and may be reloaded in another Cell)
		FWorldPartitionHelpers::DoCollectGarbage();

		FPCGWorldPartitionBuilder::SetDisablePartitionActorCreationForWorld(InWorld, /*bDisable=*/true);
	};

	// Load missing dependencies (SoftObjectPath references)	
	TSet<FWorldPartitionReference> LoadedActorDependencies;
	FPCGWorldPartitionBuilder::LoadPartitionedActorDependencies(InWorld, LoadedActorDependencies);

	// Reset Dirty flags so we can save only packages that will get modified after this line
	FPCGWorldPartitionBuilder::ResetPackageDirtyFlags();

	// Gather already loaded revertable Partition actors (before any modification occurs)
	FPCGWorldPartitionBuilder::GetRevertablePartitionActors(InWorld, Args, RevertableActors);

	// Create missing PAs and Update mappings
	FPCGWorldPartitionBuilder::SetDisablePartitionActorCreationForWorld(InWorld, /*bDisable=*/false);

	// Create Missing PAs
	bool bCellHasChanges = FPCGWorldPartitionBuilder::CreatePartitionedActors(InWorld, Args, PartitionedComponents);

	// Update Mappings
	TSet<FString> CellDeletedPackageNames;
	bCellHasChanges |= FPCGWorldPartitionBuilder::UpdatePartitionedActors(InWorld, PartitionedComponents, CellDeletedPackageNames);

	// Find Components to generate and make sure we load their bounds
	TArray<TWeakObjectPtr<UPCGComponent>> ComponentsToGenerate;
	FBox ExpandedBounds = InCellInfo.Bounds;

	UPCGSubsystem* PCGSubsystem = UWorld::GetSubsystem<UPCGSubsystem>(InWorld);
	check(PCGSubsystem);

	auto GenerateComponentFilter = [&InCellInfo, &ExpandedBounds, PCGSubsystem, this](const UPCGComponent* InComponent)
	{
		if (GeneratedComponents.Contains(InComponent))
		{
			return false;
		}
			
		bool bCheckBoundsCenter = true;
		if (InComponent->IsPartitioned())
		{
			bool bHasUnbounded = false;
			PCGHiGenGrid::FSizeArray GridSizes;
			FPCGWorldPartitionBuilder::GetGenerationGridSizes(InComponent->GetGraph(), PCGSubsystem->GetPCGWorldActor(), GridSizes, bHasUnbounded);

			if (!bHasUnbounded)
			{
				return false;
			}

			// If an Unbounded Partitioned Component exists, it needs to be generated before all its PAs, this can be wasteful on the loading side of things because we are going to expand 
			// the bounds of loading to fit this unbounded volume, if it happens to be a very large volume it can make the IterativeCell mode disadvantageous
			bCheckBoundsCenter = false;
		}
				
		// When processing a PA make sure to only collect the top level grid actors, we are then going to find all child grid actors in a next step
		if (APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(InComponent->GetOwner()))
		{
			UPCGComponent* OriginalComponent = PartitionActor->GetOriginalComponent(InComponent);
			check(OriginalComponent);
			
			// Avoid calling this multiple times
			bool bHasUnbounded = false;
			PCGHiGenGrid::FSizeArray GridSizes;
			FPCGWorldPartitionBuilder::GetGenerationGridSizes(OriginalComponent->GetGraph(), PCGSubsystem->GetPCGWorldActor(), GridSizes, bHasUnbounded);

			// Only generate if this PA represents the top level grid of the Original Component
			if (GridSizes.IsEmpty() || PartitionActor->GetPCGGridSize() != GridSizes[0])
			{
				return false;
			}
		}

		// Generate Component if its bound center is inside the currently processing cell bounds (to avoid expanding bounds too much)
		const FBox ActorBounds = PCGHelpers::GetActorBounds(InComponent->GetOwner());
		if (bCheckBoundsCenter && !InCellInfo.Bounds.IsInsideOrOnXY(ActorBounds.GetCenter()))
		{
			return false;
		}
		
		// Expand the cell bounds to include the whole actor bounds (keep Z same as original Cell bounds)
		FBox ActorXYBounds(FVector(ActorBounds.Min.X, ActorBounds.Min.Y, ExpandedBounds.Min.Z), FVector(ActorBounds.Max.X, ActorBounds.Max.Y, ExpandedBounds.Max.Z));
		ExpandedBounds += ActorXYBounds;

		// Consider this component generated
		GeneratedComponents.Add(TSoftObjectPtr<const UPCGComponent>(InComponent));

		return true;
	};

	// Collect Components to Generate a first time, this will allow us to call FPCGWorldPartitionBuilder::UpdatePartitionedActors on ParttionActors with all their overlaps
	FPCGWorldPartitionBuilder::CollectComponentsToGenerate(InWorld, Args, GenerateComponentFilter, ComponentsToGenerate);

	// Gather revertable actors from the components to generate (this should be non-partitioned components as we already called GetRevertablePartitionActors at the beginning of this function)
	FPCGWorldPartitionBuilder::GetRevertableActorsFromComponents(ComponentsToGenerate, Args, RevertableActors);

	// Make sure Partitioned (Unbounded) components are Generated first
	ComponentsToGenerate.Sort([](const TWeakObjectPtr<UPCGComponent>& WeakComponentA, const TWeakObjectPtr<UPCGComponent>& WeakComponentB)
	{
		UPCGComponent* ComponentA = WeakComponentA.Get();
		check(ComponentA);
		UPCGComponent* ComponentB = WeakComponentB.Get();
		check(ComponentB);
		
		if (ComponentA->IsPartitioned() == ComponentB->IsPartitioned())
		{
			// Default to ptr hash (we don't really care at that point as we really just want the Partitioned volumes first)
			return WeakComponentA.GetWeakPtrTypeHash() < WeakComponentB.GetWeakPtrTypeHash();
		}

		return ComponentA->IsPartitioned();
	});
		
	// Do a Bounds expansion to make sure we are generating PCG Components on areas covering their owning actor bounds (which might be a bigger area then the currently loaded cell)
	TUniquePtr<IWorldPartitionActorLoaderInterface::ILoaderAdapter> LoaderAdapter = FPCGWorldPartitionBuilder::ExpandLoadedBoundsIfNeeded(InWorld, InCellInfo.Bounds, ExpandedBounds, TEXT("Expanded Bounds"));
	
	// Gather newly loaded Partition actors before they get updated
	FPCGWorldPartitionBuilder::GetRevertablePartitionActors(InWorld, Args, RevertableActors);

	if (LoaderAdapter)
	{
		// Update Mappings for new loaded PAs from Expanded Bounds
		bCellHasChanges |= FPCGWorldPartitionBuilder::UpdatePartitionedActors(InWorld, PartitionedComponents, CellDeletedPackageNames);
	}

	// Add Components for sub grids
	TArray<UPCGComponent*> SubGridComponentsToGenerate;
	TMap<UPCGComponent*, UPCGComponent*> TaskDependencies;
	FPCGWorldPartitionBuilder::CollectSubGridComponentsToGenerate(InWorld, Args, ComponentsToGenerate, SubGridComponentsToGenerate, TaskDependencies);

	for (UPCGComponent* SubGridPCGComponent : SubGridComponentsToGenerate)
	{
		if (!GeneratedComponents.Contains(SubGridPCGComponent))
		{
			ComponentsToGenerate.Add(SubGridPCGComponent);

			// Consider this component generated
			GeneratedComponents.Add(SubGridPCGComponent);
		}
	}

	bool bErrorsOccurred = false;
	// Do Generation and Collect Packages to save
	if (ComponentsToGenerate.Num() > 0)
	{
		TArray<TArray<TWeakObjectPtr<UPCGComponent>>> GroupsToGenerate = FPCGWorldPartitionBuilder::GroupComponentsToGenerate(ComponentsToGenerate, Args);
		for (TArray<TWeakObjectPtr<UPCGComponent>>& GroupToGenerate : GroupsToGenerate)
		{
			bCellHasChanges |= FPCGWorldPartitionBuilder::GenerateComponents(GroupToGenerate, InWorld, Args, TaskDependencies, CellDeletedPackageNames, bErrorsOccurred);
			bErrorOccurredWhileGenerating |= bErrorsOccurred;
		}
	}

	if (bErrorsOccurred && !Args.bIgnoreGenerationErrors)
	{
		if (bCellHasChanges)
		{
			UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Save skipped due to errors during generation.");
		}
		return false;
	}

	if (bCellHasChanges)
	{
		// Wait for other executions that might have spawned from generating the components
		FPCGWorldPartitionBuilder::WaitOnGraphExecutor(InWorld);
		
		return SavePerCellPackages(InWorld, PackageHelper, CellDeletedPackageNames);
	}

	return true;
}

bool UPCGWorldPartitionBuilder::PostRun(UWorld* World, FPackageSourceControlHelper& PackageHelper, const bool bInRunSuccess)
{
	FPCGWorldPartitionBuilder::LogRevertedActors(RevertedActors);

	// Log final changes after combining packages (there may have been duplicates in the two deleted package lists).
	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "PostRun: Final package changes: %d new, %d modified, %d deleted, %d unchanged.", NumNewPackages, NumModifiedPackages , NumDeletedPackages, RevertedActors.Num());

	FPCGWorldPartitionBuilder::SetDisablePartitionActorCreationForWorld(World, /*bDisable=*/false);

	if (GetLoadingMode() == ELoadingMode::EntireWorld)
	{
		PartitionedComponents.Empty();
		return true;
	}

	// Builder was launched with no source control (nothing more to do)
	if (PreviousProviderName.IsNone())
	{
		return true;
	}

	// Re-establish SCC Provider
	ISourceControlModule::Get().SetProvider(PreviousProviderName);
	ISourceControlProvider::FInitResult InitResult = ISourceControlModule::Get().GetProvider().Init(ISourceControlProvider::EInitFlags::AttemptConnection);
	if (!InitResult.bIsAvailable)
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, Error, "Failed to initialize source control provider.");
		return false;
	}

	// Mark files for delete
	if (DeletedPackageNames.Num() > 0)
	{
		TArray<FString> FilesToDelete;
		FilesToDelete.Reserve(DeletedPackageNames.Num());
		Algo::Transform(DeletedPackageNames, FilesToDelete, [](const FString& PackageName) { return SourceControlHelpers::PackageFilename(PackageName); });
		if (!SourceControlHelpers::MarkFilesForDelete(FilesToDelete))
		{
			return false;
		}
	}

	TArray<FString> PackagesToCheckout;
	TArray<FString> PackagesToAdd;
	TArray<FString> PackageNames;
	PackageNames.Reserve(NewPackageNames.Num() + ModifiedPackageNames.Num());
	PackageNames.Append(NewPackageNames.Array());
	PackageNames.Append(ModifiedPackageNames.Array());

	if (!PackageHelper.GetDesiredStatesForModification(PackageNames, PackagesToCheckout, PackagesToAdd))
	{
		return false;
	}

	// Checkout modified files
	if (PackagesToCheckout.Num() > 0)
	{
		if (!PackageHelper.Checkout(PackagesToCheckout))
		{
			return false;
		}
	}

	// Add new files to source control
	if (PackagesToAdd.Num() > 0)
	{
		if (!PackageHelper.AddToSourceControl(PackagesToAdd))
		{
			return false;
		}
	}

	// Release references
	PartitionedComponents.Empty();

	TArray<FString> FilesToSubmit;
	FilesToSubmit.Append(SourceControlHelpers::PackageFilenames(DeletedPackageNames.Array()));
	FilesToSubmit.Append(SourceControlHelpers::PackageFilenames(PackagesToCheckout));
	FilesToSubmit.Append(SourceControlHelpers::PackageFilenames(PackagesToAdd));

	const FString ChangeDescription = FString::Printf(TEXT("Generated PCG components for world '%s'"), *World->GetName());
	return OnFilesModified(FilesToSubmit, ChangeDescription);
}

bool UPCGWorldPartitionBuilder::SavePerCellPackages(UWorld* InWorld, FPackageSourceControlHelper& PackageHelper, TSet<FString> CellDeletedPackages)
{
	TArray<UPackage*> DirtyPackages = FPCGWorldPartitionBuilder::GetDirtyPackages(InWorld, AllowedAssetPackages);
	TArray<UPackage*> PackagesToDelete;
	UWorldPartition* WorldPartition = InWorld->GetWorldPartition();

	uint32 CellNewPackageCount = 0;
	uint32 CellModifiedPackageCount = 0;
	uint32 CellDeletedPackageCount = 0;

	// Revert unchanged actors
	FPCGWorldPartitionBuilder::RevertUnchangedActors(DirtyPackages, Args, RevertableActors, RevertedActors);

	for (int i = DirtyPackages.Num() - 1; i >= 0; --i)
	{
		if (UPackage* DirtyPackage = DirtyPackages[i])
		{
			const FString PackageName = DirtyPackage->GetName();
			if (UPackage::IsEmptyPackage(DirtyPackage))
			{
				DeletedPackageNames.Add(PackageName);

				PackagesToDelete.Add(DirtyPackage);

				DirtyPackages.RemoveAtSwap(i);

				CellDeletedPackageCount++;
			}
			else if (DirtyPackage->HasAnyPackageFlags(PKG_NewlyCreated))
			{
				NewPackageNames.Add(PackageName);

				CellNewPackageCount++;
			}
			else if (!NewPackageNames.Contains(PackageName))
			{
				ModifiedPackageNames.Add(PackageName);

				CellModifiedPackageCount++;
			}

			CellDeletedPackages.Remove(*PackageName);
		}
	}

	// Update stats
	NumNewPackages += CellNewPackageCount;
	NumModifiedPackages += CellModifiedPackageCount;
	NumDeletedPackages += CellDeletedPackageCount;

	// Log New/Modified Actors
	FPCGWorldPartitionBuilder::LogModifiedAndNewActorsFromDirtyPackages(DirtyPackages);

	// Delete loaded packages
	if (PackagesToDelete.Num() > 0 && !PackageHelper.Delete(PackagesToDelete))
	{
		return false;
	}

	// Delete unloaded packages
	if (CellDeletedPackages.Num() > 0 && !PackageHelper.Delete(CellDeletedPackages.Array()))
	{
		return false;
	}

	// Save Dirty/New packages
	if (DirtyPackages.Num() > 0 && !SavePackages(DirtyPackages, PackageHelper))
	{
		return false;
	}

	UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Cell package changes: %d modified, %d new, %d deleted.", CellModifiedPackageCount, CellNewPackageCount, CellDeletedPackageCount);
	return true;
}

bool FPCGWorldPartitionBuilder::GenerateComponents(
	TArray<TWeakObjectPtr<UPCGComponent>>& Components,
	UWorld* InWorld,
	const FPCGWorldPartitionBuilderArgs& InArgs,
	const TMap<UPCGComponent*, UPCGComponent*>& InTaskDependencies,
	TSet<FString>& InOutDeletedActorPackages,
	bool& bOutGenerationErrors)
{
	if (!InArgs.bOneComponentAtATime)
	{
		FPCGWorldPartitionBuilder::WaitForAllAsyncEditorProcesses(InWorld);
	}

	const FPCGDetectErrorsInScope DetectErrors;

	auto WaitForComponentGeneration = [InWorld](const UPCGComponent* InComponent)
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Completing generation on PCG component on actor '%ls' label '%ls', graph '%ls'",
			*InComponent->GetOwner()->GetName(),
			*InComponent->GetOwner()->GetActorNameOrLabel(),
			*InComponent->GetGraph()->GetName());

		while (InComponent->IsGenerating())
		{
			FWorldPartitionHelpers::FakeEngineTick(InWorld);
		}

		// Can be useful to let some things flush/update after generation.
		FWorldPartitionHelpers::FakeEngineTick(InWorld);
	};

	// Will track any deleted actors in this scope
	FPCGDetectDeletedActorInScope DeletedActorScope(InWorld, InOutDeletedActorPackages);

	TMap<TObjectKey<UPCGComponent>, FPCGTaskId> GeneratedComponents;

	{
		// While we process a batch of inter-dependent components (InTaskDependencies) we want to prevent the Subsystem's GraphExecutor from clearing its outputs.
		// This is specially important when InArgs.bOneComponentAtATime is true which waits for a component generation to end before scheduling another one
		// which will tick the world and the Subsystem's GraphExecutor will become idle and clear it's outputs.
		// If that following component generation is dependant on the previous one through its InTaskDependencies then it will never be allowed to run as the outputs will no longer contain
		// the previous component's results.
		// Important: the list of component created sorted by dependency so that we can rely on the order of scheduling here
		const FPCGDisableClearResults DisableClearResults(InWorld);

		TArray<UPCGComponent*> ValidComponents;
		Algo::TransformIf(Components, ValidComponents,
			[](const TWeakObjectPtr<UPCGComponent>& InWeakComponent)
			{
				return InWeakComponent.IsValid();
			},
			[](const TWeakObjectPtr<UPCGComponent>& InWeakComponent)
			{
				return InWeakComponent.Get();
			});

		ValidComponents.Sort([](const UPCGComponent& ComponentA, const UPCGComponent& ComponentB)
		{
			return ComponentA.GetPathName() < ComponentB.GetPathName();
		});

		for (UPCGComponent* Component : ValidComponents)
		{
			FPCGTaskId TaskIdDependency = InvalidPCGTaskId;
			if (const UPCGComponent* const* TaskDependency = InTaskDependencies.Find(Component))
			{
				if (FPCGTaskId* TaskIdDependencyPtr = GeneratedComponents.Find(*TaskDependency))
				{
					TaskIdDependency = *TaskIdDependencyPtr;
				}
			}

			FPCGTaskId GenerateTaskId = FPCGWorldPartitionBuilder::GenerateComponent(Component, InWorld, InArgs, TaskIdDependency);
			if (GenerateTaskId != InvalidPCGTaskId)
			{
				GeneratedComponents.Add(Component, GenerateTaskId);
			}

			if (InArgs.bOneComponentAtATime)
			{
				WaitForComponentGeneration(Component);
			}
		}
	}

	if (!InArgs.bOneComponentAtATime)
	{
		for (const auto& KeyValuePair : GeneratedComponents)
		{
			if (const UPCGComponent* Component = KeyValuePair.Key.ResolveObjectPtr())
			{
				WaitForComponentGeneration(Component);
			}
			else
			{
				UE_LOGF(LogPCGWorldPartitionBuilder, Warning, "GeneratedComponent reference lost, abandoning.");
			}
		}
	}

	bOutGenerationErrors = DetectErrors.GetErrorOccurred();

	return !GeneratedComponents.IsEmpty();
}

TArray<TArray<TWeakObjectPtr<UPCGComponent>>> FPCGWorldPartitionBuilder::GroupComponentsToGenerate(const TArray<TWeakObjectPtr<UPCGComponent>>& InComponentsToGenerate, const FPCGWorldPartitionBuilderArgs& InArgs)
{
	TArray<TArray<TWeakObjectPtr<UPCGComponent>>> GroupsToGenerate;

	if (!InArgs.IncludeGraphNames.IsEmpty())
	{
		for (const FString& IncludeGraphName : InArgs.IncludeGraphNames)
		{
			TArray<TWeakObjectPtr<UPCGComponent>> ComponentsForGraph;
			Algo::CopyIf(InComponentsToGenerate, ComponentsForGraph, [IncludeGraphName](TWeakObjectPtr<UPCGComponent> InComponent)
			{
				if (UPCGComponent* PCGComponent = InComponent.Get())
				{
					return PCGComponent->GetGraph() && PCGComponent->GetGraph()->GetName() == IncludeGraphName;
				}

				return false;
			});

			GroupsToGenerate.Add(MoveTemp(ComponentsForGraph));
		}
	}
	else if (!InArgs.IncludeGraphs.IsEmpty())
	{
		for (const TSoftObjectPtr<UPCGGraphInterface>& IncludeGraph : InArgs.IncludeGraphs)
		{
			TArray<TWeakObjectPtr<UPCGComponent>> ComponentsForGraph;
			Algo::CopyIf(InComponentsToGenerate, ComponentsForGraph, [&IncludeGraph](TWeakObjectPtr<UPCGComponent> InComponent)
			{
				if (UPCGComponent* PCGComponent = InComponent.Get())
				{
					return TSoftObjectPtr<UPCGGraphInterface>(PCGComponent->GetGraph()) == IncludeGraph;
				}

				return false;
			});

			GroupsToGenerate.Add(MoveTemp(ComponentsForGraph));
		}
	}
	else
	{
		GroupsToGenerate.Add(InComponentsToGenerate);
	}

	return GroupsToGenerate;
}

FPCGTaskId FPCGWorldPartitionBuilder::GenerateComponent(UPCGComponent* InComponent, UWorld* InWorld, const FPCGWorldPartitionBuilderArgs& InArgs, FPCGTaskId InTaskDependency)
{
	// Separate ensures for maximum debug information.
	if (!ensure(InComponent) || !ensure(InComponent->GetGraph()))
	{
		return InvalidPCGTaskId;
	}

	ensure(InComponent->bActivated);
	ensure(!InComponent->IsManagedByRuntimeGenSystem());

	// Non-partitioned components get their editing mode updated here to make sure generation is persisted
	// @todo_pcg: revisit for 5.8:
	// Force update here to make sure ChangeTransientState gets called even if the editing mode might have been updated by previous AddGraphInstance calls.
	FPCGWorldPartitionBuilder::UpdateEditingMode(InComponent, /*bForce=*/true);

	// Force generate as components that are already generated may decline the request.
	FPCGTaskId GenerateTask = InvalidPCGTaskId;

	// The only Partitioned components we do generate in iterative mode are generated only for their Unbounded grid as the other grid sizes will be generated through the PAs directly
	if (InArgs.bIterativeCellLoading && InComponent->IsPartitioned())
	{
		GenerateTask = InComponent->GenerateLocalGetTaskId(EPCGComponentGenerationTrigger::GenerateOnDemand, /*bForce=*/true, PCGHiGenGrid::UnboundedGridSize());
	}
	else if(InTaskDependency != InvalidPCGTaskId)
	{
		GenerateTask = InComponent->GenerateLocalGetTaskId(EPCGComponentGenerationTrigger::GenerateOnDemand, /*bForce=*/true, PCGHiGenGrid::UninitializedGridSize(), { InTaskDependency });
	}
	else
	{
		GenerateTask = InComponent->GenerateLocalGetTaskId(/*bForce=*/true);
	}

	if (GenerateTask == InvalidPCGTaskId)
	{
		if (ensure(InComponent->GetOwner()))
		{
			UE_LOGF(LogPCGWorldPartitionBuilder, Warning, "Scheduling generate task failed for PCG Component on actor '%ls' label '%ls' graph '%ls'.",
				*InComponent->GetOwner()->GetName(),
				*InComponent->GetOwner()->GetActorNameOrLabel(),
				*InComponent->GetGraph()->GetName());
		}
		else
		{
			UE_LOGF(LogPCGWorldPartitionBuilder, Warning, "Scheduling generate task failed for PCG Component with no owner. Component '%ls' graph '%ls'.",
				*InComponent->GetName(),
				*InComponent->GetGraph()->GetName());
		}
	}

	return GenerateTask;
}

void FPCGWorldPartitionBuilder::WaitForAllAsyncEditorProcesses(UWorld* InWorld)
{
	InWorld->BlockTillLevelStreamingCompleted();

	// Quite a lot of activity can happen here..
	FWorldPartitionHelpers::FakeEngineTick(InWorld);

	// Make sure all actor changes are out of the way, as we are sensitive to these.
	bool bActorsStable = false;
	FDelegateHandle ActorAddedHandle = GEngine->OnLevelActorAdded().AddLambda([&bActorsStable](AActor* InActor) { bActorsStable = false; });
	FDelegateHandle ActorDeletedHandle = GEngine->OnLevelActorDeleted().AddLambda([&bActorsStable](AActor* InActor) { bActorsStable = false; });

	while (!bActorsStable)
	{
		bActorsStable = true;
		FWorldPartitionHelpers::FakeEngineTick(InWorld);
	}

	GEngine->OnLevelActorAdded().Remove(ActorAddedHandle);
	GEngine->OnLevelActorDeleted().Remove(ActorDeletedHandle);

	// Finalize asset compilation before we potentially use them during generation. Example: static mesh collision depends on built SMs.
	// Done before every generation to ensure everything up until now is compiled/built.
	FAssetCompilingManager::Get().FinishAllCompilation();

	// This may execute pending construction scripts.
	FAssetCompilingManager::Get().ProcessAsyncTasks();
}

void FPCGWorldPartitionBuilder::SetDisablePartitionActorCreationForWorld(UWorld* InWorld, bool bDisable)
{
	UPCGSubsystem::SetDisablePartitionActorCreationForWorld(InWorld, bDisable);
}

TUniquePtr<IWorldPartitionActorLoaderInterface::ILoaderAdapter> FPCGWorldPartitionBuilder::ExpandLoadedBoundsIfNeeded(UWorld* InWorld, const FBox& InCurrentBounds, const FBox& InNewBounds, const FString& InLoaderName)
{
	TUniquePtr<IWorldPartitionActorLoaderInterface::ILoaderAdapter> LoaderAdapter;
	if (!InNewBounds.Equals(InCurrentBounds))
	{
		LoaderAdapter = MakeUnique<FLoaderAdapterShape>(InWorld, InNewBounds, InLoaderName);
		LoaderAdapter->Load();

		FWorldPartitionHelpers::FakeEngineTick(InWorld);

		// Try to eliminate as much background noise as possible before getting started.
		FPCGWorldPartitionBuilder::WaitForAllAsyncEditorProcesses(InWorld);

		UE_LOGF(LogPCGWorldPartitionBuilder, Verbose, "Expanding Cell Bounds: Min %ls, Max %ls", *InNewBounds.Min.ToString(), *InNewBounds.Max.ToString());
	}

	return LoaderAdapter;
}

void FPCGWorldPartitionBuilder::GetGenerationGridSizes(const UPCGGraph* InGraph, const APCGWorldActor* InWorldActor, PCGHiGenGrid::FSizeArray& OutGridSizes, bool& bOutHasUnbounded)
{
	if (FGridSizeInfo* FoundGridSizeInfo = GraphToGridSizeInfo.Find(InGraph))
	{
		OutGridSizes = FoundGridSizeInfo->GridSizes;
		bOutHasUnbounded = FoundGridSizeInfo->bHasUnbounded;
	}
	else
	{
		check(PCGHelpers::GetGenerationGridSizes(InGraph, InWorldActor, OutGridSizes, bOutHasUnbounded));

		// Store
		FGridSizeInfo& NewGridSizeInfo = GraphToGridSizeInfo.Add(InGraph);
		NewGridSizeInfo.bHasUnbounded = bOutHasUnbounded;
		NewGridSizeInfo.GridSizes = OutGridSizes;
	}
}

void FPCGWorldPartitionBuilder::WaitOnGraphExecutor(UWorld* InWorld)
{
	UPCGSubsystem* PCGSubsystem = UWorld::GetSubsystem<UPCGSubsystem>(InWorld);
	check(PCGSubsystem);

	while (PCGSubsystem->IsAnyGraphCurrentlyExecuting())
	{
		FWorldPartitionHelpers::FakeEngineTick(InWorld);
	}
		
	FAssetCompilingManager::Get().FinishAllCompilation();
	FAssetCompilingManager::Get().ProcessAsyncTasks();
}

void FPCGWorldPartitionBuilder::ResetPackageDirtyFlags()
{
	TArray<UPackage*> DirtyPackages;

	FEditorFileUtils::GetDirtyWorldPackages(DirtyPackages);
	FEditorFileUtils::GetDirtyContentPackages(DirtyPackages);

	for (UPackage* DirtyPackage : DirtyPackages)
	{
		DirtyPackage->SetDirtyFlag(false);
	}
}

void FPCGWorldPartitionBuilder::SetDisableClearResults(UPCGSubsystem* InSubsystem, bool bInDisableClearResults)
{
	if (InSubsystem)
	{
		InSubsystem->SetDisableClearResults(bInDisableClearResults);
	}
}

bool FPCGWorldPartitionBuilder::IsActorRevertable(AActor* InActor, const FPCGWorldPartitionBuilderArgs& InArgs, FPCGCrc& OutCrc)
{
	if (!InArgs.bRevertUnchangedActors)
	{
		return false;
	}

	if (!InActor->IsPackageExternal())
	{
		return false;
	}

	UE_LOGF(LogPCGWorldPartitionBuilder, VeryVerbose, "IsActorRevertable : '%ls' : begin", *InActor->GetActorLabel());

	const bool bLocalComponents = InActor->IsA<APCGPartitionActor>();
	bool bCanRevertActor = true;
	TArray<const UPCGComponent*> PCGComponents;
	InActor->ForEachComponent(/*bIncludeFromChildActors=*/false, [bLocalComponents, &bCanRevertActor, &PCGComponents](const UActorComponent* InComponent)
	{
		const UPCGComponent* PCGComponent = Cast<UPCGComponent>(InComponent);
		if (PCGComponent && (PCGComponent->IsLocalComponent() == bLocalComponents))
		{
			PCGComponents.Add(PCGComponent);
		}
		else if (!InComponent->IsDefaultSubobject())
		{
			bCanRevertActor = false;
			UE_LOGF(LogPCGWorldPartitionBuilder, VeryVerbose, "IsActorRevertable : '%ls' : Can't revert actor because of non default component '%ls'", *InComponent->GetOwner()->GetActorLabel(), *InComponent->GetName());
		}
	});

	if (!bCanRevertActor)
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, VeryVerbose, "IsActorRevertable : '%ls' : end", *InActor->GetActorLabel());
		return false;
	}

	// Start from a random prime.
	FPCGCrc Crc(1000003);

	for (const UPCGComponent* PCGComponent : PCGComponents)
	{
		FArchiveCrc32 Ar;
		bool bComponentGenerated = PCGComponent->bGenerated;
		Ar << bComponentGenerated;
		if (bComponentGenerated)
		{
			FBox ComponentLastGeneratedBounds = PCGComponent->LastGeneratedBounds;
			Ar << ComponentLastGeneratedBounds;
		}

		Crc.Combine(Ar.GetCrc());

		PCGComponent->ForEachConstManagedResource([&Crc, &bCanRevertActor, InActor](const UPCGManagedResource* ManagedResource)
		{
			FPCGCrc ResourceCrc = ManagedResource->GetCrc();
			if (!ResourceCrc.IsValid())
			{
				bCanRevertActor = false;
				UE_LOGF(LogPCGWorldPartitionBuilder, VeryVerbose, "IsActorRevertable : '%ls' Can't revert actor because of invalid crc for resource : '%ls'", *InActor->GetActorLabel(), *ManagedResource->GetName());
			}
			else
			{
				Crc.Combine(ManagedResource->GetCrc());
				UE_LOGF(LogPCGWorldPartitionBuilder, VeryVerbose, "IsActorRevertable : '%ls' : resource '%ls' : crc '%d'", *InActor->GetActorLabel(), *ManagedResource->GetName(), ResourceCrc.GetValue());
			}
		});

		for (const FPCGTaggedData& Output : PCGComponent->GetGeneratedGraphOutput().TaggedData)
		{
			if (InArgs.OutputsToIgnoreForRevert.Contains(Output.Pin.ToString()))
			{
				UE_LOGF(LogPCGWorldPartitionBuilder, VeryVerbose, "IsActorRevertable : '%ls' : output pin ignored: '%ls'", *InActor->GetActorLabel(), *Output.Pin.ToString());
				continue;
			}

			FPCGCrc OutputCrc = Output.ComputeCrc(/*bFullDataCrc=*/true);
			UE_LOGF(LogPCGWorldPartitionBuilder, VeryVerbose, "IsActorRevertable : '%ls' : output pin: '%ls' : crc '%d'", *InActor->GetActorLabel() , *Output.Pin.ToString(), OutputCrc.GetValue());
			Crc.Combine(OutputCrc);
		}
	}
		
	if (!bCanRevertActor)
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, VeryVerbose, "IsActorRevertable : '%ls' : end", *InActor->GetActorLabel());
		return false;
	}

	OutCrc = Crc;
	UE_LOGF(LogPCGWorldPartitionBuilder, VeryVerbose, "IsActorRevertable : '%ls' : crc : '%d' : end", *InActor->GetActorLabel(), OutCrc.GetValue());
	return true;
}

void FPCGWorldPartitionBuilder::GetRevertableActorsFromComponents(TArray<TWeakObjectPtr<UPCGComponent>> InComponentsToGenerate, const FPCGWorldPartitionBuilderArgs& InArgs, TMap<TSoftObjectPtr<AActor>, FPCGCrc>& OutRevertableActors)
{
	for (TWeakObjectPtr<UPCGComponent> Component : InComponentsToGenerate)
	{
		AActor* Owner = Component.IsValid() ? Component->GetOwner() : nullptr;
		if (Owner)
		{
			FPCGCrc Crc;
			if (!OutRevertableActors.Contains(Owner) && FPCGWorldPartitionBuilder::IsActorRevertable(Owner, InArgs, Crc))
			{
				OutRevertableActors.Add(Owner, Crc);
			}
		}
	}
}

void FPCGWorldPartitionBuilder::GetRevertablePartitionActors(UWorld* InWorld, const FPCGWorldPartitionBuilderArgs& InArgs, TMap<TSoftObjectPtr<AActor>, FPCGCrc>& OutRevertableActors)
{
	for (TActorIterator<APCGPartitionActor> It(InWorld); It; ++It)
	{
		FPCGCrc Crc;
		if (!OutRevertableActors.Contains(*It) && FPCGWorldPartitionBuilder::IsActorRevertable(*It, InArgs, Crc))
		{
			OutRevertableActors.Add(*It, Crc);
		}
	}
}

void FPCGWorldPartitionBuilder::GetRevertableActorsFromDirtyPackages(TArray<UPackage*> InDirtyPackages, const FPCGWorldPartitionBuilderArgs& InArgs, TMap<TSoftObjectPtr<AActor>, FPCGCrc>& OutRevertableActors)
{
	for (UPackage* DirtyPackage : InDirtyPackages)
	{
		if (AActor* Actor = AActor::FindActorInPackage(DirtyPackage))
		{
			FPCGCrc Crc;
			if (!OutRevertableActors.Contains(Actor) && FPCGWorldPartitionBuilder::IsActorRevertable(Actor, InArgs, Crc))
			{
				OutRevertableActors.Add(Actor, Crc);
			}
		}
	}
}

void FPCGWorldPartitionBuilder::RevertUnchangedActors(TArray<UPackage*>& InOutDirtyPackages, const FPCGWorldPartitionBuilderArgs& InArgs, const TMap<TSoftObjectPtr<AActor>, FPCGCrc>& InRevertableActors, TSet<TSoftObjectPtr<AActor>>& InOutRevertedActors)
{
	TMap<TSoftObjectPtr<AActor>, FPCGCrc> DirtyRevertableActors;
	FPCGWorldPartitionBuilder::GetRevertableActorsFromDirtyPackages(InOutDirtyPackages, InArgs, DirtyRevertableActors);

	for (const auto& KVP : DirtyRevertableActors)
	{
		// Actor found in revertable actors and crc matches
		if (const FPCGCrc* PreCrc = InRevertableActors.Find(KVP.Key); PreCrc && *PreCrc == KVP.Value)
		{
			if (InOutDirtyPackages.Remove(KVP.Key->GetPackage()))
			{
				InOutRevertedActors.Add(KVP.Key);
			}
		} 
		else
		{
			// actor not found and or crc doesn't match. This can happen if actor is processed more than once in a per cell situation, one cell might not change the actor but the next one does.
			InOutRevertedActors.Remove(KVP.Key);
		}
	}
}

void FPCGWorldPartitionBuilder::GetModifiedAndNewActorsFromDirtyPackages(const TArray<UPackage*>& InDirtyPackages, TArray<AActor*>& OutModifiedActors, TArray<AActor*>& OutNewActors)
{
	for (UPackage* DirtyPackage : InDirtyPackages)
	{
		if (AActor* Actor = AActor::FindActorInPackage(DirtyPackage))
		{
			if (DirtyPackage->HasAnyPackageFlags(PKG_NewlyCreated))
			{
				OutNewActors.Add(Actor);
			}
			else
			{
				OutModifiedActors.Add(Actor);
			}
		}
	}
}

void FPCGWorldPartitionBuilder::LogRevertedActors(const TSet<TSoftObjectPtr<AActor>>& InRevertedActors)
{
	// Log actor changes
	for (TSoftObjectPtr<AActor> RevertedActorPtr : InRevertedActors)
	{
		if (AActor* RevertedActor = RevertedActorPtr.Get())
		{
			UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Unchanged actor '%ls' (name '%ls' - package '%ls')", *RevertedActor->GetActorLabel(), *RevertedActor->GetName(), *RevertedActor->GetPackage()->GetName());
		}
	}
}

void FPCGWorldPartitionBuilder::LogModifiedAndNewActorsFromDirtyPackages(const TArray<UPackage*>& InDirtyPackages)
{
	TArray<AActor*> ModifiedActors;
	TArray<AActor*> NewActors;
	FPCGWorldPartitionBuilder::GetModifiedAndNewActorsFromDirtyPackages(InDirtyPackages, ModifiedActors, NewActors);

	// Log actor changes
	for (AActor* ModifiedActor : ModifiedActors)
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Modified actor '%ls' (name '%ls' - package '%ls')", *ModifiedActor->GetActorLabel(), *ModifiedActor->GetName(), *ModifiedActor->GetPackage()->GetName());
	}

	for (AActor* NewActor : NewActors)
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Added actor '%ls' (name '%ls' - package '%ls')", *NewActor->GetActorLabel(), *NewActor->GetName(), *NewActor->GetPackage()->GetName());
	}
}

void FPCGWorldPartitionBuilder::SetChainedDispatchToLocalComponents(UWorld* InWorld, bool bInChainedDispatch)
{
	UPCGSubsystem* PCGSubsystem = UWorld::GetSubsystem<UPCGSubsystem>(InWorld);
	check(PCGSubsystem);
	PCGSubsystem->SetChainedDispatchToLocalComponents(bInChainedDispatch);
}

void FPCGWorldPartitionBuilder::UpdateEditingMode(UPCGComponent* InComponent, bool bForce)
{
	if (InComponent->GetSerializedEditingMode() == EPCGEditorDirtyMode::LoadAsPreview && (bForce || InComponent->GetEditingMode() != EPCGEditorDirtyMode::LoadAsPreview))
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Setting PCG editing mode to Load As Preview on actor '%ls' label '%ls' graph '%ls'.",
			*InComponent->GetOwner()->GetName(),
			*InComponent->GetOwner()->GetActorNameOrLabel(),
			*InComponent->GetGraph()->GetName());

		InComponent->SetEditingMode(EPCGEditorDirtyMode::LoadAsPreview, EPCGEditorDirtyMode::LoadAsPreview);
		InComponent->ChangeTransientState(EPCGEditorDirtyMode::LoadAsPreview);
	}
}

FBox FPCGWorldPartitionBuilder::GetPCGBuilderVolumeBounds(UWorld* InWorld)
{
	FBox PCGBuilderVolumeBounds(ForceInit);

	check(InWorld);

	// Override the builder bounds if pcg builder volumes exists
	for (TActorIterator<APCGBuilderVolume> It(InWorld); It; ++It)
	{
		if (APCGBuilderVolume* BuilderVolume = *It)
		{
			if (FBox Bounds = BuilderVolume->GetBounds().GetBox(); Bounds.IsValid)
			{
				PCGBuilderVolumeBounds += Bounds;
				UE_LOGF(LogPCGWorldPartitionBuilder, Display, "Found PCGBuilderVolume actor '%ls' label '%ls' bounds : %ls",
					*BuilderVolume->GetName(),
					*BuilderVolume->GetActorNameOrLabel(),
					*Bounds.ToString());
			}
			else
			{
				UE_LOGF(LogPCGWorldPartitionBuilder, Warning, "Found PCGBuilderVolume actor '%ls' label '%ls' with invalid bounds (ignored)",
					*BuilderVolume->GetName(),
					*BuilderVolume->GetActorNameOrLabel());
			}
		}
	}

	if (PCGBuilderVolumeBounds.IsValid)
	{
		UE_LOGF(LogPCGWorldPartitionBuilder, Display, "PCG builder bounds '%ls'", *PCGBuilderVolumeBounds.ToString());
	}

	return PCGBuilderVolumeBounds;
}

TArray<UPackage*> FPCGWorldPartitionBuilder::GetDirtyPackages(const UWorld* InWorld, const TSet<FString>& AllowedAssetPackages)
{
	TArray<UPackage*> DirtyPackages;
	TSet<UPackage*> ExternalPackages;

	const UPackage* WorldPackage = InWorld->GetPackage();
	const bool bIsPartitioned = InWorld->IsPartitionedWorld();

	ExternalPackages.Append(WorldPackage->GetExternalPackages());

	FEditorFileUtils::GetDirtyWorldPackages(DirtyPackages, /*ShouldIgnorePackageFunction=*/[bIsPartitioned, WorldPackage, &AllowedAssetPackages, &ExternalPackages](UPackage* DirtyPackage)
	{
		const bool bExplicitelyAllowedAsset = AllowedAssetPackages.Contains(DirtyPackage->GetName());

		// Don't allow modification of World Partition umap unless allowed in AllowedAssetPackages
		if (WorldPackage == DirtyPackage)
		{
			return bIsPartitioned ? !bExplicitelyAllowedAsset : false;
		}
		
		if (bIsPartitioned)
		{
			// WP world should only allow current world external packages (actors/folders) or explicitely allowed assets to be dirtied.
			return !bExplicitelyAllowedAsset && !ExternalPackages.Contains(DirtyPackage);
		}
		else
		{
			// Non-WP world should only allow umap to be dirtied and explicitely allowed assets.
			return !bExplicitelyAllowedAsset;
		}
	});
	
	if (!AllowedAssetPackages.IsEmpty())
	{
		TArray<UPackage*> DirtyAssetPackages;
		FEditorFileUtils::GetDirtyContentPackages(DirtyAssetPackages, /*ShouldIgnorePackageFunction=*/[&AllowedAssetPackages](UPackage* DirtyPackage)
		{
			return !AllowedAssetPackages.Contains(DirtyPackage->GetName());
		});

		DirtyPackages.Append(DirtyAssetPackages);
	}

	DirtyPackages.Remove(nullptr);
	return DirtyPackages;
}

bool FPCGWorldPartitionBuilder::Build(UWorld* InWorld, const TArray<FString>& Args)
{
	if (InWorld)
	{
		IWorldPartitionEditorModule::FRunBuilderParams Params;
		Params.BuilderClass = UPCGWorldPartitionBuilder::StaticClass();
		Params.World = InWorld;
		Params.OperationDescription = FText::FromString("Generating PCG Components...");
		
		Params.ExtraArgs = TEXT("-AllowCommandletRendering -AllowSoftwareRendering -AssetGatherAll=true");
		for (const FString& Arg : Args)
		{
			Params.ExtraArgs += " ";
			Params.ExtraArgs += Arg;
		}

		return IWorldPartitionEditorModule::Get().RunBuilder(Params);
	}

	return false;
}

bool UPCGWorldPartitionBuilder::CanBuild(const UWorld* InWorld, FName InBuildOption)
{
	return InWorld != nullptr;
}

EEditorBuildResult UPCGWorldPartitionBuilder::Build(UWorld* InWorld, FName InBuildOption)
{
	TSharedPtr<SWindow> DlgWindow =
		SNew(SWindow)
		.Title(LOCTEXT("BuildPCGWindowTitle", "Build PCG"))
		.ClientSize(SPCGBuilderDialog::DEFAULT_WINDOW_SIZE)
		.SizingRule(ESizingRule::UserSized)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.SizingRule(ESizingRule::FixedSize);

	TSharedRef<SPCGBuilderDialog> BuildDialog =
		SNew(SPCGBuilderDialog)
		.ParentWindow(DlgWindow);

	DlgWindow->SetContent(BuildDialog);

	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	FSlateApplication::Get().AddModalWindow(DlgWindow.ToSharedRef(), MainFrameModule.GetParentWindow());

	if (BuildDialog->GetDialogResult() != SPCGBuilderDialog::DialogResult::Cancel)
	{
		TArray<FString> Args;
		FString BuilderSettingStr = BuildDialog->GetBuilderSetting().ToString();
		if(!BuilderSettingStr.IsEmpty())
		{
			Args.Add("-PCGBuilderSettings=" + BuilderSettingStr);
		}
		return FPCGWorldPartitionBuilder::Build(InWorld, Args) ? EEditorBuildResult::Success : EEditorBuildResult::Skipped;
	}

	return EEditorBuildResult::Skipped;
}

FSoftObjectPath FPCGWorldPartitionBuilder::GetActorPathFromComponentPtr(TSoftObjectPtr<UPCGComponent> InComponentPtr)
{
	const FString PersistentLevelString(TEXT("PersistentLevel"));

	FSoftObjectPath ComponentPath = InComponentPtr.ToSoftObjectPath();
	FString PersistentLevel, SubObjectPath;
	if (ComponentPath.GetSubPathString().Split(TEXT("."), &PersistentLevel, &SubObjectPath))
	{
		if (PersistentLevel == PersistentLevelString)
		{
			FString ActorName;
			FString ComponentName;
			if (SubObjectPath.Split(TEXT("."), &ActorName, &ComponentName))
			{
				return FSoftObjectPath(ComponentPath.GetAssetPath(), PersistentLevelString + "." + ActorName);
			}
		}
	}

	return FSoftObjectPath();
}

void FPCGWorldPartitionBuilder::LoadPartitionedActorDependencies(UWorld* InWorld, TSet<FWorldPartitionReference>& OutLoadedActorDependencies)
{
	UWorldPartition* WorldPartition = InWorld->GetWorldPartition();

	// Loop on every loaded APCGPartitionActor (PAs):
	// 
	// Load missing Original Component Actors: this can happen when the cell getting loaded intersects with the PA but not the Original Component Volume.
	// When this happens we need to call RebuildOriginalToLocal so that the PAs internal Original to Local mappings are up to date
	//
	// Load missing Managed Actors: this will allow proper cleanup of managed resources later when processing the PAs
	for (TActorIterator<APCGPartitionActor> It(InWorld); It; ++It)
	{
		APCGPartitionActor* PartitionActor = *It;

		if (PartitionActor->GetLevel() == InWorld->PersistentLevel)
		{
			bool bRebuildOriginalToLocal = false;

			for (TObjectPtr<UPCGComponent> PCGComponent : PartitionActor->GetAllLocalPCGComponents())
			{
				if (PCGComponent)
				{
					// If the Original Component for this Local Component isn't loaded, try and load it (this is to avoid removing a dead instance because original isn't loaded yet)
					TSoftObjectPtr<UPCGComponent> OriginalPCGComponentPtr = PartitionActor->GetOriginalComponentSoftObjectPtr(PCGComponent);
					if (!OriginalPCGComponentPtr.IsNull() && !OriginalPCGComponentPtr.IsValid())
					{
						// The Original Component Path will be something like /Game/SomePath/SomeWorld.SomeWorld:PersistentLevel.ActorName.PCGComponentName
						// and we want to get /Game/SomePath/SomeWorld.SomeWorld:PersistentLevel.ActorName to load it
						FSoftObjectPath ActorPath = FPCGWorldPartitionBuilder::GetActorPathFromComponentPtr(OriginalPCGComponentPtr);
						if (ActorPath.IsValid())
						{
							if (const FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstanceByPath(ActorPath))
							{
								FWorldPartitionReference ActorReference(WorldPartition, ActorDescInstance->GetGuid());
								if (AActor* LoadedActor = ActorReference.GetActor())
								{
									OutLoadedActorDependencies.Add(ActorReference);
									UE_LOGF(LogPCGWorldPartitionBuilder, Verbose, "Loaded Original PCG Component Actor '%ls' label '%ls'.",
										*LoadedActor->GetName(),
										*LoadedActor->GetActorNameOrLabel());
									bRebuildOriginalToLocal = true;
								}
							}
						}
					}
				}
			}

			// An Original Component was loaded so we need to update the PAs internal mappings 
			if (bRebuildOriginalToLocal)
			{
				PartitionActor->RebuildOriginalToLocal();
			}

			// If this Partition Actor has some Managed Actors that aren't loaded, try and load them (this is to make sure we can delete them properly)
			TArray<TSoftObjectPtr<AActor>> ManagedActorPaths = UPCGComponent::GetManagedActorPaths(PartitionActor);
			for (const TSoftObjectPtr<AActor>& ManagedActorPath : ManagedActorPaths)
			{
				if (!ManagedActorPath.IsNull() && !ManagedActorPath.IsValid())
				{
					if (const FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstanceByPath(ManagedActorPath.ToSoftObjectPath()))
					{
						FWorldPartitionReference ActorReference(WorldPartition, ActorDescInstance->GetGuid());
						if (AActor* LoadedActor = ActorReference.GetActor())
						{
							OutLoadedActorDependencies.Add(ActorReference);
							UE_LOGF(LogPCGWorldPartitionBuilder, Verbose, "Loaded Managed Actor '%ls' label '%ls'.",
								*LoadedActor->GetName(),
								*LoadedActor->GetActorNameOrLabel());
						}
					}
				}
			}
		}
	}
}


bool FPCGWorldPartitionBuilder::CreatePartitionedActors(UWorld* InWorld, const FPCGWorldPartitionBuilderArgs& InArgs, TMap<TSoftObjectPtr<UPCGComponent>, FWorldPartitionReference>& InOutPartitionedComponents)
{
	// Scope object tracks if packages were dirtied 
	FPCGDetectDirtyPackageInScope DirtyPackageInScope;

	UPCGSubsystem* PCGSubsystem = UWorld::GetSubsystem<UPCGSubsystem>(InWorld);
	check(PCGSubsystem);

	auto ComponentFilter = [InWorld, &InOutPartitionedComponents](const UPCGComponent* InComponent)
	{
		// Only generate Partitioned Components part of the persistent level
		if (InComponent->GetComponentLevel() != InWorld->PersistentLevel)
		{
			return false;
		}

		// Only process each component once
		if (InOutPartitionedComponents.Contains(InComponent))
		{
			return false;
		}

		return InComponent->IsPartitioned();
	};

	TArray<TWeakObjectPtr<UPCGComponent>> PartitionedComponentsToProcess;
	FPCGWorldPartitionBuilder::CollectComponentsToGenerate(InWorld, InArgs, ComponentFilter, PartitionedComponentsToProcess);

	UWorldPartition* WorldPartition = InWorld->GetWorldPartition();

	for (TWeakObjectPtr<UPCGComponent> ComponentPtr : PartitionedComponentsToProcess)
	{
		if (UPCGComponent* PCGComponent = ComponentPtr.Get())
		{
			// Partitioned components need their editing mode updated before we start creating/updating PAs
			FPCGWorldPartitionBuilder::UpdateEditingMode(PCGComponent);

			check(!InOutPartitionedComponents.Contains(PCGComponent));
			InOutPartitionedComponents.Add(PCGComponent, FWorldPartitionReference(WorldPartition, PCGComponent->GetOwner()->GetActorGuid()));

			bool bHasUnbounded = false;
			PCGHiGenGrid::FSizeArray GridSizes;
			FPCGWorldPartitionBuilder::GetGenerationGridSizes(PCGComponent->GetGraph(), PCGSubsystem->GetPCGWorldActor(), GridSizes, bHasUnbounded);

			if (!PCGComponent->bGenerated && !bHasUnbounded)
			{
				PCGComponent->Modify();
				PCGComponent->bGenerated = true;
				PCGComponent->LastGeneratedBounds = PCGComponent->GetGridBounds();
			}

			if (!GridSizes.IsEmpty())
			{
				PCGSubsystem->CreatePartitionActorsWithinBounds(PCGComponent, PCGComponent->GetGridBounds(), GridSizes);
			}
		}
	}

	return DirtyPackageInScope.AnyDirtyPackage();
}

bool FPCGWorldPartitionBuilder::UpdatePartitionedActors(UWorld* InWorld, const TMap<TSoftObjectPtr<UPCGComponent>, FWorldPartitionReference>& InPartitionedComponents, TSet<FString>& InOutDeletedActorPackages)
{
	// Scope object tracks if packages were dirtied 
	FPCGDetectDirtyPackageInScope DirtyPackageInScope;

	// Will track any deleted actors in this scope
	FPCGDetectDeletedActorInScope DeletedActorScope(InWorld, InOutDeletedActorPackages);

	UPCGSubsystem* PCGSubsystem = UWorld::GetSubsystem<UPCGSubsystem>(InWorld);
	check(PCGSubsystem);

	TArray<TKeyValuePair<UPCGComponent*, TSet<TObjectPtr<APCGPartitionActor>>>> UpdatedMappings;

	// When loading existing PAs, we need to Update the mappings to add missing PCG Components
	for (auto KeyValuePair : InPartitionedComponents)
	{
		if (UPCGComponent* PartitionedComponent = KeyValuePair.Key.Get())
		{
			// Those updated mappings will be partial (only include loaded PAs) but that is ok because here we want to make sure loaded PAs get the missing graph instances and in the next loop remove the graph
			// instances from the loaded PAs no longer in the mappings
			PCGSubsystem->UpdateMappingPCGComponentPartitionActor(PartitionedComponent);

			UpdatedMappings.Add({ PartitionedComponent, PCGSubsystem->GetPCGComponentPartitionActorMappings(PartitionedComponent) });
		}
	}

	// If PartitionActor is loaded and it has invalid Original Components it means that they need to be cleaned up
	for (TActorIterator<APCGPartitionActor> It(InWorld); It; ++It)
	{
		// Cleanup Graph Instances to invalid Original Components
		APCGPartitionActor* PartitionActor = *It;

		if (PartitionActor->GetLevel() == InWorld->PersistentLevel)
		{
			// Here we cleanup dead graphs since we tried to load the Original components before, if they can't resolve it means they don't exist anymore.
			TArray<UPCGComponent*> DeadLocalComponents;

			// Test first so we can update editing mode before doing the clean up
			if (PartitionActor->CleanupDeadGraphInstancesInternal(/*bCleanupTest=*/true, &DeadLocalComponents))
			{
				for (UPCGComponent* DeadLocalComponent : DeadLocalComponents)
				{
					// Update editing mode before cleaning up to make sure cleanup gets persisted
					FPCGWorldPartitionBuilder::UpdateEditingMode(DeadLocalComponent);
				}

				PartitionActor->CleanupDeadGraphInstancesInternal(/*bCleanupTest=*/false);
			}

			// Remove Graph Instances from Partition Actors that are no longer part of the component mappings (bounds change)
			for (const TKeyValuePair<UPCGComponent*, TSet<TObjectPtr<APCGPartitionActor>>>& Mapping : UpdatedMappings)
			{
				if (!Mapping.Value.Contains(PartitionActor))
				{
					if (UPCGComponent* LocalComponent = PartitionActor->GetLocalComponent(Mapping.Key))
					{
						// Update editing mode before removing graph instance to make sure cleanup gets persisted
						FPCGWorldPartitionBuilder::UpdateEditingMode(LocalComponent);

						PartitionActor->RemoveGraphInstance(Mapping.Key);
					}
				}
			}
		}
	}

	return DirtyPackageInScope.AnyDirtyPackage();
}

void FPCGWorldPartitionBuilder::CollectSubGridComponentsToGenerate(UWorld* InWorld, const FPCGWorldPartitionBuilderArgs& InArgs, const TArray<TWeakObjectPtr<UPCGComponent>>& InComponentsToGenerate, TArray<UPCGComponent*>& OutAdditionalComponentsToGenerate, TMap<UPCGComponent*, UPCGComponent*>& OutDependencies)
{
	UPCGSubsystem* PCGSubsystem = UWorld::GetSubsystem<UPCGSubsystem>(InWorld);
	check(PCGSubsystem);

	for (TWeakObjectPtr<UPCGComponent> WeakComponentToGenerate : InComponentsToGenerate)
	{
		if (UPCGComponent* ComponentToGenerate = WeakComponentToGenerate.Get())
		{
			if (APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(ComponentToGenerate->GetOwner()))
			{
				if (UPCGComponent* OriginalComponent = PartitionActor->GetOriginalComponent(ComponentToGenerate))
				{
					// Add dependency on Unbounded generation if it is part of the components to generate
					if (InComponentsToGenerate.Contains(OriginalComponent))
					{
						OutDependencies.Add(ComponentToGenerate, OriginalComponent);
					}
					
					bool bHasUnbounded = false;
					PCGHiGenGrid::FSizeArray GridSizes;
					FPCGWorldPartitionBuilder::GetGenerationGridSizes(OriginalComponent->GetGraph(), PCGSubsystem->GetPCGWorldActor(), GridSizes, bHasUnbounded);

					check(GridSizes[0] == PartitionActor->GetPCGGridSize());
					const int32 GridSizeIndex = 0;
					CollectSubGridComponentsToGenerateInternal(PCGSubsystem, GridSizeIndex + 1, GridSizes, OriginalComponent, PartitionActor, OutAdditionalComponentsToGenerate, OutDependencies);
				}
			}
		}
	}

	// Sort by GridSize
	OutAdditionalComponentsToGenerate.Sort([](const UPCGComponent& A, const UPCGComponent& B)
	{
		return Cast<APCGPartitionActor>(A.GetOwner())->GetPCGGridSize() > Cast<APCGPartitionActor>(B.GetOwner())->GetPCGGridSize();
	});
}

void FPCGWorldPartitionBuilder::CollectSubGridComponentsToGenerateInternal(UPCGSubsystem* InPCGSubsystem, int32 InGridSizeIndex, const PCGHiGenGrid::FSizeArray& InGridSizes, UPCGComponent* InOriginalComponent, APCGPartitionActor* InPartitionActor, TArray<UPCGComponent*>& OutAdditionalComponentsToGenerate, TMap<UPCGComponent*, UPCGComponent*>& OutDependencies)
{
	if (InGridSizeIndex >= InGridSizes.Num())
	{
		return;
	}

	// Get the next Grid size overlapping components
	TArray<UPCGComponent*> LocalAdditionalComponentsToGenerate;
	InPCGSubsystem->ForAllOverlappingCells(InOriginalComponent, InPartitionActor->GetFixedBounds(), { InGridSizes[InGridSizeIndex] }, false, {},
		[&InGridSizes, InGridSizeIndex, InPartitionActor, InOriginalComponent, &LocalAdditionalComponentsToGenerate, &OutDependencies](APCGPartitionActor* SubPartitionActor, const FBox& InBounds)
		{
			check(SubPartitionActor->GetPCGGridSize() == InGridSizes[InGridSizeIndex]);
			if (InPartitionActor->GetFixedBounds().IsInsideXY(SubPartitionActor->GetFixedBounds().GetCenter()))
			{
				if (UPCGComponent* LocalComponent = SubPartitionActor->GetLocalComponent(InOriginalComponent))
				{
					LocalAdditionalComponentsToGenerate.Add(LocalComponent);
					OutDependencies.Add(LocalComponent, InPartitionActor->GetLocalComponent(InOriginalComponent));
				}
			}
			return InvalidPCGTaskId;
		});

	OutAdditionalComponentsToGenerate.Append(LocalAdditionalComponentsToGenerate);

	for (UPCGComponent* AdditionalComponentToGenerate : LocalAdditionalComponentsToGenerate)
	{
		APCGPartitionActor* SubPartitionActor = CastChecked<APCGPartitionActor>(AdditionalComponentToGenerate->GetOwner());
		CollectSubGridComponentsToGenerateInternal(InPCGSubsystem, InGridSizeIndex + 1, InGridSizes, InOriginalComponent, SubPartitionActor, OutAdditionalComponentsToGenerate, OutDependencies);
	}
}

void FPCGWorldPartitionBuilder::CollectComponentsToGenerate(UWorld* InWorld, const FPCGWorldPartitionBuilderArgs& InArgs, TFunctionRef<bool(const UPCGComponent*)> ComponentFilter, TArray<TWeakObjectPtr<UPCGComponent>>& OutComponents)
{
	check(InWorld);

	TArray<UObject*> AllComponents;
	GetObjectsOfClass(UPCGComponent::StaticClass(), AllComponents, /*bIncludeDerivedClasses=*/true);
	for (UObject* ComponentObject : AllComponents)
	{
		if (!IsValid(ComponentObject))
		{
			continue;
		}

		UPCGComponent* Component = Cast<UPCGComponent>(ComponentObject);
		if (!Component || !Component->GetOwner() || Component->GetWorld() != InWorld)
		{
			continue;
		}

		// Check actor in inclusion list if provided.
		if (!InArgs.IncludeActorIDs.IsEmpty() && !InArgs.IncludeActorIDs.Contains(Component->GetOwner()->GetName()))
		{
			continue;
		}

		// Accept based on editing mode.
		const bool bValidEditingMode = (InArgs.bGenerateEditingModeLoadAsPreviewComponents && Component->GetSerializedEditingMode() == EPCGEditorDirtyMode::LoadAsPreview)
			|| (InArgs.bGenerateEditingModeNormalComponents && Component->GetSerializedEditingMode() == EPCGEditorDirtyMode::Normal)
			|| (InArgs.bGenerateEditingModePreviewComponents && Component->GetSerializedEditingMode() == EPCGEditorDirtyMode::Preview);

		if (!bValidEditingMode)
		{
			continue;
		}

		if (!ComponentFilter(Component))
		{
			continue;
		}

		UPCGGraph* Graph = Component->GetGraph();
		if (!Graph)
		{
			UE_LOGF(LogPCGWorldPartitionBuilder, Warning, "PCG component on actor '%ls' label '%ls' has no graph assigned, skipping.",
				*Component->GetOwner()->GetName(),
				*Component->GetOwner()->GetActorNameOrLabel());
			continue;
		}

		if (!InArgs.IncludeGraphNames.IsEmpty())
		{
			bool bFound = false;
			for (const FString& IncludeGraphName : InArgs.IncludeGraphNames)
			{
				if (Graph->GetName() == IncludeGraphName)
				{
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				continue;
			}
		}
		else if (!InArgs.IncludeGraphs.IsEmpty())
		{
			bool bFound = false;
			for (const TSoftObjectPtr<UPCGGraphInterface>& IncludeGraph : InArgs.IncludeGraphs)
			{
				if (TSoftObjectPtr<UPCGGraphInterface>(Graph) == IncludeGraph)
				{
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				continue;
			}
		}

		if (!Component->bActivated)
		{
			UE_LOGF(LogPCGWorldPartitionBuilder, Display, "'Activated' toggle was set false on PCG component on actor '%ls' label '%ls' graph '%ls'. Component skipped.",
				*Component->GetOwner()->GetName(),
				*Component->GetOwner()->GetActorNameOrLabel(),
				*Component->GetGraph()->GetName());
			continue;
		}

		if (Component->IsManagedByRuntimeGenSystem())
		{
			UE_LOGF(LogPCGWorldPartitionBuilder, Display, "PCG component generation trigger is set to run-time generation on actor '%ls' label '%ls' graph '%ls'. Component skipped.",
				*Component->GetOwner()->GetName(),
				*Component->GetOwner()->GetActorNameOrLabel(),
				*Component->GetGraph()->GetName());
			continue;
		}

		OutComponents.Add(Component);
	}
}

#undef LOCTEXT_NAMESPACE
