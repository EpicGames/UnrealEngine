// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCrc.h"

#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartition/WorldPartitionHandle.h"

#include "EditorBuildUtils.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "PCGWorldPartitionBuilder.generated.h"

class UWorld;
class UPCGComponent;
class UPCGBuilderSettings;
class UPCGGraphInterface;

struct FPCGWorldPartitionCommandlineArgs
{
	/** Include components which have editing mode set to LoadAsPreview. */
	TOptional<bool> bGenerateEditingModeLoadAsPreviewComponents;

	/** Include components which have editing mode set to Normal. */
	TOptional<bool> bGenerateEditingModeNormalComponents;

	/** Include components which have editing mode set to Preview. */
	TOptional<bool> bGenerateEditingModePreviewComponents;
		
	/** If non empty, components with each graph name will be generated (if editing mode is included), in order. */
	TOptional<TArray<FString>> IncludeGraphNames;

	/** Call generate on each component and wait until completion and any async processes before generating the next. */
	TOptional<bool> bOneComponentAtATime;

	/** If non empty, only components on actors with given ID(s) will be generated. */
	TOptional<TArray<FString>> IncludeActorIDs;

	/** Submit dirty files even if errors occurred during generation. */
	TOptional<bool> bIgnoreGenerationErrors;
		
	/** Path to builder settings asset */
	TOptional<TSoftObjectPtr<UPCGBuilderSettings>> SettingAsset;

	/** Flag to generate using an iterative cell loading. Useful for large maps to limit memory consumption. */
	TOptional<bool> bIterativeCellLoading;

	/** Cell size for iterative cell loading */
	TOptional<int32> IterativeCellSize;
};

struct FPCGWorldPartitionBuilderArgs
{
	/** Include components which have editing mode set to LoadAsPreview. */
	bool bGenerateEditingModeLoadAsPreviewComponents = true;

	/** Include components which have editing mode set to Normal. */
	bool bGenerateEditingModeNormalComponents = false;

	/** Include components which have editing mode set to Preview. */
	bool bGenerateEditingModePreviewComponents = false;

	/** If non empty, components with each graph name will be generated (if editing mode is included), in order. */
	TArray<FString> IncludeGraphNames;

	/** If non empty, components with each graph asset will be generated (if editing mode is included), in order */
	TArray<TSoftObjectPtr<UPCGGraphInterface>> IncludeGraphs;

	/** Call generate on each component and wait until completion and any async processes before generating the next. */
	bool bOneComponentAtATime = false;

	/** If non empty, only components on actors with given ID(s) will be generated. */
	TArray<FString> IncludeActorIDs;

	/** Submit dirty files even if errors occurred during generation. */
	bool bIgnoreGenerationErrors = false;

	/** Flag to generate using an iterative cell loading. Useful for large maps to limit memory consumption. */
	bool bIterativeCellLoading = false;

	/** Cell size for iterative cell loading */
	int32 IterativeCellSize = 25600;

	/** Will load editor only data layers without having to specify them as included */
	bool bLoadEditorOnlyDataLayers = true;

	/** Will load runtime data layers that are default activated without having to specify them as included */
	bool bLoadActivatedRuntimeDataLayers = true;

	/** Data layers that need to be loaded when running builder. */
	TArray<TObjectPtr<UDataLayerAsset>> IncludedDataLayers;

	/** Data layers that need to be unloaded when running builder. */
	TArray<TObjectPtr<UDataLayerAsset>> ExcludedDataLayers;

	/** Experimental feature that will try and revert PCG Component Actors which haven't changed from their previous generation. */
	bool bRevertUnchangedActors = false;

	/** Outputs that can change on every generation but do not have an impact on final result should be listed here so they can be ignored when comparing results for revert. */
	TArray<FString> OutputsToIgnoreForRevert;

	static bool InitializeFrom(const FPCGWorldPartitionCommandlineArgs& CommandlineArgs, FPCGWorldPartitionBuilderArgs& OutArgs);
};

UCLASS()
class UPCGWorldPartitionBuilderHelper : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	/** Notify the running PCG Builder that this asset can be saved/submitted as part of the builder execution if it is dirty */
	UFUNCTION(BlueprintCallable, Category = "PCG|Builder")
	static void AddAssetToSubmitAllowList(const TSoftObjectPtr<UObject>& InPath);
};

/**
* Builder that triggers generation on PCG components.
* 
* Example command line:
*   ProjectName MapName -Unattended -AllowCommandletRendering -run=WorldPartitionBuilderCommandlet -Builder=PCGWorldPartitionBuilder -SCCProvider=[ProviderName] -AllowCommandletRendering -AllowSoftwareRendering -AssetGatherAll=true
* 
* Optional switches:
* 
*  -PCGBuilderSettings=/Game/Path/BuilderSettingsAssetName (default: none)
*  -IncludeGraphNames=PCG_GraphA;PCG_GraphB (default: all graphs)
*  -GenerateComponentEditingModeLoadAsPreview (default: true)
*  -GenerateComponentEditingModeNormal (default: false)
*  -GenerateComponentEditingModePreview (default: false)
*  -IgnoreGenerationErrors (default: false)
*  -IncludeActorIDs=MyActor1_UAID1234678;MyActor2_UAID1234678 (default: all actors)
*  -OneComponentAtATime (default: false)
*  -IterativeCellLoading (default: false)
*  -IterativeCellSize=25600 (default: 25600)
*/
UCLASS()
class UPCGWorldPartitionBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()

public:
	// Whether to require initialization of rendering, for now default to true. Requires a GPU present.
	// Get Texture Data benefits from rendering - means the GPU will be available as a fallback sampling method so
	// that the "CPU sampling" option does not have to be enabled on the texture. This may expand to other nodes if
	// we use GPU resources or compute more in the future.
	virtual bool RequiresCommandletRendering() const override { return true; }

	// In the future we may load a world in stages, but for now load everything to make sure we have the complete level.
	virtual ELoadingMode GetLoadingMode() const override { return Args.bIterativeCellLoading ? ELoadingMode::IterativeCells2D : ELoadingMode::EntireWorld; }
		
	void AddAssetToSubmitAllowList(const TSoftObjectPtr<UObject>& SoftObjectPath);

protected:
	virtual bool PreWorldInitialization(UWorld* World, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool PostRun(UWorld* World, FPackageSourceControlHelper& PackageHelper, const bool bInRunSuccess) override;
	
	bool RunEntireWorld(UWorld* World, FPackageSourceControlHelper& PackageHelper);
	bool RunPerCell(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper);

	virtual bool CanProcessNonPartitionedWorlds() const override { return true; }

	/** Save all the pending dirty and deleted packages. */
	bool SaveEntireWorldPackages(UWorld* World, FPackageSourceControlHelper& PackageHelper, TArray<UPackage*> DirtyPackages, TSet<FString> DeletedPackages);

	/** Save all the cell packages */
	bool SavePerCellPackages(UWorld* World, FPackageSourceControlHelper& PackageHelper, TSet<FString> CellDeletedPackages);

	friend class FPCGEditorModule;
	static bool CanBuild(const UWorld* InWorld, FName InBuildOption);
	static EEditorBuildResult Build(UWorld* InWorld, FName InBuildOption);
private:
	/** Track file states to build changelist in PostRun */
	FName PreviousProviderName;
	TSet<FString> NewPackageNames;
	TSet<FString> ModifiedPackageNames;
	TSet<FString> DeletedPackageNames;

	/** Track the generated components to avoid multiple generations when cells overlap */
	TSet<TSoftObjectPtr<const UPCGComponent>> GeneratedComponents;
	/** Processed Original Partitioned Components + Reference to owning actor to avoid unloading Original Components and causing destructive changes to Partition Actors, we keep them loaded until PostRun */
	TMap<TSoftObjectPtr<UPCGComponent>, FWorldPartitionReference> PartitionedComponents;
		
	FPCGWorldPartitionCommandlineArgs CommandlineArgs;
	FPCGWorldPartitionBuilderArgs Args;
		
	/** Non Actor assets that are allowed to be submitted by the builder */
	TSet<FString> AllowedAssetPackages;

	/** Revertable Actors */
	TMap<TSoftObjectPtr<AActor>, FPCGCrc> RevertableActors;

	/** Reverted Actors : when using the builder in per cell iteration, it is possible that one cell reverts an actor because it doesn't modify it but the next one will so keep track for stats */
	TSet<TSoftObjectPtr<AActor>> RevertedActors;

	/** Final stats */
	int32 NumNewPackages = 0;
	int32 NumModifiedPackages = 0;
	int32 NumDeletedPackages = 0;
	int32 NumRevertedPackages = 0;

	/** Flag to register if any error occurred during generation. */
	bool bErrorOccurredWhileGenerating = false;
};
