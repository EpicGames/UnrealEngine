// Copyright Epic Games, Inc. All Rights Reserved.
#include "SPCGBuilderDialog.h"

#include "PCGEditorSettings.h"

#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"

#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SPCGBuilderDialog)

#define LOCTEXT_NAMESPACE "PCGBulderDialog"

const FVector2D SPCGBuilderDialog::DEFAULT_WINDOW_SIZE = FVector2D(600, 180);

void SPCGBuilderDialog::Construct(const FArguments& InArgs)
{
	ParentWindowPtr = InArgs._ParentWindow.Get();
	Result = DialogResult::Cancel;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	TSharedPtr<IStructureDetailsView> StructureDetailsView;

	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.NotifyHook = nullptr;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bShowScrollBar = false;
	}

	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = true;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = true;
	}

	StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);
	
	Params.BuilderSetting = GetDefault<UPCGEditorProjectSettings>()->DefaultBuilderSetting;

	FStructOnScope* Struct = new FStructOnScope(FPCGBuilderDialogParams::StaticStruct(), (uint8*)&Params);
	StructureDetailsView->SetStructureData(MakeShareable(Struct));
	
	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(10.f)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(LOCTEXT("BuildPCGDescription", "Note that this will require the current map be saved & unloaded. It will be reopened once the operation is completed."))
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					StructureDetailsView->GetWidget()->AsShared()
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SSpacer)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SPCGBuilderDialog::OnBuildClicked)
					.Text(LOCTEXT("BuildButton", "Build"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SPCGBuilderDialog::OnCancelClicked)
					.Text(LOCTEXT("CancelButton", "Cancel"))
				]
			]
		]
	];
}

FReply SPCGBuilderDialog::OnBuildClicked()
{
	Result = DialogResult::Build;
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	return FReply::Handled();
}

FReply SPCGBuilderDialog::OnCancelClicked()
{
	Result = DialogResult::Cancel;
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
