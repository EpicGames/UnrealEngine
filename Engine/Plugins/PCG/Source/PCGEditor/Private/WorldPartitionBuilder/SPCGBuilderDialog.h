// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "SPCGBuilderDialog.generated.h"

class UPCGBuilderSettings;
class SWindow;

USTRUCT()
struct FPCGBuilderDialogParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Default)
	TSoftObjectPtr<UPCGBuilderSettings> BuilderSetting;
};

//////////////////////////////////////////////////////////////////////////
// SPCGBuilderDialog

class SPCGBuilderDialog : public SCompoundWidget
{
public:
	enum DialogResult
	{
		Build,
		Cancel
	};

	SLATE_BEGIN_ARGS(SPCGBuilderDialog) {}
	/** A pointer to the parent window */
	SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

	~SPCGBuilderDialog() = default;

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 */
	void Construct(const FArguments& InArgs);

	static const FVector2D DEFAULT_WINDOW_SIZE;

	DialogResult GetDialogResult() const { return Result; }
	TSoftObjectPtr<UPCGBuilderSettings> GetBuilderSetting() const { return Params.BuilderSetting; }

private:
	FReply OnBuildClicked();
	FReply OnCancelClicked();

	/** Pointer to the parent window, so we know to destroy it when done */
	TWeakPtr<SWindow> ParentWindowPtr;

	/** Dialog Result */
	DialogResult Result;

	/** Dialog Params */
	FPCGBuilderDialogParams Params;
};
