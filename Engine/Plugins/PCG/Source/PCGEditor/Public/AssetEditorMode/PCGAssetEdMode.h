// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveToolManager.h"
#include "Tools/UEdMode.h"

#include "PCGAssetEdMode.generated.h"

class FEditorViewportClient;

/**
 * Editor mode used exclusively within the PCG Asset Editor window (FPCGEditor).
 *
 * Unlike UPCGEditorMode, which is a globally registered level-editor mode with viewport focus
 * customization, auto-save control, and per-tick updates, this mode is instantiated and owned
 * directly by the asset editor. It holds a back-reference to the owning FPCGEditor and has no
 * static Register/Unregister lifecycle — it is activated and deactivated alongside the editor tab.
 */
UCLASS(Transient)
class UPCGAssetEditorMode : public UEdMode
{
	GENERATED_BODY()

public:
	/** Unique name for PCG Asset Editor Mode */
	PCGEDITOR_API const static FEditorModeID EM_PCGAssetEditorModeId;

	UPCGAssetEditorMode();

	/** Focus event when the Asset Editor enters PCG Editor Mode. */
	virtual void Enter() override;

	/** Focus event when the Asset Editor leaves PCG Editor Mode. */
	virtual void Exit() override;

	// Tool exclusivity
	PCGEDITOR_API virtual bool ShouldToolStartBeAllowed(const FString& ToolIdentifier) const override;

	/** Check if any tool is currently active */
	PCGEDITOR_API bool IsAnyToolActive() const;

	/** Set the PCG editor this mode is associated with */
	PCGEDITOR_API void SetPCGEditor(TWeakPtr<class FPCGEditor> InPCGEditor) { PCGEditor = InPCGEditor; }

	/** Get the PCG editor this mode is associated with */
	TSharedPtr<class FPCGEditor> GetPCGEditor() const { return PCGEditor.Pin(); }

protected:
	void ConfigureRealTimeViewportsOverride(bool bEnable);

	/** Reference to the PCG editor this mode is associated with */
	TWeakPtr<class FPCGEditor> PCGEditor;
	
	// @todo_pcg: Activate when the asset editor mode might need to track anything during a tool's lifecycle.

	// PCGEDITOR_API virtual void OnToolStarted(
	// 	UInteractiveToolManager* Manager,
	// 	UInteractiveTool* Tool
	// ) override;

	// PCGEDITOR_API virtual void OnToolEnded(
	// 	UInteractiveToolManager* Manager,
	// 	UInteractiveTool* Tool
	// ) override;

	// PCGEDITOR_API virtual void OnToolPostBuild(
	// 	UInteractiveToolManager* InToolManager,
	// 	EToolSide InSide,
	// 	UInteractiveTool* InBuiltTool,
	// 	UInteractiveToolBuilder* InToolBuilder,
	// 	const FToolBuilderState& ToolState
	// );
};
