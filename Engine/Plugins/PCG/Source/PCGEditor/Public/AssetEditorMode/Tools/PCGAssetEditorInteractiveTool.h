// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "UObject/WeakInterfacePtr.h"

#include "PCGAssetEditorInteractiveTool.generated.h"

class UPCGGraphInstance;
class IPCGGraphExecutionSource;
class UPCGGraph;
class UPCGSettings;
struct FToolBuilderState;

/** Expands each TSettings type into its StaticClass() pointer. */
template <typename... TSettings>
TArray<UClass*> PCGMakeSupportedSettingsArray()
{
	return TArray<UClass*>{TSettings::StaticClass()...};
}

/**
 * Declare the UPCGSettings subclasses that activate this tool.
 * Pass bare class names — ::StaticClass() is added automatically.
 * Place inside the tool class body.
 * Example:
 *   PCG_DECLARE_SUPPORTED_NODES(
 *       UPCGCreatePointsSettings,
 *       UPCGCreatePointsGridSettings
 *   )
 */
#define PCG_DECLARE_SUPPORTED_NODES(...)												\
public:																					\
	virtual TArray<UClass*> GetSupportedSettingsClasses() const override				\
	{																					\
		return PCGMakeSupportedSettingsArray<__VA_ARGS__>();							\
	}

/**
 * Generic context object stored in UContextObjectStore to pass data from FPCGEditor into a tool.
 * Populated by FPCGEditor::OnNodeToolStarted before activating any node tool.
 */
UCLASS(Transient)
class UPCGNodeToolContext : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UPCGSettings> NodeSettings = nullptr;

	/** The UObject that implements IPCGGraphExecutionSource (e.g. UPCGComponent). Kept alive via UPROPERTY. */
	UPROPERTY()
	TObjectPtr<UObject> ExecutionSourceObject = nullptr;

	/** The graph instance being edited, used to trigger re-execution notifications. */
	UPROPERTY()
	TObjectPtr<UPCGGraphInstance> GraphInstance = nullptr;

	PCGEDITOR_API IPCGGraphExecutionSource* GetExecutionSource() const;
};

/**
 * Settings for PCG Asset Editor Interactive Tool
 */
UCLASS()
class PCGEDITOR_API UPCGAssetEditorInteractiveToolSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
	
	/*
	 * @todo_pcg: add base settings required for all Viewport Tools (if any)
	 * also show the settings somewhere in the asset editor. it's currently not being shown anywhere 
	 */
};

/**
 * Base interactive tool for PCG Asset Editor
 * Provides viewport coordination and node inspection integration
 */
UCLASS(Abstract)
class PCGEDITOR_API UPCGAssetEditorInteractiveTool : public UInteractiveTool
{
	GENERATED_BODY()

public:
	// ~Begin UInteractiveTool Interface
	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;

	virtual bool HasCancel() const override { return true; }

	virtual bool HasAccept() const override { return true; }

	virtual bool CanAccept() const override { return true; }
	// ~End UInteractiveTool Interface

	/** Called by the builder immediately after construction, before Setup(). Override for custom builder-time initialization. */
	virtual void PostBuild(const FToolBuilderState& SceneState) {}

	/** Returns the UPCGSettings subclasses this tool handles. Use PCG_DECLARE_SUPPORTED_NODES to override. */
	virtual TArray<UClass*> GetSupportedSettingsClasses() const PURE_VIRTUAL(UPCGAssetEditorInteractiveTool::GetSupportedSettingsClasses, return {};)

protected:
	/** Called when tool ticks */
	virtual void OnTick(float DeltaTime) override {}

	/** Called when the tool is accepted */
	virtual void OnAccept() {}

	/** Called when the tool is cancelled */
	virtual void OnCancel() {}
	
	/** Called when tool's respective node settings is changed */
	virtual void OnNodeSettingsChanged(UPCGSettings* NodeSettings, EPCGChangeType ChangeType) {}

	UPROPERTY()
	TObjectPtr<UPCGNodeToolContext> NodeToolContext = nullptr;
	
	UPROPERTY()
	TObjectPtr<UPCGAssetEditorInteractiveToolSettings> ToolSettings = nullptr;
	
	IPCGGraphExecutionSource* ExecutionSource = nullptr;
};

/**
 * Generic builder for PCG Asset Editor Interactive Tools.
 * Set ToolClass to the desired tool subclass; the builder instantiates it and calls PostBuild.
 */
UCLASS()
class UPCGAssetEditorInteractiveToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	/** The concrete tool class to instantiate. Set this when registering a tool. */
	UPROPERTY()
	TSubclassOf<UPCGAssetEditorInteractiveTool> ToolClass = nullptr;

	// ~Begin UInteractiveToolBuilder Interface
	PCGEDITOR_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	PCGEDITOR_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	// ~End UInteractiveToolBuilder Interface
};
