// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialDataVisualization.h"
#include "PCGEditorModule.h"

#define UE_API PCGEDITOR_API

class UPCGData;

class FPCGBaseTextureDataVisualization : public IPCGSpatialDataVisualization
{
public:
	// ~Begin IPCGDataVisualization interface
	UE_API virtual TArray<TSharedPtr<FStreamableHandle>> LoadRequiredResources(const UPCGData* Data) const override;
	UE_API virtual FPCGSetupSceneFunc GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const override;
	// ~End IPCGDataVisualization interface
};

#undef UE_API
