// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"

#include "PCGCollisionVisComponent.generated.h"

UCLASS(MinimalAPI, NotPlaceable)
class UPCGCollisionVisComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<TObjectPtr<UBodySetup>> BodySetups;

	UPROPERTY()
	TArray<FTransform> BodyTransforms;

	//~Begin USceneComponent interface
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~End USceneComponent interface

	//~Begin UPrimitiveComponent interface
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual UBodySetup* GetBodySetup() override { return BodySetups.IsEmpty() ? nullptr : BodySetups[0]; }
	virtual bool IsEditorOnly() const override { return true; }
	//~End UPrimitiveComponent interface

	void SetBodySetup(UBodySetup* InBodySetup);
	void AddBodySetup(UBodySetup* InBodySetup, const FTransform& InTransform);
};