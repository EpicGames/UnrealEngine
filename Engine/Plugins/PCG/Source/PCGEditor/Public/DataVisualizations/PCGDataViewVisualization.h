// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataVisualization.h"

class UPCGData;

class FPCGDataViewVisualization : public IPCGDataVisualization
{
public:
	// ~Begin IPCGDataVisualization interface
	virtual void ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const override;
	virtual ~FPCGDataViewVisualization() override = default;
	virtual FPCGTableVisualizerInfo GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const override;
	virtual FString GetDomainDisplayNameForInspection(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const override;
	// ~End IPCGDataVisualization interface
};
