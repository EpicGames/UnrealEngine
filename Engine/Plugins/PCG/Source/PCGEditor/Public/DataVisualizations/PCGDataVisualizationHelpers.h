// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataVisualization.h"
#include "Metadata/PCGMetadataCommon.h"

class UPCGData;

namespace PCGDataVisualizationConstants
{
	/** Names of the columns in the attribute list. */
	const FName NAME_Index = FName(TEXT("$Index"));
}

namespace PCGDataVisualizationHelpers
{
	struct FColumnInfoOverrides
	{
		FText LabelOverride;
		FText TooltipOverride;
		bool bAddTypeToTooltip = true;
		bool bIsConstantValueCompressed = false;
		EPCGMetadataTypes TypeOverride = EPCGMetadataTypes::Unknown;
		TFunction<TSharedPtr<const IPCGAttributeAccessor>()> CreateAccessorFuncOverride;
		TFunction<TSharedPtr<const IPCGAttributeAccessorKeys>()> CreateAccessorKeysFuncOverride;
	};
	
	/**
	 * Add a new column to the VisualizerInfo, using the Data and Selector.
	 * By default, it will create a column with the label extracted from the selector, with accessor and keys created from the data and selector, but you can provide overrides.
	 * @param OutInfo 
	 * @param Data 
	 * @param InSelector 
	 * @param Overrides
	 */
	PCGEDITOR_API void AddColumnInfo(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides = {});
	
	PCGEDITOR_API void CreateMetadataColumnInfos(const UPCGData* Data, FPCGTableVisualizerInfo& OutInfo, const FPCGMetadataDomainID& DomainID = PCGMetadataDomainID::Default);

	PCGEDITOR_API FPCGTableVisualizerInfo CreateDefaultMetadataColumnInfos(const UPCGData* Data, const FPCGMetadataDomainID& DomainID = PCGMetadataDomainID::Default);

	template <typename T>
	void AddTypedColumnInfo_Impl(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides = {})
	{
		AddColumnInfo(OutInfo, Data, InSelector, Overrides);
	}

	template <typename T>
	void AddTypedColumnInfo(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides = {})
	{
		const FColumnInfoOverrides* OverridesToUse = &Overrides;
		FColumnInfoOverrides CopyOverrides;
		if (Overrides.TypeOverride == EPCGMetadataTypes::Unknown && Overrides.bAddTypeToTooltip && PCG::Private::IsPCGType<T>())
		{
			CopyOverrides = Overrides;
			CopyOverrides.TypeOverride = static_cast<EPCGMetadataTypes>(PCG::Private::MetadataTypes<T>::Id);
			OverridesToUse = &CopyOverrides;
		}
		
		AddTypedColumnInfo_Impl<T>(OutInfo, Data, InSelector, *OverridesToUse);
	}

	template <typename T>
	void AddPropertyEnumColumnInfo(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const UEnum* EnumClass, int64 EnumValue, const FColumnInfoOverrides& Overrides = {})
	{
		check(Data && EnumClass);

		int32 EnumIndex = EnumClass->GetIndexByValue(EnumValue);
		check(EnumIndex != INDEX_NONE);

		const FColumnInfoOverrides* OverridesToUse = &Overrides;
		FColumnInfoOverrides CopyOverrides;
		if (Overrides.TooltipOverride.IsEmpty())
		{
			CopyOverrides = Overrides;
			CopyOverrides.TooltipOverride = EnumClass->GetToolTipTextByIndex(EnumIndex);
			OverridesToUse = &CopyOverrides;
		}
		
		const FString InvariantName = EnumClass->GetNameStringByIndex(EnumIndex); // Use string version as it strips out the namespace.
		FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreatePropertySelector(*InvariantName);

		static const FString EnumMetadataDomain_MetadataFlag = PCGObjectMetadata::EnumMetadataDomain.ToString();
		
		if (EnumClass->HasMetaData(*EnumMetadataDomain_MetadataFlag, EnumIndex))
		{
			const FString MetadataDomain = EnumClass->GetMetaData(*EnumMetadataDomain_MetadataFlag, EnumIndex);
			Selector.SetDomainName(*MetadataDomain);
		}

		AddTypedColumnInfo<T>(OutInfo, Data, Selector, *OverridesToUse);
	}

	template <typename T, typename EnumType>
	void AddPropertyEnumColumnInfo(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, EnumType EnumValue, const FColumnInfoOverrides& Overrides = {})
	{
		const UEnum* EnumClass = StaticEnum<EnumType>();
		check(EnumClass);
		AddPropertyEnumColumnInfo<T>(OutInfo, Data, EnumClass, static_cast<int64>(EnumValue), Overrides);
	}

	template <> PCGEDITOR_API void AddTypedColumnInfo_Impl<FVector2D>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides);
	template <> PCGEDITOR_API void AddTypedColumnInfo_Impl<FVector>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides);
	template <> PCGEDITOR_API void AddTypedColumnInfo_Impl<FVector4>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides);
	template <> PCGEDITOR_API void AddTypedColumnInfo_Impl<FLinearColor>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides);
	template <> PCGEDITOR_API void AddTypedColumnInfo_Impl<FQuat>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides);
	template <> PCGEDITOR_API void AddTypedColumnInfo_Impl<FRotator>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides);
	template <> PCGEDITOR_API void AddTypedColumnInfo_Impl<FTransform>(FPCGTableVisualizerInfo& OutInfo, const UPCGData* Data, const FPCGAttributePropertySelector& InSelector, const FColumnInfoOverrides& Overrides);
}