// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialDataVisualization.h"

#define UE_API PCGEDITOR_API

class UPCGData;
class UPCGPointData;
struct FPCGContext;

/** Overrides collapse behavior to target a given number of points generated, to avoid too coarse debug (points way bigger than data) or enormous amount of points (points way smaller than data). */
class IPCGLandscapeDataVisualization : public IPCGSpatialDataVisualization
{
public:
	// ~Begin IPCGSpatialDataVisualization interface
	UE_API virtual const UPCGBasePointData* CollapseToDebugBasePointData(FPCGContext* Context, const UPCGData* Data) const override;
	// ~End IPCGSpatialDataVisualization interface
};

#undef UE_API
