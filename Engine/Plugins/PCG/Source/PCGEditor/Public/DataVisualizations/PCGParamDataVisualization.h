// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataVisualization.h"

#define UE_API PCGEDITOR_API

class UPCGData;

class IPCGParamDataVisualization : public IPCGDataVisualization
{
public:
	virtual void ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const { /* Do nothing. */ }
	UE_API virtual FPCGTableVisualizerInfo GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const override;
};

#undef UE_API
