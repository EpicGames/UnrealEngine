// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialDataVisualization.h"
#include "PCGEditorModule.h"

class UPCGData;

class FPCGPolygon2DDataVisualization : public IPCGSpatialDataVisualization
{
public:
	// ~Begin IPCGDataVisualization interface
	PCGEDITOR_API virtual FPCGTableVisualizerInfo GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const override;
	PCGEDITOR_API virtual FPCGSetupSceneFunc GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const override;
	// ~End IPCGDataVisualization interface
};