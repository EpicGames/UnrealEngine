// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataVisualization.h"
#include "PCGEditorModule.h"

#define UE_API PCGEDITOR_API

class AActor;
class UPCGData;
class UPCGBasePointData;
class UPCGPointData;
struct FPCGContext;
struct FPCGDebugVisualizationSettings;

/** Default implementation for spatial data. Collapses to a PointData representation. */
class IPCGSpatialDataVisualization : public IPCGDataVisualization
{
public:
	// ~Begin IPCGDataVisualization interface
	UE_API virtual void ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const override;
	UE_API virtual FPCGTableVisualizerInfo GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const override;
	
	// Show the sampled points by default
	virtual FPCGMetadataDomainID GetDefaultDomainForInspection(const UPCGData* Data) const override { return PCGMetadataDomainID::Elements; }
	UE_API virtual FString GetDomainDisplayNameForInspection(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const override;
	UE_API virtual TArray<FPCGMetadataDomainID> GetAllSupportedDomainsForInspection(const UPCGData* Data) const override;
	UE_API virtual FPCGSetupSceneFunc GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const override;
	// ~End IPCGDataVisualization interface

	UE_DEPRECATED(5.6, "Implement CollapseToDebugBasePointData instead")
	UE_API virtual const UPCGPointData* CollapseToDebugPointData(FPCGContext* Context, const UPCGData* Data) const;

	UE_API virtual const UPCGBasePointData* CollapseToDebugBasePointData(FPCGContext* Context, const UPCGData* Data) const;

	UE_API virtual void ExecuteDebugDisplayHelper(
		const UPCGData* Data,
		const FPCGDebugVisualizationSettings& DebugSettings,
		FPCGContext* Context,
		AActor* TargetActor,
		const FPCGCrc& Crc,
		const TFunction<void(class UInstancedStaticMeshComponent*)>& OnISMCCreatedCallback) const;
};

#undef UE_API
