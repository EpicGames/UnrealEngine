// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialDataVisualization.h"
#include "PCGEditorModule.h"

#define UE_API PCGEDITOR_API

class UPCGData;
struct FPCGContext;

class IPCGSplineDataVisualization : public IPCGSpatialDataVisualization
{
public:
	// ~Begin IPCGDataVisualization interface
	UE_API virtual FPCGTableVisualizerInfo GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const override;
	UE_API virtual FPCGSetupSceneFunc GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const override;
	// ~End IPCGDataVisualization interface

	// ~Begin IPCGSpatialDataVisualization interface
	/** Overrides collapse behavior to show the spline control points. */
	UE_API virtual const UPCGBasePointData* CollapseToDebugBasePointData(FPCGContext* Context, const UPCGData* Data) const override;
	// ~End IPCGSpatialDataVisualization interface
};

#undef UE_API
