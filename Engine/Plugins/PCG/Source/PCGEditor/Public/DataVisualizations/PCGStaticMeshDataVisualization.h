// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataVisualization.h"
#include "PCGEditorModule.h"

class UPCGData;
struct FPCGContext;

class IPCGStaticMeshDataVisualization : public IPCGDataVisualization
{
public:
	// ~Begin IPCGDataVisualization interface
	virtual void ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const override { /* Do nothing. */ };
	virtual FPCGTableVisualizerInfo GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const override;
	virtual TArray<TSharedPtr<FStreamableHandle>> LoadRequiredResources(const UPCGData* Data) const override;
	virtual FPCGSetupSceneFunc GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const override;
	// ~End IPCGDataVisualization interface
};
