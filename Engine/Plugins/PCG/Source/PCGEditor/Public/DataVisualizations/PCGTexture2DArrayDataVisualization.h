// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialDataVisualization.h"
#include "PCGEditorModule.h"

class UPCGData;

class FPCGTexture2DArrayDataVisualization : public IPCGSpatialDataVisualization
{
public:
	// ~Begin IPCGDataVisualization interface
	PCGEDITOR_API virtual TArray<TSharedPtr<FStreamableHandle>> LoadRequiredResources(const UPCGData* Data) const override;
	PCGEDITOR_API virtual FPCGSetupSceneFunc GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const override;
	// ~End IPCGDataVisualization interface
};
