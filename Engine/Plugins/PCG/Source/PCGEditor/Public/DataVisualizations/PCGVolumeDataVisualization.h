// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialDataVisualization.h"

#define UE_API PCGEDITOR_API

class UPCGData;
struct FPCGContext;

/** Overrides collapse behavior to target a given number of points generated, to avoid too coarse debug (points way bigger than data) or enormous amount of points (points way smaller than data). */
class FPCGPrimitiveDataVisualization : public IPCGSpatialDataVisualization
{
public:
	// ~Begin IPCGSpatialDataVisualization interface
	UE_API virtual const UPCGBasePointData* CollapseToDebugBasePointData(FPCGContext* Context, const UPCGData* Data) const override;
	UE_API FPCGSetupSceneFunc GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const override;
	// ~End IPCGSpatialDataVisualization interface

protected:
	UE_API virtual void GetComponentsAndTransforms(const UPCGData* InData, TArray<TObjectPtr<UPrimitiveComponent>>& OutComponents, TArray<FTransform>& OutComponentTransforms) const;
};

/** Specialization for volumes that don't always rely on their primitive component */
class FPCGVolumeDataVisualization : public FPCGPrimitiveDataVisualization
{
protected:
	virtual void GetComponentsAndTransforms(const UPCGData* InData, TArray<TObjectPtr<UPrimitiveComponent>>& OutComponents, TArray<FTransform>& OutComponentTransforms) const override;
};

/** Specialization for simple collision shapes */
class FPCGCollisionShapeDataVisualization : public FPCGPrimitiveDataVisualization
{
protected:
	virtual void GetComponentsAndTransforms(const UPCGData* InData, TArray<TObjectPtr<UPrimitiveComponent>>& OutComponents, TArray<FTransform>& OutComponentTransforms) const override;
};

/** Specialization for collision wrappers */
class FPCGCollisionWrapperDataVisualization : public FPCGPrimitiveDataVisualization
{
protected:
	virtual void GetComponentsAndTransforms(const UPCGData* InData, TArray<TObjectPtr<UPrimitiveComponent>>& OutComponents, TArray<FTransform>& OutComponentTransforms) const override;
};

#undef UE_API
