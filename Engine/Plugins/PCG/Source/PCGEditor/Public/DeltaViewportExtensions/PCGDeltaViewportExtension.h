// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/DataOverride/PCGDataOverride.h"
#include "Graph/PCGSourceDataContainer.h"

#include "InputCoreTypes.h"
#include "StructUtils/StructView.h"
#include "Widgets/SCompoundWidget.h"

class UPCGComponent;
struct FPCGManualEditNodeConfiguration;

/** Context passed to extensions when creating a new delta on first gizmo release. */
struct FPCGDeltaCreateContext
{
	FBox ComponentBounds = FBox(ForceInit);
	int32 OriginalElementIndex = INDEX_NONE;
};

/** Snapshot of panel state passed to extensions when the selection or configuration changes. */
struct FPCGDeltaViewportContext
{
	bool bSelectionActive = false;
	FPCGDeltaKey SelectionDeltaKey;
	FTransform SelectionTransform = FTransform::Identity;
	TWeakObjectPtr<UPCGComponent> ActivePCGComponent;
	FPCGSourceDataStorageKey ActiveStorageKey;
	TSharedPtr<const FPCGManualEditNodeConfiguration> NodeConfiguration;

	/** Index of the selected element within an insertion delta, or INDEX_NONE. */
	int32 SelectedElementIndex = INDEX_NONE;

	/** Index of the selected point in the original source data, or INDEX_NONE. */
	int32 OriginalElementIndex = INDEX_NONE;
};

/** Generic actions provided by the framework to extensions to subscribe to on creation. */
struct FPCGDeltaViewportCallbacks
{
	TFunction<void(const FPCGDeltaKey&)> RevertDelta;
	TFunction<void()> RevertAllRestorableDeltas;
	TFunction<void()> RequestListRefresh;
	TFunction<void()> ClearSelection;
	TFunction<void(const FPCGDeltaKey&, int32)> SelectElement;
};

/** A key binding requested by an extension for an input action. */
struct FPCGDeltaKeyBinding
{
	FKey Key;
	TFunction<void()> Action;
};

/** An element inserted by an extension that has no corresponding source point. Drawn and hit-tested by the visualizer. */
struct FInsertedViewportElement
{
	FPCGDeltaKey DeltaKey;
	int32 ElementIndex = INDEX_NONE;
	FTransform Transform = FTransform::Identity;
};

/**
 * Interface for manual editing viewport behavior and panel UI. Implement this to control:
 * - What UI appears in the manual edit panel for a given delta type.
 * - How elements with this delta type are drawn, selected, and manipulated in the viewport.
 */
class IPCGDeltaViewportExtension
{
public:
	virtual ~IPCGDeltaViewportExtension() = default;

	/** Returns the display name for this delta type. */
	virtual FText GetDisplayName() const = 0;

	/** The delta name used for key computation. */
	virtual FName GetDeltaName() const { return NAME_None; }

	/** Sort priority for the delta type selector. Lower values appear first. */
	virtual int32 GetSortPriority() const { return std::numeric_limits<int32>::max(); }

	/** Build the Slate widget section for this delta type. Called when the extension is swapped in. */
	virtual TSharedRef<SWidget> CreateWidget(FPCGDeltaViewportCallbacks Callbacks) = 0;

	/** Called when the panel context changes. Update buttons, labels, etc. */
	virtual void UpdateContext(const FPCGDeltaViewportContext& Context) = 0;

	/** Called when the delta context changes and lists need refreshing. */
	virtual void RefreshLists(const FPCGDeltaViewportContext& Context) = 0;

	/** Returns key bindings this extension wants the visualizer to route. Called when the extension becomes active. */
	virtual TArray<FPCGDeltaKeyBinding> GetKeyBindings() { return {}; }

	/** Color for drawing elements with this delta type in the viewport. */
	virtual FLinearColor GetDisplayColor(bool bIsSelected) const = 0;

	/** Given a source element transform and the delta struct, return the transform to display. */
	virtual FTransform GetDisplayTransform(const FTransform& SourceElementTransform, const FConstStructView DeltaStruct) const
	{
		return SourceElementTransform;
	}

	/** Given a source element's transform and a delta, return true if this delta corresponds to that source element. */
	virtual bool MatchesSourceElement(const FTransform& SourceElementTransform, const FConstStructView DeltaStruct, const double SpatialTolerance) const
	{
		return false;
	}

	/** Collect elements inserted by this extension that have no corresponding source point. The visualizer draws and hit-tests these. */
	virtual void GetInsertedElements(const FPCGDeltaCollection& Collection, TArray<FInsertedViewportElement>& OutElements) const {}

	/** Returns true if this delta type uses a TRS gizmo when selected. */
	virtual bool UsesTRSGizmo() const { return true; }

	/** Alt+drag handler. Usually used to create/insert new elements. Returns the new element index, or INDEX_NONE. */
	virtual int32 OnAltDrag(FPCGDeltaCollection& Collection, const FTransform& SourceTransform, FPCGDeltaKey& OutDeltaKey) const
	{
		return INDEX_NONE;
	}

	/** Apply the final gizmo transform to the delta struct. Returns true on success. */
	virtual bool ApplyGizmoTransform(const FTransform& NewTransform, TInstancedStruct<FPCGDeltaBase>& DeltaStruct, int32 ElementIndex) const
	{
		return false;
	}

	/** Create a new delta struct in the collection on first gizmo release (deferred creation). */
	virtual void CreateNewDelta(
		const FPCGDeltaKey& DeltaKey,
		FPCGDeltaCollection& Collection,
		const FTransform& OriginalTransform,
		const FTransform& NewTransform,
		const FPCGDeltaCreateContext& Context) const
	{}

	/** Returns the original transform stored in the delta struct. */
	virtual FTransform GetOriginalTransform(const FConstStructView DeltaStruct) const
	{
		return FTransform::Identity;
	}

	/** Collect all delta keys in the collection that this extension considers restorable. */
	virtual void CollectRestorableKeys(const FPCGDeltaCollection& Collection, TArray<FPCGDeltaKey>& OutKeys) const {}
};

/** Register a delta struct type with an IPCGDeltaViewportExtension to control viewport behavior and panel UI. */
struct FPCGDeltaViewportExtensionRegistry
{
	FPCGDeltaViewportExtensionRegistry() = default;
	FPCGDeltaViewportExtensionRegistry(const FPCGDeltaViewportExtensionRegistry&) = delete;
	FPCGDeltaViewportExtensionRegistry(FPCGDeltaViewportExtensionRegistry&&) = default;
	FPCGDeltaViewportExtensionRegistry& operator=(const FPCGDeltaViewportExtensionRegistry&) = delete;
	FPCGDeltaViewportExtensionRegistry& operator=(FPCGDeltaViewportExtensionRegistry&&) = default;
	~FPCGDeltaViewportExtensionRegistry() = default;

	/** Register an internal viewport extension. Internal extensions are provided by the PCG plugin itself. */
	PCGEDITOR_API void RegisterInternalExtension(const UScriptStruct* DeltaStruct, TUniquePtr<IPCGDeltaViewportExtension> Extension);

	/** Unregister all internal viewport extensions. */
	PCGEDITOR_API void UnregisterAllInternalExtensions();

	/** Register an external viewport extension. External registrations take priority over internal ones. */
	PCGEDITOR_API void RegisterExtension(const UScriptStruct* DeltaStruct, TUniquePtr<IPCGDeltaViewportExtension> Extension);

	/** Unregister an external viewport extension. */
	PCGEDITOR_API void UnregisterExtension(const UScriptStruct* DeltaStruct);

	/** Look up an extension for the given delta struct type, walking the struct hierarchy toward FPCGDeltaBase. */
	template <typename DeltaType>
	IPCGDeltaViewportExtension* GetExtension() const { return GetExtension(DeltaType::StaticStruct()); }

	PCGEDITOR_API IPCGDeltaViewportExtension* GetExtension(const UScriptStruct* DeltaStruct) const;

	/** Returns all registered delta struct types (from both internal and external registries, deduplicated). */
	PCGEDITOR_API TArray<const UScriptStruct*> GetRegisteredDeltaTypes() const;

private:
	/** Registry for extensions defined inside the PCG Plugin. */
	TMap<const UScriptStruct*, TUniquePtr<IPCGDeltaViewportExtension>> InternalRegistry;

	/** Registry for extensions defined outside the PCG Plugin. These take priority, allowing users to override default behavior. */
	TMap<const UScriptStruct*, TUniquePtr<IPCGDeltaViewportExtension>> ExternalRegistry;
};
