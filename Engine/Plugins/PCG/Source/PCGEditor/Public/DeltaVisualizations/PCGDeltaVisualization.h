// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataVisualization.h"

#include "StructUtils/StructView.h"

struct FPCGDeltaKey;
class SWidget;

/** Lightweight column descriptor for the delta overrides table. */
struct FPCGDeltaVisualizerColumnInfo
{
	FName Id;
	FText Label;
	FText Tooltip;
	float Width = -1.0f; // Calculated automatically if < 0
	EPCGTableVisualizerCellAlignment CellAlignment = EPCGTableVisualizerCellAlignment::Left;
};

/** Common column identifiers shared across delta visualizers. */
namespace PCGDeltaVisualization::Constants
{
	static const FName DeltaTypeId = TEXT("DeltaType");
	static const FName SignatureId = TEXT("Signature");
} // namespace PCGDeltaVisualization::Constants

/**
 * Interface for delta type visualizers. If implemented, this controls how a specific delta struct type is displayed in
 * the Data Overrides panel.
 */
class IPCGDeltaVisualization
{
public:
	virtual ~IPCGDeltaVisualization() = default;

	/** Returns the column descriptors this visualizer provides. */
	virtual TArray<FPCGDeltaVisualizerColumnInfo> GetColumnInfos() const = 0;

	/** Returns the text content for a specific cell. */
	virtual FText GetCellText(FName ColumnId, const FPCGDeltaKey& DeltaKey, FConstStructView Delta) const = 0;

	/**
	 * Returns a custom widget for a cell. Returns nullptr by default to fall back to GetCellText wrapped in STextBlock.
	 * Note: Optional.
	 */
	virtual TSharedPtr<SWidget> CreateCellWidget(FName ColumnId, const FPCGDeltaKey& DeltaKey, FConstStructView Delta) const { return nullptr; }
};
