// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeltaVisualizations/PCGDeltaVisualization.h"

class UScriptStruct;

/** Register a delta struct type with an IPCGDeltaVisualization to implement custom visualization behavior. Mirrors FPCGDataVisualizationRegistry. */
struct FPCGDeltaVisualizationRegistry
{
	FPCGDeltaVisualizationRegistry() = default;
	FPCGDeltaVisualizationRegistry(const FPCGDeltaVisualizationRegistry&) = delete;
	FPCGDeltaVisualizationRegistry(FPCGDeltaVisualizationRegistry&&) = default;
	FPCGDeltaVisualizationRegistry& operator=(const FPCGDeltaVisualizationRegistry&) = delete;
	FPCGDeltaVisualizationRegistry& operator=(FPCGDeltaVisualizationRegistry&&) = default;
	~FPCGDeltaVisualizationRegistry() = default;

	/** Register an external delta visualization. */
	PCGEDITOR_API void RegisterDeltaVisualization(const UScriptStruct* DeltaStruct, TUniquePtr<const IPCGDeltaVisualization> Visualization);

	/** Unregister an external delta visualization. */
	PCGEDITOR_API void UnregisterDeltaVisualization(const UScriptStruct* DeltaStruct);

	template <typename DeltaType>
	const IPCGDeltaVisualization* GetDeltaVisualization() const { return GetDeltaVisualization(DeltaType::StaticStruct()); }

	PCGEDITOR_API const IPCGDeltaVisualization* GetDeltaVisualization(const UScriptStruct* DeltaStruct) const;

private:
	/** Registry for delta visualizations defined inside the PCG Plugin. */
	TMap<const UScriptStruct*, TUniquePtr<const IPCGDeltaVisualization>> InternalRegistry;

	/** Registry for delta visualizations defined outside the PCG Plugin. These take priority, allowing the user to override default visualization behavior for internal types. */
	TMap<const UScriptStruct*, TUniquePtr<const IPCGDeltaVisualization>> ExternalRegistry;

	friend class FPCGEditorModule;
};
