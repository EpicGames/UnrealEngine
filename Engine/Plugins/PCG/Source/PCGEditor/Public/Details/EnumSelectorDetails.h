// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "Widgets/Input/SComboBox.h"

struct FEnumSelector;

/** Property type customization for an enum selector allowing the user to easily select from a list of available
 * enum classes and then also select a value from that class's options.
 */
class FEnumSelectorDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FEnumSelectorDetails);
	}

	/** ~Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		// Must override pure virtual function
	}

	/** ~End IPropertyTypeCustomization interface */

	/** Strips the enum class from the enum value name: EnumClass::EnumValue -> EnumValue. */
	static FText StripEnumClassName(FName EnumValueName);

protected:
	FEnumSelector* GetEnumSelector();

	const FEnumSelector* GetEnumSelector() const;

	/** Helper to convert the drop down selection into an enum class. */
	UEnum* GetSelectedEnumClass() const;

	/** Helper to convert the drop down selection into an enum value. */
	int64 GetSelectedEnumValue() const;

	/** Sets the selector enum class and calls the property update functions. */
	void SetSelectorEnumClass(UEnum* Class);

	/** Sets the selector enum value and calls the property update functions. */
	void SetSelectorEnumValue(const int64 Value);

	/** Updates the enum class from the SSearchableComboBox item type. */
	void UpdateEnumClass(const TSharedPtr<FString>& EnumString);

	/** Updates the enum value from the SComboBox item type. */
	void UpdateEnumValue(FName ValueName);

	void InitializeClassComboBox();
	void InitializeClassComboBoxSelection();

	void OnClassComboSelectionChanged(TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo);
	void OnValueComboSelectionChanged(FName InSelectedItem, ESelectInfo::Type SelectInfo);

	/** Reloads the enum values associated with an enum class, once a different enum class has been selected. */
	void ReloadValueOptions();

	/** Returns true only if the selector and enum class are both valid, false otherwise. */
	EVisibility IsValueSelectionVisible() const;

	/** Gets the display text for the currently selected enum class. */
	FText GetSelectedEnumClassText() const;

	/** Gets the display text for the currently selected enum value. */
	FText GetSelectedEnumValueText() const;

	TSharedPtr<IPropertyHandle> PropertyHandle;

	TArray<TSharedPtr<FString>> EnumClassSourceOptions;
	TArray<FName> EnumValueSourceOptions;

	TMap<TSharedPtr<FString>, UEnum*> NameStringToEnumClassMap;

	TSharedPtr<SComboBox<FName>> EnumValueComboBox;

	TSharedPtr<FString> ClassSelection;
	FName ValueSelection;
};