// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "Layout/Visibility.h"
#include "Metadata/PCGAttributePropertySelector.h"

enum class EPCGPointProperties : uint8;
enum class EPCGExtraProperties : uint8;
namespace ETextCommit { enum Type : int; }

struct FPCGAttributePropertySelector;
class SWidget;

class FPCGAttributePropertySelectorDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FPCGAttributePropertySelectorDetails);
	}

	/** ~Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override {};
	/** ~End IPropertyTypeCustomization interface */

	void AddExtractor(FName InExtractor);

protected:
	/** Return the result of GetValueData on the PropertyHandle. The output struct will be the first valid selector. */
	TTuple<FPropertyAccess::Result, const FPCGAttributePropertySelector*> GetStruct() const;

	TSharedRef<SWidget> GenerateExtraMenu();
	EVisibility ExtraMenuVisibility() const;
	bool IsEnabled() const;

	FText GetText() const;
	void SetText(const FText& NewText, ETextCommit::Type CommitInfo);
	void SetPointProperty(EPCGPointProperties EnumValue);
	void SetAttributeName(FName NewName);
	void SetExtraProperty(EPCGExtraProperties EnumValue);
	void SetSelector(FPCGAttributePropertySelector InSelector);

	TSharedPtr<IPropertyHandle> PropertyHandle;
};
