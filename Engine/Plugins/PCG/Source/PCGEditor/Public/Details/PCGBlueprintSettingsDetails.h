// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"


class FReply;

class IDetailLayoutBuilder;
class UFunction;
class UPCGBlueprintBaseElement;

class FPCGBlueprintSettingsDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ~Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** ~End IDetailCustomization interface */

private:
	// Creates a button strip in each category that contains reflected functions marked as CallInEditor
	void AddCallInEditorMethods(IDetailLayoutBuilder& DetailBuilder);

	// Executes the specified method on the selected objects
	FReply OnExecuteCallInEditorFunction(TWeakObjectPtr<UFunction> WeakFunctionPtr);

private:
	TWeakObjectPtr<UPCGBlueprintBaseElement> SelectedObject;
};
