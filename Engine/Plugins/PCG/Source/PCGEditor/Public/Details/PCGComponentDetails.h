// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

#include "PCGGraphExecutionStateInterface.h"

#include "UObject/WeakInterfacePtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FPCGComponentDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ~Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** ~End IDetailCustomization interface */

protected:
	virtual TArray<TWeakInterfacePtr<IPCGGraphExecutionSource>> GatherExecutionSourcesFromSelection(const TArray<TWeakObjectPtr<UObject>>& InObjectSelected) const;
	virtual bool AddDefaultProperties() const { return true; }
};
