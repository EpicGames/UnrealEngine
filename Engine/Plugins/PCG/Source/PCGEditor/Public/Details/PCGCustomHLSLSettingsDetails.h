// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

class FPCGEditor;

/**
 * Implements a details view customization for the Custom HLSL node settings.
 */
class FPCGCustomHLSLSettingsDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FPCGCustomHLSLSettingsDetails>();
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	TWeakPtr<FPCGEditor> GetPCGEditor(const TArray<TWeakObjectPtr<UObject>>& InObjectsBeingCustomized);
};
