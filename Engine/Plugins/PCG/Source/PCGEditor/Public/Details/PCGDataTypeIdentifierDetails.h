// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

#include "Data/Registry/PCGDataTypeIdentifier.h"

#include "Layout/Visibility.h"

class SWidget;

class FPCGDataTypeIdentifierDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FPCGDataTypeIdentifierDetails);
	}

	/** ~Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override {};
	/** ~End IPropertyTypeCustomization interface */

protected:
	FPCGDataTypeIdentifier* GetStruct();
	const FPCGDataTypeIdentifier* GetStruct() const;
	
	FText GetText() const;
	FText GetTooltip() const;
	TSharedRef<SWidget> OnGetMenuContent();

	TArray<FPCGDataTypeBaseId> HiddenTypes;
	TArray<TTuple<FPCGDataTypeBaseId, int32>> VisibleTypes;

	TSharedPtr<IPropertyHandle> PropertyHandle;
	bool bSupportComposition = false;
};
