// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "StructUtils/PropertyBag.h"
#include "TickableEditorObject.h"

class IDetailsView;
class UPCGGraphInterface;

/**
* A simple Details Customization that will provide a single property from the graph's FInstancePropertyBag
*/
class FPCGEditableUserParameterDetails : public IDetailCustomization, public FTickableEditorObject
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	//~Begin FTickableEditorObject interface
	virtual bool IsAllowedToTick() const override { return ParentDetailsView.IsValid(); }
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	//~End FTickableEditorObject interface

private:
	TWeakPtr<IDetailsView> ParentDetailsView;
	FPropertyBagPropertyDesc CachedPropertyDesc;
	TWeakObjectPtr<UPCGGraphInterface> CachedGraphInterface;

	// Time left until the next details refresh.
	float TimeUntilRefresh = -1.f;
};