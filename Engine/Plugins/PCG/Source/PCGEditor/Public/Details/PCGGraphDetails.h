// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UPCGGraph;

class FPCGGraphDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	/**
	 * Identifier the diff view sets on its details views (via FDetailsViewArgs::ViewIdentifier)
	 * to ask this customization to skip editor-action buttons (Open Graph Parameters, Run
	 * Determinism Test) that don't make sense in a read-only diff context.
	 */
	static const FName DiffViewIdentifier;

private:
	TArray<TWeakObjectPtr<UPCGGraph>> SelectedGraphs;
};
