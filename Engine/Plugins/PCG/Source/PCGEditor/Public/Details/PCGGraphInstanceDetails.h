// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

#include "Containers/Array.h"
#include "Input/Reply.h" // IWYU pragma: keep
#include "UObject/WeakObjectPtrTemplates.h"

class UPCGGraphInstance;

class FPCGGraphInstanceDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	bool SaveInstanceButtonEnabled() const;
	FReply OnSaveInstanceClicked();

	TArray<TWeakObjectPtr<UPCGGraphInstance>> SelectedGraphInstances;
};
