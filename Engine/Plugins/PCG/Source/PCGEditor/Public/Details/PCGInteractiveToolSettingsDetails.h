// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h" // IWYU pragma: keep
#include "UObject/WeakObjectPtrTemplates.h"
#include "Layout/Visibility.h"

class UPCGInteractiveToolSettings;
class UPCGGraphInstance;

class FPCGInteractiveToolSettingsDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	FPCGInteractiveToolSettingsDetails();
	virtual ~FPCGInteractiveToolSettingsDetails() override;

	void RegisterDelegates();
	void UnregisterDelegates();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
private:
	void Rebuild(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);
	
	FText GetResetToolDataButtonText() const;
	EVisibility IsResetToolDataButtonVisible() const;
	FReply ResetToolData() const;

private:
	TWeakObjectPtr<UPCGInteractiveToolSettings> ToolSettings;
	TWeakPtr<IDetailLayoutBuilder> DetailLayoutBuilder;
};
