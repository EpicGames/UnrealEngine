// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

struct FPCGRaycastFilterRuleCollection;
class SWidget;

class FPCGRaycastFilterRuleCollectionDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FPCGRaycastFilterRuleCollectionDetails>();
	}

	/** ~Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	/** ~End IPropertyTypeCustomization interface */

protected:
	void OnGenerateRaycastRuleElement(TSharedRef<IPropertyHandle> RaycastRuleArrayElementHandle, int32 ArrayIndex, IDetailChildrenBuilder& DetailChildrenBuilder, TSharedPtr<IPropertyUtilities> PropertyUtilities) const;
	
	FPCGRaycastFilterRuleCollection* GetStruct();
	const FPCGRaycastFilterRuleCollection* GetStruct() const;

	TSharedPtr<IPropertyHandle> PropertyHandle;
};
