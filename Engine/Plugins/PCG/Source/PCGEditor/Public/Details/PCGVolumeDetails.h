// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/PCGComponentDetails.h"

class IDetailCustomization;

class FPCGVolumeDetails : public FPCGComponentDetails
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	/** ~Begin FPCGComponentDetails interface */
	virtual TArray<TWeakInterfacePtr<IPCGGraphExecutionSource>> GatherExecutionSourcesFromSelection(const TArray<TWeakObjectPtr<UObject>>& InObjectSelected) const override;
	virtual bool AddDefaultProperties() const override { return false; }
	/** ~End FPCGComponentDetails interface */
};
