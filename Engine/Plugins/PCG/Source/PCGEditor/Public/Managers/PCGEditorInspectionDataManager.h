// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "PCGGraphExecutionStateInterface.h"
#include "Graph/PCGStackContext.h"

class UPCGEditorGraphNodeBase;
struct FPCGInspectionData;

struct FPCGEditorInspectionDataEntry
{
	FPCGDataCollection InspectionData;
	TWeakObjectPtr<const UPCGEditorGraphNodeBase> Node;
	FName PinName = NAME_None;
	bool bIsOutputPin = true;
	FPCGStack Stack;

	PCGEDITOR_API bool IsValid() const;
	PCGEDITOR_API void Clear();
};

struct FPCGEditorInspectionDataEntrySetupParams
{
	FPCGEditorInspectionDataEntrySetupParams(int32 InEntryIndex, const UPCGEditorGraphNodeBase* InNode, FName InPinName = NAME_None, bool bInIsOutputPin = true)
		: EntryIndex(InEntryIndex)
		, Node(InNode)
		, PinName(InPinName)
		, bIsOutputPin(bInIsOutputPin)
	{}

	FPCGEditorInspectionDataEntrySetupParams(int32 InEntryIndex, const UPCGEditorGraphNodeBase* InNode, int32 InPinIndex = INDEX_NONE, bool bInIsOutputPin = true)
		: EntryIndex(InEntryIndex)
		, Node(InNode)
		, PinIndex(InPinIndex)
		, bIsOutputPin(bInIsOutputPin)
	{}
	
	int32 EntryIndex = INDEX_NONE;
	const UPCGEditorGraphNodeBase* Node = nullptr;

	// If left to NAME_None and PinIndex is INDEX_NONE, by default it will take the first output pin.
	FName PinName = NAME_None;
	int32 PinIndex = INDEX_NONE;
	bool bIsOutputPin = true;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnInspectedStackChanged, const FPCGStack&);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInspectionEntryChanged, const FPCGEditorInspectionDataEntry&, int32);

/**
* Central class that will handle everything related to Inspection Data.
* Other widgets can ask the manager to get the inspection data for a given node,
* or change which Execution Source is currently inspected.
*/
class FPCGEditorInspectionDataManager : public TSharedFromThis<FPCGEditorInspectionDataManager>
{
public:
	// Hard-coded to 4, as it is how many attribute list view widgets we can have at maximum.
	static constexpr int32 NumberOfEntries = 4;
	
	PCGEDITOR_API void Cleanup();
	
	const FPCGStack& GetStackBeingInspected() const { return StackBeingInspected; }
	PCGEDITOR_API void SetStackBeingInspected(const FPCGStack& FullStack);

	IPCGGraphExecutionSource* GetPCGSourceBeingInspected() const { return const_cast<IPCGGraphExecutionSource*>(StackBeingInspected.GetRootSource()); }

	PCGEDITOR_API void OnSourceGenerated(IPCGGraphExecutionSource* InSource) const;

	UE_DEPRECATED(5.8, "Use GetInspectionDataPtr/InspectData instead")
	PCGEDITOR_API const FPCGDataCollection* GetInspectionData(const FPCGEditorInspectionDataEntrySetupParams& InParams) const { return nullptr; }

	PCGEDITOR_API const TSharedPtr<FPCGInspectionData> GetInspectionDataPtr(const FPCGEditorInspectionDataEntrySetupParams& InParams) const;
	PCGEDITOR_API bool InspectData(const FPCGEditorInspectionDataEntrySetupParams& InParams, TFunctionRef<void(const FPCGDataCollection&)> InspectFunc) const;
	
	PCGEDITOR_API void SetupInspectionEntry(const FPCGEditorInspectionDataEntrySetupParams& InParams);

	// Return the entry at EntryIndex. Must be within 0 and NumberOfEntries - 1.
	// If the entry is invalid at this index, return the first one (so multiple widget would show the same thing).
	PCGEDITOR_API const FPCGEditorInspectionDataEntry& GetInspectionEntry(int32 EntryIndex);

	/** Add UObject references for GC */
	PCGEDITOR_API void AddReferencedObjects(FReferenceCollector& Collector);

	FOnInspectedStackChanged OnInspectedStackChangedDelegate;
	FOnInspectionEntryChanged OnInspectionEntryChangedDelegate;

private:
	void ConstructEntry(const FPCGEditorInspectionDataEntrySetupParams& InParams, FPCGEditorInspectionDataEntry& OutEntry) const;

	bool GetInspectionDataParams(const FPCGEditorInspectionDataEntrySetupParams& InParams, FPCGStack& OutStack, UPCGPin const*& OutPin) const;
	
	FPCGSoftGraphExecutionSource PCGSourceBeingInspected;
	// Implementation note: we'll keep the last valid source inspected so we don't un-inspect on spurious selection changes
	FPCGSoftGraphExecutionSource LastValidPCGSourceBeingInspected;
	FPCGStack StackBeingInspected;
	
	TStaticArray<FPCGEditorInspectionDataEntry, NumberOfEntries> InspectionEntries;
};