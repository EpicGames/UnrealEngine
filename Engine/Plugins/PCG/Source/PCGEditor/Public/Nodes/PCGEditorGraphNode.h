// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/PCGEditorGraphNodeBase.h"

#include "PCGEditorGraphNode.generated.h"

namespace ENodeTitleType { enum Type : int; }

class UToolMenu;

UCLASS()
class UPCGEditorGraphNode : public UPCGEditorGraphNodeBase
{
	GENERATED_BODY()

public:
	static constexpr int32 MaxNodeNameCharacterCount = 128;
	static constexpr float MaxNodeTitleWidth = 256.f;

	virtual void Construct(UPCGNode* InPCGNode) override;

	// ~Begin UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void AllocateDefaultPins() override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual void OnRenameNode(const FString& NewName) override;
	// ~End UEdGraphNode interface

	/** Puts node title on node body, reducing overall node size */
	bool ShouldDrawCompact() const;

	/** Returns custom compact node icon if available */
	bool GetCompactNodeIcon(FName& OutCompactNodeIcon) const;

	/** Validate that this is an acceptable name to rename this node. */
	virtual bool OnValidateNodeTitle(const FText& NewName, FText& OutErrorMessage);
};