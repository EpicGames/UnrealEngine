// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"

#include "PCGCommon.h"
#include "Utils/PCGPreconfiguration.h"

#include "UObject/ObjectKey.h"

#include "PCGEditorGraphNodeBase.generated.h"

enum class EPCGChangeType : uint32;
enum class EPCGMetadataTypes : uint8;

struct FPCGPreconfiguredInfo;
struct FPCGStack;

class IPCGDynamicPinsProvider;
class IPCGSettingsDefaultValueProvider;
class UPCGGraph;
class UPCGGraphInterface;
class UPCGNode;
class UPCGPin;
class UPCGSettings;
class UToolMenu;

enum class EPCGPinDirection : uint8;

UCLASS()
class UPCGEditorGraphNodeBase : public UEdGraphNode
{
	GENERATED_BODY()

public:
	UPCGEditorGraphNodeBase(const FObjectInitializer& ObjectInitializer);
	virtual void Construct(UPCGNode* InPCGNode);

	// ~Begin UObject interface
	virtual void BeginDestroy() override;
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	virtual void PostLoad() override;
	// ~End UObject interface

	// ~Begin UEdGraphNode interface
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual void PrepareForCopying() override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	virtual void ReconstructNode() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FLinearColor GetNodeBodyTintColor() const override;
	virtual void PostPasteNode() override;
	virtual FText GetTooltipText() const override;
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual void OnUpdateCommentText(const FString& NewComment) override;
	virtual void OnCommentBubbleToggled(bool bInCommentBubbleVisible) override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void FindDiffs(UEdGraphNode* OtherNode, FDiffResults& Results) override;
	// ~End UEdGraphNode interface

	bool CanUserAddRemoveDynamicPins(EPCGPinDirection Direction) const;
	void OnUserAddDynamicPin(EPCGPinDirection Direction);
	bool CanUserRemoveDynamicPin(UEdGraphPin* InPinToRemove, EPCGPinDirection Direction) const;
	void OnUserRemoveDynamicPin(UEdGraphPin* InRemovedPin, EPCGPinDirection Direction);

	// @todo_pcg: Supports the dynamic pin providers only, not the soon to be deprecated PCGSettingsWithDynamicInputs.
	bool CanUserRenameDynamicPin(UEdGraphPin* InPin) const;
	void OnUserRenameDynamicPin(UEdGraphPin* InPin, FName NewLabel);

	// @todo_pcg: To be removed with deprecated PCGSettingsWithDynamicInputs
	void OnUserAddDynamicInputPin();
	bool CanUserRemoveDynamicInputPin(UEdGraphPin* InPinToRemove) const;
	void OnUserRemoveDynamicInputPin(UEdGraphPin* InRemovedPin);

	void OnConvertNode(const FPCGPreconfiguredInfo& ConversionInfo);

	UPCGNode* GetPCGNode() { return PCGNode; }
	const UPCGNode* GetPCGNode() const { return PCGNode; }
	void PostCopy();
	void RebuildAfterPaste();
	virtual void PostPaste();
	void InitNodeGuid();

	void SetInspected(bool bInIsInspecting);
	bool GetInspected() const;

	/** Whether node was culled either during compilation or at execution time. */
	void SetIsCulledFromExecution(bool bInIsCulledFromExecution) { bIsCulledFromExecution = bInIsCulledFromExecution; }
	bool IsCulledFromExecution() const { return bIsCulledFromExecution; }

	void SetTriggeredGPUUpload(bool bInTriggeredGPUUpload) { bTriggeredGPUUpload = bInTriggeredGPUUpload; }
	bool GetTriggeredGPUUpload() const { return bTriggeredGPUUpload; }

	void SetTriggeredGPUReadback(bool bInTriggeredGPUReadback) { bTriggeredGPUReadback = bInTriggeredGPUReadback; }
	bool GetTriggeredGPUReadback() const { return bTriggeredGPUReadback; }

	void SetAppliedDataOverrides(const bool bInAppliedDataOverrides) { bAppliedDataOverrides = bInAppliedDataOverrides; }
	bool GetAppliedDataOverrides() const { return bAppliedDataOverrides; }

	void SetHasDefinedOverrides(const bool bInHasDefinedOverrides) { bHasDefinedOverrides = bInHasDefinedOverrides; }
	bool GetHasDefinedOverrides() const { return bHasDefinedOverrides; }

	void SetHasOrphanedOverrides(const bool bInHasOrphanedOverrides) { bHasOrphanedOverrides = bInHasOrphanedOverrides; }
	bool GetHasOrphanedOverrides() const { return bHasOrphanedOverrides; }

	/** Increase deferred reconstruct counter, calls to ReconstructNode will flag reconstruct to happen when count hits zero */
	void EnableDeferredReconstruct();
	/** Decrease deferred reconstruct counter, ReconstructNode will be called if counter hits zero and the node is flagged for reconstruction  */
	void DisableDeferredReconstruct();

	/** Pulls current errors/warnings state from PCG subsystem. */
	EPCGChangeType UpdateErrorsAndWarnings();

	/** If the currently inspected grid size is smaller than the grid size of this node, display transparent. */
	EPCGChangeType UpdateStructuralVisualization(const IPCGGraphExecutionSource* InSourceBeingDebugged, const FPCGStack* InStackBeingInspected, bool bNewlyPlaced = false);

	EPCGChangeType UpdateGPUVisualization(const IPCGGraphExecutionSource* InSourceBeingDebugged, const FPCGStack* InStackBeingInspected);

	EPCGChangeType UpdateOverrideVisualization(const IPCGGraphExecutionSource* InSourceBeingDebugged, const FPCGStack* InStackBeingInspected);

	bool IsInteractiveNode() const;

	/** The settings support default value inline constants. */
	bool IsSettingsDefaultValuesEnabled() const;

	/** The pin has default value inline constants enabled. */
	bool IsPinDefaultValueEnabled(FName PinLabel) const;

	/** The pin has a default value and is currently active. */
	bool IsPinDefaultValueActivated(FName PinLabel) const;

	/** User activated or deactivated the inline constant value. */
	void OnUserSetPinDefaultValueActivated(FName PinLabel, bool bIsActivated) const;

	/** User is converting the default value type on the pin. */
	void ConvertPinDefaultValueMetadataType(FName PinLabel, EPCGMetadataTypes DataType) const;

	/** The target type is valid for this pin's default value and this pin is not already this type. */
	bool CanConvertToDefaultValueMetadataType(FName PinLabel, EPCGMetadataTypes DataType) const;

	/** User is resetting the inline constant value back to the default. */
	void OnUserResetPinDefaultValue(FName PinLabel, UEdGraphPin* OutPin) const;

	/** The pin can be reset back to the default value. */
	bool CanResetPinDefaultValue(FName PinLabel) const;

	/** Get the default value type on the pin. */
	EPCGMetadataTypes GetPinDefaultValueType(FName PinLabel) const;

	/** Whether to flip the order of the title lines - display generated title first and authored second. */
	bool HasFlippedTitleLines() const;

	/** Authored part of node title (like "Create Attribute X"). */
	FText GetAuthoredTitleLine() const;

	/** Generated part of node title, not user editable (like "X = 5.0"). */
	FText GetGeneratedTitleLine() const;

	/** Bitmask of inactive output pins. Bit N will be set if output pin index N is inactive. */
	uint64 GetInactiveOutputPinMask() const { return InactiveOutputPinMask; }

	/** The grid that this node executes on if higen is enabled, otherwise UninitializedGridSize(). */
	uint32 GetGenerationGridSize() const { return GenerationGridSize; }

	/** The higen grid currently being inspected if any, otherwise UninitializedGridSize(). */
	uint32 GetInspectedGridSize() const { return InspectedGridSize; }

	/** Whether the given output pin was active in the previous execution. */
	bool IsOutputPinActive(const UEdGraphPin* InOutputPin) const;

	/** Whether this OutputPin can be connected to this InputPin */
	virtual bool IsCompatible(const UPCGPin* InputPin, const UPCGPin* OutputPin, FText& OutReason) const;

	UPCGSettings* GetSettings() const;
	IPCGSettingsDefaultValueProvider* GetDefaultValueInterface() const;
	IPCGDynamicPinsProvider* GetDynamicPinsInterface() const;

	DECLARE_DELEGATE(FOnPCGEditorGraphNodeChanged);
	FOnPCGEditorGraphNodeChanged OnNodeChangedDelegate;

	DECLARE_DELEGATE_OneParam(FOnRequestPinRename, UEdGraphPin*);
	FOnRequestPinRename OnRequestPinRenameDelegate;

	/** Returns the cached wire color for the given pin. Recomputes on first call after any graph change. */
	FLinearColor GetCachedPinColor(const UPCGPin* PCGPin) const;
	void ClearPinColorCache() { CachedPinColors.Reset(); }

protected:
	static FEdGraphPinType GetPinType(const UPCGPin* InPin);

	/** Create PCG-side edges from editor pins/edges. */
	void RebuildEdgesFromPins();
	virtual void RebuildEdgesFromPins_Internal();

	void OnNodeChanged(UPCGNode* InNode, EPCGChangeType ChangeType);
	void OnPickColor();
	virtual bool CanPickColor() const { return true; }
	virtual void OnColorPicked(FLinearColor NewColor);
	void UpdateCommentBubblePinned();
	void UpdatePosition();

	void CreatePins(const TArray<UPCGPin*>& InInputPins, const TArray<UPCGPin*>& InOutputPins);

	// Performs potentially custom logic when there's a change that would trigger a reconstruct (needed for linked nodes like the named reroutes)
	virtual void ReconstructNodeOnChange() { ReconstructNode(); }

	// Custom logic to hide some pins to the user (by not creating a UI pin, even if the model pin exists).
	// Useful for deprecation
	virtual bool ShouldCreatePin(const UPCGPin* InPin) const;

	// Returns the appropriate pin name to allow for some flexibility
	virtual FText GetPinFriendlyName(const UPCGPin* InPin) const;

	UPROPERTY()
	TObjectPtr<UPCGNode> PCGNode = nullptr;

	/** A flag if the node has ever been connected with an edge.*/
	UPROPERTY()
	bool bHasEverBeenConnected = false;

	int32 DeferredReconstructCounter = 0;
	bool bDeferredReconstruct = false;
	bool bDisableReconstructFromNode = false;

	/** Whether this node was culled in the last execution. */
	bool bIsCulledFromExecution = false;

	bool bTriggeredGPUUpload = false;
	bool bTriggeredGPUReadback = false;
	bool bAppliedDataOverrides = false;
	bool bHasDefinedOverrides = false;
	bool bHasOrphanedOverrides = false;

	/** Bitmask of inactive output pins. Bit N will be set if output pin index N is inactive. */
	uint64 InactiveOutputPinMask = 0;

	/** The grid that this node executes on if higen is enabled, otherwise UninitializedGridSize(). */
	uint32 GenerationGridSize = PCGHiGenGrid::UninitializedGridSize();

	/** The higen grid currently being inspected if any, otherwise UninitializedGridSize(). */
	uint32 InspectedGridSize = PCGHiGenGrid::UninitializedGridSize();

private:
	mutable TMap<TObjectKey<UPCGPin>, FLinearColor> CachedPinColors;

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "Use GenerationGridSize")
	EPCGHiGenGrid GetGenerationGrid() const { return PCGHiGenGrid::GridSizeToGrid(GenerationGridSize); }
	UE_DEPRECATED(5.8, "Use GetInspectedGridSize")
	EPCGHiGenGrid GetInspectedGenerationGrid() const { return PCGHiGenGrid::GridSizeToGrid(InspectedGridSize); }
protected:
	UE_DEPRECATED(5.8, "Use GenerationGridSize")
	EPCGHiGenGrid GenerationGrid = EPCGHiGenGrid::Uninitialized;
	UE_DEPRECATED(5.8, "Use InspectedGridSize")
	EPCGHiGenGrid InspectedGenerationGrid = EPCGHiGenGrid::Uninitialized;
};

/** Disables reconstruct on nodes (or from a pin) and re-enables in destructor. */
struct FPCGDeferNodeReconstructScope
{
	explicit FPCGDeferNodeReconstructScope(UEdGraphPin* FromPin)
		: Node(FromPin ? Cast<UPCGEditorGraphNodeBase>(FromPin->GetOwningNode()) : nullptr)
	{
		if (Node)
		{
			Node->EnableDeferredReconstruct();
		}
	}

	explicit FPCGDeferNodeReconstructScope(UPCGEditorGraphNodeBase* InNode)
		: Node(InNode)
	{
		if (Node)
		{
			Node->EnableDeferredReconstruct();
		}
	}

	FPCGDeferNodeReconstructScope(const FPCGDeferNodeReconstructScope&) = delete;
	FPCGDeferNodeReconstructScope(FPCGDeferNodeReconstructScope&& Other)
	{
		Node = Other.Node;
		Other.Node = nullptr;
	}

	FPCGDeferNodeReconstructScope& operator=(const FPCGDeferNodeReconstructScope&) = delete;
	FPCGDeferNodeReconstructScope& operator=(FPCGDeferNodeReconstructScope&& Other)
	{
		Swap(Node, Other.Node);
		return *this;
	}

	~FPCGDeferNodeReconstructScope()
	{
		if (Node)
		{
			Node->DisableDeferredReconstruct();
			Node = nullptr;
		}
	}

private:
	UPCGEditorGraphNodeBase* Node = nullptr;
};
