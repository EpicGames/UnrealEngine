// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeltaViewportExtensions/PCGDeltaViewportExtension.h"
#include "Graph/DataOverride/PCGDataOverride.h"
#include "Graph/PCGSourceDataContainer.h"
#include "Graph/PCGStackContext.h"

#include "ComponentVisualizer.h"
#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "HitProxies.h"
#include "InputCoreTypes.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "Templates/SharedPointer.h"

class FUICommandList;
class UStaticMesh;

namespace PCG::ComponentVisualizer
{
	struct FCachedMeshWireframe;

	extern bool GEnableCulling;
	extern float GCullingDistance2D;
	extern bool GFrustumCulling;
}

struct HPCGComponentVisualizerHitProxy : public HComponentVisProxy
{
	DECLARE_HIT_PROXY(PCGEDITOR_API);

	/** Regular point constructor. */
	HPCGComponentVisualizerHitProxy(const UActorComponent* InComponent, const FPCGStack& InPinStack, int32 InDataIndex, int32 InPointIndex);

	/** Inserted element constructor. */
	HPCGComponentVisualizerHitProxy(const UActorComponent* InComponent, const FPCGStack& InPinStack, const FPCGDeltaKey& InInsertedDeltaKey, int32 InInsertedElementIndex);

	FPCGStack PinStack;
	int32 DataIndex = INDEX_NONE;
	int32 PointIndex = INDEX_NONE;

	// Inserted element fields
	bool bIsInsertedElement = false;
	int32 InsertedElementIndex = INDEX_NONE;
	FPCGDeltaKey InsertedDeltaKey;
};

class UPCGComponent;
class UPCGNode;

class FPCGComponentVisualizer : public FComponentVisualizer
{
public:
	PCGEDITOR_API FPCGComponentVisualizer();
	PCGEDITOR_API virtual ~FPCGComponentVisualizer() override;

	//~ Begin FComponentVisualizer Interface
	PCGEDITOR_API virtual void OnRegister() override;
	PCGEDITOR_API virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	PCGEDITOR_API virtual void DrawVisualizationHUD(const UActorComponent* Component, const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	PCGEDITOR_API virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click) override;
	PCGEDITOR_API virtual void EndEditing() override;
	PCGEDITOR_API virtual bool GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const override;
	PCGEDITOR_API virtual bool GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const override;
	PCGEDITOR_API virtual bool HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate, FRotator& DeltaRotate, FVector& DeltaScale) override;
	PCGEDITOR_API virtual bool HandleInputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	PCGEDITOR_API virtual TSharedPtr<SWidget> GenerateContextMenu() const override;
	//~ End FComponentVisualizer Interface

	/** Removes a specific delta by key. */
	void RestoreDelta(const FPCGDeltaKey& Key);

	/** Removes all restorable deltas from the active collection (delegates to each extension's CollectRestorableKeys). */
	void RestoreAllDeltas();

	/** Selects a delta element by key and optional sub-index. Looks up the extension from the registry. */
	void SelectElement(const FPCGDeltaKey& DeltaKey, int32 ElementIndex = INDEX_NONE);

	/** Clears the current selection (EditTarget + gizmo). Call before switching active editing nodes. */
	PCGEDITOR_API void ClearActiveSelection();

protected:
	/** Called each frame while the TRS gizmo is being dragged. Updates the cached selection transform. */
	void OnGizmoDragged(const FTransform& NewTransform);

	/** Called when the TRS gizmo drag ends. Materializes deferred deltas and applies the final transform. */
	void OnGizmoReleased(const FTransform& FinalTransform);

	/** Removes all deltas from the active editing node's collection and clears the selection. */
	void RemoveAllDeltas();

	/** Removes the currently selected delta and clears the selection. */
	void RemoveSelectedDeltas();

	/** Returns true if there is a valid PCG component and an active edit target. */
	bool HasSelection() const;

	/** Hides and destroys the active TRS gizmo. */
	void ResetGizmo();

	/** Creates a TRS gizmo at the given transform with drag/release callbacks bound to this visualizer. */
	void SetupGizmo(UInteractiveGizmoManager* GizmoManager, const FTransform& InitialTransform);

	/** Rebuilds the active key bindings from the extension's registered input actions. */
	void RefreshKeyBindings(IPCGDeltaViewportExtension* Extension);

	/** Registered Menu commands */
	TSharedPtr<FUICommandList> ComponentVisualizerActions;

	TWeakObjectPtr<UPCGNode> ActiveEditingNode;

	TWeakObjectPtr<UPCGComponent> LastPCGComponent;

	/** Cached result of resolving the mutable delta collection for the active editing node. Invalidated when the active node changes. */
	struct FPCGCachedDeltaContext
	{
		FPCGSourceDataContainer* DataContainer = nullptr;
		FPCGSourceDataStorageKey StorageKey;
		FPCGStack PinStack;
		TWeakObjectPtr<UPCGComponent> PCGComponent;

		bool IsValid() const { return DataContainer != nullptr && PCGComponent.IsValid(); }
		void Reset() { *this = {}; }
	};

	FPCGCachedDeltaContext CachedDeltaContext;

	/** Tracks the state of the currently selected delta element in the viewport, including the owning component, delta key, gizmo transform, and per-extension metadata. */
	struct FEditState
	{
		TWeakObjectPtr<UPCGComponent> PCGComponent;
		FPCGStack Stack;
		FPCGSourceDataStorageKey StorageKey;
		FPCGDeltaKey DeltaKey;
		bool bWasInDelta = false;
		FTransform Transform;
		FTransform OriginalTransform;

		bool bModified = false;

		/** True while the gizmo is actively being dragged. Used to gate the drag wireframe. */
		bool bGizmoDragging = false;

		/** True when this is a new edit delta that hasn't been stored in the collection yet (deferred until first gizmo move). */
		bool bDeltaDeferred = false;

		/** Extension governing this selection's viewport behavior. Null when no delta is active yet (deferred). */
		IPCGDeltaViewportExtension* ActiveExtension = nullptr;

		/** UScriptStruct of the delta type, for registry re-lookups when ActiveExtension is null. */
		const UScriptStruct* DeltaStructType = nullptr;

		/** Index of the selected element within an insertion delta (e.g. which inserted point is selected). */
		int32 SelectedElementIndex = INDEX_NONE;

		/** Original index into the point data array at creation time (for ElementIndex debug field). */
		int32 OriginalPointIndex = INDEX_NONE;
	};

	/** Active viewport selection. Set when a delta element is clicked. Reset on deselect or node change. */
	TOptional<FEditState> EditTarget = {};

	TWeakObjectPtr<UCombinedTransformGizmo> Gizmo;

	/** True if PCG registered the shared transform gizmo context object. */
	bool bRegisteredGizmoContextObject = false;

	/** Key bindings from the active extension, cached when the extension changes. */
	TArray<FPCGDeltaKeyBinding> ActiveKeyBindings;

	/** Persists across frames to avoid per-frame index buffer copies. */
	TUniquePtr<PCG::ComponentVisualizer::FCachedMeshWireframe> CachedWireframe;

	/** Resolved default point mesh, cached to avoid per-frame LoadSynchronous. */
	TWeakObjectPtr<UStaticMesh> CachedDefaultPointMesh;

	/** Component derived state cached across frames. Rebuilt when the executed stacks generation changes. */
	struct FManualEditComponentCache
	{
		struct FNodeStacks
		{
			TWeakObjectPtr<const UPCGNode> Node;
			TArray<FPCGStack> Stacks;
		};

		FDelegateHandle GraphGeneratedHandle;
		FDelegateHandle GraphCleanedHandle;

		uint64 PreviousStacksGeneration = 0;
		bool bDirty = true;

		TArray<FNodeStacks> ExecutedNodeStacks;
	};

	/**
	 * Returns a pointer to the cache entry for InComponent, creating one if it does not exist.
	 * NOTE: Pointer may become stale and should be used in scope only.
	 */
	FManualEditComponentCache* FindOrCreateManualEditComponentCache(const UPCGComponent* InComponent);

	/** Binds component delegate bindings that mark the cache dirty. */
	void SetupManualEditComponentCacheCallbacks(FManualEditComponentCache& Cache, const UPCGComponent* InComponent);

	/** Releases component delegate bindings. Safe to call even if the component is no longer valid. */
	void TeardownManualEditComponentCacheCallbacks(FManualEditComponentCache& Cache, const UPCGComponent* InComponent);

	/** Rebuilds the cache from the component's executed stacks map. */
	void BuildExecutedNodeStacksCache(FManualEditComponentCache& Cache, const UPCGComponent* InComponent);

	TMap<TWeakObjectPtr<const UPCGComponent>, FManualEditComponentCache> ComponentCaches;
};
