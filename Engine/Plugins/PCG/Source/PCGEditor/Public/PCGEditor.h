// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "Graph/PCGStackContext.h"
#include "Managers/PCGEditorInspectionDataManager.h"

#include "EditorUndoClient.h"
#include "GraphEditor.h"
#include "Schema/PCGEditorGraphSchema.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FAssetEditorModeManager;
class FPCGEditorInspectionDataManager;
enum class ECheckBoxState : uint8;
namespace ETextCommit { enum Type : int; }

enum class EPCGEditorPermissionMode : uint8;
class IPCGBaseSubsystem;
class FUICommandList;
class SGraphEditor;
class SGraphEditorActionMenu;
class SPCGEditorGraphAttributeListView;
class SPCGEditorGraphDebugObjectTree;
class SPCGEditorGraphDetailsView;
class SPCGEditorGraphDeterminismListView;
class SPCGEditorGraphFind;
class SPCGEditorGraphEmbeddedSubgraphsView;
class SPCGEditorGraphDataOverridesView;
class SPCGEditorGraphLogView;
class SPCGEditorGraphNodePalette;
class SPCGEditorGraphProfilingView;
class SPCGCodeEditor;
class SPCGEditorGraphUserParametersView;
class SPCGEditorViewport;
class UEdGraphNode;
class UPCGAssetEditorMode;
class UPCGComponent;
class UPCGDefaultExecutionSource;
class UPCGEditorGraph;
class UPCGEditorGraphNodeBase;
class UPCGGraph;
class UPCGSubsystem;
struct FPCGCompilerDiagnostics;
struct FAssetOpenArgs;
class FWorkspaceItem;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnInspectedNodeChanged, UPCGEditorGraphNodeBase*);

enum class UE_DEPRECATED(5.8, "Use PCGEditorTabs instead") EPCGEditorPanel
{
	Attributes1,
	Attributes2,
	Attributes3,
	Attributes4,
	DebugObjectTree,
	Determinism,
	Find,
	GraphEditor,
	Log,
	NodePalette,
	NodeSource,
	Profiling,
	PropertyDetails1,
	PropertyDetails2,
	PropertyDetails3,
	PropertyDetails4,
	UserParams,
	Viewport1,
	Viewport2,
	Viewport3,
	Viewport4
};

enum class EPCGToolbarButtons
{
	Find,
	PauseRegen,
	ForceRegen,
	AutoLayoutNodes,
	CancelExecution,
	OpenDebugObjectTreeTab,
	GraphParams,
	GraphSettings,
	Graph,
};

namespace PCGEditorTabs
{
	constexpr FLazyName GraphEditorID = TEXT("GraphEditor");
	constexpr FLazyName PropertyDetailsID[] = {
		TEXT("PropertyDetails"),
		TEXT("PropertyDetails2"),
		TEXT("PropertyDetails3"),
		TEXT("PropertyDetails4") };
	constexpr FLazyName PaletteID = TEXT("Palette");
	constexpr FLazyName DebugObjectID = TEXT("DebugObject");
	constexpr FLazyName AttributesID[] = {
		TEXT("Attributes"),
		TEXT("Attributes2"),
		TEXT("Attributes3"),
		TEXT("Attributes4") };
	constexpr FLazyName FindID = TEXT("Find");
	constexpr FLazyName DeterminismID = TEXT("Determinism");
	constexpr FLazyName ProfilingID = TEXT("Profiling");
	constexpr FLazyName LogID = TEXT("Log");
	constexpr FLazyName UserParamsID = TEXT("UserParams");
	constexpr FLazyName ViewportID[] = {
		TEXT("Viewport"),
		TEXT("Viewport2"),
		TEXT("Viewport3"),
		TEXT("Viewport4") };
	constexpr FLazyName EmbeddedSubgraphsID = TEXT("EmbeddedSubgraphs");
	constexpr FLazyName CodeEditorID = TEXT("CodeEditor");
	constexpr FLazyName DataOverridesID = TEXT("DataOverrides");

	constexpr int32 NumPropertyDetailTabs = sizeof(PropertyDetailsID) / sizeof(FLazyName);
	constexpr int32 NumAttributeTabs = sizeof(AttributesID) / sizeof(FLazyName);
	constexpr int32 NumViewportTabs = sizeof(ViewportID) / sizeof(FLazyName);

	UE_DEPRECATED(5.8, "Use CodeEditorID instead.")
	constexpr FLazyName HLSLSourceID = TEXT("HLSLSource");
}

class FPCGEditor : public FWorkflowCentricApplication, public FGCObject, public FSelfRegisteringEditorUndoClient
{
	friend class FPCGEditorModule;
	friend class FPCGManualEditPanelManager;

public:
	/** Edits the specified PCGGraph */
	PCGEDITOR_API void InitializePCGEditorGraph(UPCGGraph* InPCGGraph);
	PCGEDITOR_API UPCGEditorGraph* GetOrCreatePCGEditorGraph(UPCGGraph* InPCGGraph);
	PCGEDITOR_API void Initialize(const EToolkitMode::Type InMode, const TSharedPtr<class IToolkitHost>& InToolkitHost, UPCGGraph* InPCGGraph, UObject* InObjectToEdit = nullptr);


	/** Open a document tab */
	PCGEDITOR_API void NavigateTab(FDocumentTracker::EOpenDocumentCause InCause);
	PCGEDITOR_API TSharedPtr<SDockTab> OpenDocument(UPCGGraph* InPCGGraph, FDocumentTracker::EOpenDocumentCause InCause);
	PCGEDITOR_API void CloseDocument(UPCGGraph* InPCGGraph);
	PCGEDITOR_API FPCGEditor* OpenEditorForGraph(UPCGGraph* InPCGGraph);
	PCGEDITOR_API static void OpenAssets(const FAssetOpenArgs& OpenArgs);

	UE_DEPRECATED(5.8, "Use GetMainEditorGraph/GetFocusedEditorGraph instead")
	UPCGEditorGraph* GetPCGEditorGraph() { return GetMainEditorGraph(); }

	/** Get the root PCG editor graph being edited. */
	PCGEDITOR_API UPCGEditorGraph* GetMainEditorGraph() const;

	/** Get the focused PCG editor graph being edited. (root or embedded subgraph) */
	PCGEDITOR_API UPCGEditorGraph* GetFocusedEditorGraph() const;

	/** Gets/Creates the PCG graph editor for a given PCG graph */
	PCGEDITOR_API static UPCGEditorGraph* GetPCGEditorGraph(UPCGGraph* InGraph);
	PCGEDITOR_API static UPCGEditorGraph* GetPCGEditorGraph(const UPCGNode* InNode);
	PCGEDITOR_API static UPCGEditorGraph* GetPCGEditorGraph(const UPCGSettings* InSettings);

	UE_DEPRECATED(5.8, "Use GetMainGraph/GetFocusedGraph instead")
	const UPCGGraph* GetPCGGraph() { return GetMainGraph(); }

	/** Get the root PCG graph being edited */
	PCGEDITOR_API UPCGGraph* GetMainGraph() const;

	/** Get the focused PCG graph being edited. (root or embedded subgraph) */
	PCGEDITOR_API UPCGGraph* GetFocusedGraph() const;
	
	PCGEDITOR_API virtual UPCGAssetEditorMode* GetAssetEditorMode(int32 Index) const;

	/** Sets the execution stack that want to inspect. */
	PCGEDITOR_API void SetStackBeingInspected(const FPCGStack& FullStack);

	/** Sets the execution stack from another editor, which will set directly in the debug object tree view. */
	PCGEDITOR_API void SetStackBeingInspectedFromAnotherEditor(const FPCGStack& FullStack);

	/** Clear current inspection. */
	PCGEDITOR_API void ClearStackBeingInspected();

	/** Gets the PCG source we are debugging */
	UE_DEPRECATED(5.7, "Use GetPCGSourceBeingInspected instead")
	PCGEDITOR_API UPCGComponent* GetPCGComponentBeingInspected() const;

	PCGEDITOR_API IPCGGraphExecutionSource* GetPCGSourceBeingInspected() const;
	
	/** Gets the PCG stack we are inspecting */
	PCGEDITOR_API const FPCGStack* GetStackBeingInspected() const;

	FPCGEditorInspectionDataManager& GetInspectionDataManager() { return InspectionDataManager; }
	const FPCGEditorInspectionDataManager& GetInspectionDataManager() const { return InspectionDataManager; }

	PCGEDITOR_API void SetSourceEditorTargetObject(UObject* InObject);

	/** Focus the graph view on a specific node */
	PCGEDITOR_API void JumpToNode(const UEdGraphNode* InNode);
	PCGEDITOR_API void JumpToNode(const UPCGNode* InNode);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Get the TabID of the editor panel. */
	UE_DEPRECATED(5.8, "Use PCGEditorTabs directly")
	PCGEDITOR_API FName GetPanelID(EPCGEditorPanel Panel) const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Focuses the user on a specific panel and flashes the tab. */
	PCGEDITOR_API void BringFocusToPanel(const FName PanelID) const;
	/** Attempts to close the specific panel if it's open. */
	PCGEDITOR_API void CloseGraphPanel(const FName PanelID) const;
	/** Returns true if the selected tab is currently open. */
	PCGEDITOR_API bool IsPanelCurrentlyOpen(const FName PanelID) const;
	/** Returns true if the selected tab is currently open and focused. */
	PCGEDITOR_API bool IsPanelCurrentlyForeground(const FName PanelID) const;

	/** Returns true if the panel is available. */
	PCGEDITOR_API virtual bool IsPanelAvailable(const FName PanelID) const;

	/** Helper to get to the subsystem. */
	PCGEDITOR_API virtual IPCGBaseSubsystem* GetSubsystem() const;
	PCGEDITOR_API static UPCGSubsystem* GetWorldSubsystem();

	// ~Begin IToolkit interface
	PCGEDITOR_API virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	// ~End IToolkit interface

	// ~Begin FGCObject interface
	PCGEDITOR_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FPCGEditor");
	}
	// ~End FGCObject interface
	
	// ~Begin FEditorUndoClient interface
	PCGEDITOR_API virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;
	PCGEDITOR_API virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// ~End FEditorUndoClient interface

	// ~Begin FAssetEditorToolkit interface
	PCGEDITOR_API virtual FText GetToolkitName() const override;
	PCGEDITOR_API virtual FName GetToolkitFName() const override;
	PCGEDITOR_API virtual FText GetBaseToolkitName() const override;
	PCGEDITOR_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
	PCGEDITOR_API virtual FString GetWorldCentricTabPrefix() const override;
	PCGEDITOR_API virtual void OnClose() override;
	PCGEDITOR_API virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	// ~End FAssetEditorToolkit interface

	PCGEDITOR_API void SaveEditedObjectState();
	PCGEDITOR_API void RestoreEditedObjectState();

	/**
	 * Handles spawning a graph node in the current graph using the passed in chord.
	 *
	 * @param	InChord		Chord that was just performed.
	 * @param	InPosition	Current cursor position.
	 * @param	InGraph		Graph that chord was performed in.
	 *
	 * @return	FReply	Whether chord was handled.
	 */
	UE_DEPRECATED(5.6, "Please use the version of the function accepting FVector2f.")
	FReply OnSpawnNodeByShortcut(FInputChord InChord, const FVector2D& InPosition, UPCGEditorGraph* InGraph);
	PCGEDITOR_API FReply OnSpawnNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UPCGEditorGraph* InGraph);

	FActionMenuContent OnCreateActionMenu(UEdGraph* InGraph, const FVector2f& Location, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed OnMenuClosed);

	/** Returns contextual menu filtering mask (no mask, compatible only, accept filtering and/or conversions). */
	PCGActionsHelpers::ECompatibilityMask GetContextualCompatibilityMask() const;

	/** Can determinism be tested on the current graph */
	bool CanRunDeterminismGraphTest() const;
	/** Run the determinism test on the current graph */
	void OnDeterminismGraphTest() const;

	// Can override the schema used for this editor. By default, it's UPCGEditorSchema.
	PCGEDITOR_API virtual TSubclassOf<UPCGEditorGraphSchema> GetSchemaClass() const;

	UPCGDefaultExecutionSource* GetDefaultExecutionSource() const { return PCGDefaultExecutionSource; }

	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreatePaletteTabFactory();
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateDebugObjectTreeTabFactory();
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateFindTabFactory();
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateAttributesTabFactory(int32 Index, TSharedRef<FWorkspaceItem> Group);
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateDeterminismTabFactory();
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateProfilingTabFactory();
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateLogTabFactory();
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateCodeEditorTabFactory();
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateGraphParamsTabFactory();
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreatePropertyDetailsTabFactory(int32 Index, TSharedRef<FWorkspaceItem> Group);
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateViewportTabFactory(int32 Index, TSharedRef<FWorkspaceItem> Group);
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateEmbeddedSubgraphsTabFactory();
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateDataOverridesTabFactory();

	UE_DEPRECATED(5.8, "Use CreateCodeEditorTabFactory() instead")
	PCGEDITOR_API TSharedPtr<FWorkflowTabFactory> CreateNodeSourceTabFactory();

	/** Called when graph editor focus is changed */
	PCGEDITOR_API void OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor);

	/** Called when the graph editor tab is backgrounded */
	PCGEDITOR_API void OnGraphEditorBackgrounded(const TSharedRef<SGraphEditor>& InGraphEditor);

	PCGEDITOR_API virtual void OnNodeToolStarted(UPCGEditorGraphNodeBase* InteractiveNode);
	PCGEDITOR_API virtual void OnNodeToolEnded(UPCGEditorGraphNodeBase* InteractiveNode);

	void OnGraphPreSave(const UPCGGraph* InGraph);

	/** Brings focus to the Data Overrides tab and sets it to inspect the given node. */
	PCGEDITOR_API void OpenDataOverridesAndInspect(UPCGEditorGraphNodeBase* InNode);

	/** Selects the owner actor of the currently inspected PCG component and brings focus to the level viewport. */
	PCGEDITOR_API void FocusOwningActorInLevelViewport() const;

	/** Selects the given node in the level viewport's manual edit panel. */
	PCGEDITOR_API void SelectManualEditNode(UPCGEditorGraphNodeBase* InNode);

protected:
	PCGEDITOR_API virtual void CreateEditorModeManager() override;

	PCGEDITOR_API virtual TAttribute<FGraphAppearanceInfo> GetAppearanceInfo() const;

	UE_DEPRECATED(5.8, "Implement version with FToolbarBuilder parameter instead")
	virtual void RegisterToolbarInternal(FToolMenuSection& PCGSection) const {}

	/** Register PCG specific toolbar for the editor */
	PCGEDITOR_API virtual void RegisterToolbarInternal(FToolBarBuilder& ToolbarBuilder) const;

	UE_DEPRECATED(5.8, "Use version with FToolBarBuilder parameter instead")
	void RegisterToolbarButton(FToolMenuSection& Section, EPCGToolbarButtons Button) const {}

	/** Register PCG default editor toolbar button */
	PCGEDITOR_API void RegisterToolbarButton(FToolBarBuilder& ToolbarBuilder, EPCGToolbarButtons Button) const;

	/** Bind commands to delegates */
	PCGEDITOR_API virtual void BindCommands();

	PCGEDITOR_API virtual void OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType);

	/** Called when the selection changes in the GraphEditor */
	PCGEDITOR_API virtual void OnSelectedNodesChanged(const TSet<UObject*>& NewSelection);

	/**
	 * Called when a node is double clicked
	 *
	 * @param Node - The Node that was clicked
	 */
	PCGEDITOR_API virtual void OnNodeDoubleClicked(UEdGraphNode* Node);

	/** Toggle node inspection state for selected nodes. */
	PCGEDITOR_API void OnToggleInspected();

	/** Set inspection state for node. Can be null, in that case it will clear inspection. */
	PCGEDITOR_API void SetNodeInspected(UPCGEditorGraphNodeBase* InspectedNode, bool bValue);

	/** Whether we can toggle inspection of selected nodes */
	PCGEDITOR_API virtual bool CanToggleInspected() const;

	/** Whether we can toggle debug state of selected nodes */
	PCGEDITOR_API virtual bool CanToggleDebug() const;

	/** Whether we can toggle enabled state of selected nodes */
	PCGEDITOR_API virtual bool CanToggleEnabled() const;

	/** Create a new viewport widget for the given index. Can be subclassed to have customization around the viewport. */
	PCGEDITOR_API virtual TSharedRef<SPCGEditorViewport> CreateViewportWidget(int32 Index = 0);
	
	/** Set up AssetEditorModeManager with required Tool Contexts. */
	PCGEDITOR_API virtual void SetupModeTools(FAssetEditorModeManager* InModeTools);

	/** Returns viewport at specified index if it exists, nullptr otherwise */
	PCGEDITOR_API TSharedPtr<SPCGEditorViewport> GetViewportWidget(int32 Index) const;

	/** Returns the mode manager for the viewport at specified index, nullptr if not yet created */
	PCGEDITOR_API FAssetEditorModeManager* GetViewportModeManager(int32 Index) const;
	
	/** Auto layout the nodes for better visualization. Returns true if a node has moved.*/
	PCGEDITOR_API bool AutoLayoutFullGraph();

	// TO BE REMOVED IN 5.8
	// @todo_pcg
	// Temporary boolean to force the Attribute List view to Update even if it is closed. To be set in the Editor Ctor.
	UE_DEPRECATED(all, "Do not use.")
	bool bForceRefreshAttributeEvenIfClosed = false;

	friend class SPCGEditorGraphAttributeListView;
	friend class FPCGEditorDefaultMode;

private:
	/** Register PCG specific toolbar for the editor */
	void RegisterToolbar(FToolBarBuilder& InToolbarBuilder) const;
	
	/** Bring up the find tab */
	void OnFind();

	/** Bring up the first free details view, or if they are all locked, the first details view */
	void OpenDetailsView();

	/** Called when a details view tab is closed */
	void OnDetailsViewTabClosed(FName Id, int32 Index);

	/** Called when an attribute list view tab is closed */
	void OnAttributeListViewTabClosed(FName Id, int32 Index);

	/** Called when a viewport view tab is closed */
	void OnViewportViewTabClosed(FName Id, int32 Index);

	/** Enable/Disable automatic PCG node generation */
	void OnPauseAutomaticRegeneration_Clicked();
	/** Has the user paused automatic regeneration in the Graph Editor */
	bool IsAutomaticRegenerationPaused() const;

	/** Force a regeneration by invoking the graph notifications  */
	void OnForceGraphRegeneration_Clicked();

	/** Auto-Layout nodes in the graph for better layout */
	void OnAutoLayoutNodes_Clicked();

	/** Whether selected nodes are inspected or not */
	ECheckBoxState GetInspectedCheckState() const;

	/** Called when editor module permission mode changes */
	void OnPermissionChanged(EPCGEditorPermissionMode InPermissionMode);

	void UpdateAfterInspectedStackChanged(const FPCGStack& FullStack);

	/** Refresh visualization on every editor node in the main graph and its embedded subgraphs. */
	void RefreshEditorNodeVisualization(const IPCGGraphExecutionSource* InSource, const FPCGStack* InStack, EPCGChangeType InitialChangeType) const;

	/** Toggle node enabled state for selected nodes */
	void OnToggleEnabled();
	/** Whether selected nodes are enabled or not */
	ECheckBoxState GetEnabledCheckState() const;
	
	/** Toggle node debug state for selected nodes */
	void OnToggleDebug();
	/** Whether selected nodes are being debugged or not */
	ECheckBoxState GetDebugCheckState() const;

	/** Called when the contextual menu filtering toggle is changed. */
	void OnContextualFilteringChanged(ECheckBoxState NewState, const TWeakPtr<SGraphEditorActionMenu> WeakActionMenu, PCGActionsHelpers::ECompatibilityMask AffectedMask);

	/** Whether the contextual menu filtering is enabled or not. */
	bool IsContextualFilteringEnabled(PCGActionsHelpers::ECompatibilityMask AffectedMask) const;
	ECheckBoxState IsContextualFilteringChecked(PCGActionsHelpers::ECompatibilityMask AffectedMask) const;

	/** Enable node debug state for selected nodes and disable for others */
	void OnDebugOnlySelected();

	/** Disable node debug state for all nodes */
	void OnDisableDebugOnAllNodes();

	/** Enable manual viewport editing for the selected node, then select its owning actor and frame the viewport on it. */
	void OnEditInViewport();
	/** Returns true if the selected node supports viewport editing. */
	bool CanToggleViewportEditing() const;

	/** Mark/unmark a node for persistent viewport editing. */
	void OnMarkForViewportEditing();
	/** Returns the check box state for whether the selected node is marked for viewport editing. */
	ECheckBoxState GetMarkForViewportEditingCheckState() const;

	/** Notify the module to re-evaluate manual edit panel visibility after node state changes */
	void NotifyModuleManualEditStateChanged();

	/** Cancels the current execution of the selected graph */
	void OnCancelExecution_Clicked();

	/** Returns true if inspected graph is currently scheduled or executing */
	bool IsCurrentlyGenerating() const;

	/** Returns true if the debug object tree tab is not currently open. */
	bool IsDebugObjectTreeTabClosed() const;

	/** Opens the debug object tree tab if it is not open already. */
	void OnOpenDebugObjectTreeTab_Clicked();

	/** Can determinism be tested on the selected node(s) */
	bool CanRunDeterminismNodeTest() const;
	/** Run the determinism test on the selected node(s) */
	void OnDeterminismNodeTest() const;

	/** Open details view for the PCG object being edited */
	void OnEditGraphSettings();
	/** Whether the PCG object being edited is opened in details view or not */
	bool IsEditGraphSettingsToggled() const;

	/** Open main graph editor */
	void OnEditGraph_Clicked();
	/** Whether the main graph editor is opened */
	bool IsEditGraphTabClosed() const;

	/** Open panel to view and edit the graph parameters. */
	void OnToggleGraphParamsPanel() const;
	/** Is the graph params panel open. */
	bool IsToggleGraphParamsToggled() const;

	/** Select every node in the graph */
	void SelectAllNodes();
	/** Whether we can select every node */
	bool CanSelectAllNodes() const;

	/** Delete all selected nodes in the graph */
	void DeleteSelectedNodes();
	/** Whether we can delete all selected nodes */
	bool CanDeleteSelectedNodes() const;

	/** Copy all selected nodes in the graph */
	void CopySelectedNodes();
	/** Whether we can copy all selected nodes */
	bool CanCopySelectedNodes() const;

	/** Cut all selected nodes in the graph */
	void CutSelectedNodes();
	/** Whether we can cut all selected nodes */
	bool CanCutSelectedNodes() const;

	/** Paste nodes in the graph */
	void PasteNodes();
	/** Paste nodes in the graph at location*/
	void PasteNodesHere(const FVector2D& Location);
	/** Whether we can paste nodes */
	bool CanPasteNodes() const;

	/** Duplicate the currently selected nodes */
	void DuplicateNodes();
	/** Whether we are able to duplicate the currently selected nodes */
	bool CanDuplicateNodes() const;

	/** Exports node settings to assets */
	void OnExportNodes();

	/** Whether we are able to export the currently selected nodes */
	bool CanExportNodes() const;

	/** Converts instanced nodes to independent nodes */
	void OnConvertToStandaloneNodes();

	/** Whether we are able to convert the selected nodes to standalone */
	bool CanConvertToStandaloneNodes() const;

	/** Collapse the currently selected nodes in a subgraph */
	void OnCollapseNodesInSubgraph(bool bCollapseToEmbeddedSubgraph);
	/** Whether we can collapse nodes in a subgraph */
	bool CanCollapseNodesInSubgraph(bool bCollapseToEmbeddedSubgraph) const;

	/** User is attempting to add a dynamic pin in the given direction to a node. */
	void OnAddDynamicPin(EPCGPinDirection Direction);
	/** Returns true if the user can add a dynamic pin in the given direction to the selected node. */
	bool CanAddDynamicPin(EPCGPinDirection Direction) const;

	/** User is attempting to rename a node */
	void OnRenameNode();
	/** Whether the user can rename the selected node */
	bool CanRenameNode() const;

	/** Selects the associated usages of a given reroute declaration */
	void OnSelectNamedRerouteUsages();
	/** Whether the user can find the usages from the selection */
	bool CanSelectNamedRerouteUsages() const;

	/** Selects the associated declaration of a given reroute usage */
	void OnSelectNamedRerouteDeclaration();
	/** Whether the user can find the declaration from the selection */
	bool CanSelectNamedRerouteDeclaration() const;

	/** Jumps to source definition for the selected nodes. */
	void OnJumpToSource();

	/** Internal method that validates a few things (& logs errors) prior to executing actions. */
	bool InternalValidationOnAction();

	/** Finds editor graph node that matches the provided PCG node */
	UPCGEditorGraphNodeBase* GetEditorNode(const UPCGNode* InNode);

	void OnAlignTop();
	void OnAlignMiddle();
	void OnAlignBottom();
	void OnAlignLeft();
	void OnAlignCenter();
	void OnAlignRight();
	void OnStraightenConnections();
	void OnDistributeNodesH();
	void OnDistributeNodesV();
	void OnCreateComment();

	/** Get or Create new Graph editor widget */
	TSharedPtr<SGraphEditor> GetFocusedGraphEditorWidget() const;
	void CreateGraphEditorCommands();
	TSharedRef<SGraphEditor> CreateGraphEditorWidget(TSharedPtr<FTabInfo> InTabInfo, UPCGEditorGraph* InGraph);
	
	/** Get or Create palette widget */
	TSharedRef<SPCGEditorGraphNodePalette> GetOrCreatePaletteWidget();
	
	/** Get or Create new debug object tree widget */
	TSharedRef<SPCGEditorGraphDebugObjectTree> GetOrCreateDebugObjectTreeWidget();

	/** Get or Create new find widget */
	TSharedRef<SPCGEditorGraphFind> GetOrCreateFindWidget();

	/** Get or Create new attributes widget */
	TSharedRef<SPCGEditorGraphAttributeListView> GetOrCreateAttributesWidget(int32 Index);

	/** Get or Create a new determinism tab widget */
	TSharedRef<SPCGEditorGraphDeterminismListView> GetOrCreateDeterminismWidget();

	/** Get or Create a new profiling tab widget */
	TSharedRef<SPCGEditorGraphProfilingView> GetOrCreateProfilingWidget();

	/** Get or Create a new log capture tab widget */
	TSharedRef<SPCGEditorGraphLogView> GetOrCreateLogWidget();

	/** Get or Create a new code editor tab widget */
	TSharedRef<SPCGCodeEditor> GetOrCreateCodeEditorWidget();

	/** Get or Create a new user graph parameters tab widget */
	TSharedRef<SPCGEditorGraphUserParametersView> GetOrCreateGraphParamsWidget();

	/** Get or Create a new details tab widget */
	TSharedRef<SPCGEditorGraphDetailsView> GetOrCreatePropertyDetailsWidget(int32 Index);

	/** Get or Create a new viewport tab widget */
	TSharedRef<SPCGEditorViewport> GetOrCreateViewportWidget(int32 Index);

	/** Get or Create a new embedded subgraphs tab widget */
	TSharedRef<SPCGEditorGraphEmbeddedSubgraphsView> GetOrCreateEmbeddedSubgraphsWidget();

	/** Get or Create a new data overrides tab widget */
	TSharedRef<SPCGEditorGraphDataOverridesView> GetOrCreateDataOverridesWidget();

	/** Called when the component inspected is generated/cleaned */
	void OnSourceGenerated(IPCGGraphExecutionSource* InSource);

	/** Called to validate a node title change. */
	bool OnValidateNodeTitle(const FText& NewName, UEdGraphNode* GraphNode, FText& OutErrorMessage);

	/** Called when the title of a node is changed */
	void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	/**
	 * Try to jump to a given class (if allowed)
	 *
	 * @param Class - The Class to jump to
	 */
	void JumpToDefinition(const UClass* Class) const;

	/** Called when a PCG execution source unregisters. */
	void OnSourceUnregistered(IPCGGraphExecutionSource* Source);

	/** Called when a component finishes executing. Useful for updating debugging tools/UIs. */
	void OnSourceGenerationDone(IPCGBaseSubsystem* Subsystem, IPCGGraphExecutionSource* Source, EPCGGenerationStatus Status);

	/** Trigger any generation required to ensure debug display is up to date. */
	void UpdateDebugAfterSourceSelection(IPCGGraphExecutionSource* InOldSource, IPCGGraphExecutionSource* InNewSource, bool bNewSourceStartedInspecting);

	void RegisterDelegatesForWorld(UWorld* World);
	void UnregisterDelegatesForWorld(UWorld* World);

	void OnNodeSourceCompiled(const UPCGNode* InNode, const FPCGCompilerDiagnostics& InDiagnostics);

	void OnMapChanged(UWorld* InWorld, EMapChangeType InMapChangedType);
	void OnPostPIEStarted(bool bIsSimulating);
	void OnEndPIE(bool bIsSimulating);
	void OnGraphExecutionSourcesChanged();

	void RefreshViews();

	void UpdateDefaultExecutionSource();
	void ReleaseDefaultExecutionSource(bool bCollectGarbage = false);

	void CloseInvalidDocuments();

	PCGEDITOR_API virtual TSharedRef<FTabManager::FLayout> GetDefaultLayout() const;
	virtual void RegisterExtraTabFactories(FWorkflowAllowedTabSet& TabSet) {}

	TSharedPtr<SGraphEditor> GraphEditorWidget;
	TSharedPtr<SPCGEditorGraphDetailsView> PropertyDetailsWidgets[PCGEditorTabs::NumPropertyDetailTabs];
	TSharedPtr<SPCGEditorGraphNodePalette> PaletteWidget;
	TSharedPtr<SPCGEditorGraphDebugObjectTree> DebugObjectTreeWidget;
	TSharedPtr<SPCGEditorGraphFind> FindWidget;
	TSharedPtr<SPCGEditorGraphAttributeListView> AttributesWidgets[PCGEditorTabs::NumAttributeTabs];
	TSharedPtr<SPCGEditorGraphDeterminismListView> DeterminismWidget;
	TSharedPtr<SPCGEditorGraphProfilingView> ProfilingWidget;
	TSharedPtr<SPCGEditorGraphLogView> LogWidget;
	TSharedPtr<SPCGCodeEditor> CodeEditorWidget;
	TSharedPtr<SPCGEditorGraphUserParametersView> UserParamsWidget;
	TSharedPtr<SPCGEditorViewport> ViewportWidgets[PCGEditorTabs::NumViewportTabs];
	TSharedPtr<FAssetEditorModeManager> ViewportModeManagers[PCGEditorTabs::NumViewportTabs];
	TSharedPtr<SPCGEditorGraphEmbeddedSubgraphsView> EmbeddedSubgraphsWidget;
	TSharedPtr<SPCGEditorGraphDataOverridesView> DataOverridesWidget;

	int32 ActiveToolViewportIndex = INDEX_NONE;

	TSharedPtr<FUICommandList> GraphEditorCommands;

	TObjectPtr<UPCGDefaultExecutionSource> PCGDefaultExecutionSource = nullptr;

	TObjectPtr<UPCGGraph> MainGraph = nullptr;
	TWeakPtr<SGraphEditor> FocusedGraphEditor = nullptr;

	FPCGEditorInspectionDataManager InspectionDataManager;

	// Keep track of the last execution status to be able to break infinite loop when a source is triggered to be generated by inspection
	// aborted and re-triggered.
	TOptional<TPair<const IPCGGraphExecutionSource*, EPCGGenerationStatus>> LastExecutionStatus;

	/** Document manager for workflow tabs */
	TSharedPtr<FDocumentTracker> DocumentManager;

	/** Factory that spawns graph editors; used to look up all tabs spawned by it. */
	TWeakPtr<FDocumentTabFactory> DocumentTabFactory;

	/** Contextual menu filter toggle state variable. Starts as compatible only to show more likely choices. */
	PCGActionsHelpers::ECompatibilityMask ContextualActionMask = PCGActionsHelpers::ECompatibilityMask::Compatible;
};
