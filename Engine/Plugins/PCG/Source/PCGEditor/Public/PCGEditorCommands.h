// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class UPCGSettings;
class UEdGraph;
struct FEdGraphSchemaAction;

class FPCGEditorCommands : public TCommands<FPCGEditorCommands>
{
public:
	PCGEDITOR_API FPCGEditorCommands();

	// ~Begin TCommands<> interface
	PCGEDITOR_API virtual void RegisterCommands() override;
	// ~End TCommands<> interface

	TSharedPtr<FUICommandInfo> CollapseNodes;
	TSharedPtr<FUICommandInfo> CollapseNodesToEmbeddedSubgraph;
	TSharedPtr<FUICommandInfo> ExportNodes;
	TSharedPtr<FUICommandInfo> ConvertToStandaloneNodes;
	TSharedPtr<FUICommandInfo> Find;
	TSharedPtr<FUICommandInfo> ShowSelectedDetails;
	TSharedPtr<FUICommandInfo> PauseAutoRegeneration;
	TSharedPtr<FUICommandInfo> ForceGraphRegeneration;
	TSharedPtr<FUICommandInfo> AutoLayoutNodes;
	TSharedPtr<FUICommandInfo> OpenDebugObjectTreeTab;
	TSharedPtr<FUICommandInfo> RunDeterminismNodeTest;
	TSharedPtr<FUICommandInfo> RunDeterminismGraphTest;
	TSharedPtr<FUICommandInfo> EditGraphSettings;
	TSharedPtr<FUICommandInfo> EditGraph;
	TSharedPtr<FUICommandInfo> ToggleGraphParams;
	TSharedPtr<FUICommandInfo> CancelExecution;
	TSharedPtr<FUICommandInfo> ToggleEnabled;
	TSharedPtr<FUICommandInfo> ToggleDebug;
	TSharedPtr<FUICommandInfo> DebugOnlySelected;
	TSharedPtr<FUICommandInfo> DisableDebugOnAllNodes;
	TSharedPtr<FUICommandInfo> ToggleInspect;
	TSharedPtr<FUICommandInfo> AddInputPin;
	TSharedPtr<FUICommandInfo> AddOutputPin;
	TSharedPtr<FUICommandInfo> RenameNode;
	TSharedPtr<FUICommandInfo> SelectNamedRerouteUsages;
	TSharedPtr<FUICommandInfo> SelectNamedRerouteDeclaration;
	TSharedPtr<FUICommandInfo> JumpToSource;
	TSharedPtr<FUICommandInfo> AutoFocusViewport;
	TSharedPtr<FUICommandInfo> EditInViewport;
	TSharedPtr<FUICommandInfo> MarkForViewportEditing;
};

struct FPCGSpawnNodeCommandInfo
{
	explicit FPCGSpawnNodeCommandInfo(TSubclassOf<UPCGSettings> PCGSettingsClass) : PCGSettingsClass(PCGSettingsClass) {}

	/** Optional preconfigured settings info for a specific configuration of the node. */
	FPCGPreConfiguredSettingsInfo PreconfiguredInfo;

	/** Holds the UI Command to verify chords for this action are held. */
	TSharedPtr<FUICommandInfo> CommandInfo;

	/**
	 * Creates an action to be used for placing a node into the graph.
	 *
	 * @return A fully prepared action containing the information to spawn the node.
	 */
	TSharedPtr<FEdGraphSchemaAction> GetAction() const;

	/**
	 * Gets the PCG Settings class of the spawn node action.
	 *
	 * @return The PCG Settings class of the spawn node action.
	 */
	UClass* GetClass() const { return PCGSettingsClass; }

private:
	/** Type of settings class (node) to spawn. */
	TSubclassOf<UPCGSettings> PCGSettingsClass;
};

/** Handles spawn node commands for the PCG Graph Editor. */
class FPCGEditorSpawnNodeCommands : public TCommands<FPCGEditorSpawnNodeCommands>
{
public:
	FPCGEditorSpawnNodeCommands()
		: TCommands(TEXT("PCGEditorSpawnNodes"), NSLOCTEXT("PCGEditorSpawnNodes", "PCGEditorSpawnNodes", "PCG Editor - Spawn Nodes"), NAME_None, FAppStyle::GetAppStyleSetName())
	{}

	// ~Begin TCommands<> interface
	PCGEDITOR_API virtual void RegisterCommands() override;
	// ~End TCommands<> interface

	/**
	 * Returns a graph action assigned to the passed in chord
	 *
	 * @param InChord		The chord to use for lookup.
	 * @return				A shared pointer to the schema action.
	 */
	PCGEDITOR_API TSharedPtr<FEdGraphSchemaAction> GetGraphActionByChord(const FInputChord& InChord) const;

private:
	/** An array of all the possible commands for spawning nodes. */
	TArray<TSharedPtr<FPCGSpawnNodeCommandInfo>> SpawnNodeCommands;
};
