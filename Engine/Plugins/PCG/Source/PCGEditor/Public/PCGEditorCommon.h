// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"
#include "Misc/AssetCategoryPath.h"

#include "PCGEditorCommon.generated.h"

#define LOCTEXT_NAMESPACE "PCGEditorCommon"

UENUM()
enum class EPCGElementType : uint8
{
	Native = 1 << 0,
	Blueprint = 1 << 1,
	Subgraph = 1 << 2,
	Settings = 1 << 3,
	Asset = 1 << 4,
	Other = 1 << 5,
	All = (1 << 6) - 1
};
ENUM_CLASS_FLAGS(EPCGElementType);

/** Used to make sure UHT generates properly */
USTRUCT()
struct FPCGEditorCommonDummyStruct
{
	GENERATED_BODY()
};

namespace FPCGEditorCommon
{
	const FString ContextIdentifier = TEXT("PCGEditorContext");

	const FName SpatialDataType = FName(TEXT("Spatial Data"));
	const FName ParamDataType = FName(TEXT("Attribute Set"));
	const FName SettingsDataType = FName(TEXT("Settings Data"));
	const FName OtherDataType = FName(TEXT("Other Data"));

	const FName ConcreteDataType = FName(TEXT("Concrete Data"));
	const FName PointDataType = FName(TEXT("Point Data"));
	const FName PolyLineDataType = FName(TEXT("Poly Line Data"));
	const FName SurfaceDataType = FName(TEXT("Surface Data"));
	const FName LandscapeDataType = FName(TEXT("Landscape Data"));
	const FName BaseTextureDataType = FName(TEXT("Base Texture Data"));
	const FName TextureDataType = FName(TEXT("Texture Data"));
	const FName RenderTargetDataType = FName(TEXT("Render Target Data"));
	const FName VirtualTextureDataType = FName(TEXT("Virtual Texture Data"));
	const FName VolumeDataType = FName(TEXT("Volume Data"));
	const FName PrimitiveDataType = FName(TEXT("Primitive Data"));
	const FName DynamicMeshDataType = FName(TEXT("Dynamic Mesh Data"));
	const FName StaticMeshResourceDataType = FName(TEXT("Static Mesh Resource Data"));

	extern TAutoConsoleVariable<bool> CVarShowAdvancedAttributesFields;

	const FAssetCategoryPath PCGAssetCategoryPath(LOCTEXT("PCGCategory", "Procedural"));
	const FAssetCategoryPath PCGAssetSubCategoryPath(FPCGEditorCommon::PCGAssetCategoryPath, LOCTEXT("PCGSubCategory", "PCG"), ECategoryMenuType::Section);
	const FAssetCategoryPath PCGAdvancedAssetCategoryPath(FPCGEditorCommon::PCGAssetCategoryPath, LOCTEXT("PCGAvancedCategory", "Advanced"));

	namespace Helpers
	{
		PCGEDITOR_API void DispatchEditorToast(const FText& Text, const FText& SubText = FText{}, const float Duration = 3.f);
	}
}

#undef LOCTEXT_NAMESPACE
