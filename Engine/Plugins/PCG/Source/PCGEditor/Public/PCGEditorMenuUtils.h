// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolMenus.h"
#include "AssetRegistry/AssetData.h"

namespace PCGEditorMenuUtils
{
	PCGEDITOR_API FToolMenuSection& CreatePCGSection(UToolMenu* Menu);
	PCGEDITOR_API void CreateOrUpdatePCGAssetFromMenu(UToolMenu* Menu, TArray<FAssetData>& Assets);
}