// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/IPCGEditorModule.h"

#include "PCGGraphExecutionStateInterface.h"
#include "DeltaViewportExtensions/PCGDeltaViewportExtension.h"
#include "DeltaVisualizations/PCGDeltaVisualizationRegistry.h"

#include "Utils/PCGNodeVisualLogs.h"

#include "AssetTypeCategories.h"
#include "Containers/Ticker.h"
#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"

class FPCGLoadDataAssetChangeTracker;
class FPCGManualEditPanelManager;
class ILevelEditor;
class SPCGManualEditPanel;
class UPCGGraph;

// Logs
DECLARE_LOG_CATEGORY_EXTERN(LogPCGEditor, Log, All);

class FMenuBuilder;
class FPCGEditorGraphNodeFactory;
class IAssetTypeActions;

class FPCGEditorModule : public IPCGEditorModule
{
public:
	// ~IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override;
	// ~End IModuleInterface implementation

	// ~IPCGEditorModule implementation
	virtual TWeakPtr<IPCGEditorProgressNotification> CreateProgressNotification(const FTextFormat& TextFormat, bool bCanCancel) override;
	virtual void ReleaseProgressNotification(TWeakPtr<IPCGEditorProgressNotification> InNotification) override;
	virtual void SetOutlinerUIRefreshDelay(float InDelay) override;
	virtual const FPCGNodeVisualLogs& GetNodeVisualLogs() const override { return NodeVisualLogs; }
	virtual FPCGNodeVisualLogs& GetNodeVisualLogsMutable() override { return NodeVisualLogs; }
	virtual bool CanSelectPartitionActors() const override;
	virtual void SetPermissionMode(EPCGEditorPermissionMode InPermissionMode) override;
	virtual EPCGEditorPermissionMode GetPermissionMode() const { return PermissionMode; }
	virtual void TransferPropertyBagMetadataIntoHierarchy(UPCGGraph* Graph) const override;
	virtual UObject* CreatePropertyBagHierarchyRootObject(UPCGGraph* Graph) const override;
	virtual void ConnectPropertyBagHierarchyRootObjectModifyDelegate(UPCGGraph* Graph) const override;
	virtual void GraphIsBeingDestroyed(UPCGGraph* Graph) const override;

	static const FPCGDeltaVisualizationRegistry& GetConstDeltaVisualizationRegistry();
	static FPCGDeltaVisualizationRegistry& GetMutableDeltaVisualizationRegistry();
	static const FPCGDeltaViewportExtensionRegistry& GetConstDeltaViewportExtensionRegistry();
	static FPCGDeltaViewportExtensionRegistry& GetMutableDeltaViewportExtensionRegistry();

	DECLARE_DERIVED_EVENT(FPCGEditorModule, IPCGEditorModule::FPermissionModeChanged, FPermissionModeChanged);
	virtual FPermissionModeChanged& OnPermissionModeChanged() override { return PermissionModeChangedEvent; }

	void UpdateManualEditPanelVisibility();

	/** Returns the manual edit panel if one is currently active, or nullptr. */
	TSharedPtr<SPCGManualEditPanel> GetManualEditPanel() const;

protected:
	virtual void OnScheduleGraph(const FPCGStackContext& StackContext) override;
	virtual void OnGraphPreSave(UPCGGraph* Graph, FObjectPreSaveContext ObjectSaveContext) override;
	virtual void ClearExecutionMetadata(IPCGGraphExecutionSource* InSource) override;
	virtual void ClearExecutedStacks(const IPCGGraphExecutionSource* InRootSource) override;
	virtual void ClearExecutedStacks(const UPCGGraph* InContainingGraph) override;
	virtual	TArray<FPCGStackSharedPtr> GetExecutedStacksPtrs(const FPCGStack& BeginningWithStack) override;
	virtual TArray<FPCGStackSharedPtr> GetExecutedStacksPtrs(const IPCGGraphExecutionSource* InSource, const UPCGGraph* InSubgraph, bool bOnlyWithSubgraphAsCurrentFrame) override;
	virtual void NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType) override;
	// ~End IPCGEditorModule implementation

	void RegisterDetailsCustomizations();
	void UnregisterDetailsCustomizations();
	void RegisterMenuExtensions();
	void UnregisterMenuExtensions();
	void PopulateMenuActions(FMenuBuilder& MenuBuilder);
	void RegisterSettings();
	void UnregisterSettings();
	void RegisterPCGDataVisualizations();
	void UnregisterPCGDataVisualizations();
	void RegisterDeltaVisualizations();
	void UnregisterDeltaVisualizations();
	void RegisterDeltaViewportExtensions();
	void UnregisterDeltaViewportExtensions();
	void RegisterPinColorAndIcons();
	void UnregisterPinColorAndIcons();
	void RegisterTrackerFactories();
	void UnregisterTrackerFactories();

	void OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor);
	void RegisterOnEditorModeChange();

	/** Discovers all UPCGAssetEditorInteractiveTool subclasses and registers them. Bound to FEditorDelegates::OnEditorInitialized so all plugins are loaded first. */
	void OnEditorInitialized(double Duration);

	void OnPostEngineInit();
	void OnPreExit();

	void ExtendShowFlagsMenu();
	static int32 GetTreatEditorViewportAsGenerationSourceValue();
	static void OnTreatEditorViewportAsGenerationSourceValueChanged(int32 Value, ESelectInfo::Type SelectInfo);

	/** [EXPERIMENTAL] Used to refresh procedural instances when materials are modified which can otherwise be lost.
	* Note: This function subject to change/removal without deprecation.
	*/
	void OnSceneMaterialsModified();

	bool ShouldDisableCPUThrottling();

	virtual bool ShouldTreatViewportAsGenerationSourceInternal(const APCGWorldActor* InPCGWorldActor) const override;

	TArray<TSharedRef<IAssetTypeActions>> RegisteredAssetTypeActions;
	static EAssetTypeCategories::Type PCGAssetCategory;

	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	TSharedPtr<FPCGEditorGraphNodeFactory> GraphNodeFactory;

	FDelegateHandle ShouldDisableCPUThrottlingDelegateHandle;

	TSet<TSharedPtr<IPCGEditorProgressNotification>> ActiveNotifications;

	FPCGNodeVisualLogs NodeVisualLogs;

	/** A record of stacks that were executed. Used to populate debugging tool UIs. */
	TSet<FPCGStackSharedPtr> ExecutedStacks;
	TMap<FPCGSoftGraphExecutionSource, TSet<FPCGStackSharedPtr>> ExecutedStacksPerSource;
	mutable PCG::FSharedLock ExecutedStacksLock;

	EPCGEditorPermissionMode PermissionMode = EPCGEditorPermissionMode::All;
	FPermissionModeChanged PermissionModeChangedEvent;

	TUniquePtr<FPCGLoadDataAssetChangeTracker> LoadDataAssetChangeTracker;

	TArray<FName> VisualizersToUnregisterOnShutdown;

	TUniquePtr<FPCGManualEditPanelManager> ManualEditPanelManager;

	FPCGDeltaVisualizationRegistry DeltaVisualizationRegistry;
	FPCGDeltaViewportExtensionRegistry DeltaViewportExtensionRegistry;
};
