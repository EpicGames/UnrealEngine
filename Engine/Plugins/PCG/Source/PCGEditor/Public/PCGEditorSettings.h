// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Styling/SlateTypes.h"
#include "Templates/SubclassOf.h"

#include "PCGEditorSettings.generated.h"

class UPCGSettings;
class UPCGBuilderSettings;

struct FEdGraphPinType;

UENUM(BlueprintType)
enum class EPCGEditorDoubleClickAction : uint8
{
	DoNothing,
	ToggleInspectNode UMETA(DisplayName = "Toggle Inspection on Node"),
	ToggleDebugNode UMETA(DisplayName = "Toggle Debug on Node"),
	JumpToSourceFile UMETA(DisplayName = "View Source File (C++)")
};

UENUM()
enum class EPCGTreatViewportAsGenerationSource : uint8
{
	PCGWorldActor UMETA(DisplayName = "Use PCG World Actor flag"),
	Always,
	Never,
};

UCLASS(MinimalAPI, config=Editor, meta=(DisplayName="PCG"))
class UPCGEditorProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	PCGEDITOR_API UPCGEditorProjectSettings(const FObjectInitializer& ObjectInitializer);

	/** Default Builder Settings to use when running the PCGWorldPartitionBuilder */
	UPROPERTY(EditAnywhere, config, Category = Builder)
	TSoftObjectPtr<UPCGBuilderSettings> DefaultBuilderSetting;

	/** Actor classes to ignore for the PCG Level to Asset export */
	UPROPERTY(EditAnywhere, config, Category = "PCG Level to Asset", meta=(DisplayName = "Excluded Actor Classes"))
	TArray<TSubclassOf<AActor>> PCGLevelToAssetExcludedActorClasses;

	/** Deprecated. Replaced by per-graph UPCGGraph::bAutoRefreshOnRenderStateRefresh (and bAutoRefreshOnQualitySettingsChanged for quality events). */
	UE_DEPRECATED(5.8, "Use UPCGGraph::bAutoRefreshOnRenderStateRefresh / bAutoRefreshOnQualitySettingsChanged on each graph instead.")
	UPROPERTY(Config)
	bool bAutoRefreshGPUStaticMeshSpawners = true;
};

USTRUCT()
struct FPCGEditorAttributeListViewSettings
{
	GENERATED_BODY()

	/** Overflow policy for column headers in the Attribute List View. */
	UPROPERTY(EditAnywhere, Config, Category = "Attribute List View")
	ETextOverflowPolicy ColumnNameOverflowPolicy = ETextOverflowPolicy::Ellipsis;

	/** Overflow policy for non-string types in the Attribute List View. */
	UPROPERTY(EditAnywhere, Config, Category = "Attribute List View")
	ETextOverflowPolicy GeneralOverflowPolicy = ETextOverflowPolicy::Ellipsis;

	/** Overflow policy for 'FString' in the Attribute List View. */
	UPROPERTY(EditAnywhere, Config, Category = "Attribute List View")
	ETextOverflowPolicy StringOverflowPolicy = ETextOverflowPolicy::Ellipsis;

	/** Overflow policy for 'FName' in the Attribute List View. */
	UPROPERTY(EditAnywhere, Config, Category = "Attribute List View")
	ETextOverflowPolicy NameOverflowPolicy = ETextOverflowPolicy::Ellipsis;

	/** Overflow policy for 'FSoftObjectPath' and 'FSoftClassPath', which are HyperLinked in the Attribute List View. */
	UPROPERTY(EditAnywhere, Config, Category = "Attribute List View")
	ETextOverflowPolicy HyperLinkOverflowPolicy = ETextOverflowPolicy::MiddleEllipsis;
};

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UPCGEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	PCGEDITOR_API UPCGEditorSettings(const FObjectInitializer& ObjectInitializer);

	PCGEDITOR_API FLinearColor GetColor(UPCGSettings* InSettings) const;

	UE_DEPRECATED(5.7, "Not used anymore, Pin Colors are queried from the PCG Data Type Registry.")
	PCGEDITOR_API FLinearColor GetPinColor(const FEdGraphPinType& InPinType) const;

	/** Default node color */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor DefaultNodeColor;

	/** Instanced node body tint color */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor InstancedNodeBodyTintColor;

	/** Color used for input & output nodes */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor InputOutputNodeColor;

	/** Color used for Difference, Intersection, Projection, Union */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor SetOperationNodeColor;

	/** Color used for density remap */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor DensityOperationNodeColor;

	/** Color used for blueprints */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor BlueprintNodeColor;

	/** Color used for metadata operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor MetadataNodeColor;

	/** Color used for filter-like operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor FilterNodeColor;

	/** Color used for sampler operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor SamplerNodeColor;

	/** Color used for artifact-generating operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor SpawnerNodeColor;

	/** Color used for subgraph-like operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor SubgraphNodeColor;

	/** Color used for embedded subgraph-like operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor EmbeddedSubgraphNodeColor;

	/** Color used for Attribute Set operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel, DisplayName = "Attribute Set Node Color"))
	FLinearColor ParamDataNodeColor;

	/** Color used for debug operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor DebugNodeColor;

	/** Color used for control flow operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor ControlFlowNodeColor;
	
	/** Color used for point operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor PointOpsNodeColor;

	/** Color used for hierarchical generation operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor HierarchicalGenerationNodeColor;

	/** Color used for graph parameters operations */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor GraphParametersNodeColor;

	/** Color used for reroute nodes */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor RerouteNodeColor;

	/** Color used for dynamic mesh nodes */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor DynamicMeshNodeColor;

	/** Color used for resource nodes */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor ResourceNodeColor;

	/** Default pin color */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor DefaultPinColor;

	/** Color used for spatial data pins */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor SpatialDataPinColor;
	
	/** Color used for concrete/simple spatial data pins */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor ConcreteDataPinColor;

	/** Color used for data pins of type Point */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor PointDataPinColor;

	/** Color used for data pins of type Spline */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor PolyLineDataPinColor;

	/** Color used for data pins of type Polygon2D */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor Polygon2DDataPinColor;

	/** Color used for data pins of type Surface */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor SurfaceDataPinColor;

	/** Color used for data pins of type Landscape */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor LandscapeDataPinColor;

	/** Color used for data pins of type Base Texture */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor BaseTextureDataPinColor;
	
	/** Color used for data pins of type Texture */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor TextureDataPinColor;

	/** Color used for data pins of type Data View */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor DataViewPinColor;

	/** Color used for data pins of type Render Target */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor RenderTargetDataPinColor;

	/** Color used for data pins of type Virtual Texture */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor VirtualTextureDataPinColor;

	/** Color used for data pins of type Volume */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor VolumeDataPinColor;

	/** Color used for data pins of type Primitive */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor PrimitiveDataPinColor;

	/** Color used for data pins of type Attribute Set */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel, DisplayName = "Attribute Set Pin Color"))
	FLinearColor ParamDataPinColor;

	/** Color used for data pins of type Dynamic Mesh */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor DynamicMeshPinColor;

	/** Color used for data pins of type Static Mesh Resource */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor StaticMeshResourcePinColor;

	/** Color used for data pins of type Raw Buffer */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor RawBufferPinColor;

	/** Color used for other/unknown data types */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor UnknownDataPinColor;

	/** User-driven color overrides */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	TMap<TSubclassOf<UPCGSettings>, FLinearColor> OverrideNodeColorByClass;

	/** Default wire thickness */
	UPROPERTY(EditAnywhere, config, Category = Wire, meta = (ClampMin="1", ClampMap="9"))
	float DefaultWireThickness = 1.5f;

	/** Hover edge size emphasis */
	UPROPERTY(EditAnywhere, config, Category = Wire, meta = (ClampMin="1", ClampMax="5"))
	float HoverEdgeEmphasis = 1.5f;

	/** Debug mode edge emphasis on multi-data */
	UPROPERTY(EditAnywhere, config, Category = Wire, meta = (ClampMin="1", ClampMax="5"))
	float MultiDataEdgeDebugEmphasis = 2.5f;

	/** Debug mode edge desaturation with empty data */
	UPROPERTY(EditAnywhere, config, Category = Wire, meta = (ClampMin="0", ClampMax="1"))
	float EmptyEdgeDebugDesaturateFactor = 0.5f;

	/** Specify an action to take upon double-clicking a native PCG node. */
	UPROPERTY(EditAnywhere, Config, Category = "Workflow")
	EPCGEditorDoubleClickAction NodeDoubleClickAction = EPCGEditorDoubleClickAction::ToggleInspectNode;

	/** Verify renaming graph parameters via their local getter nodes with a pop-up confirmation window. */
	UPROPERTY(EditAnywhere, config, Category = Workflow)
	bool bConfirmLocalGraphParameterNameChanges = true;

	/** Allow selecting PCG partition actors in the viewport/outliner. */
	UPROPERTY(EditAnywhere, config, Category = "Workflow")
	bool bCanSelectPartitionActors = true;

	/** Controls if we want to treat the viewport as a runtime generation source */
	UPROPERTY(EditAnywhere, config, Category = "Editor Performance")
	EPCGTreatViewportAsGenerationSource TreatViewportAsGenerationSource = EPCGTreatViewportAsGenerationSource::PCGWorldActor;

	/** Specify if we want to disable CPU Throttling when a PCG Graph is executing, this will improve execution time when app is out of focus/minimized */
	UPROPERTY(EditAnywhere, config, Category = "Editor Performance", meta = (DisplayName = "Disable CPU throttling during graph execution"))
	bool bDisableCPUThrottlingDuringGraphExecution = false;

	/** Controls whether the "Pause PCG" button will be display in the editor menu */
	UPROPERTY(EditAnywhere, config, Category = "Editor Performance", meta = (DisplayName = "Show PCG pause button in the editor", ConfigRestartRequired = true))
	bool bShowPauseButton = false;

	/** Controls whether PCG tasks will be cancelled when unpausing PCG, which might result in dirty/stale content, but will not require significant time. */
	UPROPERTY(EditAnywhere, config, Category = "Editor Performance", meta = (DisplayName = "Unpausing cancels all PCG tasks"))
	bool bUnpauseCancelsAll = false;

	/** Controls whether the alternate PCG pause button is shown instead of the default PCG icon. */
	UPROPERTY(EditAnywhere, config, Category = "Editor Performance", meta = (DisplayName = "Use alternate pause button icon", EditCondition = "bShowPauseButton", EditConditionHides))
	bool bUseAlternatePauseButton = false;

	/** Overrides the label of the pause button while PCG is not currently being paused. The default is empty or None, which will use the default label then. */
	UPROPERTY(EditAnywhere, config, Category = "Editor Performance", meta = (DisplayName = "Overrides default name for the 'currently paused' button", EditCondition = "bShowPauseButton", EditConditionHides))
	FName OverridePausedButtonLabel = NAME_None;

	/** Overrides the label of the pause button while PCG is currently paused. The default is empty or None, which will use the default label then. */
	UPROPERTY(EditAnywhere, config, Category = "Editor Performance", meta = (DisplayName = "Overrides default name for the 'not currently paused' button", EditCondition = "bShowPauseButton", EditConditionHides))
	FName OverrideNotPausedButtonLabel = NAME_None;

	/** Overrides the tooltip on the pause button. The default is empty, which will use the default tooltip then. */
	UPROPERTY(EditAnywhere, config, Category = "Editor Performance", meta = (MultiLine = true, DisplayName = "Overrides default tooltip on the pause button", EditCondition = "bShowPauseButton", EditConditionHides))
	FString OverridePausedButtonTooltip = FString();

	/** Target number of points when trying to debug volume data or landscape data, to avoid exploding the number of points if the volume/landscape is big, and to have a good representation if the volume/landscape is small. */
	UPROPERTY(EditAnywhere, config, Category = "Editor Performance")
	int32 TargetNumPointsForDebug = 64000;

	/** Controls whether the template picker dialog will be shown when creating a new PCG graph, both in the contextual menu and in the PCG graph pickers. */
	UPROPERTY(EditAnywhere, config, Category = "Content Browser")
	bool bShowTemplatePickerOnNewGraph = true;

	/** Display node 'General' actions on the right click context menu. */
	UPROPERTY(EditAnywhere, config, Category = "Graph UI")
	bool bShowNodeGeneralActionsRightClickContextMenu = true;

	/** Display node 'Organizational' actions on the right click context menu. */
	UPROPERTY(EditAnywhere, config, Category = "Graph UI")
	bool bShowNodeOrganizationalActionsRightClickContextMenu = true;

	/** Display node 'Determinism' actions on the right click context menu. */
	UPROPERTY(EditAnywhere, config, Category = "Graph UI")
	bool bShowNodeDeterminismActionsRightClickContext = true;

	/** Attribute List View user configurations. */
	UPROPERTY(EditAnywhere, Config, Category = "Attribute List View", meta = (ShowOnlyInnerProperties))
	FPCGEditorAttributeListViewSettings AttributeListViewSettings;

	/** When inspection updates the viewport will re-focus to the data. */
	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	bool bAutoFocusViewport = true;

	/** Font size used in the HLSL Code Editor. */
	UPROPERTY(EditAnywhere, config, Category = "Code Editor", meta = (ClampMin = 6, ClampMax = 32))
	int32 CodeEditorFontSize = 9;

	// Begin deprecated properties
	UE_DEPRECATED(5.6, "bEnableNavigateToNativeNodes has been replaced by NodeDoubleClickAction.")
	UPROPERTY()
	bool bEnableNavigateToNativeNodes = true;
};
