// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"

enum ETabRole : uint8;
class FPCGEditor;
struct FSlateIcon;
class FText;
class FWorkspaceItem;
class SGraphEditor;
class SWidget;
class UPCGEditorGraph;

DECLARE_DELEGATE_OneParam(FPCGGenericTabClosed, FName);

struct FPCGGenericTabFactoryParams
{
	FPCGGenericTabFactoryParams(FName InId, TSharedPtr<FPCGEditor> InEditor, TFunction<TSharedRef<SWidget>()> InCreateTabBodyFunc)
		: Id(InId)
		, Editor(InEditor)
		, CreateTabBodyFunc(InCreateTabBodyFunc)
	{ }

	FName Id;
	TSharedPtr<FPCGEditor> Editor;
	TFunction<TSharedRef<SWidget>()> CreateTabBodyFunc;
	FPCGGenericTabClosed OnTabClosed;
	
	bool bSingleton = true;

	TOptional<ETabRole> Role;
	TOptional<FSlateIcon> Icon;
	TOptional<FText> Label;

	TSharedPtr<FWorkspaceItem> Group;
};

struct FPCGGenericTabFactory : public FWorkflowTabFactory
{
public:
	PCGEDITOR_API FPCGGenericTabFactory(const FPCGGenericTabFactoryParams& InParams);

	PCGEDITOR_API virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const;
	PCGEDITOR_API virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

private:
	void OnTabClosed(TSharedRef<SDockTab> DockTab) const;

	TFunction<TSharedRef<SWidget>()> CreateTabBodyFunc;
	FPCGGenericTabClosed OnTabClosedDelegate;
};
struct FPCGGraphEditorDocumentTabFactory : public FDocumentTabFactoryForObjects<UPCGEditorGraph>
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SGraphEditor>, FOnCreateGraphEditorWidget, TSharedPtr<FTabInfo>, UPCGEditorGraph*);
public:
	FPCGGraphEditorDocumentTabFactory(TSharedPtr<FPCGEditor> InEditorPtr, FOnCreateGraphEditorWidget InCreateGraphEditorWidgetCallback);

	virtual void OnTabActivated(TSharedPtr<SDockTab> InTab) const override;
	virtual void OnTabBackgrounded(TSharedPtr<SDockTab> InTab) const override;
	virtual void OnTabRefreshed(TSharedPtr<SDockTab> InTab) const override;
	virtual void SaveState(TSharedPtr<SDockTab> InTab, TSharedPtr<FTabPayload> InPayload) const override;
	virtual bool SupportsObjectType(UObject* InDocumentID) const override;

protected:
	static FText GetTabTitle(TWeakObjectPtr<UPCGEditorGraph> InWeakEditorGraph);

	TSharedPtr<FPCGEditor> GetPCGEditor() const;

	virtual TAttribute<FText> ConstructTabNameForObject(UPCGEditorGraph* InEditorGraph) const override;
	virtual TSharedRef<SWidget> CreateTabBodyForObject(const FWorkflowTabSpawnInfo& InInfo, UPCGEditorGraph* InEditorGraph) const override;
	virtual const FSlateBrush* GetTabIconForObject(const FWorkflowTabSpawnInfo& InInfo, UPCGEditorGraph* InEditorGraph) const override;

protected:
	FOnCreateGraphEditorWidget OnCreateGraphEditorWidget;
};
