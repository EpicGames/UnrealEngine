// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGAssetExporter.h"

#include "Engine/World.h"

#include "PCGLevelToAsset.generated.h"

#define UE_API PCGEDITOR_API

class UPackage;

UCLASS(MinimalAPI, BlueprintType, Blueprintable, meta = (ShowWorldContextPin))
class UPCGLevelToAsset : public UPCGAssetExporter
{
	GENERATED_BODY()

public:
	/** Creates/updates a PCG Asset per given world. Allows exporter subclassing by passing in a Subclass. */
	static UE_API void CreateOrUpdatePCGAssets(const TArray<FAssetData>& WorldAssets, const FPCGAssetExporterParameters& Parameters = FPCGAssetExporterParameters(), TSubclassOf<UPCGLevelToAsset> Subclass = {});

	/** Creates/Updates a PCG Asset for a specific world. Allows exporter subclassing by passing in a Subclass. Will return null if it fails, or the package that was modified on success. */
	static UE_API UPackage* CreateOrUpdatePCGAsset(TSoftObjectPtr<UWorld> World, const FPCGAssetExporterParameters& Parameters = FPCGAssetExporterParameters(), TSubclassOf<UPCGLevelToAsset> Subclass = {});

	/** Creates/Updates a PCG Asset for a specific world. Allows exporter subclassing (and settings creation by extension). Will return null if it fails, or the package that was modified on success. */
	static UE_API UPackage* CreateOrUpdatePCGAsset(UWorld* World, const FPCGAssetExporterParameters& Parameters = FPCGAssetExporterParameters(), TSubclassOf<UPCGLevelToAsset> Subclass = {});

	/** Parses the world and fills in the provided data asset. Implement this in BP to drive the generation in a custom manner. */
	UFUNCTION(BlueprintNativeEvent, meta = (DisplayName = "Export World", ForceAsFunction))
	UE_API bool BP_ExportWorld(UWorld* World, const FString& PackageName, UPCGDataAsset* Asset);

	UFUNCTION(BlueprintCallable, Category="PCG|IO")
	UE_API void SetWorld(UWorld* World);

	/** Set the world to export from a UObject. The WorldObject must be a World. Return false if the object is not a world or null. */
	UFUNCTION(BlueprintCallable, Category = "PCG|IO")
	UE_API bool SetWorldObject(UObject* WorldObject);

	UFUNCTION(BlueprintCallable, Category = "PCG|IO")
	UE_API UWorld* GetWorld() const;

protected:
	//~Being UPCGAssetExporter interface
	UE_API virtual bool ExportAsset(const FString& PackageName, UPCGDataAsset* Asset) override;
	UE_API virtual UPackage* UpdateAsset(const FAssetData& PCGAsset) override;
	//~End UPCGAssetExporter interface

	UWorld* WorldToExport = nullptr;
};

#undef UE_API
