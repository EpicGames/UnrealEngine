// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"

#include "PCGEditorCommon.h"
#include "Editor/PCGGraphCustomization.h"

#include "ConnectionDrawingPolicy.h"
#include "StructUtils/PropertyBag.h"

#include "Data/Registry/PCGDataTypeCommon.h"

#include "PCGEditorGraphSchema.generated.h"

class UPCGBaseSubgraphSettings;
enum class EPCGElementType : uint8;

class IAssetReferenceFilter;
class UPCGEditorGraph;

namespace PCGActionsHelpers
{
	enum class ECompatibilityMask
	{
		NoMask = 0,
		Compatible = 1 << std::underlying_type_t<EPCGDataTypeCompatibilityResult>(EPCGDataTypeCompatibilityResult::Compatible),
		Filter = 1 << std::underlying_type_t<EPCGDataTypeCompatibilityResult>(EPCGDataTypeCompatibilityResult::RequireFilter),
		Conversion = 1 << std::underlying_type_t<EPCGDataTypeCompatibilityResult>(EPCGDataTypeCompatibilityResult::RequireConversion),
		// note: SubtypeNotCompatible isn't used yet, but we'll keep it here for completeness
		SubtypeNotCompatible = 1 << std::underlying_type_t<EPCGDataTypeCompatibilityResult>(EPCGDataTypeCompatibilityResult::TypeCompatibleSubtypeNotCompatible)
	};
	ENUM_CLASS_FLAGS(ECompatibilityMask);
}

struct FPCGActionsFilter
{
	FPCGActionsFilter(const UEdGraph* InEdGraph, EPCGElementType InElementFilterType, FPCGGraphEditorCustomization InCustomization);

	bool Accepts(const FText& InCategory) const;

	EPCGElementType FilterType = EPCGElementType::All;
	const UPCGEditorGraph* Graph = nullptr;
	FPCGGraphEditorCustomization Customization;
};

UCLASS(MinimalAPI)
class UPCGEditorGraphSchema : public UEdGraphSchema
{
	GENERATED_BODY()

public:
	PCGEDITOR_API void GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const;

	// Return the element type filtering for the current editor. By default it is all.
	virtual EPCGElementType GetElementTypeFiltering() const { return EPCGElementType::All; }

	// Returns the settings class to use when creating a subgraph node. Override in derived schemas to provide a custom subgraph settings class.
	PCGEDITOR_API virtual TSubclassOf<UPCGBaseSubgraphSettings> GetSubgraphSettingsClass() const;
	
	// Return the graph customization for this editor. By default it is the one provided by the graph.
	PCGEDITOR_API virtual const FPCGGraphEditorCustomization& GetGraphEditorCustomization(const UEdGraph* InEdGraph) const;

	//~ Begin EdGraphSchema Interface
	PCGEDITOR_API virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	PCGEDITOR_API virtual FLinearColor GetPinColor(const UEdGraphPin* InPin) const override;
	PCGEDITOR_API virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	PCGEDITOR_API virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override;
	PCGEDITOR_API virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const override;
	PCGEDITOR_API virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	PCGEDITOR_API virtual FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	PCGEDITOR_API virtual void DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	PCGEDITOR_API virtual void DroppedAssetsOnNode(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const override;
	PCGEDITOR_API virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	PCGEDITOR_API virtual void GetAssetsNodeHoverMessage(const TArray<struct FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const override;
	PCGEDITOR_API virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const override;
	PCGEDITOR_API virtual void GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const override;
	//~ End EdGraphSchema Interface

	PCGEDITOR_API virtual const FSlateBrush* GetMetadataTypeSlateBrush(EPCGContainerType ContainerType) const;
	PCGEDITOR_API virtual FLinearColor GetMetadataTypeColor(EPCGMetadataTypes Type) const;

	PCGEDITOR_API virtual const FSlateBrush* GetPropertyBagTypeSlateBrush(EPropertyBagContainerType ContainerType) const;
	PCGEDITOR_API virtual FLinearColor GetPropertyBagTypeColor(const FPropertyBagPropertyDesc& Desc) const;

	// To customize if we want to have special brushes for Inspect and Debug on the nodes.
	virtual bool ShouldAddInspectBrushOnNode() const { return true; }
	virtual bool ShouldAddDebugBrushOnNode() const { return true; }
	virtual bool ShouldAddViewportEditBrushOnNode() const { return true; }

protected:
	PCGEDITOR_API virtual void GetGraphActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter, bool bIsContextual) const;
	PCGEDITOR_API virtual void GetNativeElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const;
	PCGEDITOR_API virtual void GetBlueprintElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const;
	PCGEDITOR_API virtual void GetSubgraphElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const;
	PCGEDITOR_API virtual void GetSettingsElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter, bool bIsContextual) const;
	PCGEDITOR_API virtual void GetExtraElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const;
	PCGEDITOR_API virtual void GetNamedRerouteUsageActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const;
	PCGEDITOR_API virtual void GetDataAssetActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const;

	PCGEDITOR_API virtual bool TryCreateConnectionInternal(UEdGraphPin* A, UEdGraphPin* B, bool bAddConversionNodeIfNeeded) const;

	PCGEDITOR_API virtual bool DroppedAssetsOnNodeInternal(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, const UEdGraphNode* Node, bool bValidate, FString& OutTooltipText) const;

private:
	static TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UEdGraph* Graph);
};

class FPCGEditorConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
public:
	FPCGEditorConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraph);
	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;

protected:
	bool UpdateParamsIfDebugging(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, FConnectionParams& Params);

	UPCGEditorGraph* Graph;
};
