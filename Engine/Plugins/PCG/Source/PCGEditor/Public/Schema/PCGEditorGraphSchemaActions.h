// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"

#include "PCGGraph.h"
#include "PCGSettings.h"
#include "Templates/SubclassOf.h"

#include "PCGEditorGraphSchemaActions.generated.h"

class UEdGraph;
class UEdGraphPin;
class UPCGBlueprintBaseElement;
class UPCGNode;
class UPCGEditorGraph;
struct FSlateBrush;

UENUM()
enum class EPCGEditorNewSettingsBehavior : uint8
{
	Normal = 0,
	ForceCopy,
	ForceInstance
};

UENUM()
enum class EPCGEditorNewPCGGraphBehavior : uint8
{
	Normal = 0,
	SubgraphNode,
	LoopNode
};

USTRUCT(MinimalAPI)
struct FPCGEditorGraphSchemaActionBase : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	using FEdGraphSchemaAction::FEdGraphSchemaAction;

	virtual FText GetMenuDescriptionOverride() const;

	virtual FSlateBrush const* GetPaletteIcon() const override;
	void SetPaletteIcon(TSharedPtr<FSlateBrush>& InPaletteIcon);

protected:
	mutable TSharedPtr<FSlateBrush> PaletteIcon = nullptr;
};

USTRUCT(MinimalAPI)
struct FPCGEditorGraphSchemaAction_NewNativeElement : public FPCGEditorGraphSchemaActionBase
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FPCGEditorGraphSchemaActionBase::FPCGEditorGraphSchemaActionBase;

	// Simple type info
	static FName StaticGetTypeId()
	{
		static FName Type("FPCGEditorGraphSchemaAction_NewNativeElement");
		return Type;
	}
	
	UPROPERTY()
	TSubclassOf<UPCGSettings> SettingsClass;

	UPROPERTY()
	FPCGPreConfiguredSettingsInfo PreconfiguredInfo;

	//~ Begin FEdGraphSchemaAction Interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual FSlateBrush const* GetPaletteIcon() const override;
	//~ End FEdGraphSchemaAction Interface

protected:
	virtual void PostCreation(UPCGNode* NewNode) {}
};

USTRUCT()
struct FPCGEditorGraphSchemaAction_NewGetParameterElement : public FPCGEditorGraphSchemaAction_NewNativeElement
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FPCGEditorGraphSchemaAction_NewNativeElement::FPCGEditorGraphSchemaAction_NewNativeElement;

	UPROPERTY()
	FPropertyBagPropertyDesc PropertyDesc;

	virtual FSlateBrush const* GetPaletteIcon() const override;

protected:
	virtual void PostCreation(UPCGNode* NewNode) override;
};

USTRUCT()
struct FPCGEditorGraphSchemaAction_NewSettingsElement : public FPCGEditorGraphSchemaActionBase
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FPCGEditorGraphSchemaActionBase::FPCGEditorGraphSchemaActionBase;

	// Simple type info
	static FName StaticGetTypeId()
	{
		static FName Type("FPCGEditorGraphSchemaAction_NewSettingsElement");
		return Type;
	}

	UPROPERTY()
	FSoftObjectPath SettingsObjectPath;

	UPROPERTY()
	EPCGEditorNewSettingsBehavior Behavior = EPCGEditorNewSettingsBehavior::Normal;

	//~ Begin FPCGEditorGraphSchemaActionBase Interface
	virtual FText GetMenuDescriptionOverride() const override;
	//~ End FPCGEditorGraphSchemaActionBase Interface

	//~ Begin FEdGraphSchemaAction Interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual FSlateBrush const* GetPaletteIcon() const override;
	//~ End FEdGraphSchemaAction Interface

	static void MakeSettingsNodesOrContextualMenu(const TSharedRef<class SWidget>& InPanel, const FVector2D& InScreenPosition, UEdGraph* InGraph, const TArray<FSoftObjectPath>& InSettingsPaths, const TArray<FVector2D>& InLocations, bool bInSelectNewNodes);
	static void MakeSettingsNodes(UPCGEditorGraph* InEditorGraph, TArray<UPCGSettings*> InSettings, TArray<FVector2D> InSettingsLocations, bool bInSelectNewNodes, bool bInCreateInstances);
	static UEdGraphNode* MakeSettingsNode(UPCGEditorGraph* InEditorGraph, UEdGraphPin* InFromPin, UPCGSettings* InSettings, FVector2D InLocation, bool bInSelectNewNode, bool bInCreateInstance);
};

USTRUCT()
struct FPCGEditorGraphSchemaAction_NewBlueprintElement : public FPCGEditorGraphSchemaActionBase
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FPCGEditorGraphSchemaActionBase::FPCGEditorGraphSchemaActionBase;
	
	// Simple type info
	static FName StaticGetTypeId()
	{
		static FName Type("FPCGEditorGraphSchemaAction_NewBlueprintElement");
		return Type;
	}

	UPROPERTY()
	FSoftClassPath BlueprintClassPath;

	UPROPERTY()
	FPCGPreConfiguredSettingsInfo PreconfiguredInfo;

	//~ Begin FEdGraphSchemaAction Interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual FSlateBrush const* GetPaletteIcon() const override;
	//~ End FEdGraphSchemaAction Interface
};

USTRUCT()
struct FPCGEditorGraphSchemaAction_NewLoadAssetElement : public FPCGEditorGraphSchemaActionBase
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FPCGEditorGraphSchemaActionBase::FPCGEditorGraphSchemaActionBase;

	// Simple type info
	static FName StaticGetTypeId()
	{
		static FName Type("FPCGEditorGraphSchemaAction_NewLoadAssetElement");
		return Type;
	}

	UPROPERTY()
	FAssetData Asset;

	UPROPERTY()
	TSubclassOf<UPCGSettings> SettingsClass;

	//~ Begin FEdGraphSchemaAction Interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Creates a new subgraph element and allows choosing whether it can be a subgraph node or a loop node. */
USTRUCT()
struct FPCGEditorGraphSchemaAction_NewSubgraphElement : public FPCGEditorGraphSchemaActionBase
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FPCGEditorGraphSchemaActionBase::FPCGEditorGraphSchemaActionBase;

	// Simple type info
	static FName StaticGetTypeId()
	{
		static FName Type("FPCGEditorGraphSchemaAction_NewSubgraphElement");
		return Type;
	}

	UPROPERTY()
	FSoftObjectPath SubgraphObjectPath;

	UPROPERTY()
	EPCGEditorNewPCGGraphBehavior Behavior = EPCGEditorNewPCGGraphBehavior::Normal;

	//~ Begin FEdGraphSchemaAction Interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

	static void MakeGraphNodesOrContextualMenu(const TSharedRef<class SWidget>& InPanel, const FVector2D& InScreenPosition, UEdGraph* InGraph, const TArray<FSoftObjectPath>& InGraphPaths, const TArray<FVector2D>& InLocations, bool bInSelectNewNodes);
	static void MakeGraphNodes(UPCGEditorGraph* InEditorGraph, TArray<UPCGGraphInterface*> InGraph, TArray<FVector2D> InGraphLocations, bool bInSelectNewNodes, bool bInCreateLoop);
	static UEdGraphNode* MakeGraphNode(UPCGEditorGraph* InEditorGraph, UEdGraphPin* InFromPin, UPCGGraphInterface* InGraph, const FVector2D& InLocation, bool bInSelectNewNode, bool bInCreateLoop);
};

/** Action to add a 'comment' node to the graph */
USTRUCT()
struct FPCGEditorGraphSchemaAction_NewComment : public FPCGEditorGraphSchemaActionBase
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FPCGEditorGraphSchemaActionBase::FPCGEditorGraphSchemaActionBase;

	// Simple type info
	static FName StaticGetTypeId()
	{
		static FName Type("FPCGEditorGraphSchemaAction_NewComment");
		return Type;
	}

	// FEdGraphSchemaAction interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction;
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface
};

USTRUCT()
struct FPCGEditorGraphSchemaAction_NewReroute : public FPCGEditorGraphSchemaActionBase
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FPCGEditorGraphSchemaActionBase::FPCGEditorGraphSchemaActionBase;

	// Simple type info
	static FName StaticGetTypeId()
	{
		static FName Type("FPCGEditorGraphSchemaAction_NewReroute");
		return Type;
	}

	// FEdGraphSchemaAction interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface
};

USTRUCT()
struct FPCGEditorGraphSchemaAction_NewNamedRerouteUsage : public FPCGEditorGraphSchemaActionBase
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FPCGEditorGraphSchemaActionBase::FPCGEditorGraphSchemaActionBase;

	// Declaration that we want to add an usage of
	UPROPERTY()
	TObjectPtr<const UPCGNode> DeclarationNode = nullptr;

	// Simple type info
	static FName StaticGetTypeId()
	{
		static FName Type("FPCGEditorGraphSchemaAction_NewNamedRerouteUsage");
		return Type;
	}

	// FEdGraphSchemaAction interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface
};

USTRUCT()
struct FPCGEditorGraphSchemaAction_NewNamedRerouteDeclaration : public FPCGEditorGraphSchemaActionBase
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FPCGEditorGraphSchemaActionBase::FPCGEditorGraphSchemaActionBase;

	// Simple type info
	static FName StaticGetTypeId()
	{
		static FName Type("FPCGEditorGraphSchemaAction_NewNamedRerouteDeclaration");
		return Type;
	}

	// FEdGraphSchemaAction interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface
};
