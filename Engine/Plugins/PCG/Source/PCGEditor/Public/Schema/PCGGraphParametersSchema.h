// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBagDetails.h"
#include "HierarchyEditor/PropertyBagHierarchyViewModel.h"

#include "PCGGraphParametersSchema.generated.h"

UCLASS(MinimalAPI)
class UPCGPropertyBagHierarchyRoot : public UPropertyBagHierarchyRoot
{
	GENERATED_BODY()
	
	// Add future properties for PCG root here
};

UCLASS(MinimalAPI)
class UPCGPropertyBagHierarchyCategory : public UPropertyBagHierarchyCategory
{
	GENERATED_BODY()
	
	// Add future properties for PCG categories here
};

UCLASS(MinimalAPI)
class UPCGPropertyBagHierarchySection : public UPropertyBagHierarchySection
{
	GENERATED_BODY()
	
	// Add future properties for PCG sections here
};

/**
 * Specific property bag schema to allow only arrays and set as containers.
 */
UCLASS(MinimalAPI)
class UPCGGraphParametersSchema : public UPropertyBagSchema
{
	GENERATED_BODY()
public:
	virtual bool SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const override;
	
	virtual UPropertyBagHierarchyRoot* GetHierarchyRoot(const TArray<UObject*>& ObjectsWithProperty) const override;
	
	virtual TArray<UScriptStruct*> GetHierarchyPropertyMetaDataTypes(const FPropertyBagPropertyDesc& Desc) const override;
	
	virtual TSubclassOf<UPropertyBagHierarchyCategory> GetHierarchyCategoryType() const override;
	
	virtual TSubclassOf<UPropertyBagHierarchySection> GetHierarchySectionType() const override;
};