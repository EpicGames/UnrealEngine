// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataVisualization.h"

#include "SAssetEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"

class FAdvancedPreviewScene;
class FPCGEditorViewportClient;
class SBox;

class SPCGEditorViewport : public SAssetEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(SPCGEditorViewport)
			: _ModeTools(nullptr)
		{}

		SLATE_ARGUMENT(FEditorModeTools*, ModeTools)
	SLATE_END_ARGS()

	PCGEDITOR_API SPCGEditorViewport();
	PCGEDITOR_API virtual ~SPCGEditorViewport();

	PCGEDITOR_API void Construct(const FArguments& InArgs);

	//~ Begin ICommonEditorViewportToolbarInfoProvider Interface
	PCGEDITOR_API virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	PCGEDITOR_API virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override {}
	//~ End ICommonEditorViewportToolbarInfoProvider Interface

	/** Rebuilds the scene using the provided resources and setup callback. */
	PCGEDITOR_API void SetupScene(const TArray<UObject*>& InResources, const FPCGSetupSceneFunc& SetupFunc);
	PCGEDITOR_API void ResetScene();
	FAdvancedPreviewScene* GetAdvancedPreviewScene() { return AdvancedPreviewScene.Get(); }

	/** Add UObject references for GC. */
	PCGEDITOR_API void AddReferencedObjects(FReferenceCollector& Collector);
	
	/** Call when a new node is selected. The next ResetScene will restore the viewport type to default and SetupScene will auto-focus. */
	PCGEDITOR_API void OnNewNodeInspected();
	
protected:
	//~ Begin SEditorViewport Interface
	PCGEDITOR_API virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	PCGEDITOR_API virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	PCGEDITOR_API virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() override;
	PCGEDITOR_API virtual void BindCommands() override;
	PCGEDITOR_API virtual void OnFocusViewportToSelection() override;
	//~ End SEditorViewport Interface

	PCGEDITOR_API void ReleaseManagedResources();

	PCGEDITOR_API void ToggleAutoFocusViewport();
	PCGEDITOR_API bool IsAutoFocusViewportChecked() const;

	virtual void OnSetupScene() {};
	virtual void OnResetScene() {};

protected:
	TSharedPtr<FPCGEditorViewportClient> EditorViewportClient = nullptr;
	TSharedPtr<FAdvancedPreviewScene> AdvancedPreviewScene = nullptr;
	
	FEditorModeTools* ModeTools = nullptr;

	/** Objects used by the scene. Must be managed for GC. */
	TArray<TObjectPtr<UObject>> ManagedResources;

	/** Used when focusing the data viewport to the visualization. */
	FBoxSphereBounds FocusBounds = FBoxSphereBounds(EForceInit::ForceInit);

	/** Used when focusing the data viewport to the visualization for orthographic views. If not set then FocusBounds is used to determin zoom. */
	TOptional<float> FocusOrthoZoom;

	/** Slot in the viewport toolbar populated by the active visualization with its ViewportToolbarWidget. */
	TSharedPtr<SBox> DynamicToolbarSlot;

private:
	/** Set by OnNewNodeInspected(); consumed by ResetScene() to restore viewport type and trigger auto-focus. */
	bool bPendingViewportReset = false;
};
