// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGraphExecutionStateInterface.h"

#include "UObject/WeakInterfacePtr.h"
#include "Widgets/SCompoundWidget.h"

class FReply;
struct EVisibility;

class SPCGExecutionSourceActionWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPCGExecutionSourceActionWidget) {}
		SLATE_ATTRIBUTE(TArray<TWeakInterfacePtr<IPCGGraphExecutionSource>>, ExecutionSources)
	SLATE_END_ARGS()

	PCGEDITOR_API void Construct(const FArguments& InArgs);

private:
	EVisibility GetGenerateButtonVisibility() const;
	EVisibility GetCancelButtonVisibility() const;
	EVisibility GetCleanupButtonVisibility() const;
	EVisibility GetRefreshButtonVisibility() const;
	EVisibility GetClearPCGLinkButtonVisibility() const;
	
	FReply OnGenerateClicked();
	FReply OnCancelClicked();
	FReply OnCleanupClicked();
	FReply OnRefreshClicked();
	FReply OnClearPCGLinkClicked();
	
	TArray<TWeakInterfacePtr<IPCGGraphExecutionSource>> ExecutionSources;
};